[CoreRedirects]
; Traits rename
+StructRedirects=(OldName="/Script/AnimNext.AnimNextDecoratorHandle", NewName="/Script/UAFAnimGraph.AnimNextTraitHandle")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextDecoratorStack", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextTraitStack")

; Parameter block/graph merge
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextParameterBlock", NewName="/Script/UAF.UAFSystem")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterBlock_EditorData", NewName="/Script/UAFUncookedOnly.UAFSystem_EditorData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterBlock_EdGraph", NewName="/Script/UAFUncookedOnly.AnimNextEdGraph")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterBlock_EdGraphNode", NewName="/Script/UAFUncookedOnly.AnimNextEdGraphNode")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterBlock_EdGraphSchema", NewName="/Script/UAFUncookedOnly.AnimNextEdGraphSchema")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterBlockParameter", NewName="/Script/UAFUncookedOnly.AnimNextVariableEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterBlockGraph", NewName="/Script/UAFUncookedOnly.AnimNextEventGraphEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraphEntry", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_TraitSchema", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphSchema")
+PropertyRedirects=(OldName="/Script/AnimNext.AnimNextScheduleEntry_ParamScope.ParameterBlocks", NewName="Parameters")
+PropertyRedirects=(OldName="/Script/AnimNext.AnimNextScheduleParamScopeEntryTask.ParameterBlocks", NewName="Parameters")

; Workspace plugin split
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextWorkspace", NewName="/Script/WorkspaceEditor.Workspace")

; Scene graph prototype compatibility
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextMeshComponent", NewName="/Script/Engine.SkeletalMeshComponent")

; AnimNextGraph -> AnimNextModule
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextGraph", NewName="/Script/UAF.UAFSystem")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_Controller", NewName="/Script/UAFUncookedOnly.AnimNextControllerBase")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_EditorData", NewName="/Script/UAFUncookedOnly.UAFSystem_EditorData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_AnimationGraphSchema", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphSchema")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_EdGraph", NewName="/Script/UAFUncookedOnly.AnimNextEdGraph")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_EdGraphNode", NewName="/Script/UAFUncookedOnly.AnimNextEdGraphNode")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_EdGraphSchema", NewName="/Script/UAFUncookedOnly.AnimNextEdGraphSchema")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterBlock_ParamSchema", NewName="/Script/UAFUncookedOnly.AnimNextEventGraphSchema")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_EventGraphSchema", NewName="/Script/UAFUncookedOnly.AnimNextEventGraphSchema")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_UnitNode", NewName="/Script/UAFUncookedOnly.AnimNextUnitNode")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextRigVMWorkspaceAssetUserData", NewName="/Script/UAFUncookedOnly.AnimNextAssetWorkspaceAssetUserData")

+PropertyRedirects=(OldName="/Script/AnimNext.AnimNextGraphInstance.Graph", NewName="Module")
+PropertyRedirects=(OldName="/Script/AnimNext.AnimNextScheduleEntry_AnimNextGraph.Graph", NewName="Module")
+PropertyRedirects=(OldName="/Script/AnimNext.AnimNextSubGraphHostTraitSharedData.SubGraph", NewName="Module")
+PropertyRedirects=(OldName="/Script/AnimNext.AnimNextScheduleGraphTask.Graph", NewName="Module")
+PropertyRedirects=(OldName="/Script/AnimNext.AnimNextScheduleGraphTask.DynamicGraph", NewName="DynamicModule")
+PropertyRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterProviderAssetRegistryExports.Parameters", NewName="Variables")

; Splitting up modules & adding variables to graphs
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextModuleWorkspaceAssetUserData", NewName="/Script/UAFUncookedOnly.AnimNextAssetWorkspaceAssetUserData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextModule_Controller", NewName="/Script/UAFUncookedOnly.AnimNextControllerBase")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextModule_AnimationGraph", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextModule_Parameter", NewName="/Script/UAFUncookedOnly.AnimNextVariableEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextModule_EventGraph", NewName="/Script/UAFUncookedOnly.AnimNextEventGraphEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_AnimationGraph", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_Parameter", NewName="/Script/UAFUncookedOnly.AnimNextVariableEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraph_EventGraph", NewName="/Script/UAFUncookedOnly.AnimNextEventGraphEntry")

+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraphAssetOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextModuleOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextVariableOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextParameterProviderAssetRegistryExports", NewName="/Script/UAFUncookedOnly.AnimNextAssetRegistryExports")

+EnumRedirects=(OldName="EAnimNextModuleInitMethod",NewName="/Script/UAF.EAnimNextModuleInitMethod")
+EnumRedirects=(OldName="EAnimNextParameterFlags",NewName="/Script/UAFUncookedOnly.EAnimNextExportedVariableFlags")

+PropertyRedirects=(OldName="/Script/AnimNext.RigVMTrait_AnimNextPublicVariables.Asset", NewName="InternalAsset")
+PropertyRedirects=(OldName="/Script/AnimNext.RigVMTrait_AnimNextPublicVariables.VariableNames", NewName="InternalVariableNames")

; Move all animation graph functionality to AnimNextAnimGraph plugin
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextAnimationGraph", NewName="/Script/UAFAnimGraph.UAFAnimGraph")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextGraphContextData", NewName="/Script/UAFAnimGraph.AnimNextGraphContextData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextGraphEntryPoint", NewName="/Script/UAFAnimGraph.AnimNextGraphEntryPoint")
+StructRedirects=(OldName="/Script/AnimNext.RigDecorator_AnimNextCppDecorator", NewName="/Script/UAFAnimGraph.RigDecorator_AnimNextCppDecorator")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextGraphEvaluatorExecuteArgument", NewName="/Script/UAFAnimGraph.AnimNextGraphEvaluatorExecuteArgument")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextGraphEvaluatorExecuteDefinition", NewName="/Script/UAFAnimGraph.AnimNextGraphEvaluatorExecuteDefinition")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextGraphEvaluator", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextGraphEvaluator")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextGraphRoot", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextGraphRoot")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextRunAnimationGraph", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextRunAnimationGraph_v1")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextShimRoot", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextShimRoot")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextTraitStack", NewName="/Script/UAFAnimGraph.RigUnit_AnimNextTraitStack")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextEntryPointHandle", NewName="/Script/UAFAnimGraph.AnimNextEntryPointHandle")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextTraitHandle", NewName="/Script/UAFAnimGraph.AnimNextTraitHandle")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextTraitSharedData", NewName="/Script/UAFAnimGraph.AnimNextTraitSharedData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextGraphInstance", NewName="/Script/UAFAnimGraph.AnimNextGraphInstance")

+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextAnimationGraphFactory", NewName="/Script/UAFAnimGraphEditor.UAFAnimGraphFactory")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AssetDefinition_AnimNextAnimationGraph", NewName="/Script/UAFAnimGraphEditor.AssetDefinition_AnimNextAnimationGraph")

+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextAnimationGraph_EditorData", NewName="/Script/UAFAnimGraphUncookedOnly.UAFAnimGraph_EditorData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextAnimationGraphSchema", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphSchema")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextAnimationGraphOutlinerData", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphOutlinerData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextAnimationGraphEntry", NewName="/Script/UAFAnimGraphUncookedOnly.AnimNextAnimationGraphEntry")

; PlayAnim -> Injection
+StructRedirects=(OldName="/Script/AnimNext.ModuleInstanceComponent", NewName="/Script/UAF.UAFModuleInstanceComponent")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextGraphInstancePtr", NewName="/Script/UAFAnimGraph.AnimNextAnimGraph")

; Pass-by-ref
+StructRedirects=(OldName="/Script/AnimNext.AnimNextDataInterfaceInstance", NewName="/Script/UAF.UAFAssetInstance")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextDataInterface", NewName="/Script/UAF.AnimNextSharedVariables")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextDataInterfaceFactory", NewName="/Script/UAFEditor.UAFSharedVariablesFactory")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextDataInterface_EditorData", NewName="/Script/UAFUncookedOnly.UAFSharedVariables_EditorData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextDataInterfaceEntry", NewName="/Script/UAFUncookedOnly.UAFSharedVariablesEntry")
+PropertyRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextSharedVariablesEntry.DataInterface", NewName="Asset")
+PropertyRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextSharedVariablesEntry.DataInterfacePath", NewName="ObjectPath")
+FunctionRedirects=(OldName="AnimNextRigVMAssetLibrary.AddDataInterface",NewName="UAFRigVMAssetLibrary.AddSharedVariables")
+FunctionRedirects=(OldName="AnimNextComponent.SetVariable",NewName="AnimNextComponent.BlueprintSetVariable")

; AnimNext plugin -> UAF plugin
+PackageRedirects=(OldName="/Script/AnimNext", NewName="/Script/UAF")
+PackageRedirects=(OldName="/Script/AnimNextEditor", NewName="/Script/UAFEditor")
+PackageRedirects=(OldName="/Script/AnimNextUncookedOnly", NewName="/Script/UAFUncookedOnly")
+PackageRedirects=(OldName="/Script/AnimNextTestSuite", NewName="/Script/UAFTestSuite")

+StructRedirects=(OldName="/Script/AnimNext.AnimNextExecuteContext", NewName="/Script/UAF.AnimNextExecuteContext")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextNativeDataInterface", NewName="/Script/UAF.AnimNextNativeDataInterface")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextGraphReferencePose", NewName="/Script/UAF.AnimNextGraphReferencePose")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextGraphLODPose", NewName="/Script/UAF.AnimNextGraphLODPose")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextBase", NewName="/Script/UAF.RigUnit_AnimNextBase")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextBeginExecution", NewName="/Script/UAF.RigUnit_AnimNextBeginExecution")
+StructRedirects=(OldName="/Script/AnimNext.RigVMTrait_AnimNextPublicVariables", NewName="/Script/UAF.RigVMTrait_AnimNextPublicVariables")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextActorComponentReferenceComponent", NewName="/Script/UAF.AnimNextActorComponentReferenceComponent")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextModuleContextData", NewName="/Script/UAF.AnimNextModuleContextData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextModuleInstance", NewName="/Script/UAF.AnimNextModuleInstance")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextSkeletalMeshComponentReferenceComponent", NewName="/Script/UAF.AnimNextSkeletalMeshComponentReferenceComponent")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextModuleAddDependencies", NewName="/Script/UAF.RigUnit_AnimNextModuleAddDependencies")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextModuleRemoveDependencies", NewName="/Script/UAF.RigUnit_AnimNextModuleRemoveDependencies")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextModuleEventBase", NewName="/Script/UAF.RigUnit_AnimNextModuleEventBase")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextExecuteBindings_GT", NewName="/Script/UAF.RigUnit_AnimNextExecuteBindings_GT")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextExecuteBindings_WT", NewName="/Script/UAF.RigUnit_AnimNextExecuteBindings_WT")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextInitializeEvent", NewName="/Script/UAF.RigUnit_AnimNextInitializeEvent")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextUserEvent", NewName="/Script/UAF.RigUnit_AnimNextUserEvent")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextPrePhysicsEvent", NewName="/Script/UAF.RigUnit_AnimNextPrePhysicsEvent")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextPostPhysicsEvent", NewName="/Script/UAF.RigUnit_AnimNextPostPhysicsEvent")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextActorLocatorFragment", NewName="/Script/UAF.AnimNextActorLocatorFragment")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextComponentLocatorFragment", NewName="/Script/UAF.AnimNextComponentLocatorFragment")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextObjectCastLocatorFragment", NewName="/Script/UAF.AnimNextObjectCastLocatorFragment")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextObjectFunctionLocatorFragment", NewName="/Script/UAF.AnimNextObjectFunctionLocatorFragment")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextObjectPropertyLocatorFragment", NewName="/Script/UAF.AnimNextObjectPropertyLocatorFragment")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextParameterBase", NewName="/Script/UAF.RigUnit_AnimNextParameterBase")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextParameterBeginExecution", NewName="/Script/UAF.RigUnit_AnimNextParameterBeginExecution")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_GetLayerParameter", NewName="/Script/UAF.RigVMDispatch_GetLayerParameter")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_GetParameter", NewName="/Script/UAF.RigVMDispatch_GetParameter")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_SetLayerParameter", NewName="/Script/UAF.RigVMDispatch_SetLayerParameter")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextFieldPath", NewName="/Script/UAF.AnimNextFieldPath")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextSoftFunctionPtr", NewName="/Script/UAF.AnimNextSoftFunctionPtr")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextVariableOverridesCollection", NewName="/Script/UAF.AnimNextVariableOverridesCollection")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_CopyModuleProxyVariables", NewName="/Script/UAF.RigUnit_CopyModuleProxyVariables")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_ResolveUniversalObjectLocator", NewName="/Script/UAF.RigUnit_ResolveUniversalObjectLocator")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_CallHoistedAccessorFunctionBase", NewName="/Script/UAF.RigVMDispatch_CallHoistedAccessorFunctionBase")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_CallHoistedAccessorFunctionNative", NewName="/Script/UAF.RigVMDispatch_CallHoistedAccessorFunctionNative")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_CallHoistedAccessorFunctionScript", NewName="/Script/UAF.RigVMDispatch_CallHoistedAccessorFunctionScript")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_CallObjectAccessorFunctionBase", NewName="/Script/UAF.RigVMDispatch_CallObjectAccessorFunctionBase")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_CallObjectAccessorFunctionNative", NewName="/Script/UAF.RigVMDispatch_CallObjectAccessorFunctionNative")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_CallObjectAccessorFunctionScript", NewName="/Script/UAF.RigVMDispatch_CallObjectAccessorFunctionScript")
+StructRedirects=(OldName="/Script/AnimNext.RigVMDispatch_GetObjectProperty", NewName="/Script/UAF.RigVMDispatch_GetObjectProperty")
+StructRedirects=(OldName="/Script/AnimNext.UAFInstanceVariableContainer", NewName="/Script/UAF.UAFInstanceVariableContainer")
+StructRedirects=(OldName="/Script/AnimNext.UAFInstanceVariableContainerProxy", NewName="/Script/UAF.UAFInstanceVariableContainerProxy")
+StructRedirects=(OldName="/Script/AnimNext.UAFInstanceVariableDataProxy", NewName="/Script/UAF.UAFInstanceVariableDataProxy")
+StructRedirects=(OldName="/Script/AnimNext.RemappedBone", NewName="/Script/UAF.RemappedBone")
+StructRedirects=(OldName="/Script/AnimNext.BoneRemapping", NewName="/Script/UAF.BoneRemapping")
+StructRedirects=(OldName="/Script/AnimNext.RemapPoseData", NewName="/Script/UAF.RemapPoseData")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextRemapPose", NewName="/Script/UAF.RigUnit_AnimNextRemapPose")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_AnimNextWriteSkeletalMeshComponentPose", NewName="/Script/UAF.RigUnit_AnimNextWriteSkeletalMeshComponentPose")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextActorTransformComponent", NewName="/Script/UAF.AnimNextActorTransformComponent")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_GetActorTransform", NewName="/Script/UAF.RigUnit_GetActorTransform")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextActorTransformComponent", NewName="/Script/UAF.AnimNextActorTransformComponent")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_GetActorTransform", NewName="/Script/UAF.RigUnit_GetActorTransform")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_IsAnimBPDriven", NewName="/Script/UAF.RigUnit_IsAnimBPDriven")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_IsRecentlyRendered", NewName="/Script/UAF.RigUnit_IsRecentlyRendered")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_MakeReferencePoseFromSkeletalMeshComponent", NewName="/Script/UAF.RigUnit_MakeReferencePoseFromSkeletalMeshComponent")
+StructRedirects=(OldName="/Script/AnimNext.RigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction", NewName="/Script/UAF.RigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction")
+StructRedirects=(OldName="/Script/AnimNext.RigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction", NewName="/Script/UAF.RigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextOverrideVariablesWorkData", NewName="/Script/UAF.AnimNextOverrideVariablesWorkData")
+StructRedirects=(OldName="/Script/AnimNext.RigUnit_OverrideVariables", NewName="/Script/UAF.RigUnit_OverrideVariables")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextVariableOverrideInfo", NewName="/Script/UAF.AnimNextVariableOverrideInfo")
+StructRedirects=(OldName="/Script/AnimNext.RigVMTrait_OverrideVariables", NewName="/Script/UAF.RigVMTrait_OverrideVariables")
+StructRedirects=(OldName="/Script/AnimNext.RigVMTrait_OverrideVariablesForAsset", NewName="/Script/UAF.RigVMTrait_OverrideVariablesForAsset")
+StructRedirects=(OldName="/Script/AnimNext.RigVMTrait_OverrideVariablesForStruct", NewName="/Script/UAF.RigVMTrait_OverrideVariablesForStruct")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextGraphState", NewName="/Script/UAF.AnimNextGraphState")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextLODPose", NewName="/Script/UAF.AnimNextLODPose")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextModuleHandle", NewName="/Script/UAF.AnimNextModuleHandle")
+StructRedirects=(OldName="/Script/AnimNext.RigVMTrait_ModuleEventDependency", NewName="/Script/UAF.RigVMTrait_ModuleEventDependency")
+StructRedirects=(OldName="/Script/AnimNext.UAFModuleInstanceComponent", NewName="/Script/UAF.UAFModuleInstanceComponent")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextTag", NewName="/Script/UAF.AnimNextTag")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextScope", NewName="/Script/UAF.AnimNextScope")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextEntryPoint", NewName="/Script/UAF.AnimNextEntryPoint")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextParamType", NewName="/Script/UAF.AnimNextParamType")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextReferencePose", NewName="/Script/UAF.AnimNextReferencePose")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextTraitEvent", NewName="/Script/UAF.AnimNextTraitEvent")
+StructRedirects=(OldName="/Script/AnimNext.UAFAssetInstance", NewName="/Script/UAF.UAFAssetInstance")
+StructRedirects=(OldName="/Script/AnimNext.UAFAssetInstanceComponent", NewName="/Script/UAF.UAFAssetInstanceComponent")
+StructRedirects=(OldName="/Script/AnimNext.UAFRigVMComponent", NewName="/Script/UAF.UAFRigVMComponent")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextSoftVariableReference", NewName="/Script/UAF.AnimNextSoftVariableReference")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextVariableReference", NewName="/Script/UAF.AnimNextVariableReference")
+StructRedirects=(OldName="/Script/AnimNext.UAFInstanceVariableData", NewName="/Script/UAF.UAFInstanceVariableData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextRigVMFunctionData", NewName="/Script/UAF.AnimNextRigVMFunctionData")
+StructRedirects=(OldName="/Script/AnimNext.AnimNextRigVMFunctionData", NewName="/Script/UAF.AnimNextRigVMFunctionData")

+ClassRedirects=(OldName="/Script/AnimNext.AnimNextConfig", NewName="/Script/UAF.AnimNextConfig")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextWorldSubsystem", NewName="/Script/UAF.AnimNextWorldSubsystem")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextModule", NewName="/Script/UAF.UAFSystem")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextComponentWorldSubsystem", NewName="/Script/UAF.AnimNextComponentWorldSubsystem")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextAddDependencyMenuContext", NewName="/Script/UAF.AnimNextAddDependencyMenuContext")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextWorldLibrary", NewName="/Script/UAF.AnimNextWorldLibrary")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextOverrideVariablesMenuContext", NewName="/Script/UAF.AnimNextOverrideVariablesMenuContext")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextRigVMAsset", NewName="/Script/UAF.AnimNextRigVMAsset")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextComponent", NewName="/Script/UAF.UAFComponent")
+ClassRedirects=(OldName="/Script/AnimNext.AnimNextSharedVariables", NewName="/Script/UAF.AnimNextSharedVariables")

+EnumRedirects=(OldName="EAnimNextModuleInitMethod", NewName="/Script/UAF.EAnimNextModuleInitMethod")
+EnumRedirects=(OldName="EAnimNextModuleEventDependencyOrdering", NewName="/Script/UAF.EAnimNextModuleEventDependencyOrdering")

+StructRedirects=(OldName="/Script/AnimNextEditor.AnimNextSchemaAction", NewName="/Script/UAFEditor.AnimNextSchemaAction")
+StructRedirects=(OldName="/Script/AnimNextEditor.AnimNextSchemaAction_RigUnit", NewName="/Script/UAFEditor.AnimNextSchemaAction_RigUnit")
+StructRedirects=(OldName="/Script/AnimNextEditor.AnimNextSchemaAction_DispatchFactory", NewName="/Script/UAFEditor.AnimNextSchemaAction_DispatchFactory")
+StructRedirects=(OldName="/Script/AnimNextEditor.AnimNextSchemaAction_Variable", NewName="/Script/UAFEditor.AnimNextSchemaAction_Variable")
+StructRedirects=(OldName="/Script/AnimNextEditor.AnimNextSchemaAction_AddComment", NewName="/Script/UAFEditor.AnimNextSchemaAction_AddComment")
+StructRedirects=(OldName="/Script/AnimNextEditor.AnimNextSchemaAction_Function", NewName="/Script/UAFEditor.AnimNextSchemaAction_Function")
+StructRedirects=(OldName="/Script/AnimNextEditor.AnimNextSchemaAction_PromoteToVariable", NewName="/Script/UAFEditor.AnimNextSchemaAction_PromoteToVariable")
+StructRedirects=(OldName="/Script/AnimNextEditor.AnimNextWorkspaceState", NewName="/Script/UAFEditor.AnimNextWorkspaceState")

+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextModuleFactory", NewName="/Script/UAFEditor.AnimNextModuleFactory")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextSharedVariablesFactory", NewName="/Script/UAFEditor.UAFSharedVariablesFactory")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextWorkspaceFactory", NewName="/Script/UAFEditor.AnimNextWorkspaceFactory")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextEditorContext", NewName="/Script/UAFEditor.AnimNextEditorContext")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextAssetFindReplaceVariables", NewName="/Script/UAFEditor.AnimNextAssetFindReplaceVariables")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AssetDefinition_AnimNextVariableEntry", NewName="/Script/UAFEditor.AssetDefinition_AnimNextVariableEntry")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AssetDefinition_AnimNextEventGraphEntry", NewName="/Script/UAFEditor.AssetDefinition_AnimNextEventGraphEntry")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AssetDefinition_AnimNextSharedVariablesEntry", NewName="/Script/UAFEditor.AssetDefinition_AnimNextSharedVariablesEntry")
+ClassRedirects=(OldName="/Script/AnimNextEditor.RigVMAssetViewMenuContext", NewName="/Script/UAFEditor.RigVMAssetViewMenuContext")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AssetDefinition_AnimNextModule", NewName="/Script/UAFEditor.AssetDefinition_AnimNextModule")
+ClassRedirects=(OldName="/Script/AnimNextEditor.PropertyBagDetailsObject", NewName="/Script/UAFEditor.PropertyBagDetailsObject")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AssetDefinition_AnimNextSharedVariables", NewName="/Script/UAFEditor.AssetDefinition_AnimNextSharedVariables")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextVariableItemMenuContext", NewName="/Script/UAFEditor.AnimNextVariableItemMenuContext")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextVariableSettings", NewName="/Script/UAFEditor.AnimNextVariableSettings")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextWorkspaceEditorMode", NewName="/Script/UAFEditor.AnimNextWorkspaceEditorMode")
+ClassRedirects=(OldName="/Script/AnimNextEditor.AnimNextWorkspaceSchema", NewName="/Script/UAFEditor.AnimNextWorkspaceSchema")

+EnumRedirects=(OldName="ESearchScope", NewName="/Script/UAFEditor.ESearchScope")

+StructRedirects=(OldName="/Script/AnimNextTestSuite.AnimNextParamTypeTestStruct", NewName="/Script/UAFTestSuite.AnimNextParamTypeTestStruct")
+StructRedirects=(OldName="/Script/AnimNextTestSuite.AnimNextTests_TestOperation", NewName="/Script/UAFTestSuite.AnimNextTests_TestOperation")
+StructRedirects=(OldName="/Script/AnimNextTestSuite.AnimNextTests_PrintResult", NewName="/Script/UAFTestSuite.AnimNextTests_PrintResult")
+StructRedirects=(OldName="/Script/AnimNextTestSuite.AnimNextTestData", NewName="/Script/UAFTestSuite.AnimNextTestData")

+ClassRedirects=(OldName="/Script/AnimNextTestSuite.AnimNextTestFuncLib", NewName="/Script/UAFTestSuite.AnimNextTestFuncLib")

+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextRigVMAssetOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextRigVMAssetOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextModuleOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextModuleOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextSharedVariablesOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextSharedVariablesOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextAssetEntryOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextAssetEntryOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextVariableOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextVariableOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextCollapseGraphsOutlinerDataBase", NewName="/Script/UAFUncookedOnly.AnimNextCollapseGraphsOutlinerDataBase")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextCollapseGraphOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextCollapseGraphOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraphFunctionOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextGraphFunctionOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextGraphOutlinerData", NewName="/Script/UAFUncookedOnly.AnimNextGraphOutlinerData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextUniversalObjectLocatorBindingData", NewName="/Script/UAFUncookedOnly.AnimNextUniversalObjectLocatorBindingData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextExportData", NewName="/Script/UAFUncookedOnly.AnimNextExportData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextExport", NewName="/Script/UAFUncookedOnly.AnimNextExport")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextVariableReferenceData", NewName="/Script/UAFUncookedOnly.AnimNextVariableReferenceData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextVariableDeclarationData", NewName="/Script/UAFUncookedOnly.AnimNextVariableDeclarationData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextManifestNodeData", NewName="/Script/UAFUncookedOnly.AnimNextManifestNodeData")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextAssetRegistryExports", NewName="/Script/UAFUncookedOnly.AnimNextAssetRegistryExports")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextVariableBinding", NewName="/Script/UAFUncookedOnly.AnimNextVariableBinding")
+StructRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextVariableBindingData", NewName="/Script/UAFUncookedOnly.AnimNextVariableBindingData")

+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextAssetWorkspaceAssetUserData", NewName="/Script/UAFUncookedOnly.AnimNextAssetWorkspaceAssetUserData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextControllerBase", NewName="/Script/UAFUncookedOnly.AnimNextControllerBase")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextEdGraph", NewName="/Script/UAFUncookedOnly.AnimNextEdGraph")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextEdGraphNode", NewName="/Script/UAFUncookedOnly.AnimNextEdGraphNode")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextEdGraphSchema", NewName="/Script/UAFUncookedOnly.AnimNextEdGraphSchema")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextRigVMAssetLibrary", NewName="/Script/UAFUncookedOnly.UAFRigVMAssetLibrary")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextRigVMAssetEditorData", NewName="/Script/UAFUncookedOnly.AnimNextRigVMAssetEditorData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextRigVMAssetSchema", NewName="/Script/UAFUncookedOnly.UAFRigVMAssetSchema")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextSharedVariableNode", NewName="/Script/UAFUncookedOnly.UAFSharedVariableNode")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextUnitNode", NewName="/Script/UAFUncookedOnly.AnimNextUnitNode")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextCategoryEntry", NewName="/Script/UAFUncookedOnly.AnimNextCategoryEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextEventGraphEntry", NewName="/Script/UAFUncookedOnly.AnimNextEventGraphEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextRigVMAssetEntry", NewName="/Script/UAFUncookedOnly.AnimNextRigVMAssetEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextSharedVariablesEntry", NewName="/Script/UAFUncookedOnly.UAFSharedVariablesEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextVariableEntry", NewName="/Script/UAFUncookedOnly.AnimNextVariableEntry")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextEventGraphSchema", NewName="/Script/UAFUncookedOnly.AnimNextEventGraphSchema")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextModule_EditorData", NewName="/Script/UAFUncookedOnly.UAFSystem_EditorData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.AnimNextSharedVariables_EditorData", NewName="/Script/UAFUncookedOnly.UAFSharedVariables_EditorData")
+ClassRedirects=(OldName="/Script/AnimNextUncookedOnly.K2Node_AnimNextComponentSetVariable", NewName="/Script/UAFUncookedOnly.K2Node_AnimNextComponentSetVariable")

+EnumRedirects=(OldName="EAnimNextSharedVariablesType", NewName="/Script/UAFUncookedOnly.EAnimNextSharedVariablesType")
+EnumRedirects=(OldName="EAnimNextExportAccessSpecifier", NewName="/Script/UAFUncookedOnly.EAnimNextExportAccessSpecifier")
+EnumRedirects=(OldName="FAnimNextUniversalObjectLocatorBindingType", NewName="/Script/UAFUncookedOnly.FAnimNextUniversalObjectLocatorBindingType")
+EnumRedirects=(OldName="EAnimNextExportedVariableFlags", NewName="/Script/UAFUncookedOnly.EAnimNextExportedVariableFlags")

; SkeletonTemplateFramework plugin -> UAF plugin
+ClassRedirects=(OldName="/Script/SkeletonTemplateFrameworkRuntime.AbstractSkeletonLabelBinding", NewName="/Script/UAF.AbstractSkeletonLabelBinding")
+ClassRedirects=(OldName="/Script/SkeletonTemplateFrameworkRuntime.AbstractSkeletonLabelCollection", NewName="/Script/UAF.AbstractSkeletonLabelCollection")
+ClassRedirects=(OldName="/Script/SkeletonTemplateFrameworkRuntime.AbstractSkeletonSetBinding", NewName="/Script/UAF.AbstractSkeletonSetBinding")
+ClassRedirects=(OldName="/Script/SkeletonTemplateFrameworkRuntime.AbstractSkeletonSetCollection", NewName="/Script/UAF.AbstractSkeletonSetCollection")

+StructRedirects=(OldName="/Script/SkeletonTemplateFrameworkRuntime.AbstractSkeleton_LabelBinding", NewName="/Script/UAF.AbstractSkeleton_LabelBinding")
+StructRedirects=(OldName="/Script/SkeletonTemplateFrameworkRuntime.AbstractSkeleton_BoneBinding", NewName="/Script/UAF.AbstractSkeleton_BoneBinding")
+StructRedirects=(OldName="/Script/SkeletonTemplateFrameworkRuntime.AbstractSkeleton_AttributeBinding", NewName="/Script/UAF.AbstractSkeleton_AttributeBinding")
+StructRedirects=(OldName="/Script/SkeletonTemplateFrameworkRuntime.AbstractSkeletonSet", NewName="/Script/UAF.AbstractSkeletonSet")

; System I/O
+StructRedirects=(OldName="/Script/UAF.AnimNextModuleHandle", NewName="/Script/UAF.UAFWeakSystemReference")
+FunctionRedirects=(OldName="AnimNextComponent.SetEnabled",NewName="ActorComponent.SetActive") 
+FunctionRedirects=(OldName="AnimNextComponent.IsEnabled",NewName="ActorComponent.IsActive")
+FunctionRedirects=(OldName="AnimNextComponent.BlueprintGetModuleHandle",NewName="UAFComponent.GetSystemReference")

; AnimNext -> UAF
+ClassRedirects=(OldName="/Script/UAF.AnimNextModule",NewName="/Script/UAF.UAFSystem")
+ClassRedirects=(OldName="/Script/UAFUncookedOnly.AnimNextModule_EditorData",NewName="/Script/UAFUncookedOnly.UAFSystem_EditorData")
+ClassRedirects=(OldName="/Script/UAFUncookedOnly.AnimNextModuleFactory",NewName="/Script/UAFUncookedOnly.UAFSystemFactory")
+ClassRedirects=(OldName="/Script/UAF.AnimNextComponent", NewName="/Script/UAF.UAFComponent")
+ClassRedirects=(OldName="/Script/UAF.AnimNextRigVMAsset",NewName="/Script/UAF.UAFRigVMAsset")
+ClassRedirects=(OldName="/Script/UAFUncookedOnly.AnimNextRigVMAssetEditorData",NewName="/Script/UAFUncookedOnly.UAFRigVMAssetEditorData")
+ClassRedirects=(OldName="/Script/UAFUncookedOnly.AnimNextRigVMAssetEntry",NewName="/Script/UAFUncookedOnly.UAFRigVMAssetEntry")
+ClassRedirects=(OldName="/Script/UAFUncookedOnly.AnimNextRigVMAssetLibrary",NewName="/Script/UAFUncookedOnly.UAFRigVMAssetLibrary")
+ClassRedirects=(OldName="/Script/UAFUncookedOnly.AnimNextRigVMAssetSchema",NewName="/Script/UAFUncookedOnly.UAFRigVMAssetSchema")
+ClassRedirects=(OldName="/Script/UAFUncookedOnly.AnimNextSharedVariables_EditorData",NewName="/Script/UAFUncookedOnly.UAFSharedVariables_EditorData")
+ClassRedirects=(OldName="/Script/UAF.AnimNextSharedVariables",NewName="/Script/UAF.UAFSharedVariables")
+ClassRedirects=(OldName="/Script/UAFUncookedOnly.AnimNextSharedVariablesEntry",NewName="/Script/UAFUncookedOnly.UAFSharedVariablesEntry")
+ClassRedirects=(OldName="/Script/UAFUncookedOnly.AnimNextSharedVariableNode",NewName="/Script/UAFUncookedOnly.UAFSharedVariableNode")
+ClassRedirects=(OldName="/Script/UAFEditor.AnimNextSharedVariablesFactory",NewName="/Script/UAFEditor.UAFSharedVariablesFactory")

; Move MaxGraphUpdateLoopCount from UAF to UAFAnimGraph
+PropertyRedirects=(OldName="/Script/UAFEditor.UAFEditorProjectSettings.MaxGraphUpdateLoopCount",NewName="/Script/UAFAnimGraphEditor.UAFAnimGraphEditorProjectSettings.MaxGraphUpdateLoopCount")