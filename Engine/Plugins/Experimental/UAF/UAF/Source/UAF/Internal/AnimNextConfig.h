// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextConfig.generated.h"

#define UE_API UAF_API

UCLASS(MinimalAPI, Config=AnimNext)
class UAnimNextConfig : public UObject
{
	GENERATED_BODY()

private:
	// UObject interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:
	// Gets all allowed asset classes that users can reference that map via GetGraphFromObject
	UFUNCTION()
	static UE_API TArray<UClass*> GetAllowedAssetClasses();
};

#undef UE_API
