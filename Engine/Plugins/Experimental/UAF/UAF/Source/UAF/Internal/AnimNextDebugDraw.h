// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PrimitiveSceneDesc.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveSceneProxyDesc.h"
#include "RigVMCore/RigVMDrawInterface.h"
#include "Misc/TransactionallySafeRWLock.h"

class UObject;
struct FRigVMDrawInterface;
struct FAnimNextModuleInstance;

namespace UE::UAF
{
	struct FModuleEventTickFunction;
	struct FSystemReference;
}

namespace UE::UAF::Debug
{

#if UE_ENABLE_DEBUG_DRAWING

// Simple scene proxy to perform debug drawing with
class FAnimNextDebugSceneProxy : public FPrimitiveSceneProxy
{
	explicit FAnimNextDebugSceneProxy(const FPrimitiveSceneProxyDesc& InProxyDesc);

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual uint32 GetMemoryFootprint() const override;
	virtual SIZE_T GetTypeHash() const override;

	FRigVMDrawInterface DrawInterface;
	bool bIsEnabled = false;

	friend struct FAnimNextModuleInstance;
	friend struct FDebugDraw;
};

// Render data for debug drawing
struct FDebugDraw
{
public:
	explicit FDebugDraw(UObject* InOwner);

	// Draw any debug items in DrawInterface
	void Draw();

	// Enable/disable debug drawing
	// Enqueues a render command to update the enabled state
	void SetEnabled(bool bInIsEnabled);

private:
	struct FCustomSceneProxyDesc : FPrimitiveSceneProxyDesc
	{
		explicit FCustomSceneProxyDesc(UObject* InOwner);

		// FPrimitiveSceneProxyDesc interface
		virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override {}

		FCustomPrimitiveData DummyCustomPrimitiveData;
	};

	void CalcBounds();

	// Unregister the primitive from the RT
	void RemovePrimitive();

	FCustomSceneProxyDesc SceneProxyDesc;
	FPrimitiveSceneDesc SceneDesc;
	FPrimitiveSceneInfoData* SceneInfoData = nullptr;
	FAnimNextDebugSceneProxy* SceneProxy = nullptr;

	FSceneInterface* Scene = nullptr;

	// Anim thread accessible draw interface
	FRigVMDrawInterface DrawInterface;

	// Anim thread accessible enabled flag
	bool bIsEnabled = false;

	// Flag indicating whether we are registered with the RT
	bool bIsRegistered = false;

	// Lock to prevent unregistration during WT drawing
	FTransactionallySafeRWLock Lock;

	friend struct ::FAnimNextModuleInstance;
	friend struct UE::UAF::FModuleEventTickFunction;
	friend UE::UAF::FSystemReference;
};

#endif

}