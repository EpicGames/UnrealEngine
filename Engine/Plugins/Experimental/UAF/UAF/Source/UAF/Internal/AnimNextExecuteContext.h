// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "Module/AnimNextModuleContextData.h"
#include "Logging/StructuredLog.h"

#include "AnimNextExecuteContext.generated.h"

struct FUAFAssetInstance;

namespace UE::UAF
{
	struct FLatentPropertyHandle;
	struct FModuleEventTickFunction;
	struct FScopedExecuteContextData;
}

USTRUCT(BlueprintType)
struct FAnimNextExecuteContext : public FRigVMExecuteContext
{
	GENERATED_BODY()

	FAnimNextExecuteContext() = default;

	virtual void Copy(const FRigVMExecuteContext* InOtherContext) override
	{
		Super::Copy(InOtherContext);

		const FAnimNextExecuteContext* OtherContext = static_cast<const FAnimNextExecuteContext*>(InOtherContext);
		ContextData = OtherContext->ContextData;
	}

	virtual void Log(const FRigVMLogSettings& InLogSettings, const FString& InMessage) const override
	{
		if (RuntimeSettings.LogFunction.IsValid())
		{
			(*RuntimeSettings.LogFunction)(InLogSettings, this, InMessage);
		}
		else
		{
			if (InLogSettings.Severity == EMessageSeverity::Error)
			{
				UE_LOGFMT(LogAnimation, Error, "{FunctionName}: {Message} [Actor: {ActorPath}]", GetFunctionName(), InMessage, FSoftObjectPath(GetOwningActor()));
			}
			else if (InLogSettings.Severity == EMessageSeverity::Warning)
			{
				UE_LOGFMT(LogAnimation, Warning, "{FunctionName}: {Message} [Actor: {ActorPath}]", GetFunctionName(), InMessage, FSoftObjectPath(GetOwningActor()));
			}
			else
			{
				UE_LOGFMT(LogAnimation, Display, "{FunctionName}: {Message} [Actor: {ActorPath}]", GetFunctionName(), InMessage, FSoftObjectPath(GetOwningActor()));
			}
		}
	}

	// Get the context data as the specified type. This will assert if the type differs from the last call to SetContextData.
	template<typename ContextType>
	const ContextType& GetContextData() const
	{
		return ContextData.Get<ContextType>();
	}

protected:
	// Setup the context data to the specified type
	void SetContextData(TConstStructView<FUAFScriptContextData> InContextData)
	{
		ContextData = InContextData;
	}

	// Context data for this execution
	TConstStructView<FUAFScriptContextData> ContextData;

	friend struct UE::UAF::FScopedExecuteContextData;
};

namespace UE::UAF
{

// Helper for applying context data prior to RigVM execution
struct FScopedExecuteContextData
{
	FScopedExecuteContextData() = delete;

	explicit FScopedExecuteContextData(FAnimNextExecuteContext& InContext, TConstStructView<FUAFScriptContextData> InContextData)
		: Context(InContext)
	{
		Context.SetContextData(InContextData);
	}

	~FScopedExecuteContextData()
	{
		Context.SetContextData(TStructView<FUAFScriptContextData>());
	}

	FAnimNextExecuteContext& Context;
};

}