// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "UObject/TopLevelAssetPath.h"

namespace UE::UAF
{

class FAnimNextModuleImpl : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void RegisterAssets();
	void UnregisterAssets();
	
	FTopLevelAssetPath UAFSystemClassPath;
};

}
