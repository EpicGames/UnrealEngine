// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/PagedArray.h"
#include "Misc/TVariant.h"
#include "AnimNextPoolHandle.h"

namespace UE::UAF
{

template<typename ElementType>
class TPool
{
private:
	struct FEntry
	{
		TVariant<ElementType, uint32> ValueOrNextFreeIndex;
		uint32 SerialNumber = 0;
	};

	TPagedArray<FEntry> Entries;

	// The number of active entries allocasted from the pool
	uint32 NumAllocatedEntries = 0;

	uint32 HeadFreeIndex = MAX_uint32;

	// Serial number is global to reduce collisions with multiple pools
	static inline uint32 SerialNumber = 0;

public:
	template<typename... ArgsType>
	TPoolHandle<ElementType> Emplace(ArgsType&&... InArgs)
	{
		SerialNumber++;
		if(SerialNumber == 0)
		{
			// Serial number wrapped, we cant have any allocated entries or we could have duplicate handles
			check(Entries.Num() == 0);
			// Skip zero serial number as this is taken to mean 'invalid'
			SerialNumber++;
		}

		TPoolHandle<ElementType> Handle;
		Handle.SerialNumber = SerialNumber;
		if(HeadFreeIndex != MAX_uint32)
		{
			// Use the next free index
			Handle.Index = HeadFreeIndex;
			HeadFreeIndex = Entries[Handle.Index].ValueOrNextFreeIndex.template Get<uint32>();
			FEntry& RecycledEntry = Entries[Handle.Index];
			RecycledEntry.SerialNumber = Handle.SerialNumber;
			RecycledEntry.ValueOrNextFreeIndex.template Emplace<ElementType>(Forward<ArgsType>(InArgs)...);
		}
		else
		{
			// Add a new element
			Handle.Index = Entries.Num();
			FEntry& NewEntry = Entries.Emplace_GetRef();
			NewEntry.SerialNumber = Handle.SerialNumber;
			NewEntry.ValueOrNextFreeIndex.template Emplace<ElementType>(Forward<ArgsType>(InArgs)...);
		}

		NumAllocatedEntries++;

		return Handle;
	}

	void Release(TPoolHandle<ElementType> InHandle, EAllowShrinking InAllowShrinking = EAllowShrinking::Default)
	{
		if(IsValidHandle(InHandle))
		{
			if(InHandle.Index == Entries.Num() - 1)
			{
				// Last entry, shrink array
				Entries.Pop(InAllowShrinking);
			}
			else
			{
				// Not last entry, call destructor & add to free list
				FEntry& Entry = Entries[InHandle.Index];
				Entry.ValueOrNextFreeIndex.template Set<uint32>(HeadFreeIndex);
				Entry.SerialNumber = 0;
				HeadFreeIndex = InHandle.Index;
			}

			check(NumAllocatedEntries != 0);
			NumAllocatedEntries--;
		}
	}

	ElementType& Get(TPoolHandle<ElementType> InHandle)
	{
		check(IsValidHandle(InHandle));
		return Entries[InHandle.Index].ValueOrNextFreeIndex.template Get<ElementType>();
	}

	const ElementType& Get(TPoolHandle<ElementType> InHandle) const
	{
		check(IsValidHandle(InHandle));
		return Entries[InHandle.Index].ValueOrNextFreeIndex.template Get<ElementType>();
	}

	ElementType* TryGet(TPoolHandle<ElementType> InHandle)
	{
		if(IsValidHandle(InHandle))
		{
			return &Entries[InHandle.Index].ValueOrNextFreeIndex.template Get<ElementType>();
		}
		return nullptr;
	}

	const ElementType* TryGet(TPoolHandle<ElementType> InHandle) const
	{
		if(IsValidHandle(InHandle))
		{
			return &Entries[InHandle.Index].ValueOrNextFreeIndex.template Get<ElementType>();
		}
		return nullptr;
	}

	uint32 Num() const
	{
		return NumAllocatedEntries;
	}

	bool IsValidHandle(TPoolHandle<ElementType> InHandle) const
	{
		return (InHandle.SerialNumber != 0 && Entries.IsValidIndex(InHandle.Index) && InHandle.SerialNumber == Entries[InHandle.Index].SerialNumber);
	}

	template<typename IteratorElementType, bool bReverse = false>
	class TIterator
	{
	public:
		TIterator(TPool<IteratorElementType>& InPool, uint32 StartIndex = 0)
			: Pool(InPool)
			, Index(StartIndex)
		{
			if(Pool.Entries.IsValidIndex(StartIndex) && Pool.Entries[StartIndex].SerialNumber == 0)
			{
				Increment();
			}
		}

		/** Advances iterator to the next valid element in the pool. */
		TIterator& operator++()
		{
			Increment();
			return *this;
		}
		TIterator operator++(int)
		{
			TIterator Temp(*this);
			Increment();
			return Temp;
		}

		FORCEINLINE IteratorElementType& operator* () const
		{
			return Pool.Entries[Index].ValueOrNextFreeIndex.template Get<ElementType>();
		}

		FORCEINLINE IteratorElementType* operator->() const
		{
			return &Pool.Entries[Index].ValueOrNextFreeIndex.template Get<ElementType>();
		}

		/** conversion to "bool" returning true if the iterator has not reached the last element. */
		FORCEINLINE explicit operator bool() const
		{
			return Pool.Entries.IsValidIndex(Index);
		}
		
		FORCEINLINE bool operator==(const TIterator& Rhs) const
		{
			return &Pool == &Rhs.Pool && Index == Rhs.Index;
		}
		
		FORCEINLINE bool operator!=(const TIterator& Rhs) const
		{
			return &Pool != &Rhs.Pool || Index != Rhs.Index;
		}

	private:
		void Increment()
		{
			while(1)
			{
				if constexpr (bReverse)
				{
					Index = Index == 0 ? Pool.Entries.Num() : Index - 1;
				}
				else
				{
					++Index;
				}
					
				if(!Pool.Entries.IsValidIndex(Index))
				{
					break;
				}
				if(Pool.Entries[Index].SerialNumber != 0)
				{
					break;
				}
			}
		}
		
		TPool<IteratorElementType>& Pool;
		uint32 Index;
	};
	
	typedef TIterator<      ElementType, false> RangedForIteratorType;
	typedef TIterator<const ElementType, false> RangedForConstIteratorType;
	typedef TIterator<      ElementType, true > RangedForReverseIteratorType;
	typedef TIterator<const ElementType, true > RangedForConstReverseIteratorType;

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */

	FORCEINLINE RangedForIteratorType             begin ()       { return RangedForIteratorType            (*this, 0); }
	FORCEINLINE RangedForConstIteratorType        begin () const { return RangedForConstIteratorType       (*this, 0); }
	FORCEINLINE RangedForIteratorType             end   ()       { return RangedForIteratorType            (*this, Entries.Num()); }
	FORCEINLINE RangedForConstIteratorType        end   () const { return RangedForConstIteratorType       (*this, Entries.Num()); }
	FORCEINLINE RangedForReverseIteratorType      rbegin()       { return RangedForReverseIteratorType     (*this, Entries.Num()); }
	FORCEINLINE RangedForConstReverseIteratorType rbegin() const { return RangedForConstReverseIteratorType(*this, Entries.Num()); }
	FORCEINLINE RangedForReverseIteratorType      rend  ()       { return RangedForReverseIteratorType     (*this, 0); }
	FORCEINLINE RangedForConstReverseIteratorType rend  () const { return RangedForConstReverseIteratorType(*this, 0); }
};

}
