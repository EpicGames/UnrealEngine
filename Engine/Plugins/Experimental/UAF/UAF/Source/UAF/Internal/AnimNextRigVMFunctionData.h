// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextRigVMFunctionData.generated.h"

USTRUCT()
struct FAnimNextRigVMFunctionData
{
	GENERATED_BODY()

	// Stable GUID identifying the function (survives renames). This corresponds to the RigVM function's GUID
	UPROPERTY()
	FGuid FunctionGuid;

	// The name of the function
	UPROPERTY()
	FName Name;

	// The mangled event name of the function
	UPROPERTY()
	FName EventName;

	// Indices into RigVM external variables for each arg
	UPROPERTY()
	TArray<int32> ArgIndices;
};