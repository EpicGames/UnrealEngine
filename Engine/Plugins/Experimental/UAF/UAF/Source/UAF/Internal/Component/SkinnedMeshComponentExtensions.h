// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Math/MathFwd.h"
#include "LODPose.h"

class USkinnedMeshComponent;

namespace UE::Anim
{

// Extensions to skinned meshes that are not public API yet
struct FSkinnedMeshComponentExtensions
{
	// Converts local to component space, flips buffers, updates bounds and dispatches to renderer
	static UAF_API void CompleteAndDispatch(
		USkinnedMeshComponent* InComponent,
		const UE::UAF::FLODPoseHeap& InLODPose,
		TUniqueFunction<void(USkinnedMeshComponent*)>&& InGameThreadCallback);
};

}
