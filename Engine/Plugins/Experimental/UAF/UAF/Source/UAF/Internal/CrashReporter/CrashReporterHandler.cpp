// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReporterHandler.h"
#include "UObject/Object.h"
#include "HAL/IConsoleManager.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#if WITH_ADDITIONAL_CRASH_CONTEXTS

namespace UE::UAF
{
static bool bCrashHandlerEnabled = false;
static FAutoConsoleVariableRef CVarCrashHandlerEnabled(
	TEXT("UAF.CrashHandlerEnabled"),
	bCrashHandlerEnabled,
	TEXT("UAF will add context into the crash reporter."),
	ECVF_Default);

namespace CrashReporter
{
	struct FCrashReporterHandlerImpl
	{
	public:
		struct FContext
		{
			explicit FContext(TNotNull<const UObject*> InOwner, TNotNull<const UObject*> InObjectOfInterest, FName InContext, uint32 InThreadID)
				: Owner(InOwner) 
				, ObjectOfInterest(InObjectOfInterest)
				, Context(InContext)
				, ThreadID(InThreadID)
			{
				ID = GenerateID();
			}

			UE_AUTORTFM_ALWAYS_OPEN
			static uint32 GenerateID()
			{
				// This ID only needs to be unique; it doesn't need to be rolled back if an AutoRTFM transaction fails.
				static std::atomic<uint32> IDGenerator = 0;
				return ++IDGenerator;
			}

			TWeakObjectPtr<const UObject> Owner;
			TWeakObjectPtr<const UObject> ObjectOfInterest;
			FName Context;
			uint32 ThreadID;
			uint32 ID;
		};

	public:
		uint32 PushInfo(TNotNull<const UObject*> InOwner, TNotNull<const UObject*> InObjectOfInterest, FName InContext)
		{
			UE::TScopeLock LockGuard(CriticalSection);
			return Contexts.Emplace_GetRef(InOwner, InObjectOfInterest, InContext, FPlatformTLS::GetCurrentThreadId()).ID;
		}

		void PopInfo(uint32 ID)
		{
			UE::TScopeLock LockGuard(CriticalSection);
			const int32 FoundIndex = Contexts.IndexOfByPredicate([ID](const FContext& Other)
				{
					return Other.ID == ID;
				});
			if (FoundIndex != INDEX_NONE)
			{
				Contexts.RemoveAtSwap(FoundIndex);
			}
		}

		void DumpInfo(FCrashContextExtendedWriter& Writer)
		{
			// Callback from the crash reporter.Avoids memory allocation in case the crash is an OOM.
			UE::TScopeLock LockGuard(CriticalSection);

			for (int32 Index = 0; Index < Contexts.Num(); ++Index)
			{
				const FContext& Context = Contexts[Index];
				Builder.Reset();
			
				if (const UObject* Owner = Context.Owner.Get())
				{
					Owner->GetFullName(Builder);
				}
				else
				{
					Builder.Append(TEXT("<Invalid>"));
				}

				if (const UObject* ObjectOfInterest = Context.ObjectOfInterest.Get())
				{
					ObjectOfInterest->GetFullName(Builder);
				}
				else
				{
					Builder.Append(TEXT("<Invalid>"));
				}

				Builder << TEXT('\n');
				Builder << Context.Context;

				Builder << TEXT('\n');
				Builder << Context.ThreadID;

				// Avoid allocating a new name for each entry.
				constexpr int32 BufferSize = 16;
				TCHAR Identifier[BufferSize];
				FCString::Snprintf(Identifier, UE_ARRAY_COUNT(Identifier), TEXT("UAF%d"), Index);
				Writer.AddString(Identifier, Builder.ToString());
			}
		}

	private:
		TArray<FContext, TInlineAllocator<32>> Contexts;
		FTransactionallySafeCriticalSection CriticalSection;
		TStringBuilder<1024> Builder;
	};


	//~ Use a shared ptr instead of manually deleting it in case we crash while unregistering the UAF module.
	static TSharedPtr<FCrashReporterHandlerImpl> Instance;
	static FDelegateHandle AdditionalCrashContextDelegateHandle;
}

void FCrashReporterHandler::Register()
{
	CrashReporter::Instance = MakeShared<CrashReporter::FCrashReporterHandlerImpl>();
	CrashReporter::AdditionalCrashContextDelegateHandle = FGenericCrashContext::OnAdditionalCrashContextDelegate().AddSP(CrashReporter::Instance.ToSharedRef(), &CrashReporter::FCrashReporterHandlerImpl::DumpInfo);
}

void FCrashReporterHandler::Unregister()
{
	FGenericCrashContext::OnAdditionalCrashContextDelegate().Remove(CrashReporter::AdditionalCrashContextDelegateHandle);
	// Keep the AdditionalCrashContextDelegateHandle valid in case it currently crashing. The pointer will be automatically removed (shared ptr) when the module unloads.
}

FCrashReporterScope::FCrashReporterScope(TNotNull<const UObject*> InOwner, TNotNull<const UObject*> ObjectOfInterest, FName Context)
{
	bWasEnabled = bCrashHandlerEnabled;
	if (bWasEnabled)
	{
		check(CrashReporter::Instance);
		ID = CrashReporter::Instance->PushInfo(InOwner, ObjectOfInterest, Context);
	}
}

FCrashReporterScope::~FCrashReporterScope()
{
	if (bWasEnabled)
	{
		CrashReporter::Instance->PopInfo(ID);
	}
}
}




#endif // WITH_ADDITIONAL_CRASH_CONTEXTS
