// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "Misc/NotNull.h"

#if WITH_ADDITIONAL_CRASH_CONTEXTS

namespace UE::UAF
{
struct FCrashReporterHandler
{
	static void Register();
	static void Unregister();
};

class FCrashReporterScope
{
public:
	explicit FCrashReporterScope(TNotNull<const UObject*> InOwner, TNotNull<const UObject*> ObjectOfInterest, FName Context);
	~FCrashReporterScope();

	FCrashReporterScope() = delete;
	FCrashReporterScope(const FCrashReporterScope&) = delete;
	FCrashReporterScope& operator=(const FCrashReporterScope&) = delete;

private:
	bool bWasEnabled = false;
	uint32 ID = 0;
};

}

#define UAF_CRASH_REPORTER_SCOPE(InOwner, InObjectOfInterest, InContextName) ::UE::UAF::FCrashReporterScope ANONYMOUS_VARIABLE(AddCrashContext) {(InOwner), (InObjectOfInterest), (#InContextName)}
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS 

#define UE_WORKSPACE_EDITOR_CRASH_REPORTER_SCOPE()
