// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextNativeDataInterface.generated.h"

class UUAFSharedVariables;

// DEPRECATED - no longer in use
USTRUCT()
struct FAnimNextNativeDataInterface
{
	GENERATED_BODY()

	FAnimNextNativeDataInterface() = default;
	virtual ~FAnimNextNativeDataInterface() = default;

	// Context passed to Initialize()
	struct FBindToFactoryObjectContext
	{
		// If this native interface was created as a result of a factory, then this is the object that the factor used as a source. Can be nullptr.
		const UObject* FactoryObject = nullptr;

		// The data interface that this native interface was constructed for
		const UUAFSharedVariables* DataInterface = nullptr;
	};

	// Initialize this native interface during factory object creation.
	// Called once on worker threads to set up this native interface's properties after it has been default-constructed.
	// The data interface instance that it will manage has not yet been constructed at this point.
	virtual void BindToFactoryObject(const FBindToFactoryObjectContext& InContext) {}
#if WITH_EDITORONLY_DATA
	// Use to upgrade older content to the trait-based approach
	virtual const UScriptStruct* GetUpgradeTraitStruct() const { return nullptr; }
#endif
};