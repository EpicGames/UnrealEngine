// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Set.h"
#include "UObject/ScriptInterface.h"

#define UE_API UAF_API

class UUAFRigVMAsset;
class IRigVMRuntimeAssetInterface;
class UUAFSystem;

namespace UE::UAF
{
	struct FSystemFactory;
}

namespace UE::UAF
{

struct FSystemBuilderContext
{
private:
	friend FSystemFactory;

	// Builds a system using the populated ComponentStructs & VariableStructs
	UE_API const UUAFSystem* Build();

public:
	// Component structs used to run the system
	TSet<const UScriptStruct*> ComponentStructs;

	// Variable structs used to communicate with the system
	TSet<const UScriptStruct*> VariableStructs;

	// All the IRigVMRuntimeAssetInterface whose variables we will reference/instance at runtime
	TSet<TScriptInterface<const IRigVMRuntimeAssetInterface>> ReferencedVariableRigVMAssets;

	// All the assets whose variables we will reference/instance at runtime
	TSet<TObjectPtr<const UUAFRigVMAsset>> ReferencedVariableAssets;
};

}

#undef UE_API