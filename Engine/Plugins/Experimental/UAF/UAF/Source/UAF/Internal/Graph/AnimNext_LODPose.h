// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ReferencePose.h"
#include "LODPose.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AttributesRuntime.h"

#include "AnimNext_LODPose.generated.h"

#define UE_API UAF_API

USTRUCT(BlueprintType, meta = (DisplayName = "ReferencePose"))
struct FAnimNextGraphReferencePose
{
	GENERATED_BODY()

	FAnimNextGraphReferencePose() = default;

	explicit FAnimNextGraphReferencePose(UE::UAF::FDataHandle& InReferencePose)
		: ReferencePose(InReferencePose)
	{
	}

	UE::UAF::FDataHandle ReferencePose;
};

USTRUCT(BlueprintType, meta = (DisplayName = "LODPose"))
struct FAnimNextGraphLODPose
{
	GENERATED_BODY()

	// Joint transforms
	UE::UAF::FLODPoseHeap LODPose;

	// Float curves
	FBlendedHeapCurve Curves;

	// Attributes
	// Note that attribute bone indices are LOD bone indices matching the LOD pose
	UE::Anim::FHeapAttributeContainer Attributes;

	/**
	 * Copy source pose into target pose, automatically remapping bones and attributes if skeletons differ.
	 * Curves are always copied directly (no skeleton dependency).
	 */
	static UE_API void CopyPose(
		const UE::UAF::FLODPoseHeap& InSourcePose, const FBlendedHeapCurve& InSourceCurves, const UE::Anim::FHeapAttributeContainer& InSourceAttributes,
		UE::UAF::FLODPoseHeap& OutTargetPose, FBlendedHeapCurve& OutTargetCurves, UE::Anim::FHeapAttributeContainer& OutTargetAttributes);

	/** Copy from individual source components, automatically remapping bones and attributes via FRemapPoseDataPool if skeletons differ. */
	UE_API void CopyFrom(const UE::UAF::FLODPoseHeap& InPose, const FBlendedHeapCurve& InCurves, const UE::Anim::FHeapAttributeContainer& InAttributes);

	/** Copy from source, automatically remapping bones and attributes via FRemapPoseDataPool if skeletons differ. */
	UE_API void CopyFrom(const FAnimNextGraphLODPose& Source);
};

#undef UE_API
