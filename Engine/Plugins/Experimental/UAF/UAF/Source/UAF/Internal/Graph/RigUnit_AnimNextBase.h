// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMStruct.h"
#include "AnimNextExecuteContext.h"
#include "RigUnit_AnimNextBase.generated.h"

/*
 * base class for all UAF rigvm nodes
 */
USTRUCT(meta=(ExecuteContext="FAnimNextExecuteContext", DocumentationPolicy = "Strict"))
struct FRigUnit_AnimNextBase : public FRigVMStruct
{
	GENERATED_BODY()
};
