// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigUnit_AnimNextBase.h"
#include "AnimNextExecuteContext.h"
#include "RigUnit_AnimNextBeginExecution.generated.h"

#define UE_API UAF_API

/**
 * Event for driving the skeleton hierarchy with variables and rig elements
 */
USTRUCT(meta=(DisplayName="Execute", Category="Events", NodeColor="1, 0, 0", Keywords="Begin,Update,Tick,Forward,Event"))
struct FRigUnit_AnimNextBeginExecution : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	virtual FName GetEventName() const override { return EventName; }
	virtual bool CanOnlyExistOnce() const override { return true; }

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Output))
	FAnimNextExecuteContext ExecuteContext;

	static UE_API FName EventName;
};

#undef UE_API
