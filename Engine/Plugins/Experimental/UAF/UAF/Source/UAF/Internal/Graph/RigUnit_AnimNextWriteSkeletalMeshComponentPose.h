// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"

#include "RigUnit_AnimNextWriteSkeletalMeshComponentPose.generated.h"

#define UE_API UAF_API

/** Writes a pose to a skeletal mesh component */
USTRUCT(meta=(DisplayName="Write Pose", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="Output,Pose,Port", RequiredComponents="AnimNextSkeletalMeshComponentReferenceComponent"))
struct FRigUnit_AnimNextWriteSkeletalMeshComponentPose : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	static void WritePose(USkeletalMeshComponent* InOutputComponent, const FAnimNextGraphLODPose& InPose);

	virtual FString GetUnitSubTitle() const { return TEXT("Skeletal Mesh Component"); };

	// Pose to write
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FAnimNextGraphLODPose Pose;

	// Mesh to write to. If this is not supplied, then the first skeletal mesh component of the current actor will be used
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

#undef UE_API