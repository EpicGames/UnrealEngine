// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"
class UInstancedSkinnedMeshComponent;

#include "RigUnit_UAFWriteInstancedSkinnedMeshPose.generated.h"

#define UE_API UAF_API

/** Writes a pose to an instanced skinned mesh component */
USTRUCT(meta=(DisplayName="Write Pose Instanced", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="Output,Pose,Port"))
struct FRigUnit_UAFWriteInstancedSkinnedMeshPose : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	virtual FString GetUnitSubTitle() const { return TEXT("Skeletal Mesh Component"); };

	// Pose to write
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FAnimNextGraphLODPose Pose;

	// Mesh to write to. 
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	TObjectPtr<class UInstancedSkinnedMeshComponent> SkinnedMeshComponent;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

#undef UE_API