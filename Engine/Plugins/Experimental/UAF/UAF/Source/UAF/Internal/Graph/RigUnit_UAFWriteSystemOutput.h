// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "UAF/ValueRuntime/ValueBundle.h"

#include "RigUnit_UAFWriteSystemOutput.generated.h"

#define UE_API UAF_API

/** Writes the output for a system */
USTRUCT(meta=(DisplayName="Write System Output", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="Output,Pose,Port", RequiredComponents="UAFSystemOutputComponent"))
struct FRigUnit_UAFWriteSystemOutput : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	// Value to write to output
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FUAFValueBundle Value;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

#undef UE_API