// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFAssetInstance.h"
#include "RigVMCore/RigVMTrait.h"
#include "RigVMTrait_AnimNextPublicVariables.generated.h"

#define UE_API UAF_API

class UUAFRigVMAsset;
class FRigVMTraitScope;
class UUAFSharedVariables;

// Represents public variables of an asset via a trait 
USTRUCT(BlueprintType)
struct FRigVMTrait_AnimNextPublicVariables : public FRigVMTrait
{
	GENERATED_BODY()

	// The asset that any programmatic pins will be derived from, if any
	UPROPERTY(meta = (Hidden))
	TObjectPtr<UUAFSharedVariables> InternalAsset = nullptr;

	// Variable names that are exposed
	UPROPERTY(meta = (Hidden))
	TArray<FName> InternalVariableNames;

	// FRigVMTrait interface
#if WITH_EDITOR
	UE_API virtual FString GetDisplayName() const override;
	UE_API virtual void GetProgrammaticPins(URigVMController* InController, int32 InParentPinIndex, const URigVMPin* InTraitPin, const FString& InDefaultValue, FRigVMPinInfoArray& OutPinArray) const override;
	UE_API virtual bool ShouldCreatePinForProperty(const FProperty* InProperty) const override;

	// Editor-implemented function ptrs
	static UE_API FString (*GetDisplayNameFunc)(const FRigVMTrait_AnimNextPublicVariables& InTrait);
	static UE_API void (*GetProgrammaticPinsFunc)(const FRigVMTrait_AnimNextPublicVariables& InTrait, URigVMController* InController, int32 InParentPinIndex, const FString& InDefaultValue, FRigVMPinInfoArray& OutPinArray);
	static UE_API bool (*ShouldCreatePinForPropertyFunc)(const FRigVMTrait_AnimNextPublicVariables& InTrait, const FProperty* InProperty);
#endif
};

#undef UE_API