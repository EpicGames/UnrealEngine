// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNext_LODPose.h"
#include "Module/UAFModuleInstanceComponent.h"

#include "UAFSystemOutputComponent.generated.h"

#define UE_API UAF_API

struct FRigUnit_UAFWriteSystemOutput;
class UMeshEntityComponentImpl_Skinned;
class FEntitySkeletalAnimationModule;
struct FUAFSystemOutputComponent;
class UUAFComponent;
struct FUAFValueBundle;

namespace UE::UAF
{
struct FVirtualValueBundle_SystemOutput;
class ISystemOutputAdapter;
}

namespace UE::UAF::Tests
{
struct FUAFSystemTests;
}

// System component used to identify output values
USTRUCT()
struct FUAFSystemOutputComponent : public FUAFModuleInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	// Get the refpose for the mesh we are bound to
	const FAnimNextGraphReferencePose& GetRefPose() const
	{
		return RefPose;
	}

	// Get the LOD we are currently outputting as
	int32 GetLOD() const
	{
		return LOD;
	}

	// Get the mesh we are currently outputting to
	USkeletalMesh* GetSkeletalMesh() const
	{
		return Mesh;
	}

	// Get the legacy mesh component we are currently outputting to
	USkeletalMeshComponent* GetSkeletalMeshComponent() const
	{
		return MeshComponent;
	}

	// Function used to callback into
	using FReadOutputFunc = TFunctionRef<void(uint32 InSerialNumber, const FAnimNextGraphLODPose& InPose, const FAnimNextGraphReferencePose& InRefPose)>;

	// Helper function used to remap a LOD pose to the output pose's refpose
	// Temporary until the value runtime is ready
	UE_API void RemapLODPoseToOutput(const FAnimNextGraphLODPose& InInputPose, FAnimNextGraphLODPose& InOutputPose);

private:
	// FUAFModuleInstanceComponent interface
	UE_API virtual void OnBeginExecution(float InDeltaTime) override;

	// Write the output
	// @param    InValue    The value to write
	void WriteOutput(const FUAFValueBundle& InValue);

	// Generate the pose etc. for rendering by a mesh.
	// If the serial numbers differ, a callback to the InFunction occurs, otherwise the call is skipped.
	// If the supplied serial number matches the one held in this component, then the pose is assumed to have not been written, so no work needs to occur
	// If the serial number supplied is zero, then the GenerateRenderData call will always be made
	// InFunction can also be skipped if other invariants are not satisfied (e.g. null or mismatched mesh)
	// @param    InSerialNumber    The current serial number that the reader holds
	// @param    InFunction        Function that will be called if conditions are met
	// @return true if the read was successful (i.e. the serial numbers were mismatched)
	UE_API bool GenerateRenderData(uint32 InSerialNumber, FReadOutputFunc InFunction) const;

	// Read the output value etc. for internal UAF use
	// If the serial numbers differ, a callback to the InFunction occurs, otherwise the call is skipped.
	// If the supplied serial number matches the one held in this component, then the pose is assumed to have not been written, so no work needs to occur
	// If the serial number supplied is zero, then the GenerateRenderData call will always be made
	// @param    InSerialNumber    The current serial number that the reader holds
	// @param    InFunction        Function that will be called if conditions are met
	// @return true if the read was successful (i.e. the serial numbers were mismatched)
	UE_API bool ReadOutput(uint32 InSerialNumber, FReadOutputFunc InFunction) const;

	// Called to bind this component to a specific system output adapter (e.g. mesh component)
	UE_API void BindSystemOutputAdapter(UE::UAF::ISystemOutputAdapter& InSystemOutputAdapter);

	// Unbinds from any system output adapter we are currently bound to
	UE_API void UnbindSystemOutputAdapter();

private:
	friend FRigUnit_UAFWriteSystemOutput;
	friend UMeshEntityComponentImpl_Skinned;
	friend FEntitySkeletalAnimationModule;
	friend UE::UAF::FVirtualValueBundle_SystemOutput;
	friend UUAFComponent;
	friend UE::UAF::Tests::FUAFSystemTests;

	// The output pose that was written this time around
	FAnimNextGraphLODPose Pose;

	// The mesh we use to determine the final pose format
	UPROPERTY(Transient)
	TObjectPtr<USkeletalMesh> Mesh;

	// Legacy skeletal mesh component, if any
	UPROPERTY(Transient)
	TObjectPtr<USkeletalMeshComponent> MeshComponent;

	// Refpose generated from the mesh
	UPROPERTY(Transient)
	FAnimNextGraphReferencePose RefPose;

	// Adapter to use for output
	UE::UAF::ISystemOutputAdapter* SystemOutputAdapter = nullptr;

	// The LOD that our system is running at this tick
	int32 LOD = 0;

	// Serial number updated each time we write the output
	uint32 SerialNumber = 0;

	// Serial number for required bones
	uint32 CurrentRequiredBonesSerial = 0;
};

#undef UE_API