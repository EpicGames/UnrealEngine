// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Module/UAFModuleInstanceComponent.h"
#include "Templates/SubclassOf.h"

#include "AnimNextActorComponentReferenceComponent.generated.h"

class UActorComponent;

// Module instance component used to provide a reference to an actor component
// Note that this does not support dynamic component updates (e.g. if we have multiple, or changing components). It grabs a ptr to the component on
// initialization and does not update it.
USTRUCT()
struct FAnimNextActorComponentReferenceComponent : public FUAFModuleInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

protected:
	// Helper function used by derived classes to initialize the component
	void OnInitializeHelper(UScriptStruct* InScriptStruct);

	// The component we reference
	UPROPERTY()
	TObjectPtr<UActorComponent> Component;

	// The type of the component to find
	UPROPERTY()
	TSubclassOf<UActorComponent> ComponentType;
};