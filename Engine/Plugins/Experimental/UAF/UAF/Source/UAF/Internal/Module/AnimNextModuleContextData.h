// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFAssetInstance.h"
#include "Script/UAFAssetContextData.h"
#include "AnimNextModuleContextData.generated.h"

struct FAnimNextModuleInstance;

// Context structure passed to script calls
USTRUCT()
struct FAnimNextModuleContextData : public FUAFAssetContextData
{
	GENERATED_BODY()

	FAnimNextModuleContextData() = default;

	UAF_API FAnimNextModuleContextData(FAnimNextModuleInstance& InInstance, FName InEventName, float InDeltaTime);

	// Get the object that the module instance is bound to, if any
	UAF_API UObject* GetObject() const;

	// Get the currently executing module.
	UAF_API FAnimNextModuleInstance& GetModuleInstance() const;
};
