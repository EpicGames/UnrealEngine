// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Module/AnimNextModule.h"
#include "AnimNextPoolHandle.h"
#include "Variables/UAFInstanceVariableDataProxy.h"
#include "UAFAssetInstance.h"
#include "Module/ModuleTickFunction.h"
#include "TraitCore/TraitEvent.h"
#include "TraitCore/TraitEventList.h"
#include "Module/UAFModuleInstanceComponent.h"
#include "StructUtils/InstancedStruct.h"
#include "Module/AnimNextModuleInitMethod.h"
#include "Module/TaskRunLocation.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Module/ModuleTaskContext.h"
#include "AnimNextModuleInstance.generated.h"

#define UE_API UAF_API

struct FAnimNextModuleInstance;
struct FRigUnit_AnimNextRunAnimationGraph_v1;
struct FRigUnit_AnimNextRunAnimationGraph_v2;
struct FRigUnit_UAFRunAsset;
class UUAFEngineSubsystem;
struct FRigUnit_AnimNextInitializeEvent;
struct FRigUnit_CopyModuleProxyVariables;
struct FUAFNotifyDispatcherComponent;
struct FAnimNextGraphInstance;
struct FUAFSystemFactoryParams;
struct FUAFWeakSystemReference;

namespace UE::UAF
{
	struct FModuleEventTickFunction;
	struct FModuleEndTickFunction;
	struct FSystemReference;
	struct FCustomSystemDeleter;
}

namespace UE::UAF::Debug
{
	struct FDebugDraw;
}

namespace UE::UAF
{
	using ModuleInstanceComponentMapType = TMap<FName, TInstancedStruct<FUAFModuleInstanceComponent>>;
}

namespace UE::UAF::Tests
{
	class FVariables_Shared;
	class FVariables_UOLBindings;
}

// Represents an instance of a UAF system.
USTRUCT()
struct FAnimNextModuleInstance : public FUAFAssetInstance
#if CPP
	, TSharedFromThis<FAnimNextModuleInstance>
#endif
{
	GENERATED_BODY()

	FAnimNextModuleInstance();

	// Standalone constructor
	UE_API FAnimNextModuleInstance(
		const UUAFSystem* InModule,
		EAnimNextModuleInitMethod InInitMethod);

	// Standalone constructor, UObject host
	UE_API FAnimNextModuleInstance(
		const UUAFSystem* InModule,
		UObject* InObject,
		EAnimNextModuleInitMethod InInitMethod);

	UE_API ~FAnimNextModuleInstance();

	// Checks to see if this entry is ticking
	bool IsEnabled() const;

	// Enables/disables the ticking of this entry
	void SetEnabled(bool bInEnabled);

	// Queues an input trait event
	// Input events will be processed in the next graph update after they are queued
	UE_API void QueueInputTraitEvent(FAnimNextTraitEventPtr Event);

	// Queues an output trait event
	// Output events will be processed at the end of the schedule tick
	UE_API void QueueOutputTraitEvent(FAnimNextTraitEventPtr Event);

	// Get the object that this module is bound to
	UObject* GetObject() const { return Object; }

	// Get the system that this instance represents
	UAF_API const UUAFSystem* GetSystem() const;

#if UE_ENABLE_DEBUG_DRAWING
	// Get the debug draw interface
	UE_API FRigVMDrawInterface* GetDebugDrawInterface();

	// Set whether to show the module's debug drawing instructions in the viewport
	UE_API void ShowDebugDrawing(bool bInShowDebugDrawing);
#endif

	// Run a simple task on the GT via FFunctionGraphTask::CreateAndDispatchWhenReady
	UE_API static void RunTaskOnGameThread(TUniqueFunction<void(void)>&& InFunction);

	// Find the tick function for the specified event
	UE::UAF::FModuleEventTickFunction* FindTickFunctionByName(FName InEventName);
	const UE::UAF::FModuleEventTickFunction* FindTickFunctionByName(FName InEventName) const;

	// Find the first 'user' tick function
	UE_API UE::UAF::FModuleEventTickFunction* FindFirstUserTickFunction();

	// Find the last 'user' tick function
	UE_API UE::UAF::FModuleEventTickFunction* FindLastUserTickFunction();

	// Run the specified script event using a module context
	UE_API void RunScriptEvent(const UE::UAF::FScriptEvent& InEvent, float InDeltaTime);

	// Run the specified script event using a module context, using name lookup alone (slower)
	UE_API void RunScriptEventByName(FName InEventName, float InDeltaTime);

	// Get the world type that this module was instantiated within
	EWorldType::Type GetWorldType() const { return WorldType; }

	// Get tick functions for this module instance
	UE_API TArrayView<UE::UAF::FModuleEventTickFunction> GetTickFunctions();

	// Get a weak, non-owning reference to this instance
	UE_API FUAFWeakSystemReference GetReference() const;

	// Queue a task to run before the first user event
	// @param	InTaskFunction		The function to run
	void QueueTask(UE::UAF::FUniqueSystemTask&& InTaskFunction)
	{
		QueueTask(FName(), MoveTemp(InTaskFunction), UE::UAF::ETaskRunLocation::Before);
	}

	// Queue a task to run at a particular system event
	// @param	InEventName			The event in the system to run the supplied task relative to. If this is NAME_None or not found then the first 'user' event will be used.
	// @param	InTaskFunction		The function to run
	// @param	InLocation			Where to run the task, before or after
	UE_API void QueueTask(FName InEventName, UE::UAF::FUniqueSystemTask&& InTaskFunction, UE::UAF::ETaskRunLocation InLocation = UE::UAF::ETaskRunLocation::Before);

	#if UAF_TRACE_ENABLED
    	// trace this module if it hasn't already been traced this frame
    	UE_API void Trace();
    #endif

private:
	// Setup the entry
	// Note that we perform initialization in two phases because we may need to access weak references to 'this' during initialization
	UE_API void Initialize(const FUAFSystemFactoryParams& InFactoryParams);

	// Clear data that binds the schedule to a runtime (e.g. tick functions) and any instance data
	void ResetBindingsAndInstanceData();

	// Remove all of our tick function's dependencies
	void RemoveAllTickDependencies();

	// Flip proxy variable buffers then copy any dirty values
	void CopyProxyVariables();

#if WITH_EDITOR
	// Resets internal state if the module we are bound to is recompiled in editor
	void OnCompileJobFinished();
#endif

	// Call module components' OnBeginExecution extension point
	void BeginExecution(float InDeltaTime);

	// Give the module components a chance to handle events and call their OnEndExecution extension point
	void EndExecution(float InDeltaTime);

	// Handles trait events at the module level.
	// Called in the module end tick function before any events are dispatched.
	void RaiseTraitEvents(const UE::UAF::FTraitEventList& EventList);

	// Set the value of the specified variable via the proxy double-buffer. Called from any thread.
	EPropertyBagResult SetProxyVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InData);

	// Accesses value of the specified variable via the proxy double-buffer. Called from any thread.
	EPropertyBagResult WriteProxyVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>)> InFunction);

	// Access a variable's value given its reference via a raw memory view.
	// This performs non-recursive locks on the system instance to ensure that concurrent access is respected, so care must be taken calling this from
	// animation worker tasks as deadlocks can occur.
	// Type must match strictly, no conversions are performed.
	// @param	InVariable			The variable to get the value of
	// @param	InType				The type of the variable
	// @param	InFunction			Function that will be called if no errors occur
	// @return see EPropertyBagResult
	EPropertyBagResult ReadVariableFromExternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TConstArrayView<uint8>)> InFunction);

	// Access the supplied system component for reading. Component will not be lazily created.
	// @param	InComponentType		The type of the component we want ot access
	// @param	InFunction			Function that will be called to access the struct
	// @return true if the struct was available, otherwise false
	bool ReadComponentFromExternal(UScriptStruct* InComponentType, TFunctionRef<void(FConstStructView InComponentStruct)> InFunction);

private:
	friend UE::UAF::FSystemReference;
	friend FUAFWeakSystemReference;
	friend UE::UAF::FModuleEventTickFunction;
	friend UE::UAF::FModuleEndTickFunction;
	friend FRigUnit_AnimNextRunAnimationGraph_v1;
	friend FRigUnit_AnimNextRunAnimationGraph_v2;
	friend FRigUnit_UAFRunAsset;
	friend UUAFEngineSubsystem;
	friend FRigUnit_AnimNextInitializeEvent;
	friend FRigUnit_CopyModuleProxyVariables;
	friend FUAFNotifyDispatcherComponent;
	friend FAnimNextGraphInstance;
	friend UE::UAF::Tests::FVariables_Shared;
	friend UE::UAF::Tests::FVariables_UOLBindings;
	friend UE::UAF::FCustomSystemDeleter;

	// Object this entry is bound to
	UPROPERTY(Transient)
	TObjectPtr<UObject> Object = nullptr;

	// Pre-allocated graph of tick functions
	TArray<UE::UAF::FModuleEventTickFunction> TickFunctions;

	// Input event list to be processed on the next update
	UE::UAF::FTraitEventList InputEventList;

	// Output event list to be processed at the end of the schedule tick
	UE::UAF::FTraitEventList OutputEventList;

	// Lock to ensure event list actions are thread safe
	FTransactionallySafeRWLock EventListLock;

	// Lock to ensure proxy access is thread safe
	FTransactionallySafeRWLock ProxyLock;

	// Lock guarding against gross concurrent access via system references (safety only: should rarely contend if dependencies are set correctly)
	FTransactionallySafeRWLock SystemReferenceLock;

	// Proxy public variables, double-buffered
	UPROPERTY(Transient)
	FUAFInstanceVariableDataProxy ProxyVariables[2];

	// Index into current buffer to write the proxy variables
	int32 ProxyWriteIndex = 0;

#if UE_ENABLE_DEBUG_DRAWING
	// Debug draw support
	// TODO: Move this to a 'module component' once that code is checked in
	TUniquePtr<UE::UAF::Debug::FDebugDraw> DebugDraw;
#endif

	enum class EInitState : uint8
	{
		NotInitialized,

		CreatingTasks,

		BindingTasks,

		SetupVariables,

		PendingInitializeEvent,

		FirstUpdate,

		Initialized
	};

	// Current initialization state
	EInitState InitState = EInitState::NotInitialized;

	enum class ERunState : uint8
	{
		NotInitialized,

		Running,

		Paused,
	};

	// Current running state
	ERunState RunState = ERunState::NotInitialized;

	// Transition to the specified init state, verifying that the current state is valid
	void TransitionToInitState(EInitState InNewState);

	// Transition to the specified run state, verifying that the current state is valid
	void TransitionToRunState(ERunState InNewState);

	// How this entry initializes
	EAnimNextModuleInitMethod InitMethod = EAnimNextModuleInitMethod::InitializeAndPauseInEditor;

	// Whether this represents an editor object 
	EWorldType::Type WorldType = EWorldType::None;

#if WITH_EDITOR
	// Whether we are currently recreating this instance because of compilation/reinstancing
	bool bIsRecreatingOnCompile : 1 = false;
	
	// Whether the module instance has had its custom destructor called (queueing it up for release at the end of the frame)
	bool bHasBeenQueuedForRelease : 1 = false;
#endif

private:
#if UAF_TRACE_ENABLED
	bool bTracedThisFrame = false;
#endif
};

template<>
struct TStructOpsTypeTraits<FAnimNextModuleInstance> : public TStructOpsTypeTraitsBase2<FAnimNextModuleInstance>
{
	enum
	{
		WithCopy = false
	};
};


#if UAF_TRACE_ENABLED
#define TRACE_UAF_MODULE(ModuleInstance) ModuleInstance.Trace();
#else
#define TRACE_UAF_MODULE(ModuleInstance)
#endif

#undef UE_API
