// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Module/AnimNextActorComponentReferenceComponent.h"

#include "AnimNextSkeletalMeshComponentReferenceComponent.generated.h"

#define UE_API UAF_API

// Module instance component used to fetch data about the current actor's skeletal mesh component
USTRUCT()
struct FAnimNextSkeletalMeshComponentReferenceComponent : public FAnimNextActorComponentReferenceComponent
{
	GENERATED_BODY()

	UE_API FAnimNextSkeletalMeshComponentReferenceComponent();
	
	UE_API USkeletalMeshComponent* GetComponent() const;

private:
	// FUAFAssetInstanceComponent interface
	virtual void OnBindToInstance() override;
};

#undef UE_API
