// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Containers/SpscQueue.h"
#include "TaskSyncManager.h"
#include "Module/ModuleTaskContext.h"
#include "Script/UAFScriptComponent.h"
#if UE_AUTORTFM
#include "Containers/TransactionallySafeSpscQueue.h"
#endif

struct FAnimNextModuleInstance;

namespace UE::UAF
{
	struct FScheduleContext;
	struct FModuleTaskContext;
}

namespace UE::UAF
{

// Multicast delegate called before module events
using FOnPreModuleEvent = TTSMulticastDelegate<void(const FModuleTaskContext& InContext)>;

struct FModuleEventTickFunction : public FTickFunction
{
	FModuleEventTickFunction()
	{
		ModuleInstance = nullptr;
		bCanEverTick = true;
		bStartWithTickEnabled = true;
		bRunOnAnyThread = true;
	}

	// FTickFunction interface
	virtual void ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) override;
	virtual FString DiagnosticMessage() override;

	void Initialize(float DeltaTime);
	void ExecuteBindings_WT(float DeltaTime);
	void Run(float DeltaTime);
	void EndTick(float DeltaTime);

#if WITH_EDITOR
	// Called standalone to run a whole module's init & work in one call. Expensive and editor only.
	// Intended to be used outside of a ticking context, such as a non-ticking world or forced initialization.
	static void InitializeAndRunModule(FAnimNextModuleInstance& InModuleInstance);
#endif

	void AddSubsequent(UObject* InObject, FTickFunction& InTickFunction);
	void RemoveSubsequent(UObject* InObject, FTickFunction& InTickFunction);
	void RemoveAllExternalSubsequents();
	void InitializeBatchedWork(class UWorld* World);

	static void RunTaskOnGameThread(TUniqueFunction<void(void)>&& InFunction);

	FAnimNextModuleInstance* ModuleInstance = nullptr;

	// Tasks queued to execute once per event
#if UE_AUTORTFM
	using FTaskQueueType = TTransactionallySafeSpscQueue<UE::UAF::FUniqueSystemTask>;
#else
	using FTaskQueueType = TMpscQueue<UE::UAF::FUniqueSystemTask>;
#endif

	FTaskQueueType PreExecuteTasks;
	FTaskQueueType PostExecuteTasks;

	// Tasks queued to execute on the game thread after the current tick completes
	TArray<TUniqueFunction<void()>> PostExecuteGameThreadTaskList;

	// Handle to use for submitting batched work to the task sync manager
	UE::Tick::FActiveSyncWorkHandle BatchedWorkHandle;

	// Multicast delegate called before this event
	FOnPreModuleEvent OnPreModuleEvent;

	// The event itself
	FScriptEvent Event;

	// Initialize event - will be called if valid
	FScriptEvent InitializeEvent;

	// WT bindings event - will be called if valid
	FScriptEvent ExecuteBindings_WTEvent;

	uint8 bLastUserEvent : 1 = false;
	uint8 bFirstUserEvent : 1 = false;
	uint8 bUserEvent : 1 = false;

	// External dependencies that this tick function needs to unregister when it is destroyed
	TArray<FTickPrerequisite> ExternalSubsequents;
};

}