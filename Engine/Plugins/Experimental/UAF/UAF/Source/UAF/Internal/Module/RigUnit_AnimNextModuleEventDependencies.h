// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigUnit_AnimNextModuleEvents.h"
#include "RigUnit_AnimNextModuleEventDependencies.generated.h"

#define UE_API UAF_API

class URigVMController;
class URigVMNode;

UCLASS()
class UAnimNextAddDependencyMenuContext : public UObject
{
	GENERATED_BODY()
public:
	TObjectPtr<URigVMController> Controller = nullptr;
	TObjectPtr<URigVMNode> Node = nullptr;
};

/*
 * the base class for all dependencies nodes
 */
USTRUCT(meta=(Hidden, Category="Dependencies", NodeColor="0, 1, 1", Keywords="Prerequisite,Subsequent,TickFunction"))
struct FRigUnit_AnimNextModuleDependenciesBase : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	FRigUnit_AnimNextModuleDependenciesBase() = default;

	// FRigVMStruct interface
	UE_API virtual TArray<FRigVMUserWorkflow> GetSupportedWorkflows(const UObject* InSubject) const override;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "Events", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

/** Adds execution dependencies between module events and other systems */
USTRUCT(meta=(DisplayName="Add Dependencies"))
struct FRigUnit_AnimNextModuleAddDependencies : public FRigUnit_AnimNextModuleDependenciesBase
{
	GENERATED_BODY()

	FRigUnit_AnimNextModuleAddDependencies() = default;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/** Removes execution dependencies between module events and some systems */
USTRUCT(meta=(DisplayName="Remove Dependencies"))
struct FRigUnit_AnimNextModuleRemoveDependencies : public FRigUnit_AnimNextModuleDependenciesBase
{
	GENERATED_BODY()

	FRigUnit_AnimNextModuleRemoveDependencies() = default;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

#undef UE_API
