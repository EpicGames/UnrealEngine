// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "AnimNextExecuteContext.h"
#include "Script/ScriptEventPhase.h"
#include "Script/ScriptEventTickFunctionBindings.h"
#include "RigUnit_AnimNextModuleEvents.generated.h"

#define UE_API UAF_API


/** Base schedule-level event, never instantiated */
USTRUCT(meta=(Abstract, Category="Events", NodeColor="1, 0, 0", Keywords="Begin,Update,Tick,Forward,Event"))
struct FRigUnit_AnimNextModuleEventBase : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	FRigUnit_AnimNextModuleEventBase() = default;

	// FRigUnit interface
	virtual FString GetUnitLabel() const override { return GetEventName().ToString(); };
	virtual bool CanOnlyExistOnce() const override { return true; }

	// Get the general ordering phase of this event, used for linearization
	virtual UE::UAF::EScriptEventPhase GetEventPhase() const { return UE::UAF::EScriptEventPhase::Execute; }

	// Overriden in derived classes to provide binding function
	virtual UE::UAF::FTickFunctionBindingFunction GetBindingFunction() const { return [](const UE::UAF::FTickFunctionBindingContext& InContext, FTickFunction& InTickFunction){}; }

	// Get the tick group for this event
	virtual ETickingGroup GetTickGroup() const { return TG_PrePhysics; }

	// Get the end tick group for this event
	virtual ETickingGroup GetEndTickGroup() const { return TG_PrePhysics; }

	// Get the integer sort key for this event (used to sort events in tick groups - smaller values get sorted earlier)
	virtual int32 GetSortOrder() const { return 0; }

	// Get whether this is a user-generated event (as opposed to a compiler-generated event)
	virtual bool IsUserEvent() const { return true; }

	// Get whether this event corresponds to a separate task (so tick functions will be generated to run it)
	virtual bool IsTask() const { return true; }

	// Get whether this event is limited to the game thread only
	virtual bool IsGameThreadTask() const { return false; }

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "Events", meta = (Output))
	FAnimNextExecuteContext ExecuteContext;
};

/** Synthetic event injected by the compiler to process any variable bindings on the game thread, not user instantiated */
USTRUCT(meta=(Hidden, Category="Internal", NodeColor="1, 0, 0"))
struct FRigUnit_AnimNextExecuteBindings_GT : public FRigUnit_AnimNextModuleEventBase
{
	GENERATED_BODY()

	FRigUnit_AnimNextExecuteBindings_GT() = default;

	static inline const FLazyName EventName = FLazyName("ExecuteBindings_GT");
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
	virtual FName GetEventName() const override final { return EventName; }
	virtual int32 GetSortOrder() const override final { return 0; }	// Sort before WT bindings
	virtual bool IsUserEvent() const override final { return false; }
	virtual bool IsGameThreadTask() const override final { return true; }

	// FRigUnit_AnimNextModuleEventBase interface
	virtual UE::UAF::EScriptEventPhase GetEventPhase() const override final { return UE::UAF::EScriptEventPhase::PreExecute; }
	UE_API virtual UE::UAF::FTickFunctionBindingFunction GetBindingFunction() const override final;
};

/** Synthetic event injected by the compiler to process any variable bindings on a worker thread, not user instantiated */
USTRUCT(meta=(Hidden, Category="Internal", NodeColor="1, 0, 0"))
struct FRigUnit_AnimNextExecuteBindings_WT : public FRigUnit_AnimNextModuleEventBase
{
	GENERATED_BODY()

	FRigUnit_AnimNextExecuteBindings_WT() = default;

	static inline const FLazyName EventName = FLazyName("ExecuteBindings_WT");
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
	virtual FName GetEventName() const override final { return EventName; }
	virtual int32 GetSortOrder() const override final { return 1; }	// Sort after GT bindings
	virtual bool IsUserEvent() const override final { return false; }
	virtual bool IsTask() const override final { return false; }

	// FRigUnit_AnimNextModuleEventBase interface
	virtual UE::UAF::EScriptEventPhase GetEventPhase() const override final { return UE::UAF::EScriptEventPhase::PreExecute; }
	UE_API virtual UE::UAF::FTickFunctionBindingFunction GetBindingFunction() const override final;
};

/** Schedule event called to set up a module */
USTRUCT(meta=(DisplayName="Initialize", Keywords="Setup,Startup,Create"))
struct FRigUnit_AnimNextInitializeEvent : public FRigUnit_AnimNextModuleEventBase
{
	GENERATED_BODY()

	static inline const FLazyName EventName = FLazyName("Initialize");

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
	virtual FName GetEventName() const override final { return EventName; }
	virtual bool IsTask() const override final { return false; }
};

/** Base event for all user-authored events. Can execute in a particular tick group (e.g. TG_PrePhysics) */
USTRUCT(meta=(Abstract))
struct FRigUnit_AnimNextUserEvent : public FRigUnit_AnimNextModuleEventBase
{
	GENERATED_BODY()

	// FRigUnit_AnimNextModuleEventBase interface
	virtual FName GetEventName() const override final { return Name; }
	virtual FString GetUnitLabel() const override { return GetEventName().ToString(); }
	UE_API virtual FString GetUnitSubTitle() const override;
	virtual bool CanOnlyExistOnce() const override { return false; }
	virtual UE::UAF::EScriptEventPhase GetEventPhase() const override final { return UE::UAF::EScriptEventPhase::Execute; }
	UE_API virtual UE::UAF::FTickFunctionBindingFunction GetBindingFunction() const override final;
	virtual ETickingGroup GetTickGroup() const override final { return TickGroup; }
	virtual ETickingGroup GetEndTickGroup() const override final { return EndTickGroup; }
	virtual int32 GetSortOrder() const override final { return SortOrder; }

	// The name of the event
	UPROPERTY(EditAnywhere, Category = "Event", meta = (Input, Constant, DetailsOnly))
	FName Name = NAME_None;

	// Sort index for ordering with other events in this tick group - smaller values get sorted earlier
	UPROPERTY(EditAnywhere, Category = "Event", meta = (Input, Constant, DetailsOnly))
	int32 SortOrder = 0;

	// The tick group the event executes in 
	UPROPERTY()
	TEnumAsByte<ETickingGroup> TickGroup = ETickingGroup::TG_PrePhysics;

	// The tick group the event completes by 
	UPROPERTY(EditAnywhere, Category = "Tick", meta = (Input, Constant, DetailsOnly))
	TEnumAsByte<ETickingGroup> EndTickGroup = ETickingGroup::TG_PrePhysics;
};

/** Schedule event called before world physics is updated */
USTRUCT(meta=(DisplayName="PrePhysics", Keywords="Start,Before"))
struct FRigUnit_AnimNextPrePhysicsEvent : public FRigUnit_AnimNextUserEvent
{
	GENERATED_BODY()

	static inline const FLazyName DefaultEventName = FLazyName("PrePhysics");

	FRigUnit_AnimNextPrePhysicsEvent()
	{
		Name = DefaultEventName;
		TickGroup = ETickingGroup::TG_PrePhysics;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/** Schedule event called after world physics is updated */
USTRUCT(meta=(DisplayName="PostPhysics", Keywords="End,After"))
struct FRigUnit_AnimNextPostPhysicsEvent : public FRigUnit_AnimNextUserEvent
{
	GENERATED_BODY()

	static inline const FLazyName DefaultEventName = FLazyName("PostPhysics");

	FRigUnit_AnimNextPostPhysicsEvent()
	{
		Name = DefaultEventName;
		TickGroup = ETickingGroup::TG_PostPhysics;
	}
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

};

#undef UE_API
