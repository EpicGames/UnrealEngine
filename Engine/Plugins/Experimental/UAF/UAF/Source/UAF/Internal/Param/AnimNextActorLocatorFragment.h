// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"
#include "AnimNextActorLocatorFragment.generated.h"

// Exposes the 'current' actor as a UOL fragment
USTRUCT()
struct FAnimNextActorLocatorFragment
{
	GENERATED_BODY()

	FAnimNextActorLocatorFragment() = default;

	UAF_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextActorLocatorFragment> FragmentType;

	friend uint32 GetTypeHash(const FAnimNextActorLocatorFragment& A)
	{
		return 0;
	}

	friend bool operator==(const FAnimNextActorLocatorFragment& A, const FAnimNextActorLocatorFragment& B)
	{
		return true;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);
	UAF_API void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);
	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FAnimNextActorLocatorFragment> : public TStructOpsTypeTraitsBase2<FAnimNextActorLocatorFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};