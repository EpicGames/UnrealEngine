// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"
#include "AnimNextComponentLocatorFragment.generated.h"

class UActorComponent;

USTRUCT()
struct FAnimNextComponentLocatorFragment
{
	GENERATED_BODY()

	FAnimNextComponentLocatorFragment() = default;

	UAF_API explicit FAnimNextComponentLocatorFragment(TSubclassOf<UActorComponent> InClass);

	// Path to the component class
	UPROPERTY()
	FSoftObjectPath Path;
	
	UAF_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextComponentLocatorFragment> FragmentType;

	friend uint32 GetTypeHash(const FAnimNextComponentLocatorFragment& A)
	{
		return GetTypeHash(A.Path);
	}

	friend bool operator==(const FAnimNextComponentLocatorFragment& A, const FAnimNextComponentLocatorFragment& B)
	{
		return A.Path == B.Path;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);
	UAF_API void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);
	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FAnimNextComponentLocatorFragment> : public TStructOpsTypeTraitsBase2<FAnimNextComponentLocatorFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};