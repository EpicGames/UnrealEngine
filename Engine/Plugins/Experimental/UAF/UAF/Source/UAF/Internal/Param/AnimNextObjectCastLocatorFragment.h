// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"
#include "AnimNextObjectCastLocatorFragment.generated.h"

USTRUCT()
struct FAnimNextObjectCastLocatorFragment
{
	GENERATED_BODY()

	FAnimNextObjectCastLocatorFragment() = default;

	explicit FAnimNextObjectCastLocatorFragment(UClass* InClass);

	// Path to the class to cast to
	UPROPERTY()
	FSoftObjectPath Path;

	UAF_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextObjectCastLocatorFragment> FragmentType;

	friend uint32 GetTypeHash(const FAnimNextObjectCastLocatorFragment& A)
	{
		return GetTypeHash(A.Path);
	}

	friend bool operator==(const FAnimNextObjectCastLocatorFragment& A, const FAnimNextObjectCastLocatorFragment& B)
	{
		return A.Path == B.Path;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);
	UAF_API void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);
	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FAnimNextObjectCastLocatorFragment> : public TStructOpsTypeTraitsBase2<FAnimNextObjectCastLocatorFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};