// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"
#include "AnimNextObjectFunctionLocatorFragment.generated.h"

USTRUCT()
struct FAnimNextObjectFunctionLocatorFragment
{
	GENERATED_BODY()

	FAnimNextObjectFunctionLocatorFragment() = default;
	
	FAnimNextObjectFunctionLocatorFragment(UFunction* InFunction);

	// Path to the function
	UPROPERTY()
	FSoftObjectPath Path;

	UAF_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextObjectFunctionLocatorFragment> FragmentType;

	friend uint32 GetTypeHash(const FAnimNextObjectFunctionLocatorFragment& A)
	{
		return GetTypeHash(A.Path);
	}

	friend bool operator==(const FAnimNextObjectFunctionLocatorFragment& A, const FAnimNextObjectFunctionLocatorFragment& B)
	{
		return A.Path == B.Path;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);
	UAF_API void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);
	static uint32 ComputePriority(const UObject* Object, const UObject* Context);
};

template<>
struct TStructOpsTypeTraits<FAnimNextObjectFunctionLocatorFragment> : public TStructOpsTypeTraitsBase2<FAnimNextObjectFunctionLocatorFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};