// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorFwd.h"
#include "AnimNextObjectPropertyLocatorFragment.generated.h"

USTRUCT()
struct FAnimNextObjectPropertyLocatorFragment
{
	GENERATED_BODY()

	FAnimNextObjectPropertyLocatorFragment() = default;

	UAF_API FAnimNextObjectPropertyLocatorFragment(TArrayView<FProperty*> InPropertyPath);

	// Path to the property, including any nested struct properties
	UPROPERTY()
	TArray<TFieldPath<FProperty>> Path;

	UAF_API static UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextObjectPropertyLocatorFragment> FragmentType;

	friend uint32 GetTypeHash(const FAnimNextObjectPropertyLocatorFragment& A)
	{
		return GetTypeHash(A.Path);
	}

	friend bool operator==(const FAnimNextObjectPropertyLocatorFragment& A, const FAnimNextObjectPropertyLocatorFragment& B)
	{
		return A.Path == B.Path;
	}

	UE::UniversalObjectLocator::FResolveResult Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const;
	UE::UniversalObjectLocator::FInitializeResult Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams);
	UAF_API void ToString(FStringBuilderBase& OutStringBuilder) const;
	UE::UniversalObjectLocator::FParseStringResult TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params);
	static uint32 ComputePriority(const UObject* Object, const UObject* Context);

private:
	FObjectProperty* GetLeafProperty() const;
	UClass* GetRootClass() const;
};

template<>
struct TStructOpsTypeTraits<FAnimNextObjectPropertyLocatorFragment> : public TStructOpsTypeTraitsBase2<FAnimNextObjectPropertyLocatorFragment>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};