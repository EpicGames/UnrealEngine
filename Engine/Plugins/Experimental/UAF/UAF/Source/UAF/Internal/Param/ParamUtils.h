// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FAnimNextParamType;
struct FUniversalObjectLocator;

namespace UE::UAF
{

struct FParamCompatibility;

struct FParamUtils
{
	// Check to see if a parameter handle is compatible with another. The order of the parameters
	// imply directionality, e.g. IsCompatible((int64)InLHS, (int32)InRHS) is allowed as no data loss 
	// occurs, but IsCompatible((int32)InLHS, (int64)InRHS) is not as B could be truncated.
	static UAF_API FParamCompatibility GetCompatibility(const FAnimNextParamType& InLHS, const FAnimNextParamType& InRHS);

	// Check whether the supplied function can be used to access/map parameters. Does not check type for validity.
	// @param	InFunction	The function to check
	// @param	InClass		The expected class to check - used to determine if hoisted functions are valid in this context.
	static UAF_API bool CanUseFunction(const UFunction* InFunction, const UClass* InExpectedClass);

	// Check whether the supplied function can be used to access/map parameters. Checks type and returns it in OutType
	// @param	InFunction		The function to check
	// @param	InClass			The expected class to check - used to determine if hoisted functions are valid in this context.
	// @param	OutType			Type of the function's return value
	static UAF_API bool CanUseFunction(const UFunction* InFunction, const UClass* InExpectedClass, FAnimNextParamType& OutType);
	
	// Check (via flags only) the supplied property can be used to access/map parameters
	static UAF_API bool CanUseProperty(const FProperty* InProperty);

	// Check whether the supplied property can be used to access/map parameters. Checks type and returns it in OutType
	static UAF_API bool CanUseProperty(const FProperty* InProperty, FAnimNextParamType& OutType);

	// Convert a UOL to an FName
	static UAF_API FName LocatorToName(const FUniversalObjectLocator& InLocator);
};

}