// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigUnit_AnimNextParameterBase.generated.h"

/** Deprecated stub - remove once content is updated */
USTRUCT(meta = (ExecuteContext = "FAnimNextExecuteContext"))
struct FRigUnit_AnimNextParameterBase : public FRigVMStruct
{
	GENERATED_BODY()
};
