// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigUnit_AnimNextParameterBase.h"
#include "AnimNextExecuteContext.h"
#include "RigUnit_AnimNextParameterBeginExecution.generated.h"

#define UE_API UAF_API

/** Deprecated stub - remove once content is updated */
USTRUCT(meta=(DisplayName="Execute", Category="Events", NodeColor="1, 0, 0", Keywords="Begin,Update,Tick,Forward,Event"))
struct FRigUnit_AnimNextParameterBeginExecution : public FRigUnit_AnimNextParameterBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	virtual FString GetUnitLabel() const override { return EntryPoint.ToString(); };
	virtual FName GetEventName() const override { return EntryPoint; }
	virtual bool CanOnlyExistOnce() const override { return true; }

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "Entry Point", meta = (Output))
	FAnimNextExecuteContext ExecuteContext;

	// The name of the entry point
	UPROPERTY(VisibleAnywhere, Category = "Entry Point", meta = (Hidden))
	FName EntryPoint = DefaultEntryPoint;

	static UE_API FName DefaultEntryPoint;
};

#undef UE_API
