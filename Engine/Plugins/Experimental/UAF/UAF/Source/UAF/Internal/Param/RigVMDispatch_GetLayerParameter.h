// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "AnimNextExecuteContext.h"
#include "RigVMDispatch_GetLayerParameter.generated.h"

#define UE_API UAF_API

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

/*
 * Gets a parameter's current value from the graph
 */
USTRUCT(meta = (Deprecated, DisplayName = "Get Graph Parameter", Category="Parameters", NodeColor = "0.8, 0, 0.2, 1"))
struct FRigVMDispatch_GetLayerParameter : public FRigVMDispatchFactory
{
	GENERATED_BODY()

	UE_API FRigVMDispatch_GetLayerParameter();

private:
	friend struct UE::UAF::UncookedOnly::FUtils;

	virtual UScriptStruct* GetExecuteContextStruct() const override { return FAnimNextExecuteContext::StaticStruct(); }
	UE_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	UE_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
#endif
	UE_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
	virtual bool IsSingleton() const override { return true; }

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &FRigVMDispatch_GetLayerParameter::Execute;
	}
	static UE_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);

	static UE_API const FName ExecuteContextName;
	static UE_API const FName ParameterName;
	static UE_API const FName ParameterIdName;
	static UE_API const FName TypeHandleName;
	static UE_API const FName ValueName;
};

#undef UE_API
