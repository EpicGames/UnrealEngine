// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "AnimNextExecuteContext.h"
#include "RigVMDispatch_GetScopedParameter.generated.h"

#define UE_API UAF_API

namespace UE::UAF::Editor
{
	class SGraphPinParam;
}

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

/*
 * Gets a parameter's value
 */
USTRUCT(meta = (Deprecated, DisplayName = "Get Parameter", Category="Parameters", NodeColor = "0.8, 0, 0.2, 1"))
struct FRigVMDispatch_GetScopedParameter : public FRigVMDispatchFactory
{
	GENERATED_BODY()

	UE_API FRigVMDispatch_GetScopedParameter();

private:
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend class UE::UAF::Editor::SGraphPinParam;

	virtual UScriptStruct* GetExecuteContextStruct() const { return FAnimNextExecuteContext::StaticStruct(); }
	UE_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	UE_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
#endif
	UE_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
	virtual bool IsSingleton() const override { return true; }

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &FRigVMDispatch_GetScopedParameter::Execute;
	}
	static UE_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);

	static UE_API const FName ParameterName;
	static UE_API const FName ValueName;
	static UE_API const FName ParameterIdName;
	static UE_API const FName TypeHandleName;
};

#undef UE_API
