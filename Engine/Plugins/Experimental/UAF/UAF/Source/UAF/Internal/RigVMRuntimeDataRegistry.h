// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMExecuteContext.h"

#define UE_API UAF_API


namespace UE::UAF
{

using FRigVMRuntimeDataID = TObjectKey<URigVM>;

struct FRigVMRuntimeData
{
	FRigVMExtendedExecuteContext Context;
};


/**
* RigVMRuntimeDataRegistry
* 
* A global registry of all existing VMs that require TLS data instantiation
* 
*/
struct FRigVMRuntimeDataRegistry final
{
	/**
	 * Finds or adds one VM runtime data instance for the passed ID in the TLS.
	 * If it exists and the Context hash is correct, returns the TLS weak pointer.
	 * If not found or the context hash is not correct, it creates a new instance in the Global storage and adds a Weak pointer to the TLS.
	 *
	 * @param RigVMRuntimeDataID The key to find or associate (if created) the instance with.
	 * @param ReferenceContext Reference context to copy data from (if not found or found but the context hash is not correct)
	 * @param bOutHasBeenUpdated Out : True If the ID did not exist or existed but the Context hash was not correct
	 * @return A weak pointer to the data as stored in the TLS map.
	 */
	static UE_API TWeakPtr<FRigVMRuntimeData> FindOrAddLocalRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID, const FRigVMExtendedExecuteContext& ReferenceContext);

	/**
	 * Finds the instance VM runtime data ID in the TLS storage
	 *
	 * @param RigVMRuntimeDataID The key to find the associated data with.
	 * @return A pointer to the data as stored in the TLS map.
	 */
	static UE_API TWeakPtr<FRigVMRuntimeData> FindLocalRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID);

	/**
	 * Adds one VM runtime data instance for the passed ID in the Global storage and adds a Weak pointer to the TLS.
	 * Adding a new instance is only allowed if it does not already exist in the TLS
	 *
	 * @param RigVMRuntimeDataID The key to associate the data with.
	 * @param ReferenceContext Reference context to copy data from
	 * @return A weak pointer to the data as stored in the TLS map.
	 */
	static UE_API TWeakPtr<FRigVMRuntimeData> AddRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID, const FRigVMExtendedExecuteContext& ReferenceContext);

	/**
	 * Destroys all the VM runtime data instances for the passed ID. This should be only called when the VM is destroyed.
	 *
	 * @param RigVMRuntimeDataID The key to find the associated data with.
	 */
	static UE_API void ReleaseAllVMRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID);

private:
	static UE_API TSharedPtr<FRigVMRuntimeData> AddGlobalRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID, const FRigVMExtendedExecuteContext& ReferenceContext);
	static UE_API void ReleaseAllGlobalRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID);

	// Post GC callback to signal a compaction has to be done (and perform it on game thread)
	static UE_API void HandlePostGarbageCollect();

	// Checks if any of the stored VM datas have been deleted and removes deleted elements. Main thread only.
	static UE_API void PerformGlobalStorageCompaction();
	// Checks if any of the stored VM datas have been deleted and removes deleted elements in the TLS storage.
	static UE_API void PerformLocalStorageCompaction();

	// Module lifetime functions
	static UE_API void Init();
	static UE_API void Destroy();

	friend class FAnimNextModuleImpl;
};

} // end namespace UE::UAF

#undef UE_API
