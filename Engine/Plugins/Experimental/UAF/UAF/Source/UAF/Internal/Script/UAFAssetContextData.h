// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFAssetInstance.h"
#include "Script/UAFScriptContextData.h"
#include "UAFAssetContextData.generated.h"

struct FAnimNextModuleInstance;

// Context structure passed to script calls
USTRUCT()
struct FUAFAssetContextData : public FUAFScriptContextData
{
	GENERATED_BODY()

	FUAFAssetContextData() = default;

	FUAFAssetContextData(FUAFAssetInstance& InInstance, FName InEventName, float InDeltaTime)
		: FUAFScriptContextData(InEventName)
		, Instance(&InInstance)
		, DeltaTime(InDeltaTime)
	{
	}

	// Get the current delta time for this execution
	float GetDeltaTime() const
	{
		return DeltaTime;
	}

	// Get the instance that is currently executing.
	FUAFAssetInstance& GetInstance() const
	{
		check(Instance != nullptr);
		return *Instance;
	}

private:
	// Instance that is currently executing.
	FUAFAssetInstance* Instance = nullptr;

	// The current delta time for this execution
	float DeltaTime = 0.0f;

	friend struct FAnimNextExecuteContext;
};
