// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFAssetInstance.h"
#include "UAFScriptContextData.generated.h"

// Base context for all scripted events
USTRUCT()
struct FUAFScriptContextData
{
	GENERATED_BODY()

	FUAFScriptContextData() = default;

	explicit FUAFScriptContextData(FName InEventName)
		: EventName(InEventName)
	{}

	// Get the currently-executing event
	FName GetEventName() const
	{
		return EventName;
	}
	
private:
	// The name of the event that is currently being executed
	FName EventName;
};