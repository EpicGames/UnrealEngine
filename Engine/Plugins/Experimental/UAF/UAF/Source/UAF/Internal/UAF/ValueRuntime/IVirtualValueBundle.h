// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#define UE_API UAF_API

struct FAnimNextGraphLODPose;

namespace UE::UAF
{
class FValueBundleHeap;
struct FTraitPtr;
struct FWeakAnimGraphReference;
}

namespace UE::UAF
{

// Implementation of a value bundle 'virtual value' that can represent a number of underlying concepts, e.g.:
// - A concrete value bundle
// - A legacy LOD pose
// - The result of reading a referenced system's output
// - An inline or owned graph instance
class IVirtualValueBundle
{
public:
	virtual ~IVirtualValueBundle() = default;

	// Get the LOD pose this bundle represents
	virtual const FAnimNextGraphLODPose* GetLODPose() const { return nullptr; }

	// Get a value bundle - valid if this bundle is a concrete value
	virtual const FValueBundleHeap* GetValueBundle() const { return nullptr; }

	// Get an anim graph reference that represents a 'virtual' value bundle - i.e. the result of evaluating the graph in-line
	virtual const FWeakAnimGraphReference* GetAnimGraphReference() const { return nullptr; }
	
	// Get an inline child ptr that represents a 'virtual' value bundle - i.e. the result of evaluating the graph fragment in-line
	virtual const FTraitPtr* GetInlineGraph() const { return nullptr; }
};

}

#undef UE_API