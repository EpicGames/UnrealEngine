// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Logging/LogCategory.h"
#include "Logging/LogVerbosity.h"

// Calls Log on the existing ExecuteContext within the current RigVMStruct scope. 
// This will either call the bound LogFunction on the ExecuteContext or fallback to a regular UE_LOG and automatically prefixes the function name to the message
#define UAF_RIGUNIT_LOG(Severity, Format, ...) \
ExecuteContext.Log(EMessageSeverity::Severity, FString::Printf((Format), ##__VA_ARGS__))

#if NO_LOGGING
	// If logging is disabled, we follow UE_LOG's behavior and only log fatal messages
	#define UAF_TRAIT_LOG(Verbosity, Format, ...) \
	{\
		if constexpr ((::ELogVerbosity::Verbosity & ::ELogVerbosity::VerbosityMask) == ::ELogVerbosity::Fatal) \
		{ \
			FString TraitName; if(Binding.GetTrait() != nullptr) { TraitName = Binding.GetTrait()->GetTraitName(); } else {TraitName = TEXT("Invalid Trait");} \
			Context.Log(ELogVerbosity::Verbosity, FString::Printf((Format), ##__VA_ARGS__), TraitName); \
		} \
	}
#else
	// Calls Context.Log within the current trait scope
	// This will automatically append the context's root graph name and the name of the owning object that runs the UAF setup, if available
	#define UAF_TRAIT_LOG(Verbosity, Format, ...) \
	FString TraitName; if(Binding.GetTrait() != nullptr) { TraitName = Binding.GetTrait()->GetTraitName(); } else {TraitName = TEXT("Invalid Trait");} \
	Context.Log(ELogVerbosity::Verbosity, FString::Printf((Format), ##__VA_ARGS__), TraitName);

#endif // NO_LOGGING

