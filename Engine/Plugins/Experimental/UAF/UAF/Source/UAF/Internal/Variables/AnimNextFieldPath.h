// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextFieldPath.generated.h"

/** Struct wrapper around TFieldPath, used to allow RigVM support */
USTRUCT()
struct FAnimNextFieldPath
{
	GENERATED_BODY()

	UPROPERTY()
	TFieldPath<FProperty> FieldPath;
};
