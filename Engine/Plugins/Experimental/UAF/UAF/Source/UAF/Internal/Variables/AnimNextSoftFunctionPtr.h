// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPtr.h"
#include "AnimNextSoftFunctionPtr.generated.h"

/** Struct wrapper around TSoftObjectPtr<UFunction>, used to allow RigVM support */
USTRUCT()
struct FAnimNextSoftFunctionPtr
{
	GENERATED_BODY()

	UPROPERTY()
	TSoftObjectPtr<UFunction> SoftObjectPtr;
};
