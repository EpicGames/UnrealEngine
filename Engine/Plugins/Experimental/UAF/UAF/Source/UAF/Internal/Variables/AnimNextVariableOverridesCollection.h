// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextVariableOverridesCollection.generated.h"

struct FRigUnit_OverrideVariables;
class FReferenceCollector;

namespace UE::UAF
{
	struct FVariableOverridesCollection;
}

USTRUCT(BlueprintType)
struct FAnimNextVariableOverridesCollection
{
	GENERATED_BODY()

	// Access the overrides for reading only
	TWeakPtr<const UE::UAF::FVariableOverridesCollection> GetOverrides() const
	{
		return Collection;
	}

	void AddStructReferencedObjects(FReferenceCollector& Collector);

private:
	// FRigUnit_OverrideVariables writes to this only, so we dont need copy-on-write and 
	// can preserve value semeantics without heavyweight copies in the VM
	friend FRigUnit_OverrideVariables;

	// Overrides that this structure wraps - ownership is with a module/graph's execute context (held in work data)
	TWeakPtr<UE::UAF::FVariableOverridesCollection> Collection;
};

template<>
struct TStructOpsTypeTraits<FAnimNextVariableOverridesCollection> : public TStructOpsTypeTraitsBase2<FAnimNextVariableOverridesCollection>
{
	enum
	{
		WithAddStructReferencedObjects = true,
	};
};
