// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "RigUnit_CopyModuleProxyVariables.generated.h"

#define UE_API UAF_API

/** Synthetic node injected by the compiler to copy proxy variables to a module instance, not user instantiated */
USTRUCT(meta=(Hidden, DisplayName = "Copy Proxy Variables", Category="Internal"))
struct FRigUnit_CopyModuleProxyVariables : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	FRigUnit_CopyModuleProxyVariables() = default;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "Events", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};

#undef UE_API
