// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "UniversalObjectLocator.h"
#include "RigUnit_ResolveUniversalObjectLocator.generated.h"

#define UE_API UAF_API

/** Synthetic node injected by the compiler to resolve a UOL to an object, not user instantiated */
USTRUCT(meta=(Hidden, DisplayName = "Resolve Locator", Category="Internal"))
struct FRigUnit_ResolveUniversalObjectLocator : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	FRigUnit_ResolveUniversalObjectLocator() = default;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The object locator to resolve
	UPROPERTY(meta = (Input))
	FUniversalObjectLocator Locator;

	// The resulting resolved object
	UPROPERTY(meta = (Output))
	TObjectPtr<UObject> Object;
};

#undef UE_API
