// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "AnimNextExecuteContext.h"
#include "RigVMDispatch_CallHoistedAccessorFunction.generated.h"

#define UE_API UAF_API

/** Synthetic dispatch injected by the compiler to get a value via a hoisted accessor UFunction, not user instantiated */
USTRUCT(meta = (Hidden, Category="Internal"))
struct FRigVMDispatch_CallHoistedAccessorFunctionBase : public FRigVMDispatchFactory
{
	GENERATED_BODY()

	UE_API FRigVMDispatch_CallHoistedAccessorFunctionBase();

	static inline const FLazyName ObjectName = FLazyName("Object");
	static inline const FLazyName FunctionName = FLazyName("Function");
	static inline const FLazyName ValueName = FLazyName("Value");

private:
	virtual UScriptStruct* GetExecuteContextStruct() const override { return FAnimNextExecuteContext::StaticStruct(); }
	UE_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual void RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
	virtual bool IsSingleton() const override { return true; }
};

/** Script function specialization of FRigVMDispatch_CallHoistedAccessorFunctionBase */
USTRUCT(meta = (Hidden, DisplayName = "Call Hoisted Native Function"))
struct FRigVMDispatch_CallHoistedAccessorFunctionNative : public FRigVMDispatch_CallHoistedAccessorFunctionBase
{
	GENERATED_BODY()

	UE_API FRigVMDispatch_CallHoistedAccessorFunctionNative();

private:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &FRigVMDispatch_CallHoistedAccessorFunctionNative::Execute;
	}
	static void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);
};

/** Script function specialization of FRigVMDispatch_CallHoistedAccessorFunctionBase */
USTRUCT(meta = (Hidden, DisplayName = "Call Hoisted Script Function"))
struct FRigVMDispatch_CallHoistedAccessorFunctionScript : public FRigVMDispatch_CallHoistedAccessorFunctionBase
{
	GENERATED_BODY()

	UE_API FRigVMDispatch_CallHoistedAccessorFunctionScript();

private:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &FRigVMDispatch_CallHoistedAccessorFunctionScript::Execute;
	}
	static void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);
};

#undef UE_API
