// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "AnimNextExecuteContext.h"
#include "RigVMDispatch_CallObjectAccessorFunction.generated.h"

#define UE_API UAF_API

/** Synthetic dispatch injected by the compiler to get a value via an accessor UFunction, not user instantiated */
USTRUCT(meta = (Hidden, Category="Internal"))
struct FRigVMDispatch_CallObjectAccessorFunctionBase : public FRigVMDispatchFactory
{
	GENERATED_BODY()

	UE_API FRigVMDispatch_CallObjectAccessorFunctionBase();

	static inline const FLazyName ObjectName = FLazyName("Object");
	static inline const FLazyName FunctionName = FLazyName("Function");
	static inline const FLazyName ValueName = FLazyName("Value");

private:
	virtual UScriptStruct* GetExecuteContextStruct() const override { return FAnimNextExecuteContext::StaticStruct(); }
	UE_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual void RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
	virtual bool IsSingleton() const override { return true; }
};

/** Script function specialization of FRigVMDispatch_CallObjectAccessorFunctionBase */
USTRUCT(meta = (Hidden, DisplayName = "Call Native Function"))
struct FRigVMDispatch_CallObjectAccessorFunctionNative : public FRigVMDispatch_CallObjectAccessorFunctionBase
{
	GENERATED_BODY()

	UE_API FRigVMDispatch_CallObjectAccessorFunctionNative();

private:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &FRigVMDispatch_CallObjectAccessorFunctionNative::Execute;
	}
	static void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);
};

/** Script function specialization of FRigVMDispatch_CallObjectAccessorFunctionBase */
USTRUCT(meta = (Hidden, DisplayName = "Call Script Function"))
struct FRigVMDispatch_CallObjectAccessorFunctionScript : public FRigVMDispatch_CallObjectAccessorFunctionBase
{
	GENERATED_BODY()

	UE_API FRigVMDispatch_CallObjectAccessorFunctionScript();

private:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &FRigVMDispatch_CallObjectAccessorFunctionScript::Execute;
	}
	static void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);
};

#undef UE_API
