// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "AnimNextExecuteContext.h"
#include "RigVMDispatch_GetObjectProperty.generated.h"

#define UE_API UAF_API

/** Synthetic dispatch injected by the compiler to get a value from an object's property, not user instantiated */
USTRUCT(meta = (Hidden, DisplayName = "Get Property", Category="Internal"))
struct FRigVMDispatch_GetObjectProperty : public FRigVMDispatchFactory
{
	GENERATED_BODY()

	UE_API FRigVMDispatch_GetObjectProperty();

	static inline const FLazyName ObjectName = FLazyName("Object");
	static inline const FLazyName PropertyName = FLazyName("Property");
	static inline const FLazyName ValueName = FLazyName("Value");
	
private:
	virtual UScriptStruct* GetExecuteContextStruct() const override { return FAnimNextExecuteContext::StaticStruct(); }
	UE_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual void RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	UE_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
	virtual bool IsSingleton() const override { return true; }

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &FRigVMDispatch_GetObjectProperty::Execute;
	}
	static UE_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);
};

#undef UE_API
