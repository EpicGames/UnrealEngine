// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Param/ParamType.h"

namespace UE::UAF
{

namespace Private
{
	void HandleLiveCodingPatchComplete();
}

// Cache of struct data to allow for easy indexing of struct properties
struct FStructDataCache
{
	// Initialize the cache
	static void Init();

	// Destroy the cache
	static void Destroy();

	// Get/cache info about a struct
	UAF_API static TSharedRef<FStructDataCache> GetStructInfo(const UScriptStruct* InStruct);

	// Information about an indexed property in a struct
	struct FPropertyInfo
	{
		const FProperty* Property = nullptr;
		FAnimNextParamType Type;
	};

	// Access indexed property data
	TConstArrayView<FPropertyInfo> GetProperties() const
	{
		return Properties;
	}

private:
	friend void Private::HandleLiveCodingPatchComplete();
	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	explicit FStructDataCache(const UScriptStruct* InStruct);

	void Rebuild();

	// The struct that this data is for 
	TWeakObjectPtr<const UScriptStruct> WeakStruct;

	// Info about properties of a struct sorted by offset
	TArray<FPropertyInfo> Properties;
};

}