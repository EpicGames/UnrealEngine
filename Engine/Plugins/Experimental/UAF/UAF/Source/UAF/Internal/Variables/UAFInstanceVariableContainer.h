// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Variables/AnimNextVariableReference.h"
#include "StructUtils/PropertyBag.h"
#include "UAFInstanceVariableContainer.generated.h"

class IRigVMRuntimeAssetInterface;
struct FRigVMExternalVariableRuntimeData;
struct FRigVMExternalVariableDef;
struct FUAFInstanceVariableContainerProxy;
struct FAnimNextModuleInstance;
struct FUAFAssetInstance;
struct FUAFInstanceVariableData;
struct FUAFInstanceVariableDataProxy;
struct FAnimNextParamType;
struct FUAFRigVMComponent;

namespace UE::UAF
{
	struct FVariableOverridesCollection;
	struct FVariableOverrides;
	struct FStructDataCache;
	struct FUAFTrace;
}

// A 'shared' variable container, either a user-defined FInstancedPropertyBag or a natively-defined struct wrapped in a FInstancedStruct
// Defines a block of asset data that:
// - Is owned by a particular asset instance
// - Is visible to nested instances (i.e. can be passed-by-reference or 'shared')
// - Can be overriden by nested instances
USTRUCT()
struct FUAFInstanceVariableContainer
{
	GENERATED_BODY()

	FUAFInstanceVariableContainer() = default;

	// GC support
	void AddStructReferencedObjects(FReferenceCollector& Collector);

private:
	friend FUAFInstanceVariableData;
	friend FUAFAssetInstance;
	friend FAnimNextModuleInstance;
	friend FUAFInstanceVariableContainerProxy;
	friend FUAFInstanceVariableDataProxy;
	template<typename, ESPMode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;
	friend UE::UAF::FUAFTrace;
	friend FUAFRigVMComponent;

	// Construct from struct data, copying the defaults. Optional overrides provided.
	explicit FUAFInstanceVariableContainer(const FUAFAssetInstance& InHostInstance, const UScriptStruct* InStruct, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides = nullptr, int32 InOverridesIndex = INDEX_NONE);

	// Construct from an asset, copying the variables. Optional overrides provided.
	explicit FUAFInstanceVariableContainer(const FUAFAssetInstance& InHostInstance, const UUAFRigVMAsset* InAsset, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides = nullptr, int32 InOverridesIndex = INDEX_NONE);

	// Construct from a IRigVMRuntimeAssetInterface, copying the variables. Optional overrides provided.
	explicit FUAFInstanceVariableContainer(const FUAFAssetInstance& InHostInstance, const IRigVMRuntimeAssetInterface* InAsset, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides = nullptr, int32 InOverridesIndex = INDEX_NONE);

	// Gets the value of the specified variable. If the type does not match exactly then a conversion will be attempted.
	EPropertyBagResult GetVariable(int32 InVariableIndex, const FAnimNextParamType& InType, TArrayView<uint8> OutResult) const;

	// Access the memory of the variable directly. Type must match strictly, no conversions are performed
	EPropertyBagResult AccessVariable(int32 InVariableIndex, const FAnimNextParamType& InType, TArrayView<uint8>& OutResult);

	// Sets the value of the specified variable. If the type does not match exactly then a conversion will be attempted.
	EPropertyBagResult SetVariable(int32 InVariableIndex, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue);

	// Find references to all variables of the specified type
	void GetAllVariablesOfType(const FAnimNextParamType& InType, TArray<FAnimNextVariableReference>& OutVariables) const;

	// Iterate over all variables calling InFunction
	bool ForEachVariable(TFunctionRef<bool(FName, const FAnimNextParamType&, int32)> InFunction, int32& InOutVariableIndex) const;

	// Get the type of the variable at the specified index
	FAnimNextParamType GetVariableType(int32 InVariableIndex) const;

	// Unchecked, non-type converting version of AccessVariable
	void AccessVariableUnchecked(int32 InVariableIndex, const FProperty*& OutProperty, TArrayView<uint8>& OutResult);

	// Get the name of the underlying object (asset, struct etc) that this instance variable container is initialized from
	FName GetFName() const;

#if WITH_EDITOR
	// Used during compilation, migrate all variables to new bags according to new defaults.
	void Migrate();
#endif

	// Resolve any overrides from an outer host
	void ResolveOverrides();

	// Indirect RigVM memory handles to either variables or overrides, filling the supplied array
	void GetRigVMMemoryForVariables(TConstArrayView<FRigVMExternalVariableDef> InVariableDefs, TArray<FRigVMExternalVariableRuntimeData>& OutRuntimeData);

	// The instance that hosts these variables
	const FUAFAssetInstance* HostInstance = nullptr;

	// The asset or struct these variables came from, if any
	using FAssetType = TObjectPtr<const UUAFRigVMAsset>;
	using FStructType = TObjectPtr<const UScriptStruct>;
	using FRigVMAssetType = TScriptInterface<const IRigVMRuntimeAssetInterface>;
	using FAssetOrStructType = TVariant<FAssetType, FStructType, FRigVMAssetType>;
	FAssetOrStructType AssetOrStructData;

	// Instance of the asset's variables or resolved overrides for the asset's variables
	using FVariablesContainerType = TVariant<FInstancedPropertyBag, FInstancedStruct>;
	FVariablesContainerType VariablesContainer;

	// Cached overrides collection, if any are to be applied
	TWeakPtr<const UE::UAF::FVariableOverridesCollection> Overrides;

	// Index into overrides collection to apply
	int32 OverridesIndex = INDEX_NONE;

	// Cached resolved overrides derived from Overrides
	TArray<uint8*> ResolvedOverrides;

	// Reference to cached struct data, if required
	TSharedPtr<UE::UAF::FStructDataCache> StructDataCache;

	// Cached variable count
	int32 NumVariables = 0;
};

template<>
struct TStructOpsTypeTraits<FUAFInstanceVariableContainer> : public TStructOpsTypeTraitsBase2<FUAFInstanceVariableContainer>
{
	enum
	{
		WithAddStructReferencedObjects = true,
	};
};