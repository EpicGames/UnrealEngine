// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Variables/UAFInstanceVariableContainer.h"
#include "Variables/UAFInstanceVariableData.h"
#include "UAFInstanceVariableDataProxy.generated.h"

class UUAFComponent;
struct FAnimNextModuleInstance;
struct FAnimNextVariableReference;

USTRUCT()
struct FUAFInstanceVariableContainerProxy : public FUAFInstanceVariableContainer
{
	GENERATED_BODY()

	FUAFInstanceVariableContainerProxy() = default;

	explicit FUAFInstanceVariableContainerProxy(const FUAFInstanceVariableContainer& InVariables);

	// Accesses the value of the specified variable, setting the dirty flag
	EPropertyBagResult AccessVariableWithDirtyFlag(int32 InVariableIndex, const FAnimNextParamType& InType, TArrayView<uint8>& OutResult);

	// Sets the value of the specified variable, setting the dirty flag
	EPropertyBagResult SetVariableWithDirtyFlag(int32 InVariableIndex, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue);

	// Dirty flags for each variable
	TBitArray<> DirtyFlags;
};

// Proxy struct for public variables for modules. Held double-buffered on module instances
// We double-buffer so updates to variables can occur during graph evaluation, with the actual update deferred to later.
// This double-buffering prevents read-back via the same mechanism at the time of writing
// @see FRigUnit_CopyModuleProxyVariables
USTRUCT()
struct FUAFInstanceVariableDataProxy
{
	GENERATED_BODY()

	// Sets up this proxy variable array from the supplied variables array
	void Initialize(const FUAFInstanceVariableData& InVariables);

	// Resets this proxy variables array to its default state
	void Reset();

	// Access the memory of the variable directly for writing. Type must match strictly, no conversions are performed.
	EPropertyBagResult WriteVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8>& OutResult);

	// Sets the value of the specified variable. If the type does not match exactly then a conversion will be attempted.
	EPropertyBagResult SetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue);

	// Access the memory of the variable directly. Type must match strictly, no conversions are performed.
	EPropertyBagResult AccessVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8>& OutValue);

	// Access the memory of the variables set directly for writing. All variables in the set are dirtied.
	bool WriteVariables(const UScriptStruct* InStruct, TFunctionRef<void(FStructView)> InFunction);

	// Copies any dirty variables over to the corresponding Variables 
	void CopyDirty();

	// Returns whether the variable has been marked as dirty
	bool IsVariableDirty(const FAnimNextVariableReference& InVariable) const;

	// Variables that this proxy corresponds to
	const FUAFInstanceVariableData* Variables = nullptr;

	// Proxy shared variable sets
	UPROPERTY()
	TArray<FUAFInstanceVariableContainerProxy> ProxyVariableSets;

	// Dirty flags for each variable set
	TBitArray<> DirtyFlags;

	// Global dirty flag
	bool bIsDirty = false;
};