// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextConfig.h"

#include "Factory/SystemFactory.h"
#include "Module/AnimNextModule.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "UAF/UAFAssetFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextConfig)

#if WITH_EDITOR

void UAnimNextConfig::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	SaveConfig();
}

#endif

TArray<UClass*> UAnimNextConfig::GetAllowedAssetClasses()
{
	TArray<UClass*> Array;
	Array.Add(UUAFSystem::StaticClass());
	Array.Append(UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses());
	return Array;
}
