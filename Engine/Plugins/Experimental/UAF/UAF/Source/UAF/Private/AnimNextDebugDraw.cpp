// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextDebugDraw.h"

#include "MeshElementCollector.h"
#include "PrimitiveSceneDesc.h"
#include "PrimitiveSceneProxyDesc.h"
#include "SceneInterface.h"
#include "RigVMCore/RigVMDrawInstruction.h"
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "Module/AnimNextModuleInstance.h"

namespace UE::UAF::Debug
{

#if UE_ENABLE_DEBUG_DRAWING

FAnimNextDebugSceneProxy::FAnimNextDebugSceneProxy(const FPrimitiveSceneProxyDesc& InProxyDesc)
	: FPrimitiveSceneProxy(InProxyDesc)
{
	bWillEverBeLit = false;

	// We do not use any streamable assets, no override is needed for Gathering
	bImplementsStreamableAssetGathering = true;
}

SIZE_T FAnimNextDebugSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FAnimNextDebugSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];

			FPrimitiveDrawInterface* PDI = Collector.GetDebugPDI(ViewIndex);

			if (bIsEnabled)
			{ 
				for (int32 InstructionIndex = 0; InstructionIndex < DrawInterface.Num(); InstructionIndex++)
				{
					const FRigVMDrawInstruction& Instruction = DrawInterface[InstructionIndex];
					if (Instruction.Positions.Num() == 0)
					{
						continue;
					}

					switch (Instruction.PrimitiveType)
					{
						case ERigVMDrawSettings::Points:
						{
							for (const FVector& Point : Instruction.Positions)
							{
								PDI->DrawPoint(Instruction.Transform.TransformPosition(Point), Instruction.Color, Instruction.Thickness, SDPG_World);
							}
							break;
						}
						case ERigVMDrawSettings::Lines:
						{
							const TArray<FVector>& Points = Instruction.Positions;
							PDI->AddReserveLines(SDPG_World, Points.Num() / 2, false, Instruction.Thickness > SMALL_NUMBER);
							for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex += 2)
							{
								PDI->DrawLine(Instruction.Transform.TransformPosition(Points[PointIndex]), Instruction.Transform.TransformPosition(Points[PointIndex + 1]), Instruction.Color, SDPG_World, Instruction.Thickness);
							}
							break;
						}
						case ERigVMDrawSettings::LineStrip:
						{
							const TArray<FVector>& Points = Instruction.Positions;
							PDI->AddReserveLines(SDPG_World, Points.Num() - 1, false, Instruction.Thickness > SMALL_NUMBER);
							for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex++)
							{
								PDI->DrawLine(Instruction.Transform.TransformPosition(Points[PointIndex]), Instruction.Transform.TransformPosition(Points[PointIndex + 1]), Instruction.Color, SDPG_World, Instruction.Thickness);
							}
							break;
						}
						case ERigVMDrawSettings::DynamicMesh:
						{
							FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
							MeshBuilder.AddVertices(Instruction.MeshVerts);
							MeshBuilder.AddTriangles(Instruction.MeshIndices);
							MeshBuilder.Draw(PDI, Instruction.Transform.ToMatrixWithScale(), Instruction.MaterialRenderProxy, SDPG_World/*SDPG_Foreground*/);
							break;
						}
					default:
						break;
					}
				}
			}
		}
	}
}

FPrimitiveViewRelevance FAnimNextDebugSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance ViewRelevance;
	ViewRelevance.bDrawRelevance = IsShown(View);
	ViewRelevance.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	ViewRelevance.bSeparateTranslucency = ViewRelevance.bNormalTranslucency = true;
	return ViewRelevance;
}

uint32 FAnimNextDebugSceneProxy::GetMemoryFootprint(void) const
{
	return(sizeof(*this) + FPrimitiveSceneProxy::GetAllocatedSize());
}

FDebugDraw::FCustomSceneProxyDesc::FCustomSceneProxyDesc(UObject* InOwner)
{
	check(InOwner);
	check(InOwner->GetWorld());

	bReceivesDecals = false;
	bVisibleInReflectionCaptures = false;
	bVisibleInRealTimeSkyCaptures = false;
	bVisibleInRayTracing = false;
	bVisibleInReflections = false;

	bCastDynamicShadow = false;
	bCastStaticShadow = false;
	bAffectDynamicIndirectLighting = false;
	bAffectDistanceFieldLighting = false;
	bCastContactShadow = false;
	bSelectable = false;
	bReceiveMobileCSMShadows = false;
	bHiddenInSceneCapture = true;
	bSupportsWorldPositionOffsetVelocity = false;

	World = InOwner->GetWorld();
	Scene = World->Scene;
	CustomPrimitiveData = &DummyCustomPrimitiveData;
}

FDebugDraw::FDebugDraw(UObject* InOwner)
	: SceneProxyDesc(InOwner)
	, SceneProxy(new FAnimNextDebugSceneProxy(SceneProxyDesc))
{
	LLM_SCOPE_BYNAME(TEXT("UAF/Debug"));

	check(IsInGameThread());

	SceneInfoData = new FPrimitiveSceneInfoData();
	SceneInfoData->SceneProxy = SceneProxy;
	SceneInfoData->OwnerLastRenderTimePtr = nullptr;
	SceneDesc.ProxyDesc = &SceneProxyDesc;
	SceneDesc.PrimitiveSceneData = SceneInfoData;
	SceneDesc.PrimitiveUObject = InOwner;

	SceneDesc.RenderMatrix = FMatrix::Identity;
	SceneDesc.AttachmentRootPosition = FVector::ZeroVector;
	SceneDesc.Bounds = FBoxSphereBounds(FVector::ZeroVector, FVector::OneVector, 1.f);
	SceneDesc.LocalBounds = SceneDesc.Bounds;

	Scene = InOwner->GetWorld()->Scene;

	Scene->AddPrimitive(&SceneDesc);
	bIsRegistered = true;
}

void FDebugDraw::RemovePrimitive()
{
	UE::TWriteScopeLock WriteLock(Lock);

	Scene->RemovePrimitive(&SceneDesc);
	bIsRegistered = false;

	// Delete the scene info data on the RT as we no longer own it
	ENQUEUE_RENDER_COMMAND(AnimNextDebugDraw)([SceneInfoData = SceneInfoData](FRHICommandList& RHICmdList)
	{
		delete SceneInfoData;
	});
}

void FDebugDraw::Draw()
{
	UE::TWriteScopeLock WriteLock(Lock);

	if (!bIsRegistered)
	{
		return;
	}

	// Recalc bounds according to what we need to draw
	CalcBounds();

	// Update the primitive transform/bounds
	Scene->UpdatePrimitiveTransform(&SceneDesc);

	// Move ownership of the draw interface to the RT
	ENQUEUE_RENDER_COMMAND(AnimNextDebugDraw)([bIsEnabled = bIsEnabled, SceneProxy = SceneProxy, DrawInterface = MoveTemp(DrawInterface)](FRHICommandList& RHICmdList) mutable
	{
		SceneProxy->bIsEnabled = bIsEnabled;
		SceneProxy->DrawInterface = MoveTemp(DrawInterface);
	});

	DrawInterface.Reset();
}

void FDebugDraw::SetEnabled(bool bInIsEnabled)
{
	check(IsInGameThread());

	if(bInIsEnabled != bIsEnabled)
	{
		// Update the RT copy of the flag if it differs
		ENQUEUE_RENDER_COMMAND(AnimNextDebugDraw)([bInIsEnabled, SceneProxy = SceneProxy](FRHICommandList& RHICmdList)
		{
			SceneProxy->bIsEnabled = bInIsEnabled;
		});
	}
	bIsEnabled = bInIsEnabled; 
}

void FDebugDraw::CalcBounds()
{
	FBox Box(ForceInit);

	// Get bounding box for the debug drawings if they are drawn 
	for (const FRigVMDrawInstruction& Instruction : DrawInterface.Instructions)
	{
		FTransform Transform = Instruction.Transform /** GetComponentToWorld()*/;
		for (const FVector& Position : Instruction.Positions)
		{
			Box += Transform.TransformPosition(Position);
		}
	}

	if (Box.IsValid)
	{
		// Points are in world space, so no need to transform.
		SceneDesc.Bounds = FBoxSphereBounds(Box);
	}
	else
	{
		SceneDesc.Bounds = FBoxSphereBounds(FVector::ZeroVector, FVector::OneVector, 1.f);
	}

	SceneDesc.LocalBounds = SceneDesc.Bounds;
}

#endif

}
