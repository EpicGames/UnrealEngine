// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextFunctionReference.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextFunctionReference)

FAnimNextFunctionReference FAnimNextFunctionReference::FromHeader(const FRigVMGraphFunctionHeader& InHeader, const UObject* InAsset)
{
	FAnimNextFunctionReference Ref;
	Ref.FunctionGuid = InHeader.Variant.Guid;
#if WITH_EDITORONLY_DATA
	// Note: prefix must match UE::UAF::UncookedOnly::FUtils::MakeFunctionWrapperEventName()
	Ref.EventName = FName(TEXT("__InternalCall_") + InHeader.Name.ToString());
#endif
	Ref.Object = InAsset;
	return Ref;
}

#if WITH_EDITORONLY_DATA
FAnimNextFunctionReference FAnimNextFunctionReference::FromEventName(FName InEventName, const UObject* InAsset)
{
	FAnimNextFunctionReference Ref;
	Ref.EventName = InEventName;
	Ref.Object = InAsset;
	return Ref;
}
#endif
