// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextModuleImpl.h"
#include "AnimNextConfig.h"
#include "AnimNextRigVMAsset.h"
#include "DataRegistry.h"
#include "IUniversalObjectLocatorModule.h"
#include "RigVMRuntimeDataRegistry.h"
#include "UniversalObjectLocator.h"
#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Animation/BlendSpace1D.h"
#include "Animation/BlendProfile.h"
#include "Component/AnimNextComponent.h"
#include "Curves/CurveFloat.h"
#include "Graph/AnimNext_LODPose.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/UAFWeakSystemReference.h"
#include "Modules/ModuleManager.h"
#include "Param/AnimNextActorLocatorFragment.h"
#include "Param/AnimNextComponentLocatorFragment.h"
#include "Param/AnimNextObjectCastLocatorFragment.h"
#include "Param/AnimNextObjectFunctionLocatorFragment.h"
#include "Param/AnimNextObjectPropertyLocatorFragment.h"
#include "Param/AnimNextTag.h"
#include "RigVMCore/RigVMRegistry.h"
#include "Variables/AnimNextFieldPath.h"
#include "Variables/AnimNextSoftFunctionPtr.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "HierarchyTable.h"
#include "BindableValue/UAFBindableTypes.h"
#include "CrashReporter/CrashReporterHandler.h"
#include "Factory/SystemFactory.h"
#include "Features/IModularFeatures.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"
#include "RewindDebugger/RewindDebuggerUAFRuntime.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Module/UAFSystemAssetData.h"
#include "UAF/UAFAssetFactory.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/BlendMask/UAFBlendMask.h"
#include "UAF/BlendProfile/UAFBlendProfile.h"

#define LOCTEXT_NAMESPACE "AnimNextModule"

namespace UE::UAF
{
#if UAF_TRACE_ENABLED
	FRewindDebuggerUAFRuntime GRewindDebuggerUAFRuntime;
#endif
	
	void FAnimNextModuleImpl::StartupModule()
	{
		GetMutableDefault<UAnimNextConfig>()->LoadConfig();
		
#if WITH_ADDITIONAL_CRASH_CONTEXTS
		FCrashReporterHandler::Register();
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS

		FRigVMRegistry& RigVMRegistry = FRigVMRegistry::Get();
		static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
		{
			{ UAnimSequence::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UBlendSpace::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UBlendSpace1D::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UScriptStruct::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UBlendProfile::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class }, // TODO: Remove
			{ UCurveFloat::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UUAFComponent::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UUAFSystem::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UUAFRigVMAsset::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UHierarchyTable::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UUAFBlendMask::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UUAFBlendProfile::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
			{ UCharacterMovementComponent::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
		};

		RigVMRegistry.RegisterObjectTypes(AllowedObjectTypes);

		static UScriptStruct* const AllowedStructTypes[] =
		{
			FAnimNextScope::StaticStruct(),
			FAnimNextEntryPoint::StaticStruct(),
			FUniversalObjectLocator::StaticStruct(),
			FAnimNextFieldPath::StaticStruct(),
			FAnimNextSoftFunctionPtr::StaticStruct(),
			FRigVMGraphFunctionHeader::StaticStruct(),
			TBaseStructure<FGuid>::Get(),
			FRigVMVariant::StaticStruct(),
			FUAFWeakSystemReference::StaticStruct(),
			FAnimNextGraphLODPose::StaticStruct(),
			FAnimNextGraphReferencePose::StaticStruct(),
			FUAFValueBundle::StaticStruct(),
			FBindableBool::StaticStruct(),
			FBindableFloat::StaticStruct(),
			FBindableDouble::StaticStruct(),
			FBindableInt32::StaticStruct(),
			FBindableInt64::StaticStruct(),
			FBindableByte::StaticStruct(),
			FBindableName::StaticStruct(),
			FBindableVector::StaticStruct(),
			FBindableQuat::StaticStruct(),
			FBindableTransform::StaticStruct(),
			FBindableEnum::StaticStruct(),
			FBindableStruct::StaticStruct(),
			FBindableObject::StaticStruct(),
		};

		RigVMRegistry.RegisterStructTypes(AllowedStructTypes);

		FSystemReference::Init();

		FDataRegistry::Init();

		FRigVMRuntimeDataRegistry::Init();

		FSystemFactory::Init();

        RegisterAssets();

		FValueRuntimeRegistry::Init();
		RegisterEngineAttributes();

#if UAF_TRACE_ENABLED
		IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &GRewindDebuggerUAFRuntime);
#endif

		UE::UniversalObjectLocator::IUniversalObjectLocatorModule& UolModule = FModuleManager::Get().LoadModuleChecked<UE::UniversalObjectLocator::IUniversalObjectLocatorModule>("UniversalObjectLocator");
		FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase::ObjectSystemReady,
			[&UolModule]
			{
				{
					UE::UniversalObjectLocator::FFragmentTypeParameters FragmentTypeParams("animobjfunc", LOCTEXT("UAFObjectFunctionFragment", "Function"));
					FragmentTypeParams.PrimaryEditorType = "AnimNextObjectFunction";
					FAnimNextObjectFunctionLocatorFragment::FragmentType = UolModule.RegisterFragmentType<FAnimNextObjectFunctionLocatorFragment>(FragmentTypeParams);
				}
				{
					UE::UniversalObjectLocator::FFragmentTypeParameters FragmentTypeParams("animobjprop", LOCTEXT("UAFObjectPropertyFragment", "Property"));
					FragmentTypeParams.PrimaryEditorType = "AnimNextObjectProperty";
					FAnimNextObjectPropertyLocatorFragment::FragmentType = UolModule.RegisterFragmentType<FAnimNextObjectPropertyLocatorFragment>(FragmentTypeParams);
				}
				{
					UE::UniversalObjectLocator::FFragmentTypeParameters FragmentTypeParams("animobjcast", LOCTEXT("UAFCastFragment", "Cast"));
					FragmentTypeParams.PrimaryEditorType = "AnimNextObjectCast";
					FAnimNextObjectCastLocatorFragment::FragmentType = UolModule.RegisterFragmentType<FAnimNextObjectCastLocatorFragment>(FragmentTypeParams);
				}
				{
					UE::UniversalObjectLocator::FFragmentTypeParameters FragmentTypeParams("animcomp", LOCTEXT("UAFComponentFragment", "UAFComponent"));
					FragmentTypeParams.PrimaryEditorType = "UAFComponent";
					FAnimNextComponentLocatorFragment::FragmentType = UolModule.RegisterFragmentType<FAnimNextComponentLocatorFragment>(FragmentTypeParams);
				}
				{
					UE::UniversalObjectLocator::FFragmentTypeParameters FragmentTypeParams("animactor", LOCTEXT("UAFActorFragment", "UAFActor"));
					FragmentTypeParams.PrimaryEditorType = "AnimNextActor";
					FAnimNextActorLocatorFragment::FragmentType = UolModule.RegisterFragmentType<FAnimNextActorLocatorFragment>(FragmentTypeParams);
				}
			});

	}

	void FAnimNextModuleImpl::ShutdownModule()
	{
		UnregisterAssets();
#if UAF_TRACE_ENABLED
		IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &GRewindDebuggerUAFRuntime);
#endif
		
		FValueRuntimeRegistry::Destroy();
		FRigVMRuntimeDataRegistry::Destroy();
		FDataRegistry::Destroy();
		FSystemFactory::Destroy();
		FSystemReference::Destroy();
		
#if WITH_ADDITIONAL_CRASH_CONTEXTS
		FCrashReporterHandler::Unregister();
#endif // WITH_ADDITIONAL_CRASH_CONTEXTS
	}

	void FAnimNextModuleImpl::RegisterAssets()
	{
		UAFSystemClassPath = FAssetDataFactory::RegisterAsset<FUAFSystemFactoryAsset_System, UUAFSystem>(
		[](const UUAFSystem* System)
			{
				return FUAFSystemFactoryAsset_System(System);
			});
	}

	void FAnimNextModuleImpl::UnregisterAssets()
	{
		FAssetDataFactory::UnregisterAsset(UAFSystemClassPath);
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::UAF::FAnimNextModuleImpl, UAF)
