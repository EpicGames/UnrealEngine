// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextRigVMAsset.h"

#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif
#include "AnimNextFunctionHandle.h"
#include "AnimNextRigVMFunctionData.h"
#include "RigVMRuntimeDataRegistry.h"
#include "Graph/RigUnit_AnimNextBeginExecution.h"
#include "UObject/AssetRegistryTagsContext.h"

#if WITH_EDITOR
#include "IAnimNextUncookedOnlyModule.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextRigVMAsset)

namespace UE::UAF::Private
{
#if WITH_EDITOR
	static UUAFRigVMAsset::FOnCompileJobEvent GOnCompileJobStarted;
	static UUAFRigVMAsset::FOnCompileJobEvent GOnCompileJobFinished;
#endif
}

UUAFRigVMAsset::UUAFRigVMAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetRigVMExtendedExecuteContext(&ExtendedExecuteContext);
}

void UUAFRigVMAsset::BeginDestroy()
{
	Super::BeginDestroy();

	if (VM)
	{
		UE::UAF::FRigVMRuntimeDataRegistry::ReleaseAllVMRuntimeData(VM);
	}
}

void UUAFRigVMAsset::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		if (EditorData)
		{
			// External editor-side APIs assume that the editordata is also loaded with the asset
			EditorData->ConditionalPreload();
		}
#endif

		ExtendedExecuteContext.InvalidateCachedMemory();

		VM = RigVM;
	}
}

void UUAFRigVMAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// External editor-side APIs assume that the editordata is also postloaded with the asset
	EditorData->ConditionalPostLoad();
#endif

	// In packaged builds, initialize the VM
	// In editor, the VM will be recompiled and initialized at UUAFRigVMAssetEditorData::HandlePackageDone::RecompileVM
#if !WITH_EDITOR
	if(VM != nullptr)
	{
		VM->ClearExternalVariables(ExtendedExecuteContext);
		VM->SetExternalVariableDefs(GetExternalVariablesImpl(false));
		VM->Initialize(ExtendedExecuteContext);
		InitializeVM(FRigUnit_AnimNextBeginExecution::EventName);
	}
#endif
}

void UUAFRigVMAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

#if WITH_EDITORONLY_DATA
	if(EditorData)
	{
		EditorData->GetAssetRegistryTags(Context);
	}
#endif

#if WITH_EDITOR
	// Allow asset user data to output tags
	for(const UAssetUserData* AssetUserDataItem : *GetAssetUserDataArray())
	{
		if (AssetUserDataItem)
		{
			AssetUserDataItem->GetAssetRegistryTags(Context);
		}
	}
#endif // WITH_EDITOR
}

void UUAFRigVMAsset::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	Super::PreDuplicate(DupParams);

#if WITH_EDITORONLY_DATA
	if (EditorData)
	{
		EditorData->PreDuplicate(DupParams);
	}
#endif
}

TArray<FRigVMExternalVariable> UUAFRigVMAsset::GetExternalVariablesImpl(bool bFallbackToBlueprint) const
{
	TArray<FRigVMExternalVariable> ExternalVariables;
#if WITH_EDITOR
	UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule& UncookedModule = UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule::Get();
	const bool bCompiledVariableDataOutOfDate = UncookedModule.DoesAssetVariablesRequireCompilation(this);
	if (bCompiledVariableDataOutOfDate && bFallbackToBlueprint)
	{
		UncookedModule.GetExternalVariablesForAsset(this, ExternalVariables);
		// Cannot rely on compiled property bag below as it might contain out-of-date information (e.g. removed or renamed variables, yet to be re-compiled)
		return ExternalVariables;
	}

	// Using of CombinedPropertyBag data is only valid if its up-to-date (or assumed up to date bFallbackToBlueprint = false)
	check(!bCompiledVariableDataOutOfDate || !bFallbackToBlueprint);
#endif // WITH_EDITOR

	if(const UPropertyBag* PropertyBag = CombinedPropertyBag.GetPropertyBagStruct())
	{
		TConstArrayView<FPropertyBagPropertyDesc> VariableDescs = PropertyBag->GetPropertyDescs();
		if(VariableDescs.Num() != 0)
		{
			uint8* Container = const_cast<uint8*>(CombinedPropertyBag.GetValue().GetMemory());
			ExternalVariables.Reserve(ExternalVariables.Num() + VariableDescs.Num());
			for(const FPropertyBagPropertyDesc& Desc : VariableDescs)
			{
				const FProperty* Property = Desc.CachedProperty;
				FRigVMExternalVariable ExternalVariable = FRigVMExternalVariable::Make(Desc.ID, Property, Container);
				if(!ExternalVariable.IsValid())
				{
					UE_LOGF(LogRigVM, Error, "%ls: Property '%ls' of type '%ls' is not supported.", *GetName(), *Property->GetName(), *Property->GetCPPType());
					continue;
				}

				ExternalVariables.Add(ExternalVariable);
			}
		}
	}

	return ExternalVariables;
}


bool UUAFRigVMAsset::ExecuteParameterlessFunction(UE::UAF::FFunctionHandle InHandle, FRigVMExtendedExecuteContext& InContext, void* OutReturnValue, int32 ReturnValueSize)
{
	check(InHandle.IsValid());
#if WITH_EDITOR
	check(InHandle.IsValidForVM(VM->GetVMHash()));
#endif

	const FAnimNextRigVMFunctionData& Function = FunctionData[InHandle.FunctionIndex];

	// Must have at least 1 arg (the return value output). Functions that read from system
	// variables (bIsInputVariable) will have additional arg indices for those reads;
	// the output is always the last arg.
	if (!ensure(Function.ArgIndices.Num() >= 1))
	{
		return false;
	}

	const int32 ReturnVarIndex = Function.ArgIndices.Last();

	// Execute the function. Guard the public context data to support re-entrant calls
	// (e.g. when GetValue is called from an anim node during an ongoing VM execution).
	// This matches the pattern used by the InvokeEntry opcode in RigVM.cpp.
	{
		FRigVMExecuteContext& ContextPublicData = InContext.GetPublicData<>();
		TGuardValue<FRigVMExecuteContext> PublicDataGuard(ContextPublicData, ContextPublicData);
		VM->ExecuteVM(InContext, Function.EventName);
	}

	// Copy the return value from the external variable to the caller's buffer.
	// Handles float<->double coercion (RigVM stores floats as doubles internally).
	const FProperty* ReturnProp = VM->GetExternalVariableDefs()[ReturnVarIndex].GetProperty();
	check(ReturnProp);
	void* SrcPtr = InContext.ExternalVariableRuntimeData[ReturnVarIndex].Memory;

	const int32 ReturnPropSize = ReturnProp->GetElementSize();
	if (ReturnPropSize == ReturnValueSize)
	{
		// Exact size match -- direct copy
		ReturnProp->CopyCompleteValue(OutReturnValue, SrcPtr);
	}
	else if (const FNumericProperty* NumProp = CastField<FNumericProperty>(ReturnProp))
	{
		if (NumProp->IsFloatingPoint())
		{
			// Float<->double coercion (RigVM stores floats as doubles internally)
			const double Value = NumProp->GetFloatingPointPropertyValue(SrcPtr);
			if (ReturnValueSize == sizeof(float))
			{
				*static_cast<float*>(OutReturnValue) = static_cast<float>(Value);
			}
			else if (ReturnValueSize == sizeof(double))
			{
				*static_cast<double*>(OutReturnValue) = Value;
			}
			else
			{
				UE_LOGF(LogAnimation, Error,
					"ExecuteParameterlessFunction: Unsupported floating point value size %u calling function %ls",
					ReturnValueSize, *Function.Name.ToString());
				return false;
			}
		}
		else if (NumProp->IsInteger() && ReturnPropSize <= ReturnValueSize)
		{
			// Integer promotion: source fits in destination (e.g. uint8 enum -> int32).
			// Zero-init destination then copy source bytes into the low end.
			FMemory::Memzero(OutReturnValue, ReturnValueSize);
			FMemory::Memcpy(OutReturnValue, SrcPtr, ReturnPropSize);
		}
		else
		{
			UE_LOGF(LogAnimation, Error,
				"ExecuteParameterlessFunction: Unsupported numerical property type %ls calling function %ls",
				*NumProp->GetCPPType(), *Function.Name.ToString());
			return false;
		}
	}
	else if (ReturnPropSize < ReturnValueSize
		&& (CastField<FEnumProperty>(ReturnProp) || CastField<FByteProperty>(ReturnProp)))
	{
		// Enum/byte promotion: source fits in destination (e.g. uint8 enum -> int32).
		// Zero-init destination then copy source bytes into the low end.
		FMemory::Memzero(OutReturnValue, ReturnValueSize);
		FMemory::Memcpy(OutReturnValue, SrcPtr, ReturnPropSize);
	}
	else
	{
		UE_LOGF(LogAnimation, Error,
			"ExecuteParameterlessFunction: return type size mismatch for '%ls' -- expected %d bytes, got %d. Binding may be stale.",
			*Function.Name.ToString(), ReturnValueSize, ReturnPropSize);
		return false;
	}

	return true;
}

void UUAFRigVMAsset::CallFunctionHandle(UE::UAF::FFunctionHandle InHandle, FRigVMExtendedExecuteContext& InContext, FInstancedPropertyBag& InArgs)
{
#if WITH_EDITOR
	check(InHandle.IsValidForVM(VM->GetVMHash()));
#endif

	const FAnimNextRigVMFunctionData& Function = FunctionData[InHandle.FunctionIndex];
	const int32 NumArgs = Function.ArgIndices.Num();

	if (!ensure(InArgs.GetNumPropertiesInBag() == NumArgs))
	{
		return;
	}

	const UPropertyBag* PropertyBag = InArgs.GetPropertyBagStruct();
	if (PropertyBag)
	{
		// Copy-in args
		void* ContainerPtr = InArgs.GetMutableValue().GetMemory();
		TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBag->GetPropertyDescs();
		for (int32 ArgIndex = 0; ArgIndex < NumArgs; ++ArgIndex)
		{
			const FPropertyBagPropertyDesc& Desc = Descs[ArgIndex];
			const int32 ExternalVariableIndex = Function.ArgIndices[ArgIndex];
			check(VM->GetExternalVariableDefs()[ExternalVariableIndex].GetProperty()->GetClass() == Desc.CachedProperty->GetClass());
			void* DestPtr = InContext.ExternalVariableRuntimeData[ExternalVariableIndex].Memory;
			void* SrcPtr = Desc.CachedProperty->ContainerPtrToValuePtr<void>(ContainerPtr);
			Desc.CachedProperty->CopyCompleteValue(DestPtr, SrcPtr);
		}

		// Run the function's wrapper event
		VM->ExecuteVM(InContext, Function.EventName);

		// Copy-out args
		for (int32 ArgIndex = 0; ArgIndex < NumArgs; ++ArgIndex)
		{
			const FPropertyBagPropertyDesc& Desc = Descs[ArgIndex];
			void* DestPtr = Desc.CachedProperty->ContainerPtrToValuePtr<void>(ContainerPtr);
			void* SrcPtr = InContext.ExternalVariableRuntimeData[Function.ArgIndices[ArgIndex]].Memory;
			Desc.CachedProperty->CopyCompleteValue(DestPtr, SrcPtr);
		}
	}
	else
	{
		// No args - just run the function's wrapper event
		VM->ExecuteVM(InContext, Function.EventName);
	}
}

UE::UAF::FFunctionHandle UUAFRigVMAsset::GetFunctionHandle(const FGuid& InFunctionGuid) const
{
	using namespace UE::UAF;

	if (!InFunctionGuid.IsValid())
	{
		return FFunctionHandle();
	}

	for (int32 FunctionIndex = 0; FunctionIndex < FunctionData.Num(); ++FunctionIndex)
	{
		const FAnimNextRigVMFunctionData& Function = FunctionData[FunctionIndex];
		if (Function.FunctionGuid == InFunctionGuid)
		{
			FFunctionHandle Handle;
			Handle.FunctionIndex = FunctionIndex;
#if WITH_EDITOR
			Handle.VMHash = VM->GetVMHash();
#endif
			return Handle;
		}
	}

	return FFunctionHandle();
}

UE::UAF::FFunctionHandle UUAFRigVMAsset::GetFunctionHandle(FName InEventName) const
{
	using namespace UE::UAF;

	for (int32 FunctionIndex = 0; FunctionIndex < FunctionData.Num(); ++FunctionIndex)
	{
		const FAnimNextRigVMFunctionData& Function = FunctionData[FunctionIndex];
		if (Function.EventName == InEventName)
		{
			FFunctionHandle Handle;
			Handle.FunctionIndex = FunctionIndex;
#if WITH_EDITOR
			Handle.VMHash = VM->GetVMHash();
#endif
			return Handle;
		}
	}

	return FFunctionHandle();
}

#if WITH_EDITOR

UUAFRigVMAsset::FOnCompileJobEvent& UUAFRigVMAsset::OnCompileJobStarted()
{
	return UE::UAF::Private::GOnCompileJobStarted;
}

UUAFRigVMAsset::FOnCompileJobEvent& UUAFRigVMAsset::OnCompileJobFinished()
{
	return UE::UAF::Private::GOnCompileJobFinished;
}

#endif
