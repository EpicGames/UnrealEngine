// Copyright Epic Games, Inc. All Rights Reserved.

#include "BindableValue/UAFBindableTypes.h"

#include "UAFAssetInstance.h"
#include "Param/ParamType.h"
#include "PropertyBindingDataView.h"
#include "PropertyBindingPath.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFBindableTypes)

//------------------------------------------------------------------------------
// Private helpers
//------------------------------------------------------------------------------

namespace UE::UAF::Private
{
	/**
	 * Resolves a SubProperty binding into OutMem (slow path).
	 * TargetProp is used only for float<->double numeric coercion; pass nullptr to skip.
	 */
	static bool ResolveSubPropertyToMem(
		const FUAFPropertyBinding& Binding,
		const FProperty*           TargetProp,
		void*                      OutMem,
		FUAFAssetInstance&         Instance)
	{
		const FProperty*       SourceProp      = Binding.SourceVariable.ResolveProperty();
		const FStructProperty* SourceStructProp = SourceProp ? CastField<FStructProperty>(SourceProp) : nullptr;
		if (!SourceStructProp || !SourceStructProp->Struct)
		{
			return false;
		}

		const int32 StructSize = SourceStructProp->Struct->GetStructureSize();
		TArray<uint8, TInlineAllocator<128>> StructBuf;
		StructBuf.SetNumUninitialized(StructSize);
		SourceStructProp->Struct->InitializeStruct(StructBuf.GetData());

		const FAnimNextParamType StructParamType(
			EPropertyBagPropertyType::Struct,
			EPropertyBagContainerType::None,
			SourceStructProp->Struct);

		const EPropertyBagResult GetResult = Instance.GetVariable(
			Binding.SourceVariable,
			StructParamType,
			TArrayView<uint8>(StructBuf.GetData(), StructSize));

		if (GetResult != EPropertyBagResult::Success)
		{
			SourceStructProp->Struct->DestroyStruct(StructBuf.GetData());
			return false;
		}

		TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>> Indirections;
		const FPropertyBindingDataView DataView(SourceStructProp->Struct, StructBuf.GetData());

		if (!Binding.SubPropertyPath.ResolveIndirectionsWithValue(DataView, Indirections) || Indirections.IsEmpty())
		{
			SourceStructProp->Struct->DestroyStruct(StructBuf.GetData());
			return false;
		}

		const FPropertyBindingPathIndirection& Leaf     = Indirections.Last();
		const FProperty*                       LeafProp = Leaf.GetProperty();
		if (!LeafProp || !Leaf.GetPropertyAddress())
		{
			SourceStructProp->Struct->DestroyStruct(StructBuf.GetData());
			return false;
		}

		const FNumericProperty* NumLeaf   = CastField<FNumericProperty>(LeafProp);
		const FNumericProperty* NumTarget = TargetProp ? CastField<FNumericProperty>(TargetProp) : nullptr;
		if (NumLeaf && NumTarget && NumLeaf->IsFloatingPoint() && NumTarget->IsFloatingPoint())
		{
			NumTarget->SetFloatingPointPropertyValue(
				OutMem,
				NumLeaf->GetFloatingPointPropertyValue(Leaf.GetPropertyAddress()));
		}
		else
		{
			LeafProp->CopySingleValue(OutMem, Leaf.GetPropertyAddress());
		}

		SourceStructProp->Struct->DestroyStruct(StructBuf.GetData());
		return true;
	}

	static bool ResolveSubStructPropertyToMem(
		const FUAFPropertyBinding& Binding,
		const UScriptStruct*       ExpectedStruct,
		void*                      OutMem,
		FUAFAssetInstance&         Instance)
	{
		const FProperty*       SourceProp      = Binding.SourceVariable.ResolveProperty();
		const FStructProperty* SourceStructProp = SourceProp ? CastField<FStructProperty>(SourceProp) : nullptr;
		if (!SourceStructProp || !SourceStructProp->Struct)
		{
			return false;
		}

		const int32 StructSize = SourceStructProp->Struct->GetStructureSize();
		TArray<uint8> StructBuf;
		StructBuf.SetNumUninitialized(StructSize);
		SourceStructProp->Struct->InitializeStruct(StructBuf.GetData());

		const FAnimNextParamType SourceParamType(
			EPropertyBagPropertyType::Struct, EPropertyBagContainerType::None,
			SourceStructProp->Struct);

		const EPropertyBagResult GetResult = Instance.GetVariable(
			Binding.SourceVariable, SourceParamType,
			TArrayView<uint8>(StructBuf.GetData(), StructSize));

		if (GetResult != EPropertyBagResult::Success)
		{
			SourceStructProp->Struct->DestroyStruct(StructBuf.GetData());
			return false;
		}

		TArray<FPropertyBindingPathIndirection, TInlineAllocator<16>> Indirections;
		const FPropertyBindingDataView DataView(SourceStructProp->Struct, StructBuf.GetData());

		if (!Binding.SubPropertyPath.ResolveIndirectionsWithValue(DataView, Indirections) || Indirections.IsEmpty())
		{
			SourceStructProp->Struct->DestroyStruct(StructBuf.GetData());
			return false;
		}

		const FPropertyBindingPathIndirection& Leaf          = Indirections.Last();
		const FStructProperty*                 LeafStructProp = CastField<FStructProperty>(Leaf.GetProperty());
		if (!LeafStructProp || !LeafStructProp->Struct || !Leaf.GetPropertyAddress())
		{
			SourceStructProp->Struct->DestroyStruct(StructBuf.GetData());
			return false;
		}

		if (!ensureMsgf(LeafStructProp->Struct == ExpectedStruct,
			TEXT("FBindableStruct SubProperty type mismatch: expected %s, leaf is %s"),
			*ExpectedStruct->GetName(), *LeafStructProp->Struct->GetName()))
		{
			SourceStructProp->Struct->DestroyStruct(StructBuf.GetData());
			return false;
		}

		LeafStructProp->CopySingleValue(OutMem, Leaf.GetPropertyAddress());
		SourceStructProp->Struct->DestroyStruct(StructBuf.GetData());
		return true;
	}

	static bool ResolveVariableToMem(
		const FAnimNextVariableReference& SourceVariable,
		const FAnimNextParamType&          TargetParamType,
		void*                              OutMem,
		int32                              OutMemSize,
		FUAFAssetInstance&                 Instance)
	{
		const EPropertyBagResult Result = Instance.GetVariable(
			SourceVariable,
			TargetParamType,
			TArrayView<uint8>(static_cast<uint8*>(OutMem), OutMemSize));
		return Result == EPropertyBagResult::Success;
	}

	/**
	 * Executes a parameterless function binding and writes the return value into OutMem.
	 * Returns true if successful, false if the function couldn't be called.
	 */
	static bool ResolveFunctionBindingToMem(
		const FUAFPropertyBinding& Binding,
		void*                      OutMem,
		int32                      OutMemSize,
		FUAFAssetInstance&         Instance)
	{
		if (Binding.SourceFunction.IsNone())
		{
			return false;
		}

		return Instance.ExecuteParameterlessFunction(Binding.SourceFunction, OutMem, OutMemSize);
	}

	/**
	 * Shared resolution logic for all simple FBindableXxx types.
	 * Handles Variable and SubProperty binding modes, falling back to ConstantValue.
	 * Scalar types (bool, float, etc.) pass TargetProp for float<->double coercion;
	 * struct types (FVector, FQuat) pass nullptr.
	 */
	template<typename T, typename BindableType>
	static T ResolveBindableValue(const BindableType& Bindable, FUAFAssetInstance* Instance)
	{
		const auto ConstantValue = Bindable.GetConstantValue();

		if (!Bindable.HasBinding() || !Instance)
		{
			return ConstantValue;
		}
		const FUAFPropertyBinding* Binding = Bindable.GetBinding();
		if (!Binding)
		{
			return ConstantValue;
		}

		switch (Binding->SourceType)
		{
		case EUAFBindingSourceType::Variable:
			{
				if (Binding->SourceVariable.IsNone())
				{
					return ConstantValue;
				}
			
				T Out = ConstantValue;
				Instance->GetVariable(Binding->SourceVariable, Out);
				return Out;
			}
		case EUAFBindingSourceType::SubProperty:
			{
				const FProperty* TargetProp = nullptr;
				if constexpr (TIsArithmetic<T>::Value)
				{
					static const FProperty* CachedProp = BindableType::StaticStruct()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(BindableType, ConstantValue));
					TargetProp = CachedProp;
				}
				// Zero-init: the leaf property may be smaller than T (e.g. uint8 sub-property → int32).
				T Out{};
				if (!ResolveSubPropertyToMem(*Binding, TargetProp, &Out, *Instance))
				{
					return ConstantValue;
				}
				return Out;
			}
		case EUAFBindingSourceType::Function:
			{
				if (Binding->SourceFunction.IsNone())
				{
					return ConstantValue;
				}
				// Zero-init: the function return type may be smaller than T (e.g. uint8 enum → int32).
				T Out{};
				if (!ResolveFunctionBindingToMem(*Binding, &Out, sizeof(T), *Instance))
				{
					return ConstantValue;
				}
				return Out;
			}
		default:
			checkNoEntry();	// Unsupported binding type
		}
		
		return ConstantValue;
	}

} // namespace UE::UAF::Private

//------------------------------------------------------------------------------
// Simple bindable type GetValue implementations
//------------------------------------------------------------------------------

bool FBindableBool::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<bool>(*this, Instance);
}

float FBindableFloat::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<float>(*this, Instance);
}

double FBindableDouble::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<double>(*this, Instance);
}

int32 FBindableInt32::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<int32>(*this, Instance);
}

int64 FBindableInt64::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<int64>(*this, Instance);
}

uint8 FBindableByte::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<uint8>(*this, Instance);
}

FName FBindableName::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<FName>(*this, Instance);
}

FVector FBindableVector::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<FVector>(*this, Instance);
}

FQuat FBindableQuat::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<FQuat>(*this, Instance);
}

FTransform FBindableTransform::GetValue(FUAFAssetInstance* Instance) const
{
	return UE::UAF::Private::ResolveBindableValue<FTransform>(*this, Instance);
}

//------------------------------------------------------------------------------
// FBindableEnum
//------------------------------------------------------------------------------

int32 FBindableEnum::GetValue(FUAFAssetInstance* Instance) const
{
	if (!Binding.IsValid() || !Instance)
	{
		return ConstantValue;
	}
	const FUAFPropertyBinding& ResolvedBinding = *Binding;

	switch (ResolvedBinding.SourceType)
	{
	case EUAFBindingSourceType::Variable:
		{
			if (ResolvedBinding.SourceVariable.IsNone())
			{
				return ConstantValue;
			}

			const FAnimNextParamType ParamType = GetEnumClass()
				? FAnimNextParamType(EPropertyBagPropertyType::Enum, EPropertyBagContainerType::None, GetEnumClass())
				: FAnimNextParamType(EPropertyBagPropertyType::Byte);

			// Determine the source enum's actual underlying byte size.
			// FByteProperty (uint8 enum) has ElementSize 1; FEnumProperty wraps a numeric property.
			int32 ReadSize = sizeof(int32);
			const FProperty* SourceProp = ResolvedBinding.SourceVariable.ResolveProperty();
			if (const FEnumProperty* EnumProp = SourceProp ? CastField<FEnumProperty>(SourceProp) : nullptr)
			{
				if (!ensureMsgf(EnumProp->GetElementSize() <= sizeof(int32),
					TEXT("FBindableEnum: enum '%s' has underlying size %u which exceeds int32; binding ignored"),
					*EnumProp->GetName(), static_cast<uint32>(EnumProp->GetElementSize())))
				{
					return ConstantValue;
				}
				ReadSize = static_cast<int32>(EnumProp->GetElementSize());
			}
			else if (const FByteProperty* ByteProp = SourceProp ? CastField<FByteProperty>(SourceProp) : nullptr)
			{
				ReadSize = static_cast<int32>(ByteProp->GetElementSize());
			}

			// Zero-init before writing — the source may be smaller than int32 (e.g. uint8 enum)
			// and we must not leave upper bytes as garbage from ConstantValue.
			int32 Out = 0;
			UE::UAF::Private::ResolveVariableToMem(ResolvedBinding.SourceVariable, ParamType, &Out, ReadSize, *Instance);
			return Out;
		}
	case EUAFBindingSourceType::SubProperty:
		{
			if (ResolvedBinding.SourceVariable.IsNone())
			{
				return ConstantValue;
			}
			int32 Out = 0;
			if (!UE::UAF::Private::ResolveSubPropertyToMem(ResolvedBinding, nullptr, &Out, *Instance))
			{
				return ConstantValue;
			}
			return Out;
		}
	case EUAFBindingSourceType::Function:
		{
			if (ResolvedBinding.SourceFunction.IsNone())
			{
				return ConstantValue;
			}
			int32 Out = 0;
			if (!UE::UAF::Private::ResolveFunctionBindingToMem(ResolvedBinding, &Out, sizeof(int32), *Instance))
			{
				return ConstantValue;
			}
			return Out;
		}
	default:
		checkNoEntry();
	}

	return ConstantValue;
}

//------------------------------------------------------------------------------
// FBindableStruct
//------------------------------------------------------------------------------

void FBindableStruct::GetValueToMem(
	FUAFAssetInstance*   Instance,
	const UScriptStruct* ExpectedStruct,
	void*                OutMem) const
{
	// No binding or null instance: copy ConstantValue
	if (!Binding.IsValid() || !Instance)
	{
		if (ensureMsgf(!ConstantValue.IsValid() || ConstantValue.GetScriptStruct() == ExpectedStruct,
			TEXT("FBindableStruct::GetValue type mismatch: expected %s, ConstantValue is %s"),
			*ExpectedStruct->GetName(),
			ConstantValue.IsValid() ? *ConstantValue.GetScriptStruct()->GetName() : TEXT("(empty)")))
		{
			if (ConstantValue.IsValid())
			{
				ExpectedStruct->CopyScriptStruct(OutMem, ConstantValue.GetMemory());
			}
			else
			{
				ExpectedStruct->InitializeStruct(OutMem);
			}
		}
		return;
	}

	const FUAFPropertyBinding& ResolvedBinding = *Binding;

	// Helper: fall back to ConstantValue or zero-init
	auto CopyConstantOrInit = [this, ExpectedStruct, OutMem]()
	{
		if (ConstantValue.IsValid())
		{
			ExpectedStruct->CopyScriptStruct(OutMem, ConstantValue.GetMemory());
		}
		else
		{
			ExpectedStruct->InitializeStruct(OutMem);
		}
	};

	switch (ResolvedBinding.SourceType)
	{
	case EUAFBindingSourceType::Variable:
		{
			if (ResolvedBinding.SourceVariable.IsNone())
			{
				CopyConstantOrInit();
				return;
			}

			const FProperty*       SourceProp      = ResolvedBinding.SourceVariable.ResolveProperty();
			const FStructProperty* SourceStructProp = SourceProp ? CastField<FStructProperty>(SourceProp) : nullptr;
			if (!SourceStructProp || !SourceStructProp->Struct)
			{
				return;
			}

			ensureMsgf(SourceStructProp->Struct == ExpectedStruct,
				TEXT("FBindableStruct::GetValue type mismatch: expected %s, source variable is %s"),
				*ExpectedStruct->GetName(), *SourceStructProp->Struct->GetName());

			const FAnimNextParamType ParamType(
				EPropertyBagPropertyType::Struct, EPropertyBagContainerType::None,
				SourceStructProp->Struct);

			Instance->GetVariable(
				ResolvedBinding.SourceVariable, ParamType,
				TArrayView<uint8>(static_cast<uint8*>(OutMem), ExpectedStruct->GetStructureSize()));
			return;
		}
	case EUAFBindingSourceType::SubProperty:
		{
			if (ResolvedBinding.SourceVariable.IsNone())
			{
				CopyConstantOrInit();
				return;
			}
			UE::UAF::Private::ResolveSubStructPropertyToMem(ResolvedBinding, ExpectedStruct, OutMem, *Instance);
			return;
		}
	case EUAFBindingSourceType::Function:
		{
			if (!ResolvedBinding.SourceFunction.IsNone() &&
				UE::UAF::Private::ResolveFunctionBindingToMem(ResolvedBinding, OutMem, ExpectedStruct->GetStructureSize(), *Instance))
			{
				return;
			}
			CopyConstantOrInit();
			return;
		}
	default:
		checkNoEntry();
	}
}

//------------------------------------------------------------------------------
// FBindableObject
//------------------------------------------------------------------------------

UObject* FBindableObject::GetValue(FUAFAssetInstance* Instance) const
{
	if (!HasBinding() || !Instance)
	{
		return ConstantValue;
	}
	const FUAFPropertyBinding* OptBinding = GetBinding();
	if (!OptBinding)
	{
		return ConstantValue;
	}

	switch (OptBinding->SourceType)
	{
	case EUAFBindingSourceType::Variable:
		{
			if (OptBinding->SourceVariable.IsNone())
			{
				return ConstantValue;
			}
			const UClass* ResolvedClass = ObjectClass ? ObjectClass.Get() : UObject::StaticClass();
			const FAnimNextParamType ParamType(
				EPropertyBagPropertyType::Object,
				EPropertyBagContainerType::None,
				ResolvedClass);
			TObjectPtr<UObject> Out = ConstantValue;
			UE::UAF::Private::ResolveVariableToMem(
				OptBinding->SourceVariable, ParamType,
				&Out, sizeof(TObjectPtr<UObject>), *Instance);
			return Out;
		}
	case EUAFBindingSourceType::SubProperty:
		{
			if (OptBinding->SourceVariable.IsNone())
			{
				return ConstantValue;
			}
			TObjectPtr<UObject> Out = ConstantValue;
			UE::UAF::Private::ResolveSubPropertyToMem(
				*OptBinding, nullptr, &Out, *Instance);
			return Out;
		}
	case EUAFBindingSourceType::Function:
		{
			if (OptBinding->SourceFunction.IsNone())
			{
				return ConstantValue;
			}
			TObjectPtr<UObject> Out = ConstantValue;
			UE::UAF::Private::ResolveFunctionBindingToMem(
				*OptBinding, &Out, sizeof(TObjectPtr<UObject>), *Instance);
			return Out;
		}
	default:
		checkNoEntry();
	}

	return ConstantValue;
}

//------------------------------------------------------------------------------
// FBindableValueBase — custom serialization
//------------------------------------------------------------------------------

bool FBindableValueBase::Serialize(FArchive& Ar)
{
	bool bHasBinding = Binding.IsValid();
	Ar << bHasBinding;

	if (bHasBinding)
	{
		if (Ar.IsLoading())
		{
			Binding = MakeUnique<FUAFPropertyBinding>();
		}
		FUAFPropertyBinding::StaticStruct()->SerializeItem(Ar, Binding.Get(), nullptr);
	}
	else if (Ar.IsLoading())
	{
		Binding.Reset();
	}

	return false; // returning false tells UE to continue with default property serialization for remaining members
}

//------------------------------------------------------------------------------
// FBindableValueBase — text export/import helpers
//------------------------------------------------------------------------------

static const TCHAR* BindingPropertyName = TEXT("__Binding");

// IMPORTANT: __Binding must ALWAYS be exported, even when no binding is set.
//
// The RigVM controller's SetPinDefaultValue (RigVMController.cpp) compares the new text
// against the pin's stored default value. For FBindable types with native import/export,
// the pin has no sub-pins and the comparison happens on the full struct text. When the
// editor customization modifies a binding (via direct raw data mutation + NotifyPostChange),
// the notification chain exports the struct to text and calls SetPinDefaultValue. The
// exported text must DIFFER from the pin's stored text so the controller detects the change:
//     if(InPin->GetDefaultValue() != ClampedDefaultValue)  // RigVMController.cpp
//
// Without __Binding=(), clearing a binding produces the same text as the stored (pre-binding)
// value, and the change is silently lost -- the pin default is never updated.
//
// DO NOT remove this always-export behavior without updating the editor binding customization
// (UAFBindableValueCustomization.cpp) to use a different change-detection mechanism.
void FBindableValueBase::ExportBindingText(FString& ValueStr, const FUAFPropertyBinding* InBinding, int32 PortFlags)
{
	FString BindingText;
	if (InBinding)
	{
		FUAFPropertyBinding::StaticStruct()->ExportText(BindingText, InBinding, nullptr, nullptr, PortFlags, nullptr, true);
	}
	else
	{
		BindingText = TEXT("()");
	}

	// Insert __Binding=(...) or __Binding=() before the closing )
	if (ValueStr.Len() > 0 && ValueStr[ValueStr.Len() - 1] == TCHAR(')'))
	{
		// Only add comma separator if there are existing properties between the parens (more than just "()")
		const bool bHasExistingContent = ValueStr.Len() > 2;
		ValueStr.InsertAt(ValueStr.Len() - 1, FString::Printf(TEXT("%s%s=%s"),
			bHasExistingContent ? TEXT(",") : TEXT(""),
			BindingPropertyName, *BindingText));
	}
}

/** Creates a copy of THIS struct's text (up to matching ')') with __Binding=(...) stripped out.
 *  The Buffer may contain text for sibling properties — only the first (...) is copied. */
static FString RemoveBindingFromText(const TCHAR* Buffer)
{
	// Find the end of this struct's text (matching closing paren)
	int32 StructLen = 0;
	if (*Buffer == TCHAR('('))
	{
		int32 Depth = 0;
		while (Buffer[StructLen])
		{
			if (Buffer[StructLen] == TCHAR('('))
			{
				Depth++;
			}
			else if (Buffer[StructLen] == TCHAR(')'))
			{
				if (--Depth <= 0)
				{
					StructLen++;
					break;
				}
			}
			StructLen++;
		}
	}
	else
	{
		StructLen = FCString::Strlen(Buffer);
	}

	// Copy only this struct's text
	FString Text(StructLen, Buffer);
	int32 BindingIdx = Text.Find(BindingPropertyName);
	if (BindingIdx == INDEX_NONE)
	{
		return Text;
	}

	// Include leading comma in removal
	int32 Start = BindingIdx;
	if (Start > 0 && Text[Start - 1] == TCHAR(','))
	{
		Start--;
	}

	// Find end by matching nested parens after __Binding=
	int32 End = BindingIdx + FCString::Strlen(BindingPropertyName);
	if (End < Text.Len() && Text[End] == TCHAR('='))
	{
		End++;
		int32 Depth = 0;
		while (End < Text.Len())
		{
			const TCHAR Ch = Text[End];
			if (Ch == TCHAR('('))
			{
				Depth++;
			}
			else if (Ch == TCHAR(')'))
			{
				if (--Depth <= 0)
				{
					End++;
					break;
				}
			}
			End++;
		}
	}
	Text.RemoveAt(Start, End - Start);
	return Text;
}

/** Advances Buffer past the outermost (...) struct text. */
static void AdvancePastStructText(const TCHAR*& Buffer)
{
	int32 Depth = 0;
	while (*Buffer)
	{
		if (*Buffer == TCHAR('('))
		{
			Depth++;
		}
		else if (*Buffer == TCHAR(')'))
		{
			if (--Depth <= 0)
			{
				Buffer++;
				return;
			}
		}
		Buffer++;
	}
}

bool FBindableValueBase::ImportBindingText(const TCHAR*& Buffer, TUniquePtr<FUAFPropertyBinding>& OutBinding, int32 PortFlags)
{
	// Find the end of THIS struct's text (the matching ')' for the opening '(').
	// The Buffer may contain text for subsequent sibling properties in a parent struct,
	// so we must NOT search beyond our own closing paren — otherwise we'd pick up
	// a __Binding from a later sibling.
	const TCHAR* SearchEnd = Buffer;
	if (*SearchEnd == TCHAR('('))
	{
		int32 Depth = 0;
		while (*SearchEnd)
		{
			if (*SearchEnd == TCHAR('('))
			{
				Depth++;
			}
			else if (*SearchEnd == TCHAR(')'))
			{
				if (--Depth <= 0)
				{
					SearchEnd++; // include the closing ')'
					break;
				}
			}
			SearchEnd++;
		}
	}

	// Search for __Binding only within this struct's text
	const TCHAR* BindingStart = nullptr;
	const int32 BindingNameLen = FCString::Strlen(BindingPropertyName);
	for (const TCHAR* Scan = Buffer; Scan + BindingNameLen <= SearchEnd; Scan++)
	{
		if (FCString::Strncmp(Scan, BindingPropertyName, BindingNameLen) == 0)
		{
			BindingStart = Scan;
			break;
		}
	}

	if (BindingStart)
	{
		// Advance past "__Binding="
		const TCHAR* ValueStart = BindingStart + BindingNameLen;
		if (*ValueStart == TCHAR('='))
		{
			ValueStart++;
			// __Binding=() is the empty marker emitted by ExportBindingText when no binding is set.
			// Treat it as no-binding rather than importing a default-constructed FUAFPropertyBinding.
			if (ValueStart[0] == TCHAR('(') && ValueStart[1] == TCHAR(')'))
			{
				OutBinding.Reset();
			}
			else
			{
				OutBinding = MakeUnique<FUAFPropertyBinding>();
				FUAFPropertyBinding::StaticStruct()->ImportText(ValueStart,
					OutBinding.Get(),
					nullptr,
					PortFlags,
					GLog,
					TEXT("FUAFPropertyBinding"),
					/*bAllowNativeOverride=*/false);
			}
		}
	}
	else
	{
		OutBinding.Reset();
	}
	return true;
}

bool FBindableValueBase::ExportTextItem(FString& ValueStr, const FBindableValueBase& DefaultValue,
	UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	// Pass `this` as both Value and Defaults to force ALL UPROPERTYs to be exported.
	// FProperty::ExportText_Direct unconditionally exports when Data==Delta (pointer equality).
	// This avoids two issues:
	// 1. The self-comparison issue when callers pass Data as both value and default
	//    (e.g. FRigVMMemoryStorageStruct::GetDataAsString).
	// 2. Properties whose value happens to match a default-constructed instance being omitted
	//    (e.g. FBindableEnum::ConstantValue == 0 matching the default int32 value).
	FBindableValueBase::StaticStruct()->ExportText(ValueStr,
		this,
		this,
		Parent,
		PortFlags,
		ExportRootScope,
		/*bAllowNativeOverride=*/false);
	ExportBindingText(ValueStr, Binding.Get(), PortFlags);
	return true;
}

bool FBindableValueBase::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags,
	UObject* Parent, FOutputDevice* ErrorText)
{
	ImportBindingText(Buffer, Binding, PortFlags);
	// Import from a clean copy without __Binding to avoid "Unknown property" warnings
	// (the RigVM pin validation treats any import warning as an error and rejects the value).
	FString CleanText = RemoveBindingFromText(Buffer);
	const TCHAR* CleanPtr = *CleanText;
	const TCHAR* Result = FBindableValueBase::StaticStruct()->ImportText(CleanPtr, this, Parent, PortFlags, ErrorText,
		TEXT("FBindableValueBase"), /*bAllowNativeOverride=*/false);
	if (Result)
	{
		AdvancePastStructText(Buffer);
	}
	return Result != nullptr;
}

//------------------------------------------------------------------------------
// Per-type ExportTextItem/ImportTextItem implementations
//------------------------------------------------------------------------------

#define IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(StructType) \
	bool StructType::ExportTextItem(FString& ValueStr, const StructType& DefaultValue, \
		UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const \
	{ \
		StructType::StaticStruct()->ExportText(ValueStr, this, this, Parent, PortFlags, ExportRootScope, /*bAllowNativeOverride=*/false); \
		ExportBindingText(ValueStr, Binding.Get(), PortFlags); \
		return true; \
	} \
	bool StructType::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, \
		UObject* Parent, FOutputDevice* ErrorText) \
	{ \
		ImportBindingText(Buffer, Binding, PortFlags); \
		FString CleanText = RemoveBindingFromText(Buffer); \
		const TCHAR* CleanPtr = *CleanText; \
		const TCHAR* Result = StructType::StaticStruct()->ImportText(CleanPtr, this, Parent, PortFlags, ErrorText, \
			TEXT(#StructType), /*bAllowNativeOverride=*/false); \
		if (Result) \
		{ \
			AdvancePastStructText(Buffer); \
		} \
		return Result != nullptr; \
	}

IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableBool)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableFloat)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableDouble)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableInt32)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableInt64)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableByte)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableName)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableVector)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableQuat)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableTransform)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableEnum)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableStruct)
IMPLEMENT_BINDABLE_TEXT_SERIALIZATION(FBindableObject)

#undef IMPLEMENT_BINDABLE_TEXT_SERIALIZATION

//------------------------------------------------------------------------------
// Per-type Identical implementations (WithIdentical)
//
// Delta serialization compares each struct property against defaults to decide
// whether to write a property tag. Without WithIdentical, the comparison only
// checks UPROPERTYs and misses the non-UPROPERTY Binding. When ConstantValue
// is at its default, the struct appears "identical" and the property tag is
// skipped -- silently dropping the binding on save.
//------------------------------------------------------------------------------

bool FBindableValueBase::BindingsIdentical(const FBindableValueBase* Other, const UScriptStruct* ConcreteStruct, uint32 PortFlags) const
{
	if (!Other)
	{
		return false;
	}

	// Compare binding presence
	if (HasBinding() != Other->HasBinding())
	{
		return false;
	}

	// Compare binding contents
	if (HasBinding())
	{
		if (!FUAFPropertyBinding::StaticStruct()->CompareScriptStruct(Binding.Get(), Other->Binding.Get(), PortFlags))
		{
			return false;
		}
	}

	// Compare all UPROPERTYs (ConstantValue, EnumClass, StructClass, etc.)
	for (TFieldIterator<FProperty> It(ConcreteStruct); It; ++It)
	{
		if (!It->Identical_InContainer(this, Other, 0, PortFlags))
		{
			return false;
		}
	}

	return true;
}

#define IMPLEMENT_BINDABLE_IDENTICAL(StructType) \
	bool StructType::Identical(const StructType* Other, uint32 PortFlags) const \
	{ \
		return BindingsIdentical(Other, StructType::StaticStruct(), PortFlags); \
	}

IMPLEMENT_BINDABLE_IDENTICAL(FBindableBool)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableFloat)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableDouble)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableInt32)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableInt64)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableByte)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableName)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableVector)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableQuat)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableTransform)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableEnum)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableStruct)
IMPLEMENT_BINDABLE_IDENTICAL(FBindableObject)

#undef IMPLEMENT_BINDABLE_IDENTICAL
