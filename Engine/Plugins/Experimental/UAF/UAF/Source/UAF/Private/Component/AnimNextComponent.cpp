// Copyright Epic Games, Inc. All Rights Reserved.

#include "Component/AnimNextComponent.h"

#include "Blueprint/BlueprintExceptionInfo.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/World.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Graph/RigUnit_AnimNextWriteSkeletalMeshComponentPose.h"
#include "Graph/UAFSystemOutputComponent.h"
#include "Module/ModuleTaskContext.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Variables/AnimNextVariableReference.h"
#include "Module/AnimNextModule.h"
#include "Module/SystemDependency.h"
#include "Module/UAFSystemAssetData.h"
#include "Module/UAFWeakSystemReference.h"
#include "UAF/UAFAssetFactory.h"
#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAF/ValueRuntime/VirtualValueBundle_ComponentOutput.h"
#include "UAF/ValueRuntime/VirtualValueBundle_SystemOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextComponent)

void UUAFComponent::OnRegister()
{
	using namespace UE::UAF;

	Super::OnRegister();

	if (AssetData.IsValid() && AssetData.Get().Validate())
	{
		AllocateSystem();

		SetupInputOutput();
		
		SetupCharacterMovementDependency();

#if WITH_EDITOR
		SystemReference.RunInitialTickInEditor();
#endif

#if UE_ENABLE_DEBUG_DRAWING
		SystemReference.ShowDebugDrawing(bShowDebugDrawing);
#endif
	}
}

void UUAFComponent::AllocateSystem()
{
	using namespace UE::UAF;

	if(AssetData.IsValid() && AssetData.Get().Validate() && !SystemReference.IsValid())
	{
		SystemReference = FSystemReference(AssetData, this, InitMethod);
	}
}

void UUAFComponent::OnUnregister()
{
	Super::OnUnregister();

	ReleaseCharacterMovementDependency();

	SystemReference.Reset();
}

void UUAFComponent::BeginPlay()
{
	Super::BeginPlay();

	UWorld* World = GetWorld();
	check(World);

	if (InitMethod == EAnimNextModuleInitMethod::InitializeAndRun
	|| (InitMethod == EAnimNextModuleInitMethod::InitializeAndPauseInEditor && World->WorldType != EWorldType::Editor && World->WorldType != EWorldType::EditorPreview))
	{
		Activate();
	}
}

void UUAFComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

	Deactivate();
}

void UUAFComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::UAFProceduralSystems)
		{
			if (Module_DEPRECATED != nullptr)
			{
				AssetData.InitializeAs<FUAFSystemFactoryAsset_System>(Module_DEPRECATED);
				Module_DEPRECATED = nullptr;
			}
		}
		else if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::UAFAssetData)
		{
			if (Asset_DEPRECATED != nullptr)
			{
				AssetData = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFSystemFactoryAsset>(Asset_DEPRECATED);
				Asset_DEPRECATED = nullptr;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
}

UE::UAF::ISystemOutputAdapter::FInputData UUAFComponent::GetInputData() const
{
	return UE::UAF::ISystemOutputAdapter::FInputData(CachedSkeletalMeshComponent.Get());
}

void UUAFComponent::SignalOutputWritten(const UE::UAF::FModuleTaskContext& InContext)
{
	if (SkeletalMeshComponentOutputMode == EUAFSkeletalMeshComponentOutputMode::WriteToSkeletalMeshComponentPose)
	{
		InContext.AccessComponent<FUAFSystemOutputComponent>([this](FUAFSystemOutputComponent& InOutputPoseComponent)
		{
			InOutputPoseComponent.GenerateRenderData(SerialNumber, [this](uint32 InSerialNumber, const FAnimNextGraphLODPose& InPose, const FAnimNextGraphReferencePose& InRefPose)
			{
				SerialNumber = InSerialNumber;

				// TODO: for now, hook the legacy output pose path. this needs a refactor of skel mesh component to work better
				FRigUnit_AnimNextWriteSkeletalMeshComponentPose::WritePose(CachedSkeletalMeshComponent.Get(), InPose);
			});
		});
	}
}

void UUAFComponent::BlueprintSetVariable(FName Name, int32 Value)
{
	checkNoEntry();
}

DEFINE_FUNCTION(UUAFComponent::execBlueprintSetVariable)
{
	using namespace UE::UAF;

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;

	P_GET_PROPERTY(FNameProperty, Name);

	Stack.StepCompiledIn<FProperty>(nullptr);
	const FProperty* ValueProp = CastField<FProperty>(Stack.MostRecentProperty);
	const void* ContainerPtr = Stack.MostRecentPropertyContainer;

	P_FINISH;

	if (!ValueProp || !ContainerPtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			NSLOCTEXT("UAFComponent", "UAFComponent_SetVariableError", "Failed to resolve the Value for Set Variable")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}

	if (Name == NAME_None)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::NonFatalError,
			NSLOCTEXT("UAFComponent", "UAFComponent_SetVariableInvalidNameWarning", "Invalid variable name supplied to Set Variable")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}

	EPropertyBagResult Result = EPropertyBagResult::Success;

	P_NATIVE_BEGIN;

	{
		FAnimNextParamType Type = FAnimNextParamType::FromProperty(ValueProp);
		const uint8* ValuePtr = ValueProp->ContainerPtrToValuePtr<uint8>(ContainerPtr);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Result = P_THIS->SetVariableInternal(FAnimNextVariableReference(Name), Type, TConstArrayView<uint8>(ValuePtr, ValueProp->GetElementSize()));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	P_NATIVE_END;

	switch (Result)
	{
	case EPropertyBagResult::Success:
		break;
	case EPropertyBagResult::TypeMismatch:
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_SetVariableTypeMismatch", "Incompatible type supplied for variable '{0}'"), FText::FromName(Name))
			);

			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			break;
		}
	case EPropertyBagResult::PropertyNotFound:
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_SetVariableNotFoundWarning", "Unknown variable name '{0}' supplied to Set Variable"), FText::FromName(Name))
			);

			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			break;
		}
	case EPropertyBagResult::OutOfBounds:
	case EPropertyBagResult::DuplicatedValue:
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_SetVariableUnknownWarning", "Unexpected internal error when calling Set Variable for '{0}'"), FText::FromName(Name))
			);

			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			break;
		}
	}
}

void UUAFComponent::BlueprintSetVariableReference(const FAnimNextVariableReference& Variable, const int32& Value)
{
	checkNoEntry();
}

void UUAFComponent::BlueprintGetVariableReference(const FAnimNextVariableReference& Variable, int32& Value)
{
	checkNoEntry();
}

DEFINE_FUNCTION(UUAFComponent::execBlueprintSetVariableReference)
{
	using namespace UE::UAF;

	P_GET_STRUCT_REF(FAnimNextVariableReference, Variable);

	const FProperty* VariableProperty = Variable.ResolveProperty();
	if (VariableProperty == nullptr)
	{
		P_FINISH;

		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_SetVariablePropertyError", "Failed to resolve the property {0} of variable reference"), FText::FromName(Variable.GetName()))
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}

	if (Variable.GetName() == NAME_None || Variable.GetObject() == nullptr)
	{
		P_FINISH;

		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::NonFatalError,
			NSLOCTEXT("UAFComponent", "UAFComponent_SetVariableInvalidWarning", "Invalid variable supplied to Set Variable")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;

	EPropertyBagResult Result = EPropertyBagResult::Success;
	{
		uint8* TempStorage = (uint8*)FMemory_Alloca_Aligned(VariableProperty->GetElementSize(), VariableProperty->GetMinAlignment());
		VariableProperty->InitializeValue(TempStorage);

		Stack.StepCompiledIn(TempStorage, VariableProperty->GetClass());

		P_FINISH;
		
		P_NATIVE_BEGIN;

		{
			FAnimNextParamType Type = FAnimNextParamType::FromProperty(VariableProperty);
			Result = P_THIS->SetVariableInternal(Variable, Type, TConstArrayView<uint8>(TempStorage, VariableProperty->GetElementSize()));
		}

		P_NATIVE_END;

		VariableProperty->DestroyValue(TempStorage);
	}

	switch (Result)
	{
	case EPropertyBagResult::Success:
		break;
	case EPropertyBagResult::TypeMismatch:
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_SetVariableTypeMismatch", "Incompatible type supplied for variable '{0}'"), FText::FromName(Variable.GetName()))
			);

			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			break;
		}
	case EPropertyBagResult::PropertyNotFound:
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_SetVariableNotFoundWarning", "Unknown variable name '{0}' supplied to Set Variable"), FText::FromName(Variable.GetName()))
			);

			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			break;
		}
	case EPropertyBagResult::OutOfBounds:
	case EPropertyBagResult::DuplicatedValue:
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_SetVariableUnknownWarning", "Unexpected internal error when calling Set Variable for '{0}'"), FText::FromName(Variable.GetName()))
			);

			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			break;
		}
	}
}

DEFINE_FUNCTION(UUAFComponent::execBlueprintGetVariableReference)
{
	using namespace UE::UAF;

	P_GET_STRUCT_REF(FAnimNextVariableReference, Variable);

	const FProperty* VariableProperty = Variable.ResolveProperty();
	if (VariableProperty == nullptr)
	{
		P_FINISH;

		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_GetVariablePropertyError", "Failed to resolve the property {0} of variable reference"), FText::FromName(Variable.GetName()))
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}
	
	if (Variable.GetName() == NAME_None || Variable.GetObject() == nullptr)
	{
		P_FINISH;

		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::NonFatalError,
			NSLOCTEXT("UAFComponent", "UAFComponent_GetVariableInvalidWarning", "Invalid variable supplied to Get Variable")
		);

		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}

	// Read wildcard Value input.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FProperty>(nullptr);

	const FProperty* TargetProperty = Stack.MostRecentProperty;
	void* TargetPtr = Stack.MostRecentPropertyAddress;

	P_FINISH;
	
	if (!TargetProperty || !TargetPtr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			NSLOCTEXT("UAFComponent", "UAFComponent_GetVariableInvalidValueWarning", "Failed to resolve the value for Get Variable")
		);
		
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}

	if (!TargetProperty->SameType(VariableProperty))
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AbortExecution,
			NSLOCTEXT("UAFComponent", "UAFComponent_GetVariablePropertyTypeMismatchWarning", "Pin property type does not match variable property type")
		);
		
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		return;
	}

	EPropertyBagResult Result = EPropertyBagResult::Success;
	
	P_NATIVE_BEGIN;
	{
		FAnimNextParamType Type = FAnimNextParamType::FromProperty(VariableProperty);

		Result = P_THIS->ReadVariableInternal(Variable, Type, [TargetProperty, TargetPtr](TConstArrayView<uint8> VariableValue)
			{
				check(TargetProperty->GetElementSize() == VariableValue.Num());
				TargetProperty->CopyCompleteValue(TargetPtr, VariableValue.GetData());
			});
	}
	P_NATIVE_END;

	switch (Result)
	{
	case EPropertyBagResult::Success:
		{
			break;
		}
	case EPropertyBagResult::TypeMismatch:
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_GetVariableTypeMismatch", "Incompatible type supplied for variable '{0}'"), FText::FromName(Variable.GetName()))
			);

			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			break;
		}
	case EPropertyBagResult::PropertyNotFound:
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_GetVariableNotFoundWarning", "Unknown variable name '{0}' supplied to Get Variable"), FText::FromName(Variable.GetName()))
			);

			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			break;
		}
	case EPropertyBagResult::OutOfBounds:
	case EPropertyBagResult::DuplicatedValue:
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::NonFatalError,
				FText::Format(NSLOCTEXT("UAFComponent", "UAFComponent_GetVariableUnknownWarning", "Unexpected internal error when calling Get Variable for '{0}'"), FText::FromName(Variable.GetName()))
			);

			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
			break;
		}
	}
}

EPropertyBagResult UUAFComponent::SetVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue)
{
	return SystemReference.SetVariable(InVariable, InType, InNewValue);
}

EPropertyBagResult UUAFComponent::GetVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8> OutNewValue)
{
	const FProperty* Property = InVariable.ResolveProperty();
	if (Property)
	{
		return SystemReference.ReadVariable(InVariable, InType, [&](TConstArrayView<uint8> VariableValue)
		{
			check(OutNewValue.Num() == VariableValue.Num());
			Property->CopyCompleteValue(OutNewValue.GetData(), VariableValue.GetData());
		});
	}
	return EPropertyBagResult::PropertyNotFound;
}

EPropertyBagResult UUAFComponent::WriteVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>)> InFunction)
{
	return SystemReference.WriteVariable(InVariable, InType, InFunction);
}

EPropertyBagResult UUAFComponent::ReadVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TConstArrayView<uint8>)> InFunction)
{
	return SystemReference.ReadVariable(InVariable, InType, InFunction);
}

void UUAFComponent::Activate(bool bReset)
{
	if (bReset || ShouldActivate())
	{
		SystemReference.SetEnabled(true);
		SetActiveFlag(true);
		OnComponentActivated.Broadcast(this, bReset);
	}
}

void UUAFComponent::Deactivate()
{
	if (!ShouldActivate())
	{
		SystemReference.SetEnabled(false);
		SetActiveFlag(false);
		OnComponentDeactivated.Broadcast(this);
	}
}

void UUAFComponent::ShowDebugDrawing(bool bInShowDebugDrawing)
{
#if UE_ENABLE_DEBUG_DRAWING
	bShowDebugDrawing = bInShowDebugDrawing;
	SystemReference.ShowDebugDrawing(bInShowDebugDrawing);
#endif
}

void UUAFComponent::QueueTask(FName InModuleEventName, TUniqueFunction<void(const UE::UAF::FModuleTaskContext&)>&& InTaskFunction, UE::UAF::ETaskRunLocation InLocation)
{
	using namespace UE::UAF;

	SystemReference.QueueTask(InModuleEventName, MoveTemp(InTaskFunction), InLocation);
}

void UUAFComponent::QueueInputTraitEvent(FAnimNextTraitEventPtr Event)
{
	using namespace UE::UAF;

	SystemReference.QueueInputTraitEvent(Event);
}

const FTickFunction* UUAFComponent::FindTickFunction(FName InEventName) const
{
	return SystemReference.FindTickFunction(InEventName);
}

void UUAFComponent::AddPrerequisite(UObject* InObject, FTickFunction& InTickFunction, FName InEventName)
{
	SystemReference.AddDependency(InObject, InTickFunction, InEventName, UE::UAF::ESystemDependency::Prerequisite);
}

void UUAFComponent::AddComponentPrerequisite(UActorComponent* InComponent, FName InEventName)
{
	if (InComponent)
	{
		AddPrerequisite(InComponent, InComponent->PrimaryComponentTick, InEventName);
	}
}

void UUAFComponent::AddSubsequent(UObject* InObject, FTickFunction& InTickFunction, FName InEventName)
{
	SystemReference.AddDependency(InObject, InTickFunction, InEventName, UE::UAF::ESystemDependency::Subsequent);
}

void UUAFComponent::AddComponentSubsequent(UActorComponent* InComponent, FName InEventName)
{
	if (InComponent)
	{
		AddSubsequent(InComponent, InComponent->PrimaryComponentTick, InEventName);
	}
}

void UUAFComponent::RemovePrerequisite(UObject* InObject, FTickFunction& InTickFunction, FName InEventName)
{
	SystemReference.RemoveDependency(InObject, InTickFunction, InEventName, UE::UAF::ESystemDependency::Prerequisite);
}

void UUAFComponent::RemoveComponentPrerequisite(UActorComponent* InComponent, FName InEventName)
{
	if (InComponent)
	{
		RemovePrerequisite(InComponent, InComponent->PrimaryComponentTick, InEventName);
	}
}

void UUAFComponent::RemoveSubsequent(UObject* InObject, FTickFunction& InTickFunction, FName InEventName)
{
	SystemReference.RemoveDependency(InObject, InTickFunction, InEventName, UE::UAF::ESystemDependency::Subsequent);
}

void UUAFComponent::RemoveComponentSubsequent(UActorComponent* InComponent, FName InEventName)
{
	if (InComponent)
	{
		RemoveSubsequent(InComponent, InComponent->PrimaryComponentTick, InEventName);
	}
}

void UUAFComponent::AddModuleEventPrerequisite(FName InEventName, UUAFComponent* OtherAnimNextComponent, FName OtherEventName)
{
	if (!OtherAnimNextComponent)
	{
		UE_LOGF(LogAnimation, Warning, "UAFComponent::AddModuleEventPrerequisite called with null OtherAnimNextComponent");
	}
	else if (OtherAnimNextComponent == this)
	{
		UE_LOGF(LogAnimation, Warning, "UAFComponent::AddModuleEventPrerequisite called using the same component");
	}
	else
	{
		SystemReference.AddSystemEventDependency(InEventName, OtherAnimNextComponent->GetSystemReference(), OtherEventName, UE::UAF::ESystemDependency::Prerequisite);
	}
}

void UUAFComponent::AddModuleEventSubsequent(FName InEventName, UUAFComponent* OtherAnimNextComponent, FName OtherEventName)
{
	if (!OtherAnimNextComponent)
	{
		UE_LOGF(LogAnimation, Warning, "UAFComponent::AddModuleEventSubsequent called with null OtherAnimNextComponent");
	}
	else if (OtherAnimNextComponent == this)
	{
		UE_LOGF(LogAnimation, Warning, "UAFComponent::AddModuleEventSubsequent called using the same component");
	}
	else
	{
		SystemReference.AddSystemEventDependency(InEventName, OtherAnimNextComponent->GetSystemReference(), OtherEventName, UE::UAF::ESystemDependency::Subsequent);
	}
}

void UUAFComponent::RemoveModuleEventPrerequisite(FName InEventName, UUAFComponent* OtherAnimNextComponent, FName OtherEventName)
{
	if (!OtherAnimNextComponent)
	{
		UE_LOGF(LogAnimation, Warning, "UAFComponent::RemoveModuleEventPrerequisite called with null OtherAnimNextComponent");
	}
	else if (OtherAnimNextComponent == this)
	{
		UE_LOGF(LogAnimation, Warning, "UAFComponent::RemoveModuleEventPrerequisite called using the same component");
	}
	else
	{
		SystemReference.RemoveSystemEventDependency(InEventName, OtherAnimNextComponent->GetSystemReference(), OtherEventName, UE::UAF::ESystemDependency::Prerequisite);
	}
}

void UUAFComponent::RemoveModuleEventSubsequent(FName InEventName, UUAFComponent* OtherAnimNextComponent, FName OtherEventName)
{
	if (!OtherAnimNextComponent)
	{
		UE_LOGF(LogAnimation, Warning, "UAFComponent::RemoveModuleEventSubsequent called with null OtherAnimNextComponent");
	}
	else if (OtherAnimNextComponent == this)
	{
		UE_LOGF(LogAnimation, Warning, "UAFComponent::RemoveModuleEventSubsequent called using the same component");
	}
	else
	{
		SystemReference.RemoveSystemEventDependency(InEventName, OtherAnimNextComponent->GetSystemReference(), OtherEventName, UE::UAF::ESystemDependency::Subsequent);
	}
}

FUAFWeakSystemReference UUAFComponent::GetSystemReference() const
{
	return FUAFWeakSystemReference(SystemReference);
}

void UUAFComponent::SetAsset(TInstancedStruct<FUAFSystemFactoryAsset>&& InAssetData)
{
	if (ensure(!IsRegistered()))
	{
		SetAssetInternal(MoveTemp(InAssetData));
	}
}

namespace UE::UAF::Private
{
struct FSystemInputBinding
{
	FAnimNextVariableReference Variable;
	TWeakObjectPtr<UUAFComponent> Component;
};

struct FComponentInputBinding
{
	FAnimNextVariableReference Variable;
	TWeakObjectPtr<USkeletalMeshComponent> Component;
};
} // namespace UE::UAF::Private

void UUAFComponent::BlueprintSetInputBinding(const FAnimNextVariableReference& InVariable, UActorComponent* InComponent)
{
	using namespace UE::UAF::Private;

	check(IsInGameThread());
	AActor* Owner = GetOwner();
	check(Owner);
	
	if (InVariable.IsValid())
	{
		if (SystemReference.IsValid())
		{
			// Fetch the current value
			FUAFValueBundle ExistingBundleValue;
			SystemReference.ReadVariable(InVariable,  FAnimNextParamType::GetType<FUAFValueBundle>(),  [&](TConstArrayView<uint8> VariableValue)
			{
				check(sizeof(ExistingBundleValue) == VariableValue.Num());
				FUAFValueBundle::StaticStruct()->CopyScriptStruct(&ExistingBundleValue, VariableValue.GetData());
			});
			
			// Handle any previously bound inputs
			TWeakObjectPtr<UActorComponent>* PreviouslyBoundActorComponentPtr = InputToComponentMappings.Find(InVariable);
			if (PreviouslyBoundActorComponentPtr)
			{
				// Cache old value
				const TWeakObjectPtr<UActorComponent> PreviouslyBoundActorComponent = *PreviouslyBoundActorComponentPtr;
				
				// Update cached binding component (if valid)
				if (InComponent)
				{
					*PreviouslyBoundActorComponentPtr = InComponent;
				}
				else
				{
					InputToComponentMappings.Remove(InVariable);
				}				
				
				TArray<TWeakObjectPtr<UActorComponent>> ValueArray;
				InputToComponentMappings.GenerateValueArray(ValueArray);
				
                if (!ValueArray.Contains(PreviouslyBoundActorComponent))
                {
                    // There are _no_ other bindings against this actor component, so we should remove tick dependencies accordingly
                    if (UActorComponent* Component = PreviouslyBoundActorComponent.Get())
                    {
	                    if (UUAFComponent* UAFComponent = Cast<UUAFComponent>(Component))
	                    {
		                    const UE::UAF::FSystemReference& OtherSystemRef = UAFComponent->SystemReference;
		                    RemoveModuleEventPrerequisite(SystemReference.GetFirstUserEventName(), UAFComponent, OtherSystemRef.GetLastUserEventName());
	                    }
	                    else if (USkeletalMeshComponent* SkelMeshComponent = Cast<USkeletalMeshComponent>(Component))
	                    {
		                    RemoveComponentPrerequisite(SkelMeshComponent, SystemReference.GetFirstUserEventName());
	                    }
                    }
                }
			}
			
			if (UUAFComponent* UAFComponent = Cast<UUAFComponent>(InComponent))
			{
				const FSystemInputBinding SystemInputBinding = { .Variable = InVariable, .Component = UAFComponent };
				
				InputToComponentMappings.Add(InVariable, UAFComponent);
				
				// Allocate the prerequisite system in case it has not been registered yet
				SystemInputBinding.Component->AllocateSystem();

				UE::UAF::FSystemReference& InputSystem = SystemInputBinding.Component->SystemReference;
				SystemReference.AddSystemEventDependency(SystemReference.GetFirstUserEventName(), FUAFWeakSystemReference(InputSystem), InputSystem.GetLastUserEventName(), UE::UAF::ESystemDependency::Prerequisite);
				
				// Queue up task to run after last user event
				SystemReference.QueueTask(SystemReference.GetLastUserEventName(), [WeakThis = TWeakObjectPtr<UUAFComponent>(this), SystemInputBinding](const UE::UAF::FModuleTaskContext& InContext)
				{
					if (UUAFComponent* This = WeakThis.Get())
					{
						if (UUAFComponent* Component = SystemInputBinding.Component.Get())
						{
							if (!InContext.AccessVariable<FUAFValueBundle>(SystemInputBinding.Variable, [Component](FUAFValueBundle& InValueBundle)
							{
								InValueBundle.SetAs<UE::UAF::FVirtualValueBundle_SystemOutput>(FUAFWeakSystemReference(Component->SystemReference));
							}))
							{
								UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Cannot bind input pose variable: '{0}'", SystemInputBinding.Variable.GetName());
							}
						}
					}
				}, UE::UAF::ETaskRunLocation::After);
			}
			else if (USkeletalMeshComponent* SkelMeshComponent = Cast<USkeletalMeshComponent>(InComponent))
			{
				const FComponentInputBinding ComponentInputBinding = { .Variable = InVariable, .Component = SkelMeshComponent };
				AddComponentPrerequisite(SkelMeshComponent, SystemReference.GetFirstUserEventName());
				
				InputToComponentMappings.Add(InVariable, SkelMeshComponent);

				SystemReference.QueueTask([WeakThis = TWeakObjectPtr<UUAFComponent>(this), ComponentInputBinding](const UE::UAF::FModuleTaskContext& InContext)
				{
					if (UUAFComponent* This = WeakThis.Get())
					{
						if (USkeletalMeshComponent* Component = ComponentInputBinding.Component.Get())
						{
							if (!InContext.AccessVariable<FUAFValueBundle>(ComponentInputBinding.Variable, [Component, This](FUAFValueBundle& InValueBundle)
							{
								InValueBundle.SetAs<UE::UAF::FVirtualValueBundle_ComponentOutput>(TWeakObjectPtr<USkeletalMeshComponent>(Component));
							}))
							{
								UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Cannot bind input component pose variable: '{0}'", ComponentInputBinding.Variable.GetName());
							}
						}
					}
				});
			}
			else if (InComponent)
			{
				ensureAlwaysMsgf(false, TEXT("UUAFComponent::BlueprintSetInputBinding, Unexpected component type %s"), InComponent->GetClass() ? *InComponent->GetClass()->GetName() : TEXT("None"));
			}
			else
			{
				// Reset to ref-pose if null component
				SystemReference.QueueTask([WeakThis = TWeakObjectPtr<UUAFComponent>(this), Variable=InVariable](const UE::UAF::FModuleTaskContext& InContext)
				{
					if (UUAFComponent* This = WeakThis.Get())
					{
						if (!InContext.AccessVariable<FUAFValueBundle>(Variable, [This](FUAFValueBundle& InValueBundle)
						{
							InValueBundle.SetAsRefPose();
						}))
						{
							UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Failed to reset value bundle to ref-pose variable: '{0}'", Variable.GetName());
						}
					}
				});
			}
		}
	}
}

void UUAFComponent::SetAssetFromObject(const UObject* InObject)
{
	TInstancedStruct<FUAFSystemFactoryAsset> AssetDataFromFactory = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject<FUAFSystemFactoryAsset>(InObject);
	if (AssetDataFromFactory.IsValid())
	{
		SetAsset(MoveTemp(AssetDataFromFactory));
	}
}

void UUAFComponent::SetAssetInternal(UE::UAF::FSystemAssetHandle&& InAssetData)
{
	AssetData = MoveTemp(InAssetData);
}

void UUAFComponent::SetAssetInternal(TConstStructView<FUAFSystemFactoryAsset> InAssetData)
{
	AssetData = InAssetData;
}

void UUAFComponent::RegisterSystem()
{
	using namespace UE::UAF;

	if (AssetData.IsValid())
	{
		check(!SystemReference.IsValid());
		SystemReference = FSystemReference(AssetData, this, InitMethod);
	}
}

void UUAFComponent::UnregisterSystem()
{
	SystemReference.Reset();
}

bool UUAFComponent::IsModuleValid()
{
	return SystemReference.IsValid();
}

void UUAFComponent::SetupInputOutput()
{
	using namespace UE::UAF::Private;

	check(IsInGameThread());

	CachedSkeletalMeshComponent = DetermineSkeletalMeshComponent();
	const bool bValidOutputComponent = (CachedSkeletalMeshComponent != nullptr);
	if (!bValidOutputComponent)
	{
		UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Could not find a valid mesh component to use for system output");
	}

	AActor* Owner = GetOwner();
	check(Owner);
	TArray<FSystemInputBinding> SystemInputBindings;
	SystemInputBindings.Reserve(Inputs.Num());
	for (const FUAFComponentInputDesc& InputDesc : Inputs)
	{
		if (InputDesc.Input.IsValid())
		{
			if (UUAFComponent* InputComponent = Cast<UUAFComponent>(InputDesc.Component.GetComponent(Owner)))
			{
				if (InputComponent == this)
				{
					UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Cannot apply self as input pose");
					continue;
				}

				SystemInputBindings.Add({ .Variable = InputDesc.Input, .Component = InputComponent });
				InputToComponentMappings.Add(InputDesc.Input, InputComponent);
			}
			else
			{
				UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Invalid input pose component specified");
			}
		}
		else
		{
			UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Invalid input pose variable specified");
		}
	}

	// Set up tick dependencies for valid inputs
	// By default we link [Input Last Event] -> [Our First Event].
	// TODO: Allow user-specified dependencies here
	for (const FSystemInputBinding& InputBinding : SystemInputBindings)
	{
		// Allocate the prerequisite system in case it has not been registered yet
		InputBinding.Component->AllocateSystem();

		UE::UAF::FSystemReference& InputSystem = InputBinding.Component->SystemReference;
		SystemReference.AddSystemEventDependency(SystemReference.GetFirstUserEventName(), FUAFWeakSystemReference(InputSystem), InputSystem.GetLastUserEventName(), UE::UAF::ESystemDependency::Prerequisite);
	}

	// Setup system as tick prerequisite for output skeletal mesh component, this should subsequently impact any follower components/manual tick dependencies setup up against the skeletal mesh component
	if (bValidOutputComponent)
	{
		AddComponentSubsequent(CachedSkeletalMeshComponent, SystemReference.GetLastUserEventName());
	}

	TArray<FComponentInputBinding> ComponentInputBindings;
	ComponentInputBindings.Reserve(Inputs.Num());
	
	for (const FUAFComponentInputDesc& InputDesc : Inputs)
	{
		if (InputDesc.Input.IsValid())
		{
			if (USkeletalMeshComponent* InputComponent = Cast<USkeletalMeshComponent>(InputDesc.Component.GetComponent(Owner)))
			{
				ComponentInputBindings.Add({ .Variable = InputDesc.Input, .Component = InputComponent });
				InputToComponentMappings.Add(InputDesc.Input, InputComponent);
			}
		}
		else
		{
			UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Invalid input pose variable specified");
		}
	}
	
	for (const FComponentInputBinding& InputBinding : ComponentInputBindings)
	{
		USkeletalMeshComponent* Component = InputBinding.Component.Get();
		AddComponentPrerequisite(Component, SystemReference.GetFirstUserEventName());
	}

	SystemReference.QueueTask([WeakThis = TWeakObjectPtr<UUAFComponent>(this), bValidOutputComponent, SystemInputBindings = MoveTemp(SystemInputBindings), ComponentInputBindings = MoveTemp(ComponentInputBindings)](const UE::UAF::FModuleTaskContext& InContext)
	{
		if (UUAFComponent* This = WeakThis.Get())
		{
			if (bValidOutputComponent)
			{
				InContext.AccessComponent<FUAFSystemOutputComponent>([This](FUAFSystemOutputComponent& UAFSystemOutputComponent)
				{
					UAFSystemOutputComponent.BindSystemOutputAdapter(*This);
				});
			}

			for (const FSystemInputBinding& InputBinding : SystemInputBindings)
			{
				if (UUAFComponent* Component = InputBinding.Component.Get())
				{
					if (!InContext.AccessVariable<FUAFValueBundle>(InputBinding.Variable, [Component](FUAFValueBundle& InValueBundle)
					{
						InValueBundle.SetAs<UE::UAF::FVirtualValueBundle_SystemOutput>(FUAFWeakSystemReference(Component->SystemReference));
					}))
					{
						UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Cannot bind system input pose variable: '{0}'", InputBinding.Variable.GetName());
					}
				}
			}

			for (const FComponentInputBinding& InputBinding : ComponentInputBindings)
			{
				if (USkeletalMeshComponent* Component = InputBinding.Component.Get())
				{
					if (!InContext.AccessVariable<FUAFValueBundle>(InputBinding.Variable, [Component, This](FUAFValueBundle& InValueBundle)
					{
						InValueBundle.SetAs<UE::UAF::FVirtualValueBundle_ComponentOutput>(TWeakObjectPtr<USkeletalMeshComponent>(Component));
					}))
					{
						UE_LOGFMT(LogAnimation, Warning, "UUAFComponent::SetupInputOutput: Cannot bind component input pose variable: '{0}'", InputBinding.Variable.GetName());
					}
				}
			}
		}
	});
}

USkeletalMeshComponent* UUAFComponent::DetermineSkeletalMeshComponent() const
{
	AActor* Owner = GetOwner();
	if (Owner == nullptr)
	{
		return nullptr;
	}
	
	if (USkeletalMeshComponent* UserSpecifiedComponent = Cast<USkeletalMeshComponent>(OutputComponent.GetComponent(Owner)))
	{
		return UserSpecifiedComponent;
	}
	return Owner->FindComponentByClass<USkeletalMeshComponent>();
}

void UUAFComponent::SetupCharacterMovementDependency()
{
	check(IsInGameThread());
	if (!bSetupCharacterMovementDependency || !SystemReference.IsValid())
	{
		return;
	}
	
	if (const ACharacter* OwnerCharacter = Cast<ACharacter>(GetOwner()))
	{
		if (const USkeletalMeshComponent* TargetComponent = DetermineSkeletalMeshComponent())
		{
			if (OwnerCharacter->GetMesh() == TargetComponent)
			{
				if (UCharacterMovementComponent* MovementComponent = OwnerCharacter->GetCharacterMovement())
				{
					AddComponentPrerequisite(MovementComponent, SystemReference.GetFirstUserEventName());
					bCharacterMovementDependencyHasBeenSet = true;
				}
			}
		}
	}
}

void UUAFComponent::ReleaseCharacterMovementDependency()
{
	check(IsInGameThread());

	if (bSetupCharacterMovementDependency && bCharacterMovementDependencyHasBeenSet && SystemReference.IsValid())
	{
		if (const ACharacter* OwnerCharacter = Cast<ACharacter>(GetOwner()))
		{
			if (UCharacterMovementComponent* MovementComponent = OwnerCharacter->GetCharacterMovement())
			{
				RemoveComponentPrerequisite(MovementComponent, SystemReference.GetFirstUserEventName());
				bCharacterMovementDependencyHasBeenSet = false;
			}
		}
	}
}
