// Copyright Epic Games, Inc. All Rights Reserved.

#include "Component/SkinnedMeshComponentExtensions.h"
#include "GenerationTools.h"
#include "Components/SkinnedMeshComponent.h"
#include "Animation/AnimTrace.h"
#include "Module/AnimNextModuleInstance.h"

namespace UE::Anim::Private
{
	// When enabled, we mark the component render data as being dirty on the main thread and we'll send the data
	// to the render thread at the end of the frame, same as AnimBP
	// When disabled, we do it inline here
	static bool GUseDeferredRenderDataUpdate = true;
	FAutoConsoleVariableRef CVar_UseDeferredRenderDataUpdate(TEXT("a.AnimNext.UseDeferredRenderDataUpdate"), GUseDeferredRenderDataUpdate, TEXT("Whether or not to defer the render data update to the end of frame"));
}

namespace UE::Anim
{

void ConvertLocalSpaceToComponentSpace(
	const USkinnedMeshComponent* InComponent,
	const UE::UAF::FLODPoseHeap& InLODPose,
	TArrayView<FTransform> OutComponentSpaceTransforms,
	TBitArray<>& OutValidMeshPoseTransforms)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_ConvertLocalSpaceToComponentSpace);

	const UE::UAF::FReferencePose& RefPose = InLODPose.GetRefPose();

	const TArrayView<const FBoneIndexType> MeshBoneIndexToParentMeshBoneIndexMap = RefPose.GetMeshBoneIndexToParentMeshBoneIndexMap();
	const TArrayView<const FBoneIndexType> LODBoneIndexToMeshBoneIndexMap = RefPose.GetLODBoneIndexToMeshBoneIndexMap(InLODPose.LODLevel);

	checkf(LODBoneIndexToMeshBoneIndexMap.Num() == InLODPose.GetNumBones(), TEXT("Buffer mismatch: %d:%d"), LODBoneIndexToMeshBoneIndexMap.Num(), InLODPose.GetNumBones());
	checkf(MeshBoneIndexToParentMeshBoneIndexMap.Num() == OutComponentSpaceTransforms.Num(), TEXT("Buffer mismatch: %d:%d"), MeshBoneIndexToParentMeshBoneIndexMap.Num(), OutComponentSpaceTransforms.Num());

	OutValidMeshPoseTransforms.Init(false, OutComponentSpaceTransforms.Num());

	const int32 NumPoseLODBones = InLODPose.GetNumBones();
	const FBoneIndexType* RESTRICT MeshBoneIndexToParentMeshBoneIndexMapPtr = MeshBoneIndexToParentMeshBoneIndexMap.GetData();
	const FBoneIndexType* RESTRICT LODBoneIndexToMeshBoneIndexMapPtr = LODBoneIndexToMeshBoneIndexMap.GetData();
	FTransform* RESTRICT ComponentSpaceData = OutComponentSpaceTransforms.GetData();

	// First bone (if we have one) is always root bone, and it doesn't have a parent.
	{
		check(LODBoneIndexToMeshBoneIndexMap[0] == 0);
		OutComponentSpaceTransforms[0] = InLODPose.LocalTransformsView[0];
		OutValidMeshPoseTransforms[0] = true;
	}

	for (int32 LODBoneIndex = 1; LODBoneIndex < NumPoseLODBones; ++LODBoneIndex)
	{
		const FTransform LocalSpaceTransform = InLODPose.LocalTransformsView[LODBoneIndex];

		const FBoneIndexType MeshBoneIndex = LODBoneIndexToMeshBoneIndexMapPtr[LODBoneIndex];
		const FBoneIndexType ParentMeshBoneIndex = MeshBoneIndexToParentMeshBoneIndexMapPtr[MeshBoneIndex];

		const FTransform* RESTRICT ParentComponentSpaceTransform = ComponentSpaceData + ParentMeshBoneIndex;

		FTransform* RESTRICT ComponentSpaceTransform = ComponentSpaceData + MeshBoneIndex;

		FTransform::Multiply(ComponentSpaceTransform, &LocalSpaceTransform, ParentComponentSpaceTransform);

		OutValidMeshPoseTransforms[MeshBoneIndex] = true;

		ComponentSpaceTransform->NormalizeRotation();

		checkSlow(ComponentSpaceTransform->IsRotationNormalized());
		checkSlow(!ComponentSpaceTransform->ContainsNaN());
	}

	const int32 MeshLOD = InComponent->GetPredictedLODLevel();
	if (MeshLOD < InLODPose.LODLevel)
	{
		// If we evaluate the animation with lower quality than the visual mesh, we might
		// have some transforms that we didn't compute. We'll grab the reference pose for them.
		const TArrayView<const FBoneIndexType> LODBoneIndexToMeshBoneIndexMapForMeshLOD = RefPose.GetLODBoneIndexToMeshBoneIndexMap(MeshLOD);

		const int32 NumMeshLODBones = LODBoneIndexToMeshBoneIndexMapForMeshLOD.Num();
		check(NumPoseLODBones <= NumMeshLODBones);

		const FBoneIndexType* RESTRICT LODBoneIndexToMeshBoneIndexMapForMeshLODPtr = LODBoneIndexToMeshBoneIndexMapForMeshLOD.GetData();

		for (int32 LODBoneIndex = NumPoseLODBones; LODBoneIndex < NumMeshLODBones; ++LODBoneIndex)
		{
			const FTransform LocalSpaceTransform = RefPose.GetRefPoseTransform(LODBoneIndex);

			const FBoneIndexType MeshBoneIndex = LODBoneIndexToMeshBoneIndexMapForMeshLODPtr[LODBoneIndex];
			const FBoneIndexType ParentMeshBoneIndex = MeshBoneIndexToParentMeshBoneIndexMapPtr[MeshBoneIndex];

			const FTransform* RESTRICT ParentComponentSpaceTransform = ComponentSpaceData + ParentMeshBoneIndex;

			FTransform* RESTRICT ComponentSpaceTransform = ComponentSpaceData + MeshBoneIndex;

			FTransform::Multiply(ComponentSpaceTransform, &LocalSpaceTransform, ParentComponentSpaceTransform);

			ComponentSpaceTransform->NormalizeRotation();

			OutValidMeshPoseTransforms[MeshBoneIndex] = true;

			checkSlow(ComponentSpaceTransform->IsRotationNormalized());
			checkSlow(!ComponentSpaceTransform->ContainsNaN());
		}
	}
}

void FSkinnedMeshComponentExtensions::CompleteAndDispatch(
	USkinnedMeshComponent* InComponent,
	const UE::UAF::FLODPoseHeap& InLODPose,
	TUniqueFunction<void(USkinnedMeshComponent*)>&& InGameThreadCallback)
{
	bool bNeedsChildTransformUpdate = false;

	// Fill the component space transform buffer
	TArrayView<FTransform> ComponentSpaceTransforms = InComponent->GetEditableComponentSpaceTransforms();
	if (ComponentSpaceTransforms.Num() > 0)
	{
		ConvertLocalSpaceToComponentSpace(InComponent, InLODPose, ComponentSpaceTransforms, InComponent->GetMutableValidMeshPoseTransforms());
		InComponent->bValidMeshPoseTransformsUpToDate = true;

		// Flag buffer for flip
		InComponent->bNeedToFlipSpaceBaseBuffers = true;

		InComponent->FlipEditableSpaceBases();
		InComponent->bHasValidBoneTransform = true;

		if (!UE::Anim::Private::GUseDeferredRenderDataUpdate)
		{
			InComponent->InvalidateCachedBounds();
			InComponent->UpdateBounds();

			// Send updated transforms & bounds to the renderer
			InComponent->SendRenderDynamicData_Concurrent();
			InComponent->SendRenderTransform_Concurrent();
		}

		bNeedsChildTransformUpdate = InComponent->bHasSocketAttachments;

		if (const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InComponent))
		{
			TRACE_SKELETAL_MESH_COMPONENT(SkeletalMeshComponent);
		}
	}

	if (bNeedsChildTransformUpdate || InGameThreadCallback || UE::Anim::Private::GUseDeferredRenderDataUpdate)
	{
		FAnimNextModuleInstance::RunTaskOnGameThread(
			[
				WeakComponent = TWeakObjectPtr<USkinnedMeshComponent>(InComponent),
				bNeedsChildTransformUpdate,
				GameThreadCallback = MoveTemp(InGameThreadCallback)
			]()
			{
				USkinnedMeshComponent* Component = WeakComponent.Get();
				if (Component == nullptr)
				{
					return;
				}

				SCOPED_NAMED_EVENT(UAF_SkinnedMesh_CompleteAndDispatch_GameThread, FColor::Orange);
				if (bNeedsChildTransformUpdate)
				{
					Component->UpdateChildTransforms(EUpdateTransformFlags::OnlyUpdateIfUsingSocket);
				}

				if (GameThreadCallback)
				{
					GameThreadCallback(Component);
				}

				if (UE::Anim::Private::GUseDeferredRenderDataUpdate)
				{
					Component->InvalidateCachedBounds();
					Component->UpdateBounds();

					// Skip components that have a system tag as they will likely be modified later.
					if (Component->ComponentTags.IsEmpty())
					{
						const bool bReadyForEarlyUpdate = true;
						Component->MarkForNeededEndOfFrameUpdate(bReadyForEarlyUpdate);
					}

					// Need to send new bounds to the render thread
					Component->MarkRenderTransformDirty();

					// New bone positions need to be sent to render thread
					Component->MarkRenderDynamicDataDirty();
				}
			});
	}
}

}
