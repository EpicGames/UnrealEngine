// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistry.h"

#include "Misc/CoreDelegates.h"
#include "RemapPoseDataPool.h"
#include "Misc/ScopeRWLock.h"
#include "DataRegistryTypes.h"
#include "GenerationTools.h"
#include "UObject/UObjectGlobals.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "BoneContainer.h"
#include "AssetRegistry/AssetData.h"

namespace // Private
{

UE::UAF::FDataRegistry* GAnimationDataRegistry = nullptr;
constexpr int32 BASIC_TYPE_ALLOC_BLOCK = 1000;
FDelegateHandle PostGarbageCollectHandle;
FDelegateHandle DataRegistryPreExitHandle;

static TAutoConsoleVariable<float> CVar_GCCleanupTimeBudget(TEXT("a.AnimNext.GCCleanupTimeBudget"), 100.0f / (1000.0f * 1000.0f), TEXT("Amount of time in seconds that we can use when reclaiming memory post-GC"));

} // end namespace

namespace UE::UAF
{

/*static*/ void FDataRegistry::Init()
{
	if (GAnimationDataRegistry == nullptr)
	{
		GAnimationDataRegistry = new FDataRegistry();

		PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(&FDataRegistry::HandlePostGarbageCollect);
		DataRegistryPreExitHandle = FCoreDelegates::OnEnginePreExit.AddStatic(&FDataRegistry::OnPreExit);
	}
}

/*static*/ void FDataRegistry::Destroy()
{
	if (GAnimationDataRegistry != nullptr)
	{
		FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGarbageCollectHandle);
		PostGarbageCollectHandle.Reset();

		FCoreDelegates::OnEnginePreExit.Remove(DataRegistryPreExitHandle);
		DataRegistryPreExitHandle.Reset();

		if (!GAnimationDataRegistry->GCComponentKeysToCleanup.IsEmpty() ||
			!GAnimationDataRegistry->GCMeshKeysToCleanup.IsEmpty())
		{
			GAnimationDataRegistry->RunGCCleanup(INDEX_NONE);

			FTSTicker::RemoveTicker(GAnimationDataRegistry->GCCleanupTickerHandle);
		}

		GAnimationDataRegistry->ReleaseReferencePoseData(); // release any registered poses

		check(GAnimationDataRegistry->AllocatedBlocks.Num() == 0); // any other data should have been released at this point
		check(GAnimationDataRegistry->StoredData.Num() == 0);

		delete GAnimationDataRegistry;
		GAnimationDataRegistry = nullptr;
	}
}

void FDataRegistry::OnPreExit()
{
	checkf(GAnimationDataRegistry, TEXT("Animation Data Registry is not instanced. It is only valid to access this while the engine module is loaded."));

	if (!GAnimationDataRegistry->GCComponentKeysToCleanup.IsEmpty() ||
		!GAnimationDataRegistry->GCMeshKeysToCleanup.IsEmpty())
	{
		GAnimationDataRegistry->RunGCCleanup(INDEX_NONE);

		FTSTicker::RemoveTicker(GAnimationDataRegistry->GCCleanupTickerHandle);
	}

	GAnimationDataRegistry->ReleaseReferencePoseData(); // release any registered poses
}

FDataRegistry* FDataRegistry::Get()
{
	checkf(GAnimationDataRegistry, TEXT("Animation Data Registry is not instanced. It is only valid to access this while the engine module is loaded."));
	return GAnimationDataRegistry;
}


/*static*/ void FDataRegistry::HandlePostGarbageCollect()
{
	SCOPED_NAMED_EVENT(UAF_DataRegistry_PostGC, FColor::Blue);

	// Clean up stale entries in the remap pose data pool (game thread instance)
	FRemapPoseDataPool::Get().GarbageCollect();

	// Compact the registry on GC
	if (GAnimationDataRegistry)
	{
		FRWScopeLock ComponentLock(GAnimationDataRegistry->SkeletalMeshComponentReferencePosesLock, SLT_Write);
		FRWScopeLock MeshLock(GAnimationDataRegistry->SkeletalMeshReferencePosesLock, SLT_Write);

		if (GAnimationDataRegistry->SkeletalMeshComponentReferencePoses.IsEmpty() &&
			GAnimationDataRegistry->SkeletalMeshReferencePoses.IsEmpty())
		{
			// Nothing to clean up
			return;
		}

		// Start our cleanup
		// If cleanup was already in progress, we purge whatever remains (shouldn't happen unless we force GC to run)
		if (!GAnimationDataRegistry->GCComponentKeysToCleanup.IsEmpty() ||
			!GAnimationDataRegistry->GCMeshKeysToCleanup.IsEmpty())
		{
			GAnimationDataRegistry->RunGCCleanup(INDEX_NONE);
		}

		if (GAnimationDataRegistry->GCCleanupTickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(GAnimationDataRegistry->GCCleanupTickerHandle);
		}

		// Copy our keys into a temporary array, we'll process them in small batches over time
		GAnimationDataRegistry->SkeletalMeshReferencePoses.GenerateKeyArray(GAnimationDataRegistry->GCMeshKeysToCleanup);
		GAnimationDataRegistry->SkeletalMeshComponentReferencePoses.GenerateKeyArray(GAnimationDataRegistry->GCComponentKeysToCleanup);
		GAnimationDataRegistry->GCComponentKeysProcessedIndex = 0;
		GAnimationDataRegistry->GCMeshKeysProcessedIndex = 0;

		const float TickerDelay = 0.0f;
		GAnimationDataRegistry->GCCleanupTickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("UAF DataRegistry Post-GC Cleanup"), TickerDelay, GCCleanupTicker);
	}
}

bool FDataRegistry::GCCleanupTicker(float DeltaTime)
{
	bool bIsDone;
	{
		FRWScopeLock Lock(GAnimationDataRegistry->SkeletalMeshComponentReferencePosesLock, SLT_Write);
		bIsDone = GAnimationDataRegistry->RunGCCleanup(CVar_GCCleanupTimeBudget.GetValueOnAnyThread());
	}

	const bool bFireTickerAgainAfterDelay = !bIsDone;
	return bFireTickerAgainAfterDelay;
}

bool FDataRegistry::RunGCCleanup(double TimeBudget)
{
	SCOPED_NAMED_EVENT(UAF_DataRegistry_GCCleanup, FColor::Blue);

	if (TimeBudget <= 0.0)
	{
		// No budget specified, process everything
		TimeBudget = UE_DOUBLE_BIG_NUMBER;
	}

	const double StartTime = FPlatformTime::Seconds();

	while (GCComponentKeysProcessedIndex < GCComponentKeysToCleanup.Num())
	{
		const TObjectKey<USkeletalMeshComponent>& SkeletalMeshComponentKey = GCComponentKeysToCleanup[GCComponentKeysProcessedIndex];
		if (SkeletalMeshComponentKey.ResolveObjectPtr() == nullptr)
		{
			SkeletalMeshComponentReferencePoses.Remove(SkeletalMeshComponentKey);
		}

		GCComponentKeysProcessedIndex++;

		if ((FPlatformTime::Seconds() - StartTime) > TimeBudget)
		{
			break;
		}
	}

	while (GCMeshKeysProcessedIndex < GCMeshKeysToCleanup.Num())
	{
		const TObjectKey<USkeletalMesh>& SkeletalMeshKey = GCMeshKeysToCleanup[GCMeshKeysProcessedIndex];
		if (SkeletalMeshKey.ResolveObjectPtr() == nullptr)
		{
			SkeletalMeshReferencePoses.Remove(SkeletalMeshKey);
		}

		GCMeshKeysProcessedIndex++;

		if ((FPlatformTime::Seconds() - StartTime) > TimeBudget)
		{
			break;
		}
	}

	if (GCComponentKeysProcessedIndex == GCComponentKeysToCleanup.Num() &&
		GCMeshKeysProcessedIndex == GCMeshKeysToCleanup.Num())
	{
		// We are done
		GCComponentKeysToCleanup.Empty();
		GCMeshKeysToCleanup.Empty();
		GCComponentKeysProcessedIndex = 0;
		GCMeshKeysProcessedIndex = 0;
		return true;
	}

	// Not done yet
	return false;
}

FDataHandle FDataRegistry::RegisterReferencePose(USkeletalMeshComponent* SkeletalMeshComponent)
{
	FDataHandle Handle = AllocateData<FAnimNextReferencePose>(1);

	FAnimNextReferencePose& AnimationReferencePose = Handle.GetRef<FAnimNextReferencePose>();

	FGenerationTools::GenerateReferencePose(SkeletalMeshComponent, SkeletalMeshComponent->GetSkeletalMeshAsset(), AnimationReferencePose);

	// register even if it fails to generate (register an empty ref pose)

	const TObjectKey<USkeletalMeshComponent> Key(SkeletalMeshComponent);
	const uint32 KeyHash = GetTypeHash(Key);

	{
		FRWScopeLock Lock(SkeletalMeshComponentReferencePosesLock, SLT_Write);

		// First check if another thread grabbed the write lock before us to do the same work
		if (const FReferencePoseData* ExistingPoseData = SkeletalMeshComponentReferencePoses.FindByHash(KeyHash, Key))
		{
			// Another thread beat us to it, discard the work we did and re-use what is cached
			return ExistingPoseData->AnimationDataHandle;
		}

		// Only register the delegate if we are the one to add to the map
		FDelegateHandle DelegateHandle = SkeletalMeshComponent->RegisterOnLODRequiredBonesUpdate_Member(FOnLODRequiredBonesUpdate::CreateRaw(this, &FDataRegistry::OnLODRequiredBonesUpdate));

		SkeletalMeshComponentReferencePoses.AddByHash(KeyHash, Key, FReferencePoseData(Handle, DelegateHandle));
	}

	return Handle;
}

FDataHandle FDataRegistry::RegisterReferencePose(USkeletalMesh* SkeletalMesh)
{
	FDataHandle Handle = AllocateData<FAnimNextReferencePose>(1);

	FAnimNextReferencePose& AnimationReferencePose = Handle.GetRef<FAnimNextReferencePose>();

	FGenerationTools::GenerateReferencePose(nullptr, SkeletalMesh, AnimationReferencePose);

	// register even if it fails to generate (register an empty ref pose)

	const TObjectKey<USkeletalMesh> Key(SkeletalMesh);
	const uint32 KeyHash = GetTypeHash(Key);

	{
		FRWScopeLock Lock(SkeletalMeshReferencePosesLock, SLT_Write);

		// First check if another thread grabbed the write lock before us to do the same work
		if (const FReferencePoseData* ExistingPoseData = SkeletalMeshReferencePoses.FindByHash(KeyHash, Key))
		{
			// Another thread beat us to it, discard the work we did and re-use what is cached
			return ExistingPoseData->AnimationDataHandle;
		}
		
		SkeletalMeshReferencePoses.AddByHash(KeyHash, Key, FReferencePoseData(Handle, FDelegateHandle()));
	}

	return Handle;
}

void FDataRegistry::OnLODRequiredBonesUpdate(USkeletalMeshComponent* SkeletalMeshComponent, int32 LODLevel, const TArray<FBoneIndexType>& LODRequiredBones)
{
	// TODO : Check if the LDO bomes are different from the currently calculated ReferencePose data (for now just delete the cached data)
	RemoveReferencePose(SkeletalMeshComponent);
}

FDataHandle FDataRegistry::GetOrGenerateReferencePose(USkeletalMeshComponent* SkeletalMeshComponent)
{
	FDataHandle ReturnHandle;

	{
		FRWScopeLock Lock(SkeletalMeshComponentReferencePosesLock, SLT_ReadOnly);

		if (const FReferencePoseData* ReferencePoseData = SkeletalMeshComponentReferencePoses.Find(SkeletalMeshComponent))
		{
			ReturnHandle = ReferencePoseData->AnimationDataHandle;
		}
	}
	
	if (ReturnHandle.IsValid() == false)
	{
		ReturnHandle = RegisterReferencePose(SkeletalMeshComponent);
	}

	return ReturnHandle;
}


FDataHandle FDataRegistry::GetOrGenerateReferencePose(USkeletalMesh* SkeletalMesh)
{
	FDataHandle ReturnHandle;

	{
		FRWScopeLock Lock(SkeletalMeshReferencePosesLock, SLT_ReadOnly);

		if (const FReferencePoseData* ReferencePoseData = SkeletalMeshReferencePoses.Find(SkeletalMesh))
		{
			ReturnHandle = ReferencePoseData->AnimationDataHandle;
		}
	}
	
	if (ReturnHandle.IsValid() == false)
	{
		ReturnHandle = RegisterReferencePose(SkeletalMesh);
	}

	return ReturnHandle;
}

void FDataRegistry::RemoveReferencePose(USkeletalMeshComponent* SkeletalMeshComponent)
{
	if (SkeletalMeshComponent != nullptr)
	{
		FRWScopeLock Lock(SkeletalMeshComponentReferencePosesLock, SLT_Write);

		FReferencePoseData ReferencePoseData;
		if (SkeletalMeshComponentReferencePoses.RemoveAndCopyValue(SkeletalMeshComponent, ReferencePoseData))
		{
			SkeletalMeshComponent->UnregisterOnLODRequiredBonesUpdate_Member(ReferencePoseData.DelegateHandle);
		}
	}
}

void FDataRegistry::RemoveReferencePose(USkeletalMesh* SkeletalMesh)
{
	if (SkeletalMesh != nullptr)
	{
		FRWScopeLock Lock(SkeletalMeshReferencePosesLock, SLT_Write);
		SkeletalMeshReferencePoses.Remove(SkeletalMesh);
	}
}

void FDataRegistry::RegisterData(const FName& Id, const FDataHandle& AnimationDataHandle)
{
	FRWScopeLock Lock(StoredDataLock, SLT_Write);
	StoredData.Add(Id, AnimationDataHandle);
}

void FDataRegistry::UnregisterData(const FName& Id)
{
	FRWScopeLock Lock(StoredDataLock, SLT_Write);
	StoredData.Remove(Id);
}

FDataHandle FDataRegistry::GetRegisteredData(const FName& Id) const
{
	FDataHandle ReturnHandle;

	{
		FRWScopeLock Lock(StoredDataLock, SLT_ReadOnly);

		if (const FDataHandle* HandlePtr = StoredData.Find(Id))
		{
			ReturnHandle = *HandlePtr;
		}
	}

	return ReturnHandle;
}

void FDataRegistry::FreeAllocatedBlock(Private::FAllocatedBlock* AllocatedBlock)
{
	FRWScopeLock ScopedAllocatedBlocksLock(AllocatedBlocksLock, SLT_Write);

	if (ensure(AllocatedBlock != nullptr && AllocatedBlocks.Find(AllocatedBlock)))
	{
		if (AllocatedBlock->Memory != nullptr)
		{
			void* Memory = AllocatedBlock->Memory;

			{
				FRWScopeLock ScopedDataTypeDefsLock(DataTypeDefsLock, SLT_ReadOnly);
				// We are empty on app exit and thus cannot run destructors (param type is invalid)
				// When this occurs, we leak memory on purpose
				const FDataTypeDef* TypeDef = !DataTypeDefs.IsEmpty() ? DataTypeDefs.Find(AllocatedBlock->Type) : nullptr;
				if (!DataTypeDefs.IsEmpty()
					&& ensure(TypeDef != nullptr))
				{
					TypeDef->DestroyTypeFn((uint8*)AllocatedBlock->Memory, AllocatedBlock->NumElem);
				}
			}

			FMemory::Free(AllocatedBlock->Memory); // TODO : This should come from preallocated chunks, use malloc / free for now
			AllocatedBlock->Memory = nullptr;

			AllocatedBlocks.Remove(AllocatedBlock);
			delete AllocatedBlock; // TODO : avoid memory fragmentation
		}
	}
}

// Remove any ReferencePoses and unregister all the SkeletalMeshComponent delegates (if any still alive)
void FDataRegistry::ReleaseReferencePoseData()
{
	{
		FRWScopeLock Lock(SkeletalMeshComponentReferencePosesLock, SLT_Write);

		for (auto Iter = GAnimationDataRegistry->SkeletalMeshComponentReferencePoses.CreateIterator(); Iter; ++Iter)
		{
			const TObjectKey<USkeletalMeshComponent>& SkeletalMeshComponentPtr = Iter.Key();
			const FReferencePoseData& ReferencePoseData = Iter.Value();

			if (USkeletalMeshComponent* SkeletalMeshComponent = SkeletalMeshComponentPtr.ResolveObjectPtr())
			{
				SkeletalMeshComponent->UnregisterOnLODRequiredBonesUpdate_Member(ReferencePoseData.DelegateHandle);
			}
		}

		SkeletalMeshComponentReferencePoses.Empty();
	}
	{
		FRWScopeLock Lock(SkeletalMeshReferencePosesLock, SLT_Write);
		SkeletalMeshReferencePoses.Empty();
	}
}

} // end namespace UE::UAF
