// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryTypes.h"
#include "DataRegistry.h"

namespace UE::UAF
{

FDataHandle::~FDataHandle()
{
	if (AllocatedBlock != nullptr)
	{
		check(AllocatedBlock->GetRefCount() > 0);

		const int32 CurrentCount = AllocatedBlock->Release();
		check(CurrentCount >= 0);

		if (CurrentCount == 0)
		{
			FDataRegistry::Get()->FreeAllocatedBlock(AllocatedBlock);
		}
	}
}

} // namespace UE::UAF
