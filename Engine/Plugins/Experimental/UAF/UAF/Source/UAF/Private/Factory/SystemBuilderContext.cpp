// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/SystemBuilderContext.h"
#include "Module/AnimNextModule.h"
#include "UAFAssetInstanceComponent.h"

namespace UE::UAF
{

const UUAFSystem* FSystemBuilderContext::Build()
{
	// Create our system
	UUAFSystem* System = NewObject<UUAFSystem>(GetTransientPackage(), NAME_None, RF_Transient);

	// Copy in variables & default components
	System->ReferencedVariableStructs = VariableStructs.Array();
	for (const UScriptStruct* ComponentStruct : ComponentStructs)
	{
		if (ensure(ComponentStruct->IsChildOf<FUAFAssetInstanceComponent>()))
		{
			TInstancedStruct<FUAFAssetInstanceComponent> InstancedStruct;
			InstancedStruct.InitializeAsScriptStruct(ComponentStruct);
			System->Components.Add(MoveTemp(InstancedStruct));
		}
	}

	System->ReferencedVariableAssets = ReferencedVariableAssets.Array();
	System->ReferencedVariableRigVMAssets = ReferencedVariableRigVMAssets.Array();

	// Clear async flag to make the object visible to GC
	(void)System->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
	return System;
}

}