// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/SystemFactory.h"

#include "Factory/SystemBuilderContext.h"
#include "Containers/StripedMap.h"
#include "Factory/UAFSystemBuilder.h"
#include "Module/AnimNextModule.h"
#include "Misc/CoreDelegates.h"
#include "UAF/UAFAssetData.h"

namespace UE::UAF::Private
{

static FDelegateHandle GSystemFactoryPreExitHandle;

// References to all generated factory systems, keyed by method
static TStripedMap<32, uint64, TWeakObjectPtr<const UUAFSystem>> GSystemMap;

// All registered object -> param mappings
static TMap<FTopLevelAssetPath, TPair<FUAFSystemFactoryParams, FSystemFactory::FParamsInitializer>> GDefaultParamsForObject;
static TArray<TSubScriptStructOf<FUAFSystemFactoryAsset>> GAllFactoryStructs;

const TPair<FUAFSystemFactoryParams, FSystemFactory::FParamsInitializer>* FindSystemParamsForAssetData(TConstStructView<FUAFSystemFactoryAsset> AssetData)
{
	return GDefaultParamsForObject.Find(AssetData.GetScriptStruct()->GetStructPathName());
}
}

namespace UE::UAF
{

uint64 ISystemBuilder::GetKey() const
{
	return 0;
}

const UUAFSystem* FSystemFactory::BuildSystem(const ISystemBuilder& InBuilder)
{
	const uint64 Key = InBuilder.GetKey();
	if (Key == ISystemBuilder::InvalidKey)
	{
		return nullptr;
	}

	auto ProduceSystem = [&InBuilder]() -> TWeakObjectPtr<const UUAFSystem>
	{
		FSystemBuilderContext Context;
		if (InBuilder.Build(Context))
		{
			return Context.Build();
		}
		return nullptr;
	};

	TWeakObjectPtr<const UUAFSystem> System;
	auto ApplyWeakSystem = [&ProduceSystem, &System](TWeakObjectPtr<const UUAFSystem>& InFoundSystem)
	{
		if (InFoundSystem.IsStale() || InFoundSystem.Get() == nullptr)
		{
			// Found a stale weak ptr, so regenerate (this may be a procedural system or a previously loaded system that has been GCed)
			InFoundSystem = ProduceSystem();
		}
		System = InFoundSystem;
	};

	Private::GSystemMap.FindOrProduceAndApplyForWrite(Key, ProduceSystem, ApplyWeakSystem);
	return System.Get();
}

FUAFSystemFactoryParams FSystemFactory::GetDefaultParamsForAsset(TConstStructView<FUAFSystemFactoryAsset> Asset)
{
	check(Asset.IsValid());
	const TPair<FUAFSystemFactoryParams, FParamsInitializer>* ParamsTask = Private::FindSystemParamsForAssetData(Asset);
	checkf(ParamsTask != nullptr, 	TEXT("Couldn't find UAF system factory params for asset data type %s"), *Asset.GetScriptStruct()->GetStructPathName().ToString());

	FUAFSystemFactoryParams ParamsCopy = ParamsTask->Key;
	if (ParamsCopy.Builder.ComponentStructs.Num() > 0 || ParamsCopy.Builder.VariablesStructs.Num() > 0)
	{
		if (ParamsTask->Value)
		{
			ParamsTask->Value(Asset, ParamsCopy);
		}
	}
	return ParamsCopy;
}

void FSystemFactory::RegisterAsset(const TSubScriptStructOf<FUAFSystemFactoryAsset>& ScriptStruct, FUAFSystemFactoryParams&& InParams,
	FParamsInitializer&& InInitializer)
{
	checkf(Private::GAllFactoryStructs.Contains(ScriptStruct) == false, TEXT("Attempting to register ScriptStruct %s twice"), *ScriptStruct->GetStructPathName().ToString());
	
	Private::GDefaultParamsForObject.Add(ScriptStruct->GetStructPathName(), {MoveTemp(InParams), MoveTemp(InInitializer)});
	Private::GAllFactoryStructs.Add(ScriptStruct);
}

TConstArrayView<TSubScriptStructOf<FUAFSystemFactoryAsset>> FSystemFactory::GetRegisteredAssetDataTypes()
{
	return Private::GAllFactoryStructs;
}

void FSystemFactory::Init()
{
	Private::GSystemFactoryPreExitHandle = FCoreDelegates::OnEnginePreExit.AddStatic(&FSystemFactory::OnPreExit);
}

void FSystemFactory::Destroy()
{
	FCoreDelegates::OnEnginePreExit.Remove(Private::GSystemFactoryPreExitHandle);
	Private::GSystemFactoryPreExitHandle.Reset();
}

void FSystemFactory::OnPreExit()
{
	Private::GSystemMap.Empty();
	Private::GDefaultParamsForObject.Empty();
	Private::GAllFactoryStructs.Empty();
}

}
