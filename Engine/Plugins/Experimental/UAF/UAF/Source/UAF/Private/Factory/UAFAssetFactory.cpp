// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/UAFAssetFactory.h"

#include "UAF/UAFAssetData.h"
#include "Templates/SubclassOf.h"

namespace UE::UAF
{
TMap<FTopLevelAssetPath, FAssetDataFactory::FInitializerInfo> FAssetDataFactory::AssetInitializers;
TArray<FTopLevelAssetPath> FAssetDataFactory::AllSupportedObjectClassPaths;

const FAssetDataFactory::FInitializerInfo* FAssetDataFactory::FindAssetInitializer(const UClass* Class)
{
	check(Class != nullptr);
	FInitializerInfo* InitializerInfo;
	const UClass* SearchClass = Class;
	do
	{
		InitializerInfo = AssetInitializers.Find(SearchClass->GetClassPathName());
		SearchClass = SearchClass->GetSuperClass();
	}
	while (InitializerInfo == nullptr && SearchClass != nullptr);
	return InitializerInfo;
}

FTopLevelAssetPath FAssetDataFactory::RegisterAsset(TSubScriptStructOf<FUAFAssetData> AssetDataType, UClass* ObjectClass, FAssetInitializer&& Initializer)
{
	const FTopLevelAssetPath ClassPathName = ObjectClass->GetClassPathName();
	checkf(AssetInitializers.Contains(ClassPathName) == false, TEXT("Attempting to register type %s in UAF::FAssetFactory twice"), *ObjectClass->GetFullName());

	AssetInitializers.Add(ClassPathName, FInitializerInfo(AssetDataType, Initializer));
	AllSupportedObjectClassPaths.Add(ClassPathName);
	return ClassPathName;
}

TInstancedStruct<FUAFAssetData> FAssetDataFactory::CreateUAFAssetDataFromObject(const UObject* Object)
{
	check(Object != nullptr);
	
	const FInitializerInfo* InitializerInfo = FindAssetInitializer(Object->GetClass());
	if (ensureMsgf(InitializerInfo != nullptr, TEXT("Unsupported object type (%s) passed to UAF::FAssetFactory"), *Object->GetClass()->GetFullName()))
	{
		return InitializerInfo->Initializer(Object);
	}

	return TInstancedStruct<FUAFAssetData>();
}

void FAssetDataFactory::UnregisterAsset(const FTopLevelAssetPath& InObjectClassPath)
{
	check(InObjectClassPath.IsValid());

	int Removed = AssetInitializers.Remove(InObjectClassPath);
	check(Removed > 0);
	Removed = AllSupportedObjectClassPaths.Remove(InObjectClassPath);
	check(Removed > 0);
}

TArray<UClass*> FAssetDataFactory::GetRegisteredObjectClasses()
{
	TSet<UClass*> AllSupportedObjectClasses;
	AllSupportedObjectClasses.Reserve(AllSupportedObjectClassPaths.Num());
	for (const FTopLevelAssetPath& ClassName : AllSupportedObjectClassPaths)
	{
		if (UClass* Class = FindObject<UClass>(nullptr, ClassName.ToString()))
		{
			AllSupportedObjectClasses.Add(Class);
			
			// Add all derived classes 
			TArray<UClass*> DerivedClasses;
			GetDerivedClasses(Class, DerivedClasses, /*bRecursive=*/true);
			AllSupportedObjectClasses.Append(DerivedClasses);
		}
	}
	return AllSupportedObjectClasses.Array();
}

	
TArray<UClass*> FAssetDataFactory::GetRegisteredObjectClasses(TSubScriptStructOf<FUAFAssetData> BaseAssetType)
{
	TArray<UClass*> CompatibleClasses;
	CompatibleClasses.Reserve(AllSupportedObjectClassPaths.Num());

	for (auto Info : AssetInitializers)
	{
		if (Info.Value.AssetDataType->IsChildOf(BaseAssetType.Get()))
		{
			if (UClass* Class = FindObject<UClass>(nullptr, Info.Key.ToString()))
			{
				CompatibleClasses.Add(Class);
				
				// Add any valid subclasses of this class that are not registered themselves or have a closer parent with a registered initializer
				GatherDerivedCompatibleObjectClasses(Class, CompatibleClasses);
			}
		}
	}

	return CompatibleClasses;
}
	
void FAssetDataFactory::GatherDerivedCompatibleObjectClasses(const UClass* SuperClass, TArray<UClass*>& InOutCompatibleClasses)
{
	if (!SuperClass)
	{
		return;
	}
	
	TArray<UClass*> DerivedClasses;
	GetDerivedClasses(SuperClass, DerivedClasses, false);
	for (UClass* DerivedClass : DerivedClasses)
	{
		// If the derived class does not have a registered initializer it's a valid subclass we want to add and explore it's derived classes as well
		if (DerivedClass && !AssetInitializers.Contains(DerivedClass->GetClassPathName()))
		{
			InOutCompatibleClasses.Add(DerivedClass);
			GatherDerivedCompatibleObjectClasses(DerivedClass, InOutCompatibleClasses);
		}
	}
}
	
}


