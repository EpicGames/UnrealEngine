// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/UAFSimpleSystemBuilder.h"

#include "AnimNextRigVMAsset.h"
#include "Factory/SystemBuilderContext.h"
#include "UAFAssetInstanceComponent.h"
#include "Hash/xxhash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFSimpleSystemBuilder)

bool FUAFSimpleSystemBuilder::Build(UE::UAF::FSystemBuilderContext& InContext) const
{
	using namespace UE::UAF;

	InContext.ComponentStructs.Reserve(ComponentStructs.Num());
	for (const TInstancedStruct<FUAFAssetInstanceComponent>& ComponentStruct : ComponentStructs)
	{
		InContext.ComponentStructs.Add(ComponentStruct.GetScriptStruct());
	}
	
	InContext.VariableStructs.Reserve(VariablesStructs.Num());
	for (const FInstancedStruct& VariablesStruct : VariablesStructs)
	{
		InContext.VariableStructs.Add(VariablesStruct.GetScriptStruct());
	}

	InContext.ReferencedVariableAssets.Append(ReferencedVariableAssets);
	InContext.ReferencedVariableRigVMAssets.Append(ReferencedVariableRigVMAssets);

	return true;
}

void FUAFSimpleSystemBuilder::AddComponentInstancedStruct(TInstancedStruct<FUAFAssetInstanceComponent>&& InInstancedStruct)
{
	using namespace UE::UAF;

	auto FindExistingStruct = [&InInstancedStruct](const TInstancedStruct<FUAFAssetInstanceComponent>& InExistingStruct)
	{
		return InInstancedStruct.GetScriptStruct() == InExistingStruct.GetScriptStruct();
	};

	if (TInstancedStruct<FUAFAssetInstanceComponent>* ExistingStruct = ComponentStructs.FindByPredicate(FindExistingStruct))
	{
		*ExistingStruct = InInstancedStruct;
	}
	else
	{
		ComponentStructs.Emplace(MoveTemp(InInstancedStruct));
	}

	InvalidateKey();
}

void FUAFSimpleSystemBuilder::AddComponentStructView(TConstStructView<FUAFAssetInstanceComponent> InStructView)
{
	using namespace UE::UAF;

	auto FindExistingStruct = [&InStructView](const TInstancedStruct<FUAFAssetInstanceComponent>& InExistingStruct)
	{
		return InStructView.GetScriptStruct() == InExistingStruct.GetScriptStruct();
	};

	if (TInstancedStruct<FUAFAssetInstanceComponent>* ExistingStruct = ComponentStructs.FindByPredicate(FindExistingStruct))
	{
		*ExistingStruct = InStructView;
	}
	else
	{
		ComponentStructs.Emplace(InStructView);
	}

	InvalidateKey();
}

void FUAFSimpleSystemBuilder::AddVariablesInstancedStruct(FInstancedStruct&& InInstancedStruct)
{
	using namespace UE::UAF;

	auto FindExistingStruct = [&InInstancedStruct](const FInstancedStruct& InExistingStruct)
	{
		return InInstancedStruct.GetScriptStruct() == InExistingStruct.GetScriptStruct();
	};

	if (FInstancedStruct* ExistingStruct = VariablesStructs.FindByPredicate(FindExistingStruct))
	{
		*ExistingStruct = InInstancedStruct;
	}
	else
	{
		VariablesStructs.Emplace(MoveTemp(InInstancedStruct));
	}

	InvalidateKey();
}

void FUAFSimpleSystemBuilder::AddVariablesStructView(FConstStructView InStructView)
{
	using namespace UE::UAF;

	auto FindExistingStruct = [&InStructView](const FInstancedStruct& InExistingStruct)
	{
		return InStructView.GetScriptStruct() == InExistingStruct.GetScriptStruct();
	};

	if (FInstancedStruct* ExistingStruct = VariablesStructs.FindByPredicate(FindExistingStruct))
	{
		*ExistingStruct = InStructView;
	}
	else
	{
		VariablesStructs.Emplace(InStructView);
	}

	InvalidateKey();
}

void FUAFSimpleSystemBuilder::AddVariablesStruct(const TObjectPtr<const UScriptStruct>& InScriptStruct)
{
	using namespace UE::UAF;

	auto FindExistingStruct = [&InScriptStruct](const FInstancedStruct& InExistingStruct)
		{
			return InScriptStruct == InExistingStruct.GetScriptStruct();
		};

	if (FInstancedStruct* ExistingStruct = VariablesStructs.FindByPredicate(FindExistingStruct))
	{
		// Already added: nothing to do
		return;
	}

	FInstancedStruct DefaultInstance;
	DefaultInstance.InitializeAs(InScriptStruct);
	VariablesStructs.Emplace(DefaultInstance);
	
	InvalidateKey();
}

void FUAFSimpleSystemBuilder::AddVariablesRigVMAsset(const TScriptInterface<const IRigVMRuntimeAssetInterface>& RigVMAssetInterface)
{
	ReferencedVariableRigVMAssets.Add(RigVMAssetInterface);
	InvalidateKey();
}

void FUAFSimpleSystemBuilder::AddVariablesAsset(const TObjectPtr<const UUAFRigVMAsset>& UAFAsset)
{
	ReferencedVariableAssets.Add(UAFAsset);
	InvalidateKey();
}

uint64 FUAFSimpleSystemBuilder::RecalculateKey() const
{
	FXxHash64Builder Hash;

	const int32 NumComponents = ComponentStructs.Num();
	Hash.Update(&NumComponents, sizeof(NumComponents));
	for (const TInstancedStruct<FUAFAssetInstanceComponent>& ComponentStruct : ComponentStructs)
	{
		const UScriptStruct* Struct = ComponentStruct.GetScriptStruct();
		Hash.Update(&Struct, sizeof(Struct));
	}

	const int32 NumVariables = VariablesStructs.Num();
	Hash.Update(&NumVariables, sizeof(NumVariables));
	for (const FInstancedStruct& VariablesStruct : VariablesStructs)
	{
		const UScriptStruct* Struct = VariablesStruct.GetScriptStruct();
		Hash.Update(&Struct, sizeof(Struct));
	}

	const int32 NumReferencedVariableAssets = ReferencedVariableAssets.Num();
	Hash.Update(&NumReferencedVariableAssets, sizeof(NumReferencedVariableAssets));
	for (TObjectPtr<const UUAFRigVMAsset> ReferencedVariableAsset : ReferencedVariableAssets)
	{
		uint64 ObjHash = GetTypeHash(ReferencedVariableAsset);
		Hash.Update(&ObjHash, sizeof(ObjHash));
	}

	const int32 NumReferencedVariableRigVMAssets = ReferencedVariableRigVMAssets.Num();
	Hash.Update(&NumReferencedVariableRigVMAssets, sizeof(NumReferencedVariableRigVMAssets));
	for (const TScriptInterface<const IRigVMRuntimeAssetInterface>& ReferencedVariableRigVMAsset : ReferencedVariableRigVMAssets)
	{
		uint64 ObjHash = GetTypeHash(ReferencedVariableRigVMAsset);
		Hash.Update(&ObjHash, sizeof(ObjHash));
	}

	return Hash.Finalize().Hash;
}
