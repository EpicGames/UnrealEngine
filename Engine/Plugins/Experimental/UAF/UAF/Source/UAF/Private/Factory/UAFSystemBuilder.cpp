// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/UAFSystemBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFSystemBuilder)

uint64 FUAFSystemBuilder::GetKey() const
{
	if (CachedKey == UE::UAF::ISystemBuilder::InvalidKey)
	{
		CachedKey = RecalculateKey();
	}
	return CachedKey;
}
