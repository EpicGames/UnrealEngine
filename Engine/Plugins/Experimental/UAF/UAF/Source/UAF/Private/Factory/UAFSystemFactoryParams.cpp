// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factory/UAFSystemFactoryParams.h"
#include "UAFAssetInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFSystemFactoryParams)

void FUAFSystemFactoryParams::InitializeInstance(FUAFAssetInstance& InInstance) const
{
	for (const TInstancedStruct<FUAFAssetInstanceComponent>& ComponentStruct : Builder.ComponentStructs)
	{
		FUAFAssetInstanceComponent* Component = InInstance.TryGetComponent(ComponentStruct.GetScriptStruct());
		if (ensure(Component != nullptr))
		{
			ComponentStruct.GetScriptStruct()->CopyScriptStruct(Component, ComponentStruct.GetMemory());
		}
	}

	for (const FInstancedStruct& VariablesStruct : Builder.VariablesStructs)
	{
		InInstance.AccessVariablesStruct(VariablesStruct.GetScriptStruct(), [&VariablesStruct](FStructView InStructView)
		{
			check(VariablesStruct.GetScriptStruct() == InStructView.GetScriptStruct());
			InStructView.GetScriptStruct()->CopyScriptStruct(InStructView.GetMemory(), VariablesStruct.GetMemory());
		});
	}
	
	const UE::UAF::FInstanceTaskContext TaskContext(InInstance);
	for (const UE::UAF::FInstanceTask& InitializeTask : InitializeTasks)
	{
		InitializeTask(TaskContext);
	}
}

void FUAFSystemFactoryParams::Reset()
{
	Builder.Reset();
	InitializeTasks.Empty();
}
