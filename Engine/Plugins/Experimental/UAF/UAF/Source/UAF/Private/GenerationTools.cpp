// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenerationTools.h"

#include "ReferencePose.h"
#include "LODPose.h"
#include "AnimationRuntime.h"
#include "Animation/AnimNodeBase.h"
#include "BoneContainer.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "SkeletalMeshSceneProxy.h"
#include "Engine/SkinnedAssetCommon.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AnimNextStats.h"
#include "Animation/AttributesContainer.h"

DEFINE_STAT(STAT_AnimNext_GenerateReferencePose);
DEFINE_STAT(STAT_AnimNext_RemapPose_FromAnimBP);
DEFINE_STAT(STAT_AnimNext_RemapPose_ToAnimBP);
DEFINE_STAT(STAT_AnimNext_RemapPose_ToLocalTransforms);
DEFINE_STAT(STAT_AnimNext_ConvertLocalSpaceToComponentSpace);

DEFINE_LOG_CATEGORY_STATIC(LogAnimGenerationTools, Log, All)


namespace UE::UAF
{

struct FCompareBoneIndexType
{
	FORCEINLINE bool operator()(const FBoneIndexType& A, const FBoneIndexType& B) const
	{
		return A < B;
	}
};


/*static*/ bool FGenerationTools::GenerateReferencePose(const USkeletalMeshComponent* SkeletalMeshComponent
	, USkeletalMesh* SkeletalMesh
	, FReferencePose& OutAnimationReferencePose)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_GenerateReferencePose);
	
	using namespace UE::UAF;

	bool ReferencePoseGenerated = false;

	if (SkeletalMesh == nullptr)
	{
		return false;
	}

	UE_LOGF(LogAnimGenerationTools, VeryVerbose, "Generating CanonicalBoneSet for SkeletalMesh %ls.", *SkeletalMesh->GetPathName());

	const FSkeletalMeshRenderData* SkelMeshRenderData = (SkeletalMeshComponent != nullptr)
		? SkeletalMeshComponent->GetSkeletalMeshRenderData()
		: SkeletalMesh->GetResourceForRendering();

	if (!SkelMeshRenderData)
	{
		UE_LOGF(LogAnimGenerationTools, Warning, "Error generating CanonicalBoneSet for SkeletalMesh %ls. No SkeletalMeshRenderData.", *SkeletalMesh->GetPathName());
		return false;
	}

	const TIndirectArray<FSkeletalMeshLODRenderData>& LODRenderData = SkelMeshRenderData->LODRenderData;
	const int32 NumLODs = LODRenderData.Num();

	TArray<FGenerationLODData> GenerationLODData;
	GenerationLODData.Reserve(NumLODs);
	GenerationLODData.AddDefaulted(NumLODs);

	TArray<FGenerationLODData> ComponentSpaceGenerationLODData;
	ComponentSpaceGenerationLODData.Reserve(NumLODs);
	ComponentSpaceGenerationLODData.AddDefaulted(NumLODs);

	if (NumLODs > 0)
	{
		// Generate LOD0 bones
		TArray<FBoneIndexType>& RequiredBones_LOD0 = GenerationLODData[0].RequiredBones;

		constexpr int32 LOD0Index = 0;
		GenerateRawLODData(SkeletalMeshComponent, SkeletalMesh, LOD0Index, LODRenderData, RequiredBones_LOD0, ComponentSpaceGenerationLODData[LOD0Index].RequiredBones);

		// Now calculate the LODs > 1
		constexpr int32 StartLOD = 1;
		GenerateLODData(SkeletalMeshComponent, SkeletalMesh, StartLOD, NumLODs, LODRenderData, RequiredBones_LOD0, GenerationLODData, ComponentSpaceGenerationLODData);

		// Add missing bones to parent LODs for skeletal meshes that arrive with malformed LOD setups
		FixLODRequiredBones(NumLODs, SkeletalMesh, GenerationLODData, ComponentSpaceGenerationLODData);

		// Check if the sockets are set to always animate, else the component space requires separated data (different bone indexes)
		bool bCanGenerateSingleBonesList = CheckSkeletalAllMeshSocketsAlwaysAnimate(SkeletalMesh);

		if (bCanGenerateSingleBonesList)
		{
			TArray<FBoneIndexType> LODBoneIndexToMeshBoneIndexMap;
			TArray<FBoneIndexType> ComponentSpaceOrderedBoneList;

			if (bCanGenerateSingleBonesList)
			{
				bCanGenerateSingleBonesList &= GenerateOrderedBoneList(SkeletalMesh, GenerationLODData, LODBoneIndexToMeshBoneIndexMap);
				//bCanGenerateSingleBonesList &= GenerateOrderedBoneList(SkeletalMesh, ComponentSpaceGenerationLODData, ComponentSpaceOrderedBoneList); // Commented : right now we only support skeletal meshes with all the sockets set to always animate
				//bCanGenerateSingleBonesList &= LODBoneIndexToMeshBoneIndexMap == ComponentSpaceOrderedBoneList; // Commented : right now we only support skeletal meshes with all the sockets set to always animate
			}

			if (bCanGenerateSingleBonesList)
			{
				const USkeleton* Skeleton = SkeletalMesh->GetSkeleton();

				OutAnimationReferencePose.GenerationFlags = EReferencePoseGenerationFlags::FastPath;
				OutAnimationReferencePose.SkeletalMeshComponent = SkeletalMeshComponent;
				OutAnimationReferencePose.SkeletalMesh = SkeletalMesh;
				OutAnimationReferencePose.Skeleton = Skeleton;

				TArray<int32> LODNumBones;
				LODNumBones.Reset(NumLODs);
				for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
				{
					LODNumBones.Add(GenerationLODData[LODIndex].RequiredBones.Num());
				}

				// Removing const here because the linkup lazily builds the mapping and caches it
				const FSkeletonToMeshLinkup& LinkupTable = const_cast<USkeleton*>(Skeleton)->FindOrAddMeshLinkupData(SkeletalMesh);

				// Generate a Skeleton to LOD look up table
				const int32 NumSkelBones = Skeleton->GetReferenceSkeleton().GetNum();
				const int32 NumMeshBones = SkeletalMesh->GetRefSkeleton().GetNum();
				const int32 NumOrderedBones = LODBoneIndexToMeshBoneIndexMap.Num();

				TArray<FBoneIndexType> SkeletonBoneIndexToLODBoneIndexMap;
				TArray<FBoneIndexType> MeshBoneIndexToLODBoneIndexMap;
				TArray<FBoneIndexType> LODBoneIndexToSkeletonBoneIndexMap;
				TMap<FName, FBoneIndexType> BoneNameToLODIndexMap;
				TArray<FBoneIndexType> MeshBoneIndexToLODBoneIndex;
				SkeletonBoneIndexToLODBoneIndexMap.Init(INDEX_NONE, NumSkelBones);
				MeshBoneIndexToLODBoneIndexMap.Init(INDEX_NONE, NumMeshBones);
				LODBoneIndexToSkeletonBoneIndexMap.Init(INDEX_NONE, NumOrderedBones);
				BoneNameToLODIndexMap.Reserve(NumOrderedBones);
				MeshBoneIndexToLODBoneIndex.Init(INDEX_NONE, NumMeshBones);

				for (int32 LODBoneIndex = 0; LODBoneIndex < NumOrderedBones; ++LODBoneIndex)
				{
					// The ordered list contains skeletal mesh bone indices sorted by LOD
					const FMeshPoseBoneIndex MeshBoneIndex(LODBoneIndexToMeshBoneIndexMap[LODBoneIndex]);

					// Remap our skeletal mesh bone index into the skeleton bone index we output for
					FSkeletonPoseBoneIndex SkeletonBoneIndex(INDEX_NONE);
					if(LinkupTable.MeshToSkeletonTable.IsValidIndex(MeshBoneIndex.GetInt()))
					{
						SkeletonBoneIndex = FSkeletonPoseBoneIndex(LinkupTable.MeshToSkeletonTable[MeshBoneIndex.GetInt()]);
					}

					if(SkeletonBoneIndexToLODBoneIndexMap.IsValidIndex(SkeletonBoneIndex.GetInt()))
					{
						SkeletonBoneIndexToLODBoneIndexMap[static_cast<FBoneIndexType>(SkeletonBoneIndex.GetInt())] = LODBoneIndex;
						LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex] = static_cast<FBoneIndexType>(SkeletonBoneIndex.GetInt());
					}
					else
					{
						LODBoneIndexToSkeletonBoneIndexMap[LODBoneIndex] = INDEX_NONE;
					}

					if(MeshBoneIndexToLODBoneIndexMap.IsValidIndex(MeshBoneIndex.GetInt()))
					{
						MeshBoneIndexToLODBoneIndexMap[static_cast<FBoneIndexType>(MeshBoneIndex.GetInt())] = LODBoneIndex;
					}

					FName BoneName = SkeletalMesh->GetRefSkeleton().GetBoneName(MeshBoneIndex.GetInt());
					if (BoneName.IsValid())
					{
						BoneNameToLODIndexMap.Add(BoneName, LODBoneIndex);
					}

					MeshBoneIndexToLODBoneIndex[MeshBoneIndex.GetInt()] = LODBoneIndex;
				}

				const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();

				TArray<FBoneIndexType> LODBoneIndexToParentLODBoneIndexMap;
				LODBoneIndexToParentLODBoneIndexMap.Init(INDEX_NONE, NumOrderedBones);

				for (int32 LODBoneIndex = 0; LODBoneIndex < NumOrderedBones; ++LODBoneIndex)
				{
					const int32 MeshBoneIndex = LODBoneIndexToMeshBoneIndexMap[LODBoneIndex];
					const int32 ParentMeshBoneIndex = RefSkeleton.GetParentIndex(MeshBoneIndex);
					const int32 ParentLODBoneIndex = ParentMeshBoneIndex != INDEX_NONE ? MeshBoneIndexToLODBoneIndex[ParentMeshBoneIndex] : INDEX_NONE;

					LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex] = ParentLODBoneIndex;
				}

				UE::Anim::FBulkCurveFlags CurveFlags;
				Skeleton->ForEachCurveMetaData([&CurveFlags](const FName& InCurveName, const FCurveMetaData& InMetaData)
					{
						UE::Anim::ECurveElementFlags Flags = UE::Anim::ECurveElementFlags::None;

						if (InMetaData.Type.bMaterial)
						{
							Flags |= UE::Anim::ECurveElementFlags::Material;
						}

						if (InMetaData.Type.bMorphtarget)
						{
							Flags |= UE::Anim::ECurveElementFlags::MorphTarget;
						}

						if (Flags != UE::Anim::ECurveElementFlags::None)
						{
							// Add curve with any relevant flags to bulk flags
							CurveFlags.Add(InCurveName, Flags);
						}
					});

				// Override any metadata with the skeletal mesh
				if (const UAnimCurveMetaData* MetaDataUserData = SkeletalMesh->GetAssetUserData<UAnimCurveMetaData>())
				{
					UE::Anim::FBulkCurveFlags MeshCurveFlags;

					// Apply morph target flags to any morph curves
					MetaDataUserData->ForEachCurveMetaData([&MeshCurveFlags](FName InCurveName, const FCurveMetaData& InCurveMetaData)
						{
							UE::Anim::ECurveElementFlags Flags = UE::Anim::ECurveElementFlags::None;

							if (InCurveMetaData.Type.bMaterial)
							{
								Flags |= UE::Anim::ECurveElementFlags::Material;
							}

							if (InCurveMetaData.Type.bMorphtarget)
							{
								Flags |= UE::Anim::ECurveElementFlags::MorphTarget;
							}

							if (Flags != UE::Anim::ECurveElementFlags::None)
							{
								MeshCurveFlags.Add(InCurveName, Flags);
							}
						});

					UE::Anim::FNamedValueArrayUtils::Union(CurveFlags, MeshCurveFlags);
				}

				OutAnimationReferencePose.Initialize(
					RefSkeleton,
					{ LODBoneIndexToParentLODBoneIndexMap },
					{ LODBoneIndexToMeshBoneIndexMap },
					{ LODBoneIndexToSkeletonBoneIndexMap },
					{ SkeletonBoneIndexToLODBoneIndexMap },
					{ MeshBoneIndexToLODBoneIndexMap },
					LODNumBones,
					BoneNameToLODIndexMap,
					CurveFlags,
					bCanGenerateSingleBonesList);

				ReferencePoseGenerated = true;
			}
		}
	}

	return ReferencePoseGenerated;
}

/*static*/ void FGenerationTools::GenerateRawLODData(const USkeletalMeshComponent* SkeletalMeshComponent
	, const USkeletalMesh* SkeletalMesh
	, const int32 LODIndex
	, const TIndirectArray<FSkeletalMeshLODRenderData>& LODRenderData
	, TArray<FBoneIndexType>& OutRequiredBones
	, TArray<FBoneIndexType>& OutFillComponentSpaceTransformsRequiredBones)
{
	const FSkeletalMeshLODRenderData& LODModel = LODRenderData[LODIndex];

	if (LODRenderData[LODIndex].RequiredBones.Num() > 0)
	{
		// Start with the LODModel RequiredBones (precalculated LOD data)
		OutRequiredBones = LODModel.RequiredBones;

		// Add the Virtual bones from the skeleton
		USkeletalMeshComponent::GetRequiredVirtualBones(SkeletalMesh, OutRequiredBones);

		// Add any bones used by physics SkeletalBodySetups
		const UPhysicsAsset* const PhysicsAsset = SkeletalMeshComponent != nullptr ? SkeletalMeshComponent->GetPhysicsAsset() : SkeletalMesh->GetPhysicsAsset();
		// If we have a PhysicsAsset, we also need to make sure that all the bones used by it are always updated, as its used
		// by line checks etc. We might also want to kick in the physics, which means having valid bone transforms.
		if (PhysicsAsset)
		{
			USkeletalMeshComponent::GetPhysicsRequiredBones(SkeletalMesh, PhysicsAsset, OutRequiredBones);
		}

		// TODO - Make sure that bones with per-poly collision are also always updated.

		// If we got a SkeletalMeshComponent, we can exclude invisible bones
		if (SkeletalMeshComponent != nullptr)
		{
			USkeletalMeshComponent::ExcludeHiddenBones(SkeletalMeshComponent, SkeletalMesh, OutRequiredBones);
		}


		// Get socket bones set to animate and bones required to fill the component space base transforms
		TArray<FBoneIndexType> NeededBonesForFillComponentSpaceTransforms;
		USkeletalMeshComponent::GetSocketRequiredBones(SkeletalMesh, OutRequiredBones, NeededBonesForFillComponentSpaceTransforms);

		// If we got a SkeletalMeshComponent, we can include shadow shapes referenced bones
		if (SkeletalMeshComponent != nullptr)
		{
			USkeletalMeshComponent::GetShadowShapeRequiredBones(SkeletalMeshComponent, OutRequiredBones);
		}

		// Ensure that we have a complete hierarchy down to those bones.
		// This is needed because when we add bones (i.e. physics), the parent might not be in the list
		FAnimationRuntime::EnsureParentsPresent(OutRequiredBones, SkeletalMesh->GetRefSkeleton());

		OutFillComponentSpaceTransformsRequiredBones.Reset(OutRequiredBones.Num() + NeededBonesForFillComponentSpaceTransforms.Num());
		OutFillComponentSpaceTransformsRequiredBones = OutRequiredBones;

		NeededBonesForFillComponentSpaceTransforms.Sort();
		USkeletalMeshComponent::MergeInBoneIndexArrays(OutFillComponentSpaceTransformsRequiredBones, NeededBonesForFillComponentSpaceTransforms);
		FAnimationRuntime::EnsureParentsPresent(OutFillComponentSpaceTransformsRequiredBones, SkeletalMesh->GetRefSkeleton());
	}
}

/*static*/ void FGenerationTools::GenerateLODData(const USkeletalMeshComponent* SkeletalMeshComponent
	, const USkeletalMesh* SkeletalMesh
	, const int32 StartLOD
	, const int32 NumLODs
	, const TIndirectArray<FSkeletalMeshLODRenderData>& LODRenderData
	, const TArray<FBoneIndexType>& RequiredBones_LOD0
	, TArray<FGenerationLODData>& GenerationLODData
	, TArray<FGenerationLODData>& GenerationComponentSpaceLODData)
{
	for (int32 LODIndex = StartLOD; LODIndex < NumLODs; ++LODIndex)
	{
		TArray<FBoneIndexType>& RequiredBones = GenerationLODData[LODIndex].RequiredBones;
		TArray<FBoneIndexType>& RequiredComponentSpaceBones = GenerationComponentSpaceLODData[LODIndex].RequiredBones;
		GenerateRawLODData(SkeletalMeshComponent, SkeletalMesh, LODIndex, LODRenderData, RequiredBones, RequiredComponentSpaceBones);

		const int32 ParentLODIndex = LODIndex - 1;

		CalculateDifferenceFromParentLOD(LODIndex, GenerationLODData);
		CalculateDifferenceFromParentLOD(LODIndex, GenerationComponentSpaceLODData);
	}
}

// Calculate the bone indexes difference from LOD0 for LODIndex
/*static*/ void FGenerationTools::CalculateDifferenceFromParentLOD(int32 LODIndex, TArray<FGenerationLODData>& GenerationLODData)
{
	const int32 ParentLODIndex = LODIndex - 1;

	const TArray<FBoneIndexType>& RequiredBones_LOD0 = GenerationLODData[0].RequiredBones;
	const TArray<FBoneIndexType>& RequiredBones = GenerationLODData[LODIndex].RequiredBones;
	const TArray<FBoneIndexType>& RequiredBones_ParentLOD = GenerationLODData[ParentLODIndex].RequiredBones;
	
	TArray<FBoneIndexType>& ExcludedBonesFromLOD0 = GenerationLODData[LODIndex].ExcludedBones;
	TArray<FBoneIndexType>& ExcludedBonesFromPrevLOD = GenerationLODData[LODIndex].ExcludedBonesFromPrevLOD;

	DifferenceBoneIndexArrays(RequiredBones_LOD0, RequiredBones, ExcludedBonesFromLOD0);
	DifferenceBoneIndexArrays(RequiredBones_ParentLOD, RequiredBones, ExcludedBonesFromPrevLOD);
}

static FString GetBoneNameSafe(const USkeletalMesh* SkeletalMesh, const uint32 BoneIndex)
{
	FString BoneName;

	if (const USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
	{
		if (Skeleton->GetReferenceSkeleton().IsValidIndex(BoneIndex))
		{
			BoneName = Skeleton->GetReferenceSkeleton().GetBoneName(BoneIndex).ToString();
		}
	}

	return BoneName;
};

void FGenerationTools::FixLODRequiredBones(const int32 NumLODs
	, const USkeletalMesh* SkeletalMesh
	, TArray<FGenerationLODData>& GenerationLODData
	, TArray<FGenerationLODData>& GenerationComponentSpaceLODData)
{
	TArray<FBoneIndexType> MissingBones;

	for (int32 LODIndex = NumLODs - 1; LODIndex > 0; --LODIndex)
	{
		MissingBones.Reset();

		FGenerationLODData& LODData = GenerationLODData[LODIndex];
		FGenerationLODData& PrevLODData = GenerationLODData[LODIndex - 1];

		// Check if all required bones by the current LOD are part of the parent LOD as well.
		// If a bone is present at a lower LOD while missing from the parent LOD, we will automatically add it to the parent LOD.
		for (FBoneIndexType BoneIdx : LODData.RequiredBones)
		{
			if (!PrevLODData.RequiredBones.Contains(BoneIdx))
			{
				MissingBones.Add(BoneIdx);

				UE_LOGF(LogAnimGenerationTools, Warning, "SkeletalMesh LOD %d does not contain bone [%ls] required by LOD %d. Please update the skeletal mesh asset [%ls] or its corresponding LOD settings asset."
					, LODIndex - 1
					, *GetBoneNameSafe(SkeletalMesh, BoneIdx)
					, LODIndex
					, *SkeletalMesh->GetPathName());
			}
		}

		if (MissingBones.Num() > 0) // Update the arrays and keep bones sorted
		{
			FGenerationLODData& ComponentSpacePrevLODData = GenerationLODData[LODIndex - 1];

			USkeletalMeshComponent::MergeInBoneIndexArrays(PrevLODData.RequiredBones, MissingBones);
			USkeletalMeshComponent::MergeInBoneIndexArrays(ComponentSpacePrevLODData.RequiredBones, MissingBones);

			if (LODIndex > 1) // No need to remove excluded bones at LOD 1, as parent is LOD 0 and has none excluded
			{
				const int32 NumElem = MissingBones.Num();
				for (int32 i = NumElem - 1; i >= 0; --i)
				{
					const FBoneIndexType BoneIndex = MissingBones[i];

					PrevLODData.ExcludedBones.Remove(BoneIndex);
					PrevLODData.ExcludedBonesFromPrevLOD.Remove(BoneIndex);
					ComponentSpacePrevLODData.ExcludedBones.Remove(BoneIndex);
					ComponentSpacePrevLODData.ExcludedBonesFromPrevLOD.Remove(BoneIndex);
				}
			}
		}
	}
}

/*static*/ bool FGenerationTools::CheckExcludedBones(const int32 NumLODs
	, const TArray<FGenerationLODData>& GenerationLODData
	, const USkeletalMesh* SkeletalMesh)
{
	bool bCanGenerateSingleBonesList = true;

	for (int32 LODIndex = NumLODs - 1; LODIndex >= 1; --LODIndex)
	{
		const FGenerationLODData& LODData = GenerationLODData[LODIndex];
		const FGenerationLODData& PrevLODData = GenerationLODData[LODIndex - 1];

		const bool bPrevSmaller = (PrevLODData.ExcludedBones.Num() <= LODData.ExcludedBones.Num());
		if (bPrevSmaller == false)
		{
			bCanGenerateSingleBonesList = false;
			UE_LOGF(LogAnimGenerationTools, Warning, "SkeletalMesh %ls canonical ordered bone set can not be stored as single bones list. LOD %d does not contain all the bones of LOD %d", *SkeletalMesh->GetPathName(), LODIndex, LODIndex - 1);
			break;
		}

		for (TArray<FBoneIndexType>::TConstIterator LODExcludedBonesIt(PrevLODData.ExcludedBones); LODExcludedBonesIt; ++LODExcludedBonesIt)
		{
			if (LODData.ExcludedBones.Contains(*LODExcludedBonesIt) == false)
			{
				bCanGenerateSingleBonesList = false;
				UE_LOGF(LogAnimGenerationTools, Warning, "SkeletalMesh %ls canonical ordered bone set can not be stored in LOD order. LOD %d does not contain all the bones of LOD %d, like e.g. '%ls'.", *SkeletalMesh->GetPathName(), LODIndex, LODIndex - 1, *GetBoneNameSafe(SkeletalMesh, *LODExcludedBonesIt));
				break;
			}
		}
	}
	return bCanGenerateSingleBonesList;
}

/*static*/ bool FGenerationTools::GenerateOrderedBoneList(const USkeletalMesh* SkeletalMesh
	, TArray<FGenerationLODData>& GenerationLODData
	, TArray<FBoneIndexType>& OrderedBoneList)
{
	bool bCanFastPath = true;

	OrderedBoneList = GenerationLODData[0].RequiredBones;

	const int32 NumLODs = GenerationLODData.Num();

	// Compute the common set of bones for all LODS (remove excluded bones for LODS > 0)
	for (int32 LODIndex = 1; LODIndex < NumLODs; ++LODIndex)
	{
		const TArray<FBoneIndexType>& ExcludedBonesFromPrevLOD = GenerationLODData[LODIndex].ExcludedBonesFromPrevLOD;

		const int32 NumExcludedBones = ExcludedBonesFromPrevLOD.Num();
		for (int i = NumExcludedBones - 1; i >= 0; --i)
		{
			OrderedBoneList.Remove(ExcludedBonesFromPrevLOD[i]);
		}
	}

	// Add the ExcludedBonesFromPrevLOD of each LOD, in inverse order 
	for (int32 LODIndex = NumLODs - 1; LODIndex > 0; --LODIndex)
	{
		for (TArray<FBoneIndexType>::TConstIterator SetIt(GenerationLODData[LODIndex].ExcludedBonesFromPrevLOD); SetIt; ++SetIt)
		{
			OrderedBoneList.Add(*SetIt);
		}
	}

	// Check if all the bones have the parents before themselves in the array
	const int NumBones = OrderedBoneList.Num();
	for (int i = 0; i < NumBones; ++i)
	{
		const FBoneIndexType BoneIndex = OrderedBoneList[i];
		const int32 BoneIndexParentIndex = SkeletalMesh->GetRefSkeleton().GetParentIndex(BoneIndex);

		if (BoneIndexParentIndex >= 0)
		{
			int32 ParentIndexAtOrderedBoneList = -1;
			OrderedBoneList.Find(BoneIndexParentIndex, ParentIndexAtOrderedBoneList);

			if (ParentIndexAtOrderedBoneList >= i)
			{
				bCanFastPath = false;
				UE_LOGF(LogAnimGenerationTools, Warning, "Warning : SkeletalMesh [%ls] has an invalid LOD setup.", *SkeletalMesh->GetPathName());
				break;
			}
		}
	}

	return bCanFastPath;
}

/**
 *	Utility for taking two arrays of bone indices, which must be strictly increasing, and finding the A - B.
 *	That is - any items left in A, after removing B.
 */
/*static*/ void FGenerationTools::DifferenceBoneIndexArrays(const TArray<FBoneIndexType>& A, const TArray<FBoneIndexType>& B, TArray<FBoneIndexType>& Output)
{
	int32 APos = 0;
	int32 BPos = 0;

	while (APos < A.Num())
	{
		// check if any elements left in B
		if (BPos < B.Num())
		{
			// If A Value < B Value, we have to add A Value to the output (these indexes are not in the substract array)
			if (A[APos] < B[BPos])
			{
				Output.Add(A[APos]);
				APos++;
			}
			// If APos value == BPos value, we have to skip A Value in the output (we want to substract B values from A)
			// We increase BPos as we assume no duplicated indexes in the arrays
			else if (A[APos] == B[BPos])
			{
				APos++;
				BPos++;
			}
			// If APos value > BPos value, we have to increase BPos, until any of the other two conditions are valid again or we finish the elements in B
			else
			{
				BPos++;
			}
		}
		// If B is finished (no more elements), we just keep adding A to the output
		else
		{
			Output.Add(A[APos]);
			APos++;
		}
	}
}

/*static*/ bool FGenerationTools::CheckSkeletalAllMeshSocketsAlwaysAnimate(const USkeletalMesh* SkeletalMesh)
{
	bool bAllSocketsAlwaysAnimate = true;

	TArray<USkeletalMeshSocket*> ActiveSocketList = SkeletalMesh->GetActiveSocketList();
	for (const USkeletalMeshSocket* Socket : ActiveSocketList)
	{
		const int32 BoneIndex = SkeletalMesh->GetRefSkeleton().FindBoneIndex(Socket->BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			if (Socket->bForceAlwaysAnimated == false)
			{
				UE_LOGF(LogSkeletalMesh, Warning, "SkeletalMesh %ls canonical ordered bone set can not be stored as single bones list. Socket [%ls] is not set to always animate.", *SkeletalMesh->GetPathName(), *Socket->GetName());
				bAllSocketsAlwaysAnimate = false;
			}
		}
	}

	return bAllSocketsAlwaysAnimate;
}

// Converts AnimBP pose to AnimNext Pose
// This function expects both poses to have the same LOD (number of bones and indexes)
// The target pose should be assigned to the correct reference pose prior to this call
/*static*/ void FGenerationTools::RemapPose(const FPoseContext& SourcePose, FLODPose& TargetPose)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_RemapPose_FromAnimBP);
	
	const FBoneContainer& BoneContainer = SourcePose.Pose.GetBoneContainer();
	const FReferencePose& RefPose = TargetPose.GetRefPose();
	const TArrayView<const FBoneIndexType> LODBoneIndexes = RefPose.GetLODBoneIndexToMeshBoneIndexMap(TargetPose.LODLevel);
	const int32 NumLODBones = LODBoneIndexes.Num();

	check(TargetPose.GetNumBones() == NumLODBones);

	for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
	{
		// Reference pose holds a list of skeletal mesh bone indices sorted by LOD
		const FMeshPoseBoneIndex MeshBoneIndex(LODBoneIndexes[LODBoneIndex]);

		// Remap our skeletal mesh bone index into the skeleton bone index we output for
		const FSkeletonPoseBoneIndex SkeletonBoneIndex = BoneContainer.GetSkeletonPoseIndexFromMeshPoseIndex(MeshBoneIndex);
		ensure(SkeletonBoneIndex.IsValid());	// We expect the skeletal mesh bone to map to a valid skeleton bone

		// Remap our skeleton bone index into the compact pose bone index we output for
		const FCompactPoseBoneIndex CompactBoneIndex = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonBoneIndex);

		if (ensure(CompactBoneIndex.IsValid()))	// We expect the skeleton bone to map to a valid compact pose bone
		{
			TargetPose.LocalTransformsView[LODBoneIndex] = SourcePose.Pose[CompactBoneIndex];
		}
		else
		{
			// This bone is part of the LOD but isn't part of the required bones
		}
	}
}

// Converts AnimNext pose to AnimBP Pose
// This function expects both poses to have the same LOD (number of bones and indexes)
// The target pose should be assigned to the correct reference pose prior to this call
/*static*/ void FGenerationTools::RemapPose(const FLODPose& SourcePose, FPoseContext& TargetPose)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_RemapPose_ToAnimBP);
	
	const FBoneContainer& BoneContainer = TargetPose.Pose.GetBoneContainer();
	const FReferencePose& RefPose = SourcePose.GetRefPose();
	const TArrayView<const FBoneIndexType> LODBoneIndexes = RefPose.GetLODBoneIndexToMeshBoneIndexMap(SourcePose.LODLevel);
	const int32 NumLODBones = LODBoneIndexes.Num();

	check(SourcePose.GetNumBones() == NumLODBones);

	for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
	{
		// Reference pose holds a list of skeletal mesh bone indices sorted by LOD
		const FMeshPoseBoneIndex MeshBoneIndex(LODBoneIndexes[LODBoneIndex]);

		// Remap our skeletal mesh bone index into the skeleton bone index we output for
		const FSkeletonPoseBoneIndex SkeletonBoneIndex = BoneContainer.GetSkeletonPoseIndexFromMeshPoseIndex(MeshBoneIndex);
		ensure(SkeletonBoneIndex.IsValid());	// We expect the skeletal mesh bone to map to a valid skeleton bone

		// Remap our skeleton bone index into the compact pose bone index we output for
		const FCompactPoseBoneIndex CompactBoneIndex = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonBoneIndex);

		if (ensure(CompactBoneIndex.IsValid()))	// We expect the skeleton bone to map to a valid compact pose bone
		{
			TargetPose.Pose[CompactBoneIndex] = SourcePose.LocalTransformsView[LODBoneIndex];
		}
		else
		{
			// This bone is part of the LOD but isn't part of the required bones
		}
	}
}

// Converts AnimNext pose to local space transform array
// This function expects the output pose to have the same or a greater number of bones (as it may be being calculated
// for a lower LOD
// The target pose should be assigned to the correct reference pose prior to this call, as transforms will not be filled
// in by this call if they are not affected by the current LOD.
/*static*/ void FGenerationTools::RemapPose(const FLODPose& SourcePose, TArrayView<FTransform> TargetTransforms)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_RemapPose_ToLocalTransforms);
	
	const FReferencePose& RefPose = SourcePose.GetRefPose();
	const TArrayView<const FBoneIndexType> LODBoneIndexes = RefPose.GetLODBoneIndexToMeshBoneIndexMap(SourcePose.LODLevel);
	const int32 NumLODBones = LODBoneIndexes.Num();

	check(SourcePose.GetNumBones() == NumLODBones);

#if DEFAULT_SOA && DEFAULT_SOA_VIEW
	const double* RESTRICT RotationPtr = reinterpret_cast<const double*>(SourcePose.LocalTransformsView.Rotations.GetData());
	const double* RESTRICT RotationEndPtr = RotationPtr + NumLODBones * 4;

	// Our SoA buffer is contiguous
	// Because the translations/scales have the same size (FVector), each entry is a fixed offset apart and we can use a single ptr/offset pair
	const double* RESTRICT TranslationPtr = reinterpret_cast<const double*>(SourcePose.LocalTransformsView.Translations.GetData());
	const int64 ScaleOffset = reinterpret_cast<const double*>(SourcePose.LocalTransformsView.Scales3D.GetData()) - TranslationPtr;

	const FBoneIndexType* RESTRICT LODBoneIndexPtr = LODBoneIndexes.GetData();

	while (RotationPtr < RotationEndPtr)
	{
		VectorRegister4Double Rotation = VectorLoadAligned(RotationPtr);
		VectorRegister4Double Translation = VectorSet_W0(VectorLoad(TranslationPtr));
		VectorRegister4Double Scale = VectorSet_W0(VectorLoad(TranslationPtr + ScaleOffset));

		const int32 DestTransformIndex = *LODBoneIndexPtr;
		TargetTransforms[DestTransformIndex] = FTransform(Rotation, Translation, Scale);

		RotationPtr += 4;
		TranslationPtr += 3;
		LODBoneIndexPtr++;
	}
#else
	for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
	{
		TargetTransforms[LODBoneIndexes[LODBoneIndex]] = SourcePose.LocalTransformsView[LODBoneIndex];
	}
#endif
}

void FGenerationTools::RemapAttributes(
	const FLODPose& LODPose,
	const UE::Anim::FHeapAttributeContainer& InAttributes,
	UE::Anim::FMeshAttributeContainer& OutAttributes)
{
	/** LODPose index to MeshBone index */
	TMap<FCompactPoseBoneIndex, FMeshPoseBoneIndex> LODToMeshBoneIndexMapping;
	const uint32 NumberOfTypes = InAttributes.GetUniqueTypes().Num();

	const TArrayView<const FBoneIndexType> LODToMeshBoneMapping = LODPose.RefPose->GetLODBoneIndexToMeshBoneIndexMap(LODPose.LODLevel);
	for (uint32 TypeIndex = 0; TypeIndex < NumberOfTypes; ++TypeIndex)
	{
		const TArray<int32>& BoneIndices = InAttributes.GetUniqueTypedBoneIndices(TypeIndex);

		for (const int32& BoneIndex : BoneIndices)
		{
			const FBoneIndexType RemappedIndex = LODToMeshBoneMapping[BoneIndex];
			LODToMeshBoneIndexMapping.Add(FCompactPoseBoneIndex(BoneIndex), FMeshPoseBoneIndex(RemappedIndex));
		}
	}

	OutAttributes.CopyFrom(InAttributes, LODToMeshBoneIndexMapping);
}

void FGenerationTools::RemapAttributes(const FLODPose& LODPose, const UE::Anim::FStackAttributeContainer& InAttributes, UE::Anim::FMeshAttributeContainer& OutAttributes)
{
	/** LODPose index to MeshBone index */
	TMap<FCompactPoseBoneIndex, FMeshPoseBoneIndex> LODToMeshBoneIndexMapping;
	const uint32 NumberOfTypes = InAttributes.GetUniqueTypes().Num();

	const TArrayView<const FBoneIndexType> LODToMeshBoneMapping = LODPose.RefPose->GetLODBoneIndexToMeshBoneIndexMap(LODPose.LODLevel);
	for (uint32 TypeIndex = 0; TypeIndex < NumberOfTypes; ++TypeIndex)
	{
		const TArray<int32>& BoneIndices = InAttributes.GetUniqueTypedBoneIndices(TypeIndex);

		for (const int32& BoneIndex : BoneIndices)
		{
			const FBoneIndexType RemappedIndex = LODToMeshBoneMapping[BoneIndex];
			LODToMeshBoneIndexMapping.Add(FCompactPoseBoneIndex(BoneIndex), FMeshPoseBoneIndex(RemappedIndex));
		}
	}

	OutAttributes.CopyFrom(InAttributes, LODToMeshBoneIndexMapping);
}

void FGenerationTools::RemapAttributes(const FLODPose& LODPose, const UE::Anim::FMeshAttributeContainer& InAttributes, UE::Anim::FStackAttributeContainer& OutAttributes)
{
	/** MeshBone to LODPose index */
	TMap<FMeshPoseBoneIndex, FCompactPoseBoneIndex> MeshBoneToLODIndexMapping;
	const uint32 NumberOfTypes = InAttributes.GetUniqueTypes().Num();

	const int32 NumLODBones = LODPose.RefPose->GetNumBonesForLOD(LODPose.LODLevel);
	const TArrayView<const FBoneIndexType> MeshBoneToLODMapping = LODPose.RefPose->GetMeshBoneIndexToLODBoneIndexMap();
	for (uint32 TypeIndex = 0; TypeIndex < NumberOfTypes; ++TypeIndex)
	{
		const TArray<int32>& BoneIndices = InAttributes.GetUniqueTypedBoneIndices(TypeIndex);

		for (const int32& BoneIndex : BoneIndices)
		{
			const FBoneIndexType RemappedIndex = MeshBoneToLODMapping[BoneIndex];
			if (RemappedIndex < NumLODBones)
			{
				MeshBoneToLODIndexMapping.Add(FMeshPoseBoneIndex(BoneIndex), FCompactPoseBoneIndex(RemappedIndex));
			}
		}
	}

	OutAttributes.CopyFrom(InAttributes, MeshBoneToLODIndexMapping);
}

template<typename BoneIndexType, typename InAllocator>
static void RemapAttributesImpl(
	const FLODPose& LODPose,
	const UE::Anim::TAttributeContainer<BoneIndexType, InAllocator>& InAttributes,
	FPoseContext& OutPose)
{
	const TArrayView<const FBoneIndexType> LODBoneIndexToMeshBoneIndexMap = LODPose.GetLODBoneIndexToMeshBoneIndexMap();
	const FBoneContainer& BoneContainer = OutPose.Pose.GetBoneContainer();

	for (const TWeakObjectPtr<UScriptStruct> WeakScriptStruct : InAttributes.GetUniqueTypes())
	{
		const UScriptStruct* ScriptStruct = WeakScriptStruct.Get();
		const int32 TypeIndex = InAttributes.FindTypeIndex(ScriptStruct);
		if (TypeIndex != INDEX_NONE)
		{
			const TArray<UE::Anim::TWrappedAttribute<InAllocator>, FDefaultAllocator>& SourceValues = InAttributes.GetValues(TypeIndex);
			const TArray<UE::Anim::FAttributeId, FDefaultAllocator>& AttributeIds = InAttributes.GetKeys(TypeIndex);

			// Try and remap all the source attributes to their respective new bone indices
			for (int32 EntryIndex = 0; EntryIndex < AttributeIds.Num(); ++EntryIndex)
			{
				const UE::Anim::FAttributeId& AttributeId = AttributeIds[EntryIndex];

				const int32 LODBoneIndex = AttributeId.GetIndex();
				const FMeshPoseBoneIndex MeshBoneIndex(LODBoneIndexToMeshBoneIndexMap[LODBoneIndex]);

				// Remap our skeletal mesh bone index into the skeleton bone index we output for
				const FSkeletonPoseBoneIndex SkeletonBoneIndex = BoneContainer.GetSkeletonPoseIndexFromMeshPoseIndex(MeshBoneIndex);
				ensure(SkeletonBoneIndex.IsValid());	// We expect the skeletal mesh bone to map to a valid skeleton bone

				// Remap our skeleton bone index into the compact pose bone index we output for
				const FCompactPoseBoneIndex CompactBoneIndex = BoneContainer.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonBoneIndex);

				if (ensure(CompactBoneIndex.IsValid()))	// We expect the skeleton bone to map to a valid compact pose bone
				{
					const UE::Anim::FAttributeId NewInfo(AttributeId.GetName(), CompactBoneIndex);
					uint8* NewAttribute = OutPose.CustomAttributes.FindOrAdd(ScriptStruct, NewInfo);
					ScriptStruct->CopyScriptStruct(NewAttribute, SourceValues[EntryIndex].template GetPtr<void>());
				}
				else
				{
					// This bone is part of the LOD but isn't part of the required bones
				}
			}
		}
	}
}

void FGenerationTools::RemapAttributes(
	const FLODPose& LODPose,
	const UE::Anim::FHeapAttributeContainer& InAttributes,
	FPoseContext& OutPose)
{
	RemapAttributesImpl(LODPose, InAttributes, OutPose);
}

void FGenerationTools::RemapAttributes(
	const FLODPose& LODPose,
	const UE::Anim::FStackAttributeContainer& InAttributes,
	FPoseContext& OutPose)
{
	RemapAttributesImpl(LODPose, InAttributes, OutPose);
}

template<typename InBoneIndexType, typename InAllocator, typename OutBoneIndexType, typename OutAllocator>
static void RemapCompactPoseAttributesToLODPoseAttributes(
	const FBoneContainer& InBoneContainer,
	const UE::Anim::TAttributeContainer<InBoneIndexType, InAllocator>& InAttributes,
	const TArrayView<const FBoneIndexType>& SkeletonBoneIndexToLODBoneIndexMap,
	UE::Anim::TAttributeContainer<OutBoneIndexType, OutAllocator>& OutAttributes
	)
{
	for (const TWeakObjectPtr<UScriptStruct> WeakScriptStruct : InAttributes.GetUniqueTypes())
	{
		const UScriptStruct* ScriptStruct = WeakScriptStruct.Get();
		const int32 TypeIndex = InAttributes.FindTypeIndex(ScriptStruct);
		if (TypeIndex != INDEX_NONE)
		{
			const TArray<UE::Anim::TWrappedAttribute<InAllocator>, FDefaultAllocator>& SourceValues = InAttributes.GetValues(TypeIndex);
			const TArray<UE::Anim::FAttributeId, FDefaultAllocator>& AttributeIds = InAttributes.GetKeys(TypeIndex);

			// Try and remap all the source attributes to their respective new bone indices
			for (int32 EntryIndex = 0; EntryIndex < AttributeIds.Num(); ++EntryIndex)
			{
				const UE::Anim::FAttributeId& AttributeId = AttributeIds[EntryIndex];

				// Obtain the compact pose bone index from the attribute
				const FCompactPoseBoneIndex CompactBoneIndex(AttributeId.GetIndex());
				ensure(CompactBoneIndex.IsValid());

				// Remap the compact pose bone index to the skeleton index
				const FSkeletonPoseBoneIndex SkeletonBoneIndex = InBoneContainer.GetSkeletonPoseIndexFromCompactPoseIndex(CompactBoneIndex);
				ensure(SkeletonBoneIndex.IsValid());	// We expect the skeletal mesh bone to map to a valid skeleton bone
				
				// Remap our skeleton bone index into the LOD Pose bone index we output for
				const FBoneIndexType& LODBoneIndex = SkeletonBoneIndexToLODBoneIndexMap[SkeletonBoneIndex.GetInt()];
				if (ensure(LODBoneIndex != INDEX_NONE))	// We expect the skeleton bone to map to a valid LOD Pose bone index
				{
					const UE::Anim::FAttributeId NewInfo(AttributeId.GetName(), LODBoneIndex, AttributeId.GetNamespace());
					uint8* NewAttribute = OutAttributes.FindOrAdd(ScriptStruct, NewInfo);
					ScriptStruct->CopyScriptStruct(NewAttribute, SourceValues[EntryIndex].template GetPtr<void>());
				}
				else
				{
					// This bone is part of the LOD but isn't part of the required bones
				}
			}
		}
	}
}

void FGenerationTools::RemapAttributes(
	const FPoseContext& OutPose,
	const FLODPose& LODPose,
	UE::Anim::FHeapAttributeContainer& OutAttributes
	)
{
	RemapCompactPoseAttributesToLODPoseAttributes(OutPose.Pose.GetBoneContainer(), OutPose.CustomAttributes, LODPose.GetSkeletonBoneIndexToLODBoneIndexMap(), OutAttributes);
}

void FGenerationTools::RemapAttributes(
	const FPoseContext& OutPose,
	const FLODPose& LODPose,
	UE::Anim::FStackAttributeContainer& OutAttributes
	)
{
	RemapCompactPoseAttributesToLODPoseAttributes(OutPose.Pose.GetBoneContainer(), OutPose.CustomAttributes, LODPose.GetSkeletonBoneIndexToLODBoneIndexMap(), OutAttributes);
}

void FGenerationTools::ConvertLocalSpaceToComponentSpace(
												 TConstArrayView<FBoneIndexType> InMeshBoneIndexToParentMeshBoneIndexMap,
												 TConstArrayView<FTransform> InBoneSpaceTransforms,
												 TConstArrayView<FBoneIndexType> InLODBoneIndexToMeshBoneIndexMap,
												 TArrayView<FTransform> OutComponentSpaceTransforms)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_ConvertLocalSpaceToComponentSpace);

	checkf(InMeshBoneIndexToParentMeshBoneIndexMap.Num() == InBoneSpaceTransforms.Num(), TEXT("Buffer mismatch: %d:%d"), InMeshBoneIndexToParentMeshBoneIndexMap.Num(), InBoneSpaceTransforms.Num());
	checkf(InMeshBoneIndexToParentMeshBoneIndexMap.Num() == OutComponentSpaceTransforms.Num(), TEXT("Buffer mismatch: %d:%d"), InMeshBoneIndexToParentMeshBoneIndexMap.Num(), OutComponentSpaceTransforms.Num());

	const FBoneIndexType* RESTRICT LODBoneIndexToMeshBoneIndexMapPtr = InLODBoneIndexToMeshBoneIndexMap.GetData();
	const FBoneIndexType* RESTRICT MeshBoneIndexToParentMeshBoneIndexMapPtr = InMeshBoneIndexToParentMeshBoneIndexMap.GetData();
	const FTransform* RESTRICT LocalTransformsData = InBoneSpaceTransforms.GetData();
	FTransform* RESTRICT ComponentSpaceData = OutComponentSpaceTransforms.GetData();

	// First bone (if we have one) is always root bone, and it doesn't have a parent.
	{
		check(InLODBoneIndexToMeshBoneIndexMap.Num() == 0 || InLODBoneIndexToMeshBoneIndexMap[0] == 0);
		OutComponentSpaceTransforms[0] = InBoneSpaceTransforms[0];
	}

	const int32 NumLODBones = InLODBoneIndexToMeshBoneIndexMap.Num();
	for (int32 LODBoneIndex = 1; LODBoneIndex < NumLODBones; ++LODBoneIndex)
	{
		const FBoneIndexType MeshBoneIndex = LODBoneIndexToMeshBoneIndexMapPtr[LODBoneIndex];
		const FBoneIndexType ParentMeshBoneIndex = MeshBoneIndexToParentMeshBoneIndexMapPtr[MeshBoneIndex];

		FTransform* RESTRICT ComponentSpaceTransform = ComponentSpaceData + MeshBoneIndex;
		const FTransform* RESTRICT ParentComponentSpaceTransform = ComponentSpaceData + ParentMeshBoneIndex;
		const FTransform* RESTRICT LocalSpaceTransform = LocalTransformsData + MeshBoneIndex;

		FTransform::Multiply(ComponentSpaceTransform, LocalSpaceTransform, ParentComponentSpaceTransform);

		ComponentSpaceTransform->NormalizeRotation();

		checkSlow(ComponentSpaceTransform->IsRotationNormalized());
		checkSlow(!ComponentSpaceTransform->ContainsNaN());
	}
}

} // end namespace UE::UAF



namespace UE::UAF::Tools::ConsoleCommands
{
	struct FHelper
	{
		static TArray<FBoneIndexType> ComputeExcludedBones(const USkeletalMesh* SkeletalMesh, const TArray<FBoneIndexType>& LODRequiredBones, const TArray<FBoneIndexType>& NextLODRequiredBones)
		{
			TArray<FBoneIndexType> ExcludedBones;

			UE::UAF::FGenerationTools::DifferenceBoneIndexArrays(LODRequiredBones, NextLODRequiredBones, ExcludedBones);

			return ExcludedBones;
		}

		static void CheckSkeletalMeshesLODs()
		{
			using namespace UE::UAF;

			TArray<FAssetData> Assets;
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
			AssetRegistry.GetAssetsByClass(USkeletalMesh::StaticClass()->GetClassPathName(), Assets);

			const int32 NumAssets = Assets.Num();
			for (int32 Idx = 0; Idx < NumAssets; Idx++)
			{
				USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Assets[Idx].GetAsset());
				if (SkeletalMesh != nullptr)
				{
					UE::UAF::FReferencePose OutAnimationReferencePose;

					if (FGenerationTools::GenerateReferencePose(nullptr, SkeletalMesh, OutAnimationReferencePose))
					{
						UE_LOGF(LogAnimGenerationTools, VeryVerbose, "[%d of %d] SkeletalMesh %ls BoneReferencePose generated.", Idx + 1, NumAssets, *SkeletalMesh->GetPathName());
					}
				}
				else
				{
					UE_LOGF(LogAnimGenerationTools, VeryVerbose, "[%d of %d] SkeletalMesh is null. Asset : %ls could not be loaded.", Idx + 1, NumAssets, *Assets[Idx].AssetName.ToString());
				}
			}
		}
	};


	FAutoConsoleCommand CheckSkeletalMeshesLODs(TEXT("uaf.tools.checkskeletalmesheslods"), TEXT(""), FConsoleCommandDelegate::CreateLambda([]()
	{
		FHelper::CheckSkeletalMeshesLODs();
	}));
} // end namespace UE::UAF::Tools::ConsoleCommands
