// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNext_LODPose.h"
#include "RemapPoseDataPool.h"
#include "ReferencePose.h"

void FAnimNextGraphLODPose::CopyPose(
	const UE::UAF::FLODPoseHeap& InSourcePose, const FBlendedHeapCurve& InSourceCurves, const UE::Anim::FHeapAttributeContainer& InSourceAttributes,
	UE::UAF::FLODPoseHeap& OutTargetPose, FBlendedHeapCurve& OutTargetCurves, UE::Anim::FHeapAttributeContainer& OutTargetAttributes)
{
	const UE::UAF::FReferencePose* SourceRef = InSourcePose.RefPose;
	const UE::UAF::FReferencePose* TargetRef = OutTargetPose.RefPose;

	if (SourceRef != nullptr && TargetRef != nullptr && UE::UAF::FRemapPoseDataPool::NeedsRemapping(*SourceRef, *TargetRef))
	{
		const FRemapPoseData& RemapData = UE::UAF::FRemapPoseDataPool::Get().GetRemapData(*SourceRef, *TargetRef);
		RemapData.RemapPose(InSourcePose, OutTargetPose);
		RemapData.RemapAttributes(InSourcePose, InSourceAttributes, OutTargetPose, OutTargetAttributes);
	}
	else
	{
		OutTargetPose.CopyFrom(InSourcePose);
		OutTargetAttributes.CopyFrom(InSourceAttributes);
	}

	OutTargetCurves.CopyFrom(InSourceCurves);
}

void FAnimNextGraphLODPose::CopyFrom(const UE::UAF::FLODPoseHeap& InPose, const FBlendedHeapCurve& InCurves, const UE::Anim::FHeapAttributeContainer& InAttributes)
{
	CopyPose(InPose, InCurves, InAttributes, LODPose, Curves, Attributes);
}

void FAnimNextGraphLODPose::CopyFrom(const FAnimNextGraphLODPose& Source)
{
	CopyFrom(Source.LODPose, Source.Curves, Source.Attributes);
}
