// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_AnimNextBeginExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextBeginExecution)

FName FRigUnit_AnimNextBeginExecution::EventName = TEXT("Update");

FRigUnit_AnimNextBeginExecution_Execute()
{
}
