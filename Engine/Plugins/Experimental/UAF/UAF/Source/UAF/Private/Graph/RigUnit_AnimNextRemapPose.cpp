// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_AnimNextRemapPose.h"

#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "UAFLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextRemapPose)
DEFINE_STAT(STAT_AnimNext_RigUnit_RemapPose);

FRigUnit_AnimNextRemapPose_Execute()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_RigUnit_RemapPose);

	using namespace UE::UAF;

	if(!TargetAnimGraphRefPose.ReferencePose.IsValid())
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not remap pose - Target Ref Pose is not valid."));
		return;
	}
	
	if(!Pose.LODPose.IsValid())
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not remap pose - LOD Pose is not valid."));
		return;
	}

	FMemMark MemMark(FMemStack::Get());

	const UE::UAF::FReferencePose& TargetRefPose = TargetAnimGraphRefPose.ReferencePose.GetRef<UE::UAF::FReferencePose>();

	// CopyFrom handles both same-skeleton (fast copy) and cross-skeleton (remap via pool) transparently
	Result.LODPose.PrepareForLOD(TargetRefPose, Pose.LODPose.LODLevel, /*bSetRefPose=*/false, Pose.LODPose.IsAdditive());
	Result.CopyFrom(Pose);
}
