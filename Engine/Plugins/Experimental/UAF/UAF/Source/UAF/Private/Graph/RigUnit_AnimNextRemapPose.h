// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "AnimNextStats.h"

#include "RigUnit_AnimNextRemapPose.generated.h"

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF RigUnit: Remap Pose"), STAT_AnimNext_RigUnit_RemapPose, STATGROUP_AnimNext, UAF_API);

/** Remaps an anim graph pose from to another skeletal mesh component.
 *  Note: Cross-skeleton remapping now happens automatically in FAnimNextGraphLODPose::CopyFrom
 *  and FKeyframeState::CopyFrom via the FRemapPoseDataPool. This node is only needed for
 *  explicit control over where remapping occurs in the system and might be deprecated later on.
 */
USTRUCT(meta=(DisplayName="Remap Pose", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="Output,Pose,Port"))
struct FRigUnit_AnimNextRemapPose : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// Pose to read
	UPROPERTY(EditAnywhere, DisplayName = "Input Pose", Category = "Graph", meta = (Input))
	FAnimNextGraphLODPose Pose;

	UPROPERTY(EditAnywhere, DisplayName = "Converted Pose", Category = "Graph", meta = (Output))
	FAnimNextGraphLODPose Result;

	UPROPERTY(EditAnywhere, DisplayName = "Target Ref Pose", Category = "Graph", meta = (Input))
	FAnimNextGraphReferencePose TargetAnimGraphRefPose;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;

};
