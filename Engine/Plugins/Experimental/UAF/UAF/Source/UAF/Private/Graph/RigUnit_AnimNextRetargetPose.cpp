// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_AnimNextRetargetPose.h"

#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "AnimationRuntime.h"
#include "RemapPoseDataPool.h"
#include "RetargetingTools.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/Skeleton.h"
#include "Animation/SkeletonRemapping.h"
#include "UAFLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextRetargetPose)
DEFINE_STAT(STAT_AnimNext_RigUnit_RetargetPose);

FRigUnit_AnimNextRetargetPose_Execute()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_RigUnit_RetargetPose);

	using namespace UE::UAF;

	if(!TargetAnimGraphRefPose.ReferencePose.IsValid())
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not retarget pose - Target Ref Pose is not valid."));
		return;
	}

	if (!Pose.LODPose.IsValid())
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not retarget pose - LOD Pose is not valid."));
		return;
	}

	FMemMark MemMark(FMemStack::Get());

	const UE::UAF::FLODPoseHeap& SourcePose = Pose.LODPose;
	const UE::UAF::FReferencePose& SourceRefPose = SourcePose.GetRefPose();
	const USkeletalMesh* SourceMesh = SourceRefPose.SkeletalMesh.Get();

	const UE::UAF::FReferencePose& TargetRefPose = TargetAnimGraphRefPose.ReferencePose.GetRef<UE::UAF::FReferencePose>();
	UE::UAF::FLODPoseHeap& TargetPose = Result.LODPose;
	const USkeletalMesh* TargetMesh = TargetRefPose.SkeletalMesh.Get();
	if (SourceMesh == TargetMesh)
	{
		TargetPose.PrepareForLOD(TargetRefPose, SourcePose.LODLevel, /*bSetRefPose=*/false, SourcePose.IsAdditive());
		// Just copy the pose to the target in case there is nothing to convert.
		TargetPose.CopyFrom(SourcePose);

		// No need to retarget, early out.
		return;
	}

	const USkeleton* SourceSkeleton = SourceMesh->GetSkeleton();
	const USkeleton* TargetSkeleton = TargetMesh->GetSkeleton();
	
	const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);
	const bool bUseSourceRetargetModes = TargetSkeleton->GetUseRetargetModesFromCompatibleSkeleton();
	const bool bDisableRetargeting = TargetPose.GetDisableRetargeting();
	
	// 1) Remap bones + attributes + curves in one call via the pool.
	Result.LODPose.PrepareForLOD(TargetRefPose, SourcePose.LODLevel, /*bSetRefPose=*/false, SourcePose.IsAdditive());
	Result.CopyFrom(Pose);

	// 2) Get remap data from the pool for retarget steps (O(1) lookup, already cached by CopyFrom).
	const FRemapPoseData& RemapPoseData = UE::UAF::FRemapPoseDataPool::Get().GetRemapData(SourceRefPose, TargetRefPose);

	// 3) Build or update per-bone retargeting pairs for current source/target LODs.
	if (RetargetPoseData.PairsInitForSourcePoseLOD != SourcePose.LODLevel ||
		RetargetPoseData.PairsInitForTargetPoseLOD != TargetPose.LODLevel)
	{
		FRetargetingTools::BuildRetargetingPairs(
			SourcePose, TargetPose,
			RemapPoseData,
			SourceSkeleton, TargetSkeleton,
			bUseSourceRetargetModes,
			bDisableRetargeting,
			RetargetPoseData.SkeletonPairs, RetargetPoseData.AnimScalePairs, RetargetPoseData.AnimRelativePairs, RetargetPoseData.OrientAndScalePairs);

		RetargetPoseData.PairsInitForSourcePoseLOD = SourcePose.LODLevel;
		RetargetPoseData.PairsInitForTargetPoseLOD = TargetPose.LODLevel;
	}

	// 4) Retarget reference pose to align target skeleton base transforms.
	FRetargetingTools::ReferencePoseRetarget(
		SourceSkeleton,
		TargetSkeleton,
		SkeletonRemapping,
		bUseSourceRetargetModes,
		bDisableRetargeting,
		TargetPose);

	// 5) Apply per-bone translation retargeting using retargeting pairs.
	FRetargetingTools::RetargetPose(
		TargetRefPose,
		SkeletonRemapping,
		RetargetPoseData.SkeletonPairs, RetargetPoseData.AnimScalePairs, RetargetPoseData.AnimRelativePairs, RetargetPoseData.OrientAndScalePairs,
		SourceSkeleton->GetRefLocalPoses(),
		TargetPose);
}