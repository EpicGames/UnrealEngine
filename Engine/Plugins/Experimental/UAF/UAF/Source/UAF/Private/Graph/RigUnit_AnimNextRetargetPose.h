// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "AnimEncoding.h"
#include "AnimNextStats.h"

#include "RigUnit_AnimNextRetargetPose.generated.h"

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF RigUnit: Retarget Pose"), STAT_AnimNext_RigUnit_RetargetPose, STATGROUP_AnimNext, UAF_API);

USTRUCT(BlueprintType)
struct FRetargetPoseData
{
	GENERATED_BODY()

	BoneTrackArray SkeletonPairs;
	BoneTrackArray AnimScalePairs;
	BoneTrackArray AnimRelativePairs;
	BoneTrackArray OrientAndScalePairs;

	// Pairs initialized for the given source and target pose LODs.
	// These are used to determine if we need to re-initialize the pairs after a LOD switch in either of the two poses.
	int32 PairsInitForSourcePoseLOD = -1;
	int32 PairsInitForTargetPoseLOD = -1;
};

/** Translational retargeting node. */
USTRUCT(meta=(DisplayName="Retarget Pose", Category="Animation Graph", NodeColor="1, 0, 1", Keywords="Output,Pose,Port"))
struct FRigUnit_AnimNextRetargetPose : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();
	
	UPROPERTY(EditAnywhere, DisplayName = "Source Pose", Category = "Graph", meta = (Input))
	FAnimNextGraphLODPose Pose;

	UPROPERTY(EditAnywhere, DisplayName = "Retargeted Pose", Category = "Graph", meta = (Output))
	FAnimNextGraphLODPose Result;

	UPROPERTY(EditAnywhere, DisplayName = "Target Ref Pose", Category = "Graph", meta = (Input))
	FAnimNextGraphReferencePose TargetAnimGraphRefPose;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;

	UPROPERTY(meta = (Hidden))
	FRetargetPoseData RetargetPoseData;
};
