// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_AnimNextWriteSkeletalMeshComponentPose.h"

#include "AnimationRuntime.h"
#include "Animation/NamedValueArray.h"
#include "AnimNextStats.h"
#include "GenerationTools.h"
#include "Component/SkinnedMeshComponentExtensions.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Logging/StructuredLog.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/AnimNextSkeletalMeshComponentReferenceComponent.h"
#include "UAFLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextWriteSkeletalMeshComponentPose)

DEFINE_STAT(STAT_AnimNext_Write_Pose);

FRigUnit_AnimNextWriteSkeletalMeshComponentPose_Execute()
{
	USkeletalMeshComponent* OutputComponent = SkeletalMeshComponent;

	// Defer to module component if no component is supplied
	if(OutputComponent == nullptr)
	{
		const FAnimNextModuleContextData& ModuleContextData = ExecuteContext.GetContextData<FAnimNextModuleContextData>();
		FAnimNextModuleInstance& ModuleInstance = ModuleContextData.GetModuleInstance();
		const FAnimNextSkeletalMeshComponentReferenceComponent& ComponentReference = ModuleInstance.GetOrAddComponent<FAnimNextSkeletalMeshComponentReferenceComponent>();
		OutputComponent = ComponentReference.GetComponent();
	}

	WritePose(OutputComponent, Pose);
}

void FRigUnit_AnimNextWriteSkeletalMeshComponentPose::WritePose(USkeletalMeshComponent* InOutputComponent, const FAnimNextGraphLODPose& InPose)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_Write_Pose);

	using namespace UE::UAF;
	FTaskTagScope TaskTag(ETaskTag::EParallelGameThread);

	if(InOutputComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "Could not write to skeletal mesh component - No valid skeletal mesh component found.");
		return;
	}

	if(!InPose.LODPose.IsValid())
	{
		UE_LOGF(LogAnimation, Warning, "Could not write to skeletal mesh component - LOD Pose is not valid.");
		return;
	}
	
	USkeletalMesh* SkeletalMesh = InOutputComponent->GetSkeletalMeshAsset();
	if(SkeletalMesh == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "Could not write to skeletal mesh component - No valid SkeletalMesh on [SK: %ls][Owner: %ls]",
			*InOutputComponent->GetName(), *GetNameSafe(InOutputComponent->GetOuter()));
		return;
	}

	// We cannot write to the skeletal mesh component if it is using anim BP
	if(InOutputComponent->bEnableAnimation)
	{
		UE_LOGF(LogAnimation, Warning, "Could not write to skeletal mesh component - bEnableAnimation is true[SK: %ls][Owner: %ls]",
			*InOutputComponent->GetName(), *GetNameSafe(InOutputComponent->GetOuter()));
		return;
	}

	FDataHandle RefPoseHandle = FDataRegistry::Get()->GetOrGenerateReferencePose(InOutputComponent);
	const UE::UAF::FReferencePose& RefPose = RefPoseHandle.GetRef<UE::UAF::FReferencePose>();

	// We cannot use two different reference poses because we want to avoid an expensive remap operation
	// You should remap the pose explicitly if this is what you need
	if (&RefPose != &InPose.LODPose.GetRefPose())
	{
		UE_LOGF(LogAnimation, Warning, "Could not write to skeletal mesh component - The input pose and the skeletal mesh component use different reference poses [SK: %ls][Owner: %ls]",
			*SkeletalMesh->GetName(), *GetNameSafe(InOutputComponent->GetOuter()));

		return;
	}

	FMemMark MemMark(FMemStack::Get());
	USkinnedMeshComponent::FRenderStateLockScope LockScope(InOutputComponent);

	FBlendedHeapCurve PrevCurves;

	// Clear our curves and attributes or we'll have leftovers from our last write.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Swap(PrevCurves, InOutputComponent->AnimCurves);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	InOutputComponent->GetEditableCustomAttributes().Empty();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InOutputComponent->AnimCurves.CopyFrom(InPose.Curves);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InOutputComponent->GetEditableAnimationCurves().CopyFrom(InPose.Curves);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	TUniqueFunction<void(USkinnedMeshComponent*)> GameThreadCallback;

	// Extract our material parameters and morph target curves
	if (InPose.Curves.Num() != 0 && RefPose.CurveFlags.Num() != 0)
	{
		FBlendedCurve ExpiredCurves;
		ExpiredCurves.Reserve(64);			// Expired curves are uncommon

		// Reset any material curves we produced in the last update but not in this update
		UE::Anim::FNamedValueArrayUtils::Subtraction(PrevCurves, InPose.Curves,
			[&ExpiredCurves](const UE::Anim::FCurveElement& InCurveElement)
			{
				ExpiredCurves.Add(InCurveElement);
			});

		FBlendedHeapCurve ExpiredMaterialCurves;
		ExpiredMaterialCurves.Reserve(ExpiredCurves.Num());

		FBlendedHeapCurve MaterialCurves;
		MaterialCurves.Reserve(RefPose.CurveFlags.Num());

		TMap<FName, float> MorphTargetCurves;
		MorphTargetCurves.Reserve(RefPose.CurveFlags.Num());

		UE::Anim::FNamedValueArrayUtils::Intersection(ExpiredCurves, RefPose.CurveFlags,
			[&ExpiredMaterialCurves](const UE::Anim::FCurveElement& InCurveElement, const UE::Anim::FCurveElementFlags& InCurveFlagsElement)
			{
				if (EnumHasAnyFlags(InCurveFlagsElement.Flags | InCurveElement.Flags, UE::Anim::ECurveElementFlags::Material))
				{
					ExpiredMaterialCurves.Add(InCurveElement);
				}
			});

		UE::Anim::FNamedValueArrayUtils::Intersection(InPose.Curves, RefPose.CurveFlags,
			[&MaterialCurves, &MorphTargetCurves](const UE::Anim::FCurveElement& InCurveElement, const UE::Anim::FCurveElementFlags& InCurveFlagsElement)
			{
				if (EnumHasAnyFlags(InCurveFlagsElement.Flags | InCurveElement.Flags, UE::Anim::ECurveElementFlags::MorphTarget))
				{
					MorphTargetCurves.Add(InCurveElement.Name, InCurveElement.Value);
				}

				if (EnumHasAnyFlags(InCurveFlagsElement.Flags | InCurveElement.Flags, UE::Anim::ECurveElementFlags::Material))
				{
					MaterialCurves.Add(InCurveElement);
				}
			});

		if (ExpiredMaterialCurves.Num() != 0 || MaterialCurves.Num() != 0 || !MorphTargetCurves.IsEmpty())
		{
			GameThreadCallback = [
				ExpiredMaterialCurves = MoveTemp(ExpiredMaterialCurves),
				MaterialCurves = MoveTemp(MaterialCurves),
				MorphTargetCurves = MoveTemp(MorphTargetCurves)
				](USkinnedMeshComponent* InComponent)
				{
					USkeletalMeshComponent* SMComponent = CastChecked<USkeletalMeshComponent>(InComponent);

					SMComponent->ResetMorphTargetCurves();

					if (!MorphTargetCurves.IsEmpty())
					{
						// TODO: We should avoid using a TMap for this, a named value array would be a better fit
						USkeletalMesh* SkeletalMesh = SMComponent->GetSkeletalMeshAsset();
						FAnimationRuntime::AppendActiveMorphTargets(SkeletalMesh, MorphTargetCurves, SMComponent->ActiveMorphTargets, SMComponent->MorphTargetWeights);
					}

					// now update morph target curves that are added via SetMorphTarget
					SMComponent->UpdateMorphTargetOverrideCurves();

					// Reset material curves that are no longer animated to their default value
					ExpiredMaterialCurves.ForEachElement([SMComponent](const UE::Anim::FCurveElement& MaterialCurve)
						{
							float DefaultValue = SMComponent->GetScalarParameterDefaultValue(MaterialCurve.Name);
							SMComponent->SetScalarParameterValueOnMaterials(MaterialCurve.Name, DefaultValue);
						});

					// Update material curves that were animated
					MaterialCurves.ForEachElement([SMComponent](const UE::Anim::FCurveElement& MaterialCurve)
						{
							SMComponent->SetScalarParameterValueOnMaterials(MaterialCurve.Name, MaterialCurve.Value);
						});
				};
		}
	}

	// If we don't have any animated curves, make sure we reset the morph targets and update the ones from gameplay
	if (!GameThreadCallback)
	{
		GameThreadCallback = [](USkinnedMeshComponent* InComponent)
			{
				USkeletalMeshComponent* SMComponent = CastChecked<USkeletalMeshComponent>(InComponent);

				SMComponent->ResetMorphTargetCurves();

				// now update morph target curves that are added via SetMorphTarget
				SMComponent->UpdateMorphTargetOverrideCurves();
			};
	}

	// Attributes require remapping since the indices are LOD indices and we want mesh indices
	FGenerationTools::RemapAttributes(InPose.LODPose, InPose.Attributes, InOutputComponent->GetEditableCustomAttributes());

	// Convert and dispatch to renderer
	UE::Anim::FSkinnedMeshComponentExtensions::CompleteAndDispatch(
		InOutputComponent,
		InPose.LODPose,
		MoveTemp(GameThreadCallback));
}