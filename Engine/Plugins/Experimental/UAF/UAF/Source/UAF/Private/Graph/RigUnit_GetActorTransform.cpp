// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_GetActorTransform.h"

#include "Component/AnimNextComponent.h"
#include "Module/AnimNextModuleInstance.h"
#include "UAFLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetActorTransform)

void FAnimNextActorTransformComponent::OnBindToInstance()
{
	FUAFAssetInstance& AssetInstance = GetAssetInstance();
	if (AssetInstance.GetRootInstance() == nullptr)
	{
		return;
	}

	UUAFComponent* AnimNextComponent = Cast<UUAFComponent>(AssetInstance.GetRootInstance()->GetObject());

	FAnimNextModuleInstance::RunTaskOnGameThread(
		[this, WeakAnimNextComponent = TWeakObjectPtr<UUAFComponent>(AnimNextComponent)]()
		{
			UUAFComponent* Component = WeakAnimNextComponent.Get();
			if(Component == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FAnimNextActorTransformComponent: UAF Component is invalid.");
				return;
			}

			AActor* Owner = Component->GetOwner();
			if(Owner == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FAnimNextActorTransformComponent: Owner of %ls is invalid.", *Component->GetName());
				return;
			}

			USceneComponent* SceneComponent = Owner->GetRootComponent();
			if(SceneComponent == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FAnimNextActorTransformComponent: %ls does not have a valid RootComponent", *Owner->GetName());
				return;
			}

			// Sync to current value
			// Note here we assume its safe to capture 'this' since the lifetime of this lambda should be tied to the existence of the module component
			// Components live until the system dies, so that implies that tasks queued on a system can access components on that system by raw ptr
			ActorTransform = Owner->GetActorTransform();

			SceneComponent->TransformUpdated.AddWeakLambda(Component, [Component](USceneComponent* UpdatedComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
			{
				AActor* UpdatedComponentOwner = UpdatedComponent->GetOwner();
				if(UpdatedComponentOwner == nullptr)
				{
					return;
				}

				Component->QueueTask(NAME_None, [Transform = UpdatedComponentOwner->GetActorTransform()](const UE::UAF::FModuleTaskContext& InContext)
				{
					FAnimNextModuleInstance* ModuleInstance = InContext.GetModuleInstance();
					if (FAnimNextActorTransformComponent* ActorTransformComponent = ModuleInstance->TryGetComponent<FAnimNextActorTransformComponent>())
					{
						ActorTransformComponent->ActorTransform = Transform;
					}
				});
			});
		});
}

FRigUnit_GetActorTransform_Execute()
{
	using namespace UE::UAF;
	const FUAFAssetContextData& AssetContextData = ExecuteContext.GetContextData<FUAFAssetContextData>();
	FUAFAssetInstance& AssetInstance = AssetContextData.GetInstance();
	if (AssetInstance.GetRootInstance() == nullptr)
    {
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not get actor transform - Root module instance is not valid."));
		return;
    }
	const FAnimNextActorTransformComponent& TransformComponent = AssetInstance.GetRootInstance()->GetOrAddComponent<FAnimNextActorTransformComponent>();
	Transform = TransformComponent.GetActorTransform();
}
