// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "Module/UAFModuleInstanceComponent.h"

#include "RigUnit_GetActorTransform.generated.h"

struct FAnimNextModuleInstance;

// Module component used to marshal actor transform data for RigVM units to use
USTRUCT()
struct FAnimNextActorTransformComponent : public FUAFModuleInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	// Get the transform of the actor we are bound to
	const FTransform& GetActorTransform() const { return ActorTransform; }

private:
	// FUAFAssetInstanceComponent interface
	virtual void OnBindToInstance() override;

	// Cached actor transform
	FTransform ActorTransform = FTransform::Identity;
};

/** Gets the transform of the actor hosting this AnimNext module */
USTRUCT(meta=(DisplayName="Get Host Transform", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="Actor, Transform", RequiredComponents="AnimNextActorTransformComponent"))
struct FRigUnit_GetActorTransform : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	virtual FString GetUnitSubTitle() const { return TEXT("Actor"); };

	// The transform of our host actor
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	FTransform Transform = FTransform::Identity;
};
