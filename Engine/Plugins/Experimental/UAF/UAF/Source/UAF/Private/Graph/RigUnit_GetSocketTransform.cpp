// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_GetSocketTransform.h"
#include "UAFLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_GetSocketTransform)

FRigUnit_GetSocketTransform_Execute()
{
	if (SceneComponent)
	{
		Result = SceneComponent->GetSocketTransform(SocketName, TransformSpace);
	}
	else
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not get transform for socket [%s] - SceneComponent is not valid."), *SocketName.ToString());
	}
}
