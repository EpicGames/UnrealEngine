// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/RigUnit_AnimNextBase.h"
#include "Components/SceneComponent.h"

#include "RigUnit_GetSocketTransform.generated.h"

// Returns the transform for the specified socket. Note that this is not thread-safe, so must be
// called at a point where you know that the target socket is not being updated.
USTRUCT(meta=(DisplayName="Get Socket Transform", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="Socket, Bone, Transform"))
struct FRigUnit_GetSocketTransform : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The scene component to look up a socket transform for
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	TObjectPtr<USceneComponent> SceneComponent = nullptr;

	// The name of the socket transform to retrieve
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	FName SocketName = NAME_None;

	// The space to retrieve the transform in
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	TEnumAsByte<ERelativeTransformSpace> TransformSpace = ERelativeTransformSpace::RTS_World;

	// The resulting socket transform
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	FTransform Result = FTransform::Identity;
};
