// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_IsAnimBPDriven.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_IsAnimBPDriven)

FRigUnit_IsAnimBPDriven_Execute()
{
	Result = MeshComponent != nullptr && MeshComponent->bEnableAnimation;
}
