// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"

#include "RigUnit_IsAnimBPDriven.generated.h"

/** Gets whether or not a skeletal mesh component is being driven by an Animation Blueprint */
USTRUCT(meta = (DisplayName = "Is AnimBP Driven", Category = "Mesh", NodeColor = "0, 1, 1", Keywords = "Mesh"))
struct FRigUnit_IsAnimBPDriven : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The mesh component to check
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	TObjectPtr<USkeletalMeshComponent> MeshComponent;

	// True if the skeletal mesh component is being driven by an Animation Blueprint
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	bool Result = false;
};
