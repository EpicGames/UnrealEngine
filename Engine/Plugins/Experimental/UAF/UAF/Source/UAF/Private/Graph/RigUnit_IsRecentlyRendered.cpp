// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_IsRecentlyRendered.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_IsRecentlyRendered)

FRigUnit_IsRecentlyRendered_Execute()
{
	if (const USkinnedMeshComponent* Component = MeshComponent)
	{
		if (const FSkeletalMeshObject* MeshObject = Component->GetMeshObject())
		{
			// We are considered recently rendered if we are, or we are set to always tick & refresh, or we haven't been updated yet
			Result =
				Component->bRecentlyRendered ||
				// TODO: Enable this once the content is ready
				//Component->VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones ||
				!MeshObject->bHasBeenUpdatedAtLeastOnce;
		}
		else
		{
			Result = false;
		}
	}
	else
	{
		Result = false;
	}
}
