// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/RigUnit_AnimNextBase.h"

#include "RigUnit_IsRecentlyRendered.generated.h"

/** Gets recently rendered flag from the input skinned mesh component */
USTRUCT(meta = (DisplayName = "Is Recently Rendered", Category = "Mesh", NodeColor = "0, 1, 1", Keywords = "Mesh"))
struct FRigUnit_IsRecentlyRendered : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// Mesh to query
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	TObjectPtr<USkinnedMeshComponent> MeshComponent;

	// The recently rendered flag of the input mesh
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Output))
	bool Result = false;
};
