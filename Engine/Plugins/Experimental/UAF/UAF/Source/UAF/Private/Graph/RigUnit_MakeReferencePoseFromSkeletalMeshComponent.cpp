// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_MakeReferencePoseFromSkeletalMeshComponent.h"

#include "AnimNextStats.h"
#include "GenerationTools.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/AnimNextSkeletalMeshComponentReferenceComponent.h"
#include "UAFLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_MakeReferencePoseFromSkeletalMeshComponent)

DEFINE_STAT(STAT_AnimNext_Make_RefPose);

FRigUnit_MakeReferencePoseFromSkeletalMeshComponent_Execute()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_Make_RefPose);

	using namespace UE::UAF;

	USkeletalMeshComponent* InputComponent = SkeletalMeshComponent;

	// Defer to module component if no component is supplied
	if(InputComponent == nullptr)
	{
		const FAnimNextModuleContextData& ModuleContextData = ExecuteContext.GetContextData<FAnimNextModuleContextData>();
		FAnimNextModuleInstance& ModuleInstance = ModuleContextData.GetModuleInstance();
		const FAnimNextSkeletalMeshComponentReferenceComponent& ComponentReference = ModuleInstance.GetOrAddComponent<FAnimNextSkeletalMeshComponentReferenceComponent>();
		InputComponent = ComponentReference.GetComponent();
	}

	if(InputComponent == nullptr)
	{
		UAF_RIGUNIT_LOG(Warning, TEXT("Could not make ref pose - Skeletal mesh component is not valid."));
		return;
	}

	ReferencePose.ReferencePose = FDataRegistry::Get()->GetOrGenerateReferencePose(InputComponent);
}
