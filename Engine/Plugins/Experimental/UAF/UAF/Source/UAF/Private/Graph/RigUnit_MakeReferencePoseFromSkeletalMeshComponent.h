// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExecuteContext.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"

#include "RigUnit_MakeReferencePoseFromSkeletalMeshComponent.generated.h"

/** Makes a reference pose from a skeletal mesh component */
USTRUCT(meta=(DisplayName="Make Reference Pose", Category="Animation Graph", NodeColor="0, 1, 1", Keywords="Output,Pose,Port", RequiredComponents="AnimNextSkeletalMeshComponentReferenceComponent"))
struct FRigUnit_MakeReferencePoseFromSkeletalMeshComponent : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	virtual FString GetUnitSubTitle() const { return TEXT("Skeletal Mesh Component"); };

	// Reference pose to write
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input, Output))
	FAnimNextGraphReferencePose ReferencePose;

	// Mesh to use to generate the reference pose. If this is not supplied, then the first skeletal mesh component of the current actor will be used
	UPROPERTY(EditAnywhere, Category = "Graph", meta = (Input))
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	// The execution result
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;
};
