// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_UAFWriteInstancedSkinnedMeshPose.h"

#include "AnimationRuntime.h"
#include "Animation/NamedValueArray.h"
#include "Animation/AnimRuntimeTransformProviderData.h"
#include "AnimNextStats.h"
#include "GenerationTools.h"
#include "Component/SkinnedMeshComponentExtensions.h"
#include "Components/InstancedSkinnedMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Logging/StructuredLog.h"
#include "Module/AnimNextModuleInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_UAFWriteInstancedSkinnedMeshPose)

namespace UE::Anim
{

extern void ConvertLocalSpaceToComponentSpace(
	const USkinnedMeshComponent* InComponent,
	const UE::UAF::FLODPoseHeap& InLODPose,
	TArrayView<FTransform> OutComponentSpaceTransforms,
	TBitArray<>& OutValidMeshPoseTransforms);

}

FRigUnit_UAFWriteInstancedSkinnedMeshPose_Execute()
{
	FTaskTagScope TaskTag(ETaskTag::EParallelGameThread);
	TRACE_CPUPROFILER_EVENT_SCOPE(AnimNext::WriteInstancedSkinnedMeshPose);

	if (!SkinnedMeshComponent)
	{
		return;
	}

	USkinnedAsset* SkinnedAsset = SkinnedMeshComponent->GetSkinnedAsset();
	if (!SkinnedAsset)
	{
		return;
	}

	UAnimRuntimeTransformProviderData* AnimRuntimeData = Cast<UAnimRuntimeTransformProviderData>(SkinnedMeshComponent->GetTransformProvider());
	if (!AnimRuntimeData)
	{
		return;
	}

	if (!Pose.LODPose.IsValid() || Pose.LODPose.LODLevel != 0)
	{
		return;
	}

	const TArray<FMatrix44f>& RefBasesInvMatrix = SkinnedAsset->GetRefBasesInvMatrix();

	TArray<FTransform> ComponentSpaceTransforms;
	TArray<FCompressedBoneTransform> ReferenceToLocalTransforms;
	ComponentSpaceTransforms.SetNumUninitialized(RefBasesInvMatrix.Num());
	ReferenceToLocalTransforms.SetNumUninitialized(RefBasesInvMatrix.Num());

	TBitArray<> DummyValidMeshPoseTransforms;
	UE::Anim::ConvertLocalSpaceToComponentSpace(SkinnedMeshComponent, Pose.LODPose, ComponentSpaceTransforms, DummyValidMeshPoseTransforms);

	for (int32 Index = 0; Index < ComponentSpaceTransforms.Num(); ++Index)
	{
		FMatrix44f Transform = (FMatrix44f)ComponentSpaceTransforms[Index].ToMatrixWithScale();
		VectorMatrixMultiply(&Transform, &RefBasesInvMatrix[Index], &Transform);
		StoreCompressedBoneTransform(&ReferenceToLocalTransforms[Index], Transform);
	}

	for (int32 InstanceIndex = 0; InstanceIndex < SkinnedMeshComponent->GetInstanceCount(); ++InstanceIndex)
	{
		FPrimitiveInstanceId InstanceId = SkinnedMeshComponent->GetInstanceId(InstanceIndex);
		int32 TrackIndex;

		if (!SkinnedMeshComponent->GetInstanceAnimationIndex(InstanceId, TrackIndex))
		{
			continue;
		}

		if (!AnimRuntimeData->IsValidIndex(TrackIndex))
		{
			continue;
		}

		FAnimRuntimeTrackAllocation Allocation = AnimRuntimeData->UpdateTrack(TrackIndex, EPreviousBoneTransformUpdateMode::None);

		if (!Allocation.IsValid())
		{
			continue;
		}

		Allocation.Current->SetRevisionNumber(GFrameNumber);
		TArrayView<FCompressedBoneTransform> OutReferenceToLocal = Allocation.Current->GetTransforms();
		check(OutReferenceToLocal.Num() == ReferenceToLocalTransforms.Num());

		for (int32 TransformIndex = 0; TransformIndex < ReferenceToLocalTransforms.Num(); ++TransformIndex)
		{
			OutReferenceToLocal[TransformIndex] = ReferenceToLocalTransforms[TransformIndex];
		}
	}

	FAnimNextModuleInstance::RunTaskOnGameThread([WeakComponent = TWeakObjectPtr<UInstancedSkinnedMeshComponent>(SkinnedMeshComponent)]()
		{
			UInstancedSkinnedMeshComponent* Component = WeakComponent.Get();
			if (Component == nullptr)
			{
				return;
			}

			Component->InvalidateCachedBounds();
			Component->UpdateBounds();
			Component->MarkRenderTransformDirty();
			Component->MarkRenderInstancesDirty();
		});
}