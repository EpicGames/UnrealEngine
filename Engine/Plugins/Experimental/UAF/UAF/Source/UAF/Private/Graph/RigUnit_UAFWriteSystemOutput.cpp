// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigUnit_UAFWriteSystemOutput.h"

#include "AnimNextStats.h"
#include "Graph/UAFSystemOutputComponent.h"
#include "Module/AnimNextModuleInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_UAFWriteSystemOutput)

FRigUnit_UAFWriteSystemOutput_Execute()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_Write_Pose);

	using namespace UE::UAF;

	const FAnimNextModuleContextData& ModuleContextData = ExecuteContext.GetContextData<FAnimNextModuleContextData>();
	FAnimNextModuleInstance& ModuleInstance = ModuleContextData.GetModuleInstance();
	FUAFSystemOutputComponent& OutputPoseComponent = ModuleInstance.GetOrAddComponent<FUAFSystemOutputComponent>();
	OutputPoseComponent.WriteOutput(Value);
}
