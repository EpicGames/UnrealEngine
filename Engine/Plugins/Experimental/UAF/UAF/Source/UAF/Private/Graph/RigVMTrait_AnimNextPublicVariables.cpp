// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/RigVMTrait_AnimNextPublicVariables.h"
#include "AnimNextRigVMAsset.h"
#include "Logging/StructuredLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMTrait_AnimNextPublicVariables)

#if WITH_EDITOR
FString (*FRigVMTrait_AnimNextPublicVariables::GetDisplayNameFunc)(const FRigVMTrait_AnimNextPublicVariables& InTrait);
void (*FRigVMTrait_AnimNextPublicVariables::GetProgrammaticPinsFunc)(const FRigVMTrait_AnimNextPublicVariables& InTrait, URigVMController* InController, int32 InParentPinIndex, const FString& InDefaultValue, FRigVMPinInfoArray& OutPinArray);
bool (*FRigVMTrait_AnimNextPublicVariables::ShouldCreatePinForPropertyFunc)(const FRigVMTrait_AnimNextPublicVariables& InTrait, const FProperty* InProperty);

FString FRigVMTrait_AnimNextPublicVariables::GetDisplayName() const
{
	check(GetDisplayNameFunc);
	return GetDisplayNameFunc(*this);
}

void FRigVMTrait_AnimNextPublicVariables::GetProgrammaticPins(URigVMController* InController, int32 InParentPinIndex, const URigVMPin* InTraitPin, const FString& InDefaultValue, FRigVMPinInfoArray& OutPinArray) const
{
	check(GetProgrammaticPinsFunc);
	GetProgrammaticPinsFunc(*this, InController, InParentPinIndex, InDefaultValue, OutPinArray);
}

bool FRigVMTrait_AnimNextPublicVariables::ShouldCreatePinForProperty(const FProperty* InProperty) const
{
	check(ShouldCreatePinForPropertyFunc);
	return ShouldCreatePinForPropertyFunc(*this, InProperty);
}
#endif

