// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/UAFSystemOutputComponent.h"

#include "AnimationRuntime.h"
#include "ISystemOutputAdapter.h"
#include "ReferencePose.h"
#include "Engine/SkeletalMesh.h"
#include "Graph/AnimNext_LODPose.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/ModuleTaskContext.h"
#include "UAF/ValueRuntime/IVirtualValueBundle.h"
#include "UAF/ValueRuntime/ValueBundle.h"

void FUAFSystemOutputComponent::OnBeginExecution(float InDeltaTime)
{
	// Grab all the info we need to run the system this time around & generate/get the reference pose
	if (SystemOutputAdapter != nullptr)
	{
		UE::UAF::ISystemOutputAdapter::FInputData InputData = SystemOutputAdapter->GetInputData();
		USkeletalMesh* OldMesh = Mesh;
		Mesh = InputData.SkeletalMesh;
		MeshComponent = InputData.SkeletalMeshComponent;
		LOD = InputData.LOD;

		if (MeshComponent)
		{
			if (MeshComponent->GetSkeletalMeshAsset() != OldMesh)
			{
				RefPose.ReferencePose = UE::UAF::FDataRegistry::Get()->GetOrGenerateReferencePose(MeshComponent);
			}
		}
		else if (Mesh)
		{
			if (Mesh != OldMesh)
			{
				RefPose.ReferencePose = UE::UAF::FDataRegistry::Get()->GetOrGenerateReferencePose(Mesh);
			}
		}
		else
		{
			RefPose.ReferencePose = UE::UAF::FDataHandle();
		}
	}
}

void FUAFSystemOutputComponent::WriteOutput(const FUAFValueBundle& InValue)
{
	SerialNumber++;

	// If serial number wrapped, skip zero so client serials that have just been created will always mismatch
	if (SerialNumber == 0)
	{
		SerialNumber++;
	}
	
	if (InValue.IsRefPose())
	{
		Pose.LODPose.SetRefPose();
	}
	else
	{
		UE::UAF::IVirtualValueBundle& Impl = InValue.GetImplChecked();

		// TODO: Value bundle version of this
		if (const FAnimNextGraphLODPose* ValuePose = Impl.GetLODPose())
		{
			RemapLODPoseToOutput(*ValuePose, Pose);
		}
	}

	if (SystemOutputAdapter != nullptr)
	{
		SystemOutputAdapter->SignalOutputWritten(UE::UAF::FModuleTaskContext(GetModuleInstance()));
	}
}

bool FUAFSystemOutputComponent::GenerateRenderData(uint32 InSerialNumber, FReadOutputFunc InFunction) const
{
	// Zero serial indicates no valid pose to output, so early out
	if (SerialNumber == 0)
	{
		return false;
	}

	// Always call the read function when the supplied serial is zero, otherwise only call it on mismatched serial numbers
	if (InSerialNumber != 0 && InSerialNumber == SerialNumber)
	{
		return false;
	}

	if (!ensure(Mesh != nullptr))
	{
		return false;
	}

	if (SystemOutputAdapter == nullptr)
	{
		return false;
	}

	// Validate that the input we are transmitting matches the expected mesh of the adapter
	// It is the assumed responsibility of client systems to ensure consistency of the mesh across the frame.
	// It may be necessary when switching mesh mid-frame to recreate a new system to perform the work again.
	// Note LOD can vary across the frame due to streaming and unfenced other LOD calculations, so clients must deal with LODs that are
	// inconsistent between OnBeginExecution's GetInputData and this call
	UE::UAF::ISystemOutputAdapter::FInputData InputData = SystemOutputAdapter->GetInputData();
	if (ensure(InputData.SkeletalMesh == Mesh))
	{
		InFunction(SerialNumber, Pose, RefPose);
		return true;
	}

	return false;
}

bool FUAFSystemOutputComponent::ReadOutput(uint32 InSerialNumber, FReadOutputFunc InFunction) const
{
	// Zero serial indicates no valid pose to output, so early out
	if (SerialNumber == 0)
	{
		return false;
	}

	// Always call the read function when the supplied serial is zero, otherwise only call it on mismatched serial numbers
	if (InSerialNumber != 0 && InSerialNumber == SerialNumber)
	{
		return false;
	}

	InFunction(SerialNumber, Pose, RefPose);
	return true;
}

void FUAFSystemOutputComponent::BindSystemOutputAdapter(UE::UAF::ISystemOutputAdapter& InSystemOutputAdapter)
{
	ensure(SystemOutputAdapter == nullptr);	// Should unbind first
	SystemOutputAdapter = &InSystemOutputAdapter;
}

void FUAFSystemOutputComponent::UnbindSystemOutputAdapter()
{
	ensure(SystemOutputAdapter != nullptr);	// Should bind first
	SystemOutputAdapter = nullptr;
}

void FUAFSystemOutputComponent::RemapLODPoseToOutput(const FAnimNextGraphLODPose& InInputPose, FAnimNextGraphLODPose& InOutputPose)
{
	const UE::UAF::FReferencePose* InputRefPose = InInputPose.LODPose.RefPose;
	const UE::UAF::FReferencePose* OutputRefPose = RefPose.ReferencePose.IsValid() ? RefPose.ReferencePose.GetPtr<UE::UAF::FReferencePose>() : nullptr;
	if (InputRefPose != nullptr && OutputRefPose != nullptr)
	{
		if (InputRefPose != OutputRefPose)
		{
			// Cross-skeleton: prepare the output with the target reference pose, then copy/remap via shared pool
			InOutputPose.LODPose.PrepareForLOD(*OutputRefPose, OutputRefPose->GetSourceLODLevel(), true, InInputPose.LODPose.IsAdditive());
			InOutputPose.CopyFrom(InInputPose);
		}
		else
		{
			// Same skeleton: direct copy, no remap needed
			InOutputPose = InInputPose;
		}
	}
}