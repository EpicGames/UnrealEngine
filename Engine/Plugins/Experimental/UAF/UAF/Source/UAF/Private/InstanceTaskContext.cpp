// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceTaskContext.h"

#include "UAFAssetInstance.h"

namespace UE::UAF
{

FInstanceTaskContext::FInstanceTaskContext(FUAFAssetInstance& InInstance)
	: Instance(InInstance)
{
}

bool FInstanceTaskContext::AccessVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>&)> InFunction) const
{
	TArrayView<uint8> Data;
	EPropertyBagResult Result = Instance.Variables.AccessVariable(InVariable, InType, Data);
	if (Result == EPropertyBagResult::Success)
	{
		InFunction(Data);
		return true;
	}
	return false;
}

bool FInstanceTaskContext::SetVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue) const
{
	return Instance.Variables.SetVariable(InVariable, InType, InNewValue) == EPropertyBagResult::Success;
}

bool FInstanceTaskContext::AccessVariablesStructInternal(const UScriptStruct* InStruct, TFunctionRef<void(FStructView)> InFunction) const
{
	return Instance.Variables.AccessVariablesStructInternal(InStruct, InFunction);
}

};
 