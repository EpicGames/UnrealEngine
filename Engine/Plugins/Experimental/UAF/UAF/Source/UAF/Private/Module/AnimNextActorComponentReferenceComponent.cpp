// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/AnimNextActorComponentReferenceComponent.h"

#include "Component/AnimNextComponent.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/UAFWeakSystemReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextActorComponentReferenceComponent)

void FAnimNextActorComponentReferenceComponent::OnInitializeHelper(UScriptStruct* InScriptStruct)
{
	check(InScriptStruct->IsChildOf(FAnimNextActorComponentReferenceComponent::StaticStruct()));

	FAnimNextModuleInstance* ModuleInstance = GetModuleInstancePtr();
	if (ModuleInstance == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FAnimNextActorComponentReferenceComponent: Could not retrieve component reference of type [%ls] - Module Instance is not valid.", *GetNameSafe(ComponentType));
		return;
	}

	UUAFComponent* AnimNextComponent = Cast<UUAFComponent>(ModuleInstance->GetObject());

	FAnimNextModuleInstance::RunTaskOnGameThread(
		[Reference = ModuleInstance->GetReference(), WeakAnimNextComponent = TWeakObjectPtr<UUAFComponent>(AnimNextComponent), ComponentType = ComponentType, InScriptStruct]()
		{
			UUAFComponent* PinnedAnimNextComponent = WeakAnimNextComponent.Get();
			if(PinnedAnimNextComponent == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FAnimNextActorComponentReferenceComponent: Could not retrieve component reference of type [%ls] - UAF Component is not valid.", *GetNameSafe(ComponentType));
				return;
			}

			AActor* Owner = PinnedAnimNextComponent->GetOwner();
			if(Owner == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FAnimNextActorComponentReferenceComponent: Could not retrieve component reference of type [%ls] - Owner of UAF Component is not valid.", *GetNameSafe(ComponentType));
				return;
			}

			UActorComponent* FoundComponent = Owner->FindComponentByClass(ComponentType);
			Reference.QueueTask(NAME_None, [FoundComponent, InScriptStruct](const UE::UAF::FModuleTaskContext& InContext)
			{
				InContext.TryAccessComponent(InScriptStruct, [FoundComponent](FUAFAssetInstanceComponent& InComponent)
				{
					static_cast<FAnimNextActorComponentReferenceComponent&>(InComponent).Component = FoundComponent;
				});
			});
		});
}
