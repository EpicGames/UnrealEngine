// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/AnimNextModule.h"
#include "AnimNextPool.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "Serialization/MemoryReader.h"
#include "Graph/RigUnit_AnimNextBeginExecution.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectResource.h"

#if WITH_EDITOR	
#include "Engine/ExternalAssetDependencyGatherer.h"
REGISTER_ASSETDEPENDENCY_GATHERER(FExternalAssetDependencyGatherer, UUAFSystem);
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextModule)

UUAFSystem::UUAFSystem(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExtendedExecuteContext.SetContextPublicDataStruct(FAnimNextExecuteContext::StaticStruct());
}

void UUAFSystem::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if(Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextModuleRefactor)
		{
			// Skip over shared archive buffer if we are loading from an older version
			if (const FLinkerLoad* Linker = GetLinker())
			{
				const int32 LinkerIndex = GetLinkerIndex();
				const FObjectExport& Export = Linker->ExportMap[LinkerIndex];
				Ar.Seek(Export.SerialOffset + Export.SerialSize);
			}
		}

		if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextGraphAccessSpecifiers)
		{
			DefaultState_DEPRECATED.State = PropertyBag_DEPRECATED;
		}

		if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::UAFProceduralSystems)
		{
			for (TObjectPtr<UScriptStruct> RequiredComponent : RequiredComponents_DEPRECATED)
			{
				TInstancedStruct<FUAFAssetInstanceComponent> Component;
				Component.InitializeAsScriptStruct(RequiredComponent);
				Components.Add(MoveTemp(Component));
			}
		}
	}
#endif
}
