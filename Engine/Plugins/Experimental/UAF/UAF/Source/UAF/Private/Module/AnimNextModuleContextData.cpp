// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/AnimNextModuleContextData.h"
#include "Module/AnimNextModuleInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextModuleContextData)

FAnimNextModuleContextData::FAnimNextModuleContextData(FAnimNextModuleInstance& InInstance, FName InEventName, float InDeltaTime)
	: FUAFAssetContextData(InInstance, InEventName, InDeltaTime)
{
}

UObject* FAnimNextModuleContextData::GetObject() const
{
	return GetInstance().As<FAnimNextModuleInstance>().GetObject();
}

FAnimNextModuleInstance& FAnimNextModuleContextData::GetModuleInstance() const
{
	return GetInstance().As<FAnimNextModuleInstance>();
}