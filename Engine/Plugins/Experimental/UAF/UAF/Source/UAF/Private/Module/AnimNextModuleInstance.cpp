// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/AnimNextModuleInstance.h"

#if UE_ENABLE_DEBUG_DRAWING
#include "AnimNextDebugDraw.h"
#endif
#include "AnimNextPool.h"
#include "Engine/World.h"
#include "AnimNextStats.h"
#include "Async/TaskGraphInterfaces.h"
#include "SceneInterface.h"
#include "Script/UAFScriptComponent.h"
#include "Algo/TopologicalSort.h"
#include "CrashReporter/CrashReporterHandler.h"
#include "Factory/SystemFactory.h"
#include "Factory/UAFSystemFactoryParams.h"
#include "Logging/StructuredLog.h"
#include "Misc/EnumerateRange.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Module/ModuleTickFunction.h"
#include "Module/SystemReference.h"
#include "Module/UAFWeakSystemReference.h"
#include "UObject/UObjectIterator.h"
#include "RewindDebugger/UAFTrace.h"
#include "Script/UAFScriptComponent.h"
#include "Variables/AnimNextVariableReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextModuleInstance)

DEFINE_STAT(STAT_AnimNext_InitializeInstance);

FAnimNextModuleInstance::FAnimNextModuleInstance() = default;

FAnimNextModuleInstance::FAnimNextModuleInstance(
		const UUAFSystem* InModule,
		EAnimNextModuleInitMethod InInitMethod)
	: FAnimNextModuleInstance(InModule, nullptr, InInitMethod)
{
}

FAnimNextModuleInstance::FAnimNextModuleInstance(
		const UUAFSystem* InModule,
		UObject* InObject,
		EAnimNextModuleInitMethod InInitMethod)
	: FUAFAssetInstance(StaticStruct())
	, Object(InObject)
	, InitState(EInitState::NotInitialized)
	, RunState(ERunState::NotInitialized)
	, InitMethod(InInitMethod)
{
	using namespace UE::UAF;

	ensure(InModule);

	Asset = InModule;

#if UE_ENABLE_DEBUG_DRAWING
	if(Object && Object->GetWorld())
	{
		DebugDraw = MakeUnique<UE::UAF::Debug::FDebugDraw>(Object);
	}
#endif
}

FAnimNextModuleInstance::~FAnimNextModuleInstance()
{
	ResetBindingsAndInstanceData();

#if UE_ENABLE_DEBUG_DRAWING
	DebugDraw.Reset();
#endif

	Object = nullptr;
	Asset = nullptr;
}

void FAnimNextModuleInstance::Initialize(const FUAFSystemFactoryParams& InFactoryParams)
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_InitializeInstance);
	
	using namespace UE::UAF;

	check(IsInGameThread());

	const UUAFSystem* System = GetSystem();
	if (System)
	{
#if WITH_EDITOR
		if (System->CompilationState == EAnimNextRigVMAssetState::CompiledWithErrors)
		{
			return;
		}

		for (const UUAFRigVMAsset* StaticallyReferencedAsset : System->ReferencedVariableAssets)
		{
			if (!StaticallyReferencedAsset || StaticallyReferencedAsset->CompilationState == EAnimNextRigVMAssetState::CompiledWithErrors)
			{
				return;
			}
		}
#endif
	}
	else
	{
		return;
	}

	UWorld* World = Object != nullptr ? Object->GetWorld() : nullptr;
	if(World)
	{
		WorldType = World->WorldType;
	}

	// Copy compiled-in components first, as we may have a script component subclass
	CopyDefaultComponents();

	// Get the script events component & all the events that are implemented
	// Note: Calling TryGetComponent here as it will not bind to host automatically.
	//       Binding would involve interaction with variables, which are not handled until after this call
	TConstArrayView<FScriptEventInfo> ImplementedEventInfo;
	if (FUAFScriptComponent* ScriptComponent = TryGetComponent<FUAFScriptComponent>())
	{
		ScriptComponent->Instance = this;
		ImplementedEventInfo = ScriptComponent->GetScriptEvents();
	}

	// Setup tick function graph using module events
	if (ImplementedEventInfo.Num() > 0)
	{
		TransitionToInitState(EInitState::CreatingTasks);

		if (World)
		{
			// Allocate tick functions
			TickFunctions.Reserve(ImplementedEventInfo.Num());
			bool bFoundFirstUserEvent = false;
			FModuleEventTickFunction* PrevTickFunction = nullptr;
			for (int32 EventIndex = 0; EventIndex < ImplementedEventInfo.Num(); EventIndex++)
			{
				const FScriptEventInfo& ModuleEvent = ImplementedEventInfo[EventIndex];
				if (!ModuleEvent.bIsTask)
				{
					continue;
				}

				FModuleEventTickFunction& TickFunction = TickFunctions.AddDefaulted_GetRef();
				TickFunction.bRunOnAnyThread = !ModuleEvent.bIsGameThreadTask;
				TickFunction.ModuleInstance = this;
				TickFunction.Event = ModuleEvent.Event;
				TickFunction.TickGroup = ModuleEvent.TickGroup;
				TickFunction.EndTickGroup = ModuleEvent.EndTickGroup;
				TickFunction.bUserEvent = ModuleEvent.bUserEvent;

				// Perform custom setup
				FTickFunctionBindingContext Context(*this, Object, World);
				if (ModuleEvent.Binding.IsSet())
				{
					ModuleEvent.Binding(Context, TickFunction);
				}

				// Establish linear dependency chain
				if (PrevTickFunction != nullptr)
				{
					TickFunction.AddPrerequisite(Object, *PrevTickFunction);
				}
				PrevTickFunction = &TickFunction;

				// Set up dependencies, if any
				for (const TInstancedStruct<FRigVMTrait_ModuleEventDependency>& DependencyInstance : System->Dependencies)
				{
					const FRigVMTrait_ModuleEventDependency* Dependency = DependencyInstance.GetPtr<FRigVMTrait_ModuleEventDependency>();
					if (Dependency != nullptr && Dependency->EventName == ModuleEvent.Event.GetEventName())
					{
						FModuleDependencyContext ModuleDependencyContext(Object, TickFunction);
						Dependency->OnAddDependency(ModuleDependencyContext);
					}
				}

				if (ModuleEvent.bUserEvent && !ModuleEvent.bIsGameThreadTask)
				{
					// Hook up with Task Sync Manager if possible
					TickFunction.InitializeBatchedWork(World);
				}

				if (ModuleEvent.bUserEvent && !bFoundFirstUserEvent)
				{
					TickFunction.bFirstUserEvent = true;
					bFoundFirstUserEvent = true;

					// Set this first user event to run the initialize event, if it exists
					auto IsInitializeEvent = [](const FScriptEventInfo& InEvent)
					{
						return InEvent.Event.GetEventName() == FRigUnit_AnimNextInitializeEvent::EventName;
					};

					if (const FScriptEventInfo* InitializeEvent = ImplementedEventInfo.FindByPredicate(IsInitializeEvent))
					{
						TickFunction.InitializeEvent = InitializeEvent->Event;
					}

					// Set this first user event to run the bindings event, if it exists
					auto IsExecuteBindingsEvent = [](const FScriptEventInfo& InEvent)
					{
						return InEvent.Event.GetEventName() == FRigUnit_AnimNextExecuteBindings_WT::EventName;
					};

					if (const FScriptEventInfo* BindingsEvent = ImplementedEventInfo.FindByPredicate(IsExecuteBindingsEvent))
					{
						TickFunction.ExecuteBindings_WTEvent = BindingsEvent->Event;
					}
				}
			}

			// Find the last user event - 'end' logic will be called from here
			for (int32 EventIndex = TickFunctions.Num() - 1; EventIndex >= 0; EventIndex--)
			{
				FModuleEventTickFunction& TickFunction = TickFunctions[EventIndex];
				if (TickFunction.bUserEvent)
				{
					TickFunction.bLastUserEvent = true;
					break;
				}
			}
		}

		TransitionToInitState(EInitState::BindingTasks);

		// Register our tick functions
		if(World)
		{
			ULevel* Level = World->PersistentLevel;
			for (FModuleEventTickFunction& TickFunction : TickFunctions)
			{
				TickFunction.RegisterTickFunction(Level);
			}
		}

		TransitionToInitState(EInitState::SetupVariables);

		// TODO: code in EInitState::SetupVariables phase below can probably move to FModuleEventTickFunction::Initialize

		// Initialize variables
#if WITH_EDITOR
		if(bIsRecreatingOnCompile && Variables.bHasBeenInitialized)
		{
			MigrateVariables();
		}
		else
#endif
		{
			InitializeVariables();
		}

		// Apply factory params to further customize/initialize variables/components
		if (InFactoryParams.IsValid())
		{
			InFactoryParams.InitializeInstance(*this);
		}

		ProxyVariables[0].Initialize(Variables);
		ProxyVariables[1].Initialize(Variables);

		// Ensure components are bound now we can execute user code
		BindDefaultComponents();

		TransitionToInitState(EInitState::PendingInitializeEvent);
		TransitionToRunState(ERunState::Running);

		// Just pause now if we arent needing an initial update
		if(InitMethod == EAnimNextModuleInitMethod::None)
		{
			SetEnabled(false);
		}
	}
}

void FAnimNextModuleInstance::RemoveAllTickDependencies()
{
	using namespace UE::UAF;

	check(IsInGameThread());

	for (FModuleEventTickFunction& TickFunction : TickFunctions)
	{
		TickFunction.RemoveAllExternalSubsequents();
	}
}

void FAnimNextModuleInstance::ResetBindingsAndInstanceData()
{
	using namespace UE::UAF;

	check(IsInGameThread());

	TransitionToInitState(EInitState::NotInitialized);
	TransitionToRunState(ERunState::NotInitialized);

#if WITH_EDITOR
	if(!bIsRecreatingOnCompile)
#endif
	{
		for (FModuleEventTickFunction& TickFunction : TickFunctions)
		{
			// We should have released all external dependencies by now via RemoveAllTickDependencies
			check(TickFunction.ExternalSubsequents.Num() == 0);
			TickFunction.UnRegisterTickFunction();
		}
		ProxyVariables[0].Reset();
		ProxyVariables[1].Reset();
	}

	TickFunctions.Reset();

	ReleaseComponents();
}

void FAnimNextModuleInstance::QueueInputTraitEvent(FAnimNextTraitEventPtr Event)
{
	InputEventList.Push(MoveTemp(Event));
}

void FAnimNextModuleInstance::QueueOutputTraitEvent(FAnimNextTraitEventPtr Event)
{
	OutputEventList.Push(MoveTemp(Event));
}

bool FAnimNextModuleInstance::IsEnabled() const
{
	using namespace UE::UAF;

	check(IsInGameThread());

	return RunState == ERunState::Running;
}

void FAnimNextModuleInstance::SetEnabled(bool bInEnabled)
{
	using namespace UE::UAF;

	check(IsInGameThread());

	// Ensure that we synchronize activation state with actor component if we are hosted there
	if (UActorComponent* Component = Cast<UActorComponent>(Object))
	{
		Component->SetActiveFlag(bInEnabled);
	}

	if(RunState == ERunState::Paused || RunState == ERunState::Running)
	{
		for (FModuleEventTickFunction& TickFunction : TickFunctions)
		{
			TickFunction.SetTickFunctionEnable(bInEnabled);
		}

		TransitionToRunState(bInEnabled ? ERunState::Running : ERunState::Paused);
	}
}

void FAnimNextModuleInstance::TransitionToInitState(EInitState InNewState)
{
	switch(InNewState)
	{
	case EInitState::NotInitialized:
		check(InitState == EInitState::NotInitialized || InitState == EInitState::PendingInitializeEvent || InitState == EInitState::SetupVariables || InitState == EInitState::FirstUpdate || InitState == EInitState::Initialized);
		break;
	case EInitState::CreatingTasks:
		check(InitState == EInitState::NotInitialized);
		break;
	case EInitState::BindingTasks:
		check(InitState == EInitState::CreatingTasks);
		break;
	case EInitState::SetupVariables:
		check(InitState == EInitState::BindingTasks);
		break;
	case EInitState::PendingInitializeEvent:
		check(InitState == EInitState::SetupVariables);
		break;
	case EInitState::FirstUpdate:
		check(InitState == EInitState::PendingInitializeEvent);
		break;
	case EInitState::Initialized:
		check(InitState == EInitState::FirstUpdate);
		break;
	default:
		checkNoEntry();
	}

	InitState = InNewState;
}

void FAnimNextModuleInstance::TransitionToRunState(ERunState InNewState)
{
	switch(InNewState)
	{
	case ERunState::Running:
		check(RunState == ERunState::NotInitialized || RunState == ERunState::Paused || RunState == ERunState::Running);
		break;
	case ERunState::Paused:
		check(RunState == ERunState::Paused || RunState == ERunState::Running);
		break;
	case ERunState::NotInitialized:
		check(RunState == ERunState::NotInitialized || RunState == ERunState::Paused || RunState == ERunState::Running);
		break;
	default:
		checkNoEntry();
	}

	RunState = InNewState;
}

void FAnimNextModuleInstance::CopyProxyVariables()
{
	int32 ProxyReadIndex = 0;
	{
		UE::TWriteScopeLock WriteLock(ProxyLock);

		// Flip the write buffer index
		ProxyReadIndex = ProxyWriteIndex;
		ProxyWriteIndex = 1 - ProxyWriteIndex;
	}

	FUAFInstanceVariableDataProxy& ProxyVariablesRead = ProxyVariables[ProxyReadIndex];
	ProxyVariablesRead.CopyDirty();
}

#if UAF_TRACE_ENABLED
void FAnimNextModuleInstance::Trace()
{
	if (!bTracedThisFrame)
	{
		TRACE_UAF_VARIABLES(this, Object);
		bTracedThisFrame = true;
	}
}
#endif

const UUAFSystem* FAnimNextModuleInstance::GetSystem() const
{
	return CastChecked<UUAFSystem>(Asset, ECastCheckedType::NullAllowed);
}

#if WITH_EDITOR
void FAnimNextModuleInstance::OnCompileJobFinished()
{
	using namespace UE::UAF;
	
	UAF_CRASH_REPORTER_SCOPE(Object, GetSystem(), OnCompileJobFinished);

	FGuardValue_Bitfield(bIsRecreatingOnCompile, true);

	ResetBindingsAndInstanceData();
	Initialize(FUAFSystemFactoryParams());
}
#endif

#if UE_ENABLE_DEBUG_DRAWING
FRigVMDrawInterface* FAnimNextModuleInstance::GetDebugDrawInterface()
{
	return DebugDraw ? &DebugDraw->DrawInterface : nullptr;
}

void FAnimNextModuleInstance::ShowDebugDrawing(bool bInShowDebugDrawing)
{
	if(DebugDraw)
	{
		DebugDraw->SetEnabled(bInShowDebugDrawing);
	}
}
#endif

void FAnimNextModuleInstance::RunTaskOnGameThread(TUniqueFunction<void(void)>&& InFunction)
{
	UE::UAF::FModuleEventTickFunction::RunTaskOnGameThread(MoveTemp(InFunction));
}

UE::UAF::FModuleEventTickFunction* FAnimNextModuleInstance::FindTickFunctionByName(FName InEventName)
{
	using namespace UE::UAF;

	return const_cast<FModuleEventTickFunction*>(const_cast<const FAnimNextModuleInstance*>(this)->FindTickFunctionByName(InEventName));
}

const UE::UAF::FModuleEventTickFunction* FAnimNextModuleInstance::FindTickFunctionByName(FName InEventName) const
{
	using namespace UE::UAF;

	for(const FModuleEventTickFunction& TickFunction : TickFunctions)
	{
		if (TickFunction.Event.GetEventName() == InEventName)
		{
			return &TickFunction;
		}
	}
	return nullptr;
}

void FAnimNextModuleInstance::BeginExecution(float InDeltaTime)
{
	// Give each component a chance to begin execution
	for(TInstancedStruct<FUAFAssetInstanceComponent>& InstancedStruct : Components)
	{
		FUAFModuleInstanceComponent* Component = InstancedStruct.GetMutablePtr<FUAFModuleInstanceComponent>();
		if (Component == nullptr)
		{
			continue;
		}
		Component->OnBeginExecution(InDeltaTime);
	}
}

void FAnimNextModuleInstance::EndExecution(float InDeltaTime)
{
	// Give the module a chance to handle events
	RaiseTraitEvents(OutputEventList);

	// Give each component a chance to finalize execution
	for(TInstancedStruct<FUAFAssetInstanceComponent>& InstancedStruct : Components)
	{
		FUAFModuleInstanceComponent* Component = InstancedStruct.GetMutablePtr<FUAFModuleInstanceComponent>();
		if (Component == nullptr)
		{
			continue;
		}
		Component->OnEndExecution(InDeltaTime);
	}
}

void FAnimNextModuleInstance::RaiseTraitEvents(const UE::UAF::FTraitEventList& EventList)
{
	for(TInstancedStruct<FUAFAssetInstanceComponent>& InstancedStruct : Components)
	{
		FUAFModuleInstanceComponent* Component = InstancedStruct.GetMutablePtr<FUAFModuleInstanceComponent>();
		if (Component == nullptr)
		{
			continue;
		}

		// Event handlers can raise events and as such the list may change while we iterate
		// However, if an event is added while we iterate, we will not visit it
		const int32 NumEvents = EventList.Num();
		for (int32 EventIndex = 0; EventIndex < NumEvents; ++EventIndex)
		{
			const FAnimNextTraitEventPtr Event = EventList[EventIndex];
			if (Event->IsValid())
			{
				Component->OnTraitEvent(*Event);
			}
		}
	}
}

EPropertyBagResult FAnimNextModuleInstance::SetProxyVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InData)
{
	UE::TWriteScopeLock WriteLock(ProxyLock);
	FUAFInstanceVariableDataProxy& Proxy = ProxyVariables[ProxyWriteIndex];
	return Proxy.SetVariable(InVariable, InType, InData);
}

EPropertyBagResult FAnimNextModuleInstance::WriteProxyVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>)> InFunction)
{
	UE::TWriteScopeLock WriteLock(ProxyLock);
	FUAFInstanceVariableDataProxy& Proxy = ProxyVariables[ProxyWriteIndex];
	TArrayView<uint8> Data;
	EPropertyBagResult Result = Proxy.WriteVariable(InVariable, InType, Data);
	if (Result == EPropertyBagResult::Success)
	{
		InFunction(Data);
	}
	return Result;
}

EPropertyBagResult FAnimNextModuleInstance::ReadVariableFromExternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TConstArrayView<uint8>)> InFunction)
{
	// First check if this variable has a dirty value in the write proxy buffer.
	// If so, an external system has set this value before it has been copied to the live variables so we should return it.
	// If not, continue reading the value from the live variables.
	{
		UE::TReadScopeLock ReadLock(ProxyLock);
		FUAFInstanceVariableDataProxy& Proxy = ProxyVariables[ProxyWriteIndex];
		if (Proxy.bIsDirty)
		{
			if (Proxy.IsVariableDirty(InVariable))
			{
				TArrayView<uint8> Data;
				EPropertyBagResult Result = Proxy.AccessVariable(InVariable, InType, Data);
				if (Result == EPropertyBagResult::Success)
				{
					InFunction(Data);
					return EPropertyBagResult::Success;
				}
			}
		}
	}

	UE::TReadScopeLock Lock(SystemReferenceLock);

	TArrayView<uint8> Data;
	EPropertyBagResult Result = Variables.AccessVariable(InVariable, InType, Data);
	if (Result == EPropertyBagResult::Success)
	{
		InFunction(Data);
	}
	return Result;
}

bool FAnimNextModuleInstance::ReadComponentFromExternal(UScriptStruct* InComponentType, TFunctionRef<void(FConstStructView InComponentStruct)> InFunction)
{
	UE::TReadScopeLock Lock(SystemReferenceLock);

	if (const FUAFAssetInstanceComponent* Component = TryGetComponent(InComponentType))
	{
		InFunction(FConstStructView(InComponentType, (uint8*)Component));
		return true;
	}
	return false;
}

void FAnimNextModuleInstance::RunScriptEvent(const UE::UAF::FScriptEvent& InEvent, float InDeltaTime)
{
	if (!InEvent.IsCallable())
	{
		return;
	}

	if (FUAFScriptComponent* ScriptComponent = TryGetComponent<FUAFScriptComponent>())
	{
		FAnimNextModuleContextData ContextData(*this, InEvent.GetEventName(), InDeltaTime);
		ScriptComponent->CallEvent(InEvent, ContextData);
	}
}

void FAnimNextModuleInstance::RunScriptEventByName(FName InEventName, float InDeltaTime)
{
	if (FUAFScriptComponent* ScriptComponent = TryGetComponent<FUAFScriptComponent>())
	{
		FAnimNextModuleContextData ContextData(*this, InEventName, InDeltaTime);
		ScriptComponent->CallEventByName(ContextData);
	}
}

TArrayView<UE::UAF::FModuleEventTickFunction> FAnimNextModuleInstance::GetTickFunctions()
{
	return TickFunctions;
}

FUAFWeakSystemReference FAnimNextModuleInstance::GetReference() const
{
	FUAFWeakSystemReference Handle;
	Handle.WeakPtr = SharedThis(const_cast<FAnimNextModuleInstance*>(this));
	return Handle;
}

UE::UAF::FModuleEventTickFunction* FAnimNextModuleInstance::FindFirstUserTickFunction()
{
	UE::UAF::FModuleEventTickFunction* FoundTickFunction = TickFunctions.FindByPredicate([](const UE::UAF::FModuleEventTickFunction& InTickFunction)
	{
		return InTickFunction.bFirstUserEvent;
	});

	return FoundTickFunction;
}

UE::UAF::FModuleEventTickFunction* FAnimNextModuleInstance::FindLastUserTickFunction()
{
	const int32 LastUserIndex = TickFunctions.FindLastByPredicate([](const UE::UAF::FModuleEventTickFunction& InTickFunction)
	{
		return InTickFunction.bUserEvent;
	});

	return LastUserIndex != INDEX_NONE ? &TickFunctions[LastUserIndex] : nullptr;
}

void FAnimNextModuleInstance::QueueTask(FName InEventName, UE::UAF::FUniqueSystemTask&& InTaskFunction, UE::UAF::ETaskRunLocation InLocation)
{
	using namespace UE::UAF;

#if UE_AUTORTFM
	// As we only have a SPSC queue under AutoRTFM, we limit enquing tasks to the game thread for now
	check(IsInGameThread());
#endif

	FModuleEventTickFunction* FoundTickFunction = nullptr;
	if(TickFunctions.Num() > 0)
	{
		if(!InEventName.IsNone())
		{
			// Match according to event desc
			FoundTickFunction = TickFunctions.FindByPredicate([InEventName](const FModuleEventTickFunction& InTickFunction)
			{
				return InTickFunction.Event.GetEventName() == InEventName;
			});
		}

		if (FoundTickFunction == nullptr)
		{
			// Fall back to first user function
			FoundTickFunction = TickFunctions.FindByPredicate([](const UE::UAF::FModuleEventTickFunction& InTickFunction)
			{
				return InTickFunction.bFirstUserEvent;
			});
		}
	}

	FModuleEventTickFunction::FTaskQueueType* Queue = nullptr;
	if (FoundTickFunction)
	{
		switch (InLocation)
		{
		case ETaskRunLocation::Before:
			Queue = &FoundTickFunction->PreExecuteTasks;
			break;
		case ETaskRunLocation::After:
			Queue = &FoundTickFunction->PostExecuteTasks;
			break;
		}
	}

	if (Queue)
	{
		Queue->Enqueue(MoveTemp(InTaskFunction));
	}
	else
	{
		UE_LOGFMT(LogAnimation, Warning, "QueueTask: Could not find event '{EventName}' in module '{ModuleName}'", InEventName, GetAssetName());
	}
}
