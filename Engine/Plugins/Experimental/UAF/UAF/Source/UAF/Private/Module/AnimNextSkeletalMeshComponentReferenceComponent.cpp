// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/AnimNextSkeletalMeshComponentReferenceComponent.h"

#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextSkeletalMeshComponentReferenceComponent)

FAnimNextSkeletalMeshComponentReferenceComponent::FAnimNextSkeletalMeshComponentReferenceComponent()
{
	ComponentType = USkeletalMeshComponent::StaticClass();
}

void FAnimNextSkeletalMeshComponentReferenceComponent::OnBindToInstance()
{
	OnInitializeHelper(StaticStruct());
}

USkeletalMeshComponent* FAnimNextSkeletalMeshComponentReferenceComponent::GetComponent() const
{
	return Cast<USkeletalMeshComponent>(Component);
}
