// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/ModuleTaskContext.h"

#include "Module/AnimNextModuleInstance.h"

namespace UE::UAF
{

FModuleTaskContext::FModuleTaskContext(FAnimNextModuleInstance& InModuleInstance)
	: FInstanceTaskContext(InModuleInstance)
	, ModuleInstance(InModuleInstance)
{
}

void FModuleTaskContext::QueueInputTraitEvent(FAnimNextTraitEventPtr Event) const
{
	ModuleInstance.QueueInputTraitEvent(MoveTemp(Event));
}

void FModuleTaskContext::QueueOutputTraitEvent(FAnimNextTraitEventPtr Event) const
{
	ModuleInstance.QueueOutputTraitEvent(MoveTemp(Event));
}

void FModuleTaskContext::TryAccessComponent(UScriptStruct* InComponentType, TFunctionRef<void(FUAFAssetInstanceComponent&)> InFunction) const
{
	FUAFAssetInstanceComponent* Component = static_cast<FUAFAssetInstanceComponent*>(ModuleInstance.TryGetComponent(InComponentType));
	if(Component == nullptr)
	{
		return;
	}

	InFunction(*Component);
}

void FModuleTaskContext::AccessComponent(UScriptStruct* InComponentType, TFunctionRef<void(FUAFAssetInstanceComponent&)> InFunction) const
{
	FUAFAssetInstanceComponent& Component = ModuleInstance.GetOrAddComponent(InComponentType);
	InFunction(Component);
}

FAnimNextModuleInstance* const FModuleTaskContext::GetModuleInstance() const
{
	return &ModuleInstance;
}

}
