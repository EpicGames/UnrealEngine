// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/ModuleTickFunction.h"

#include "AnimNextDebugDraw.h"
#include "AnimNextStats.h"
#include "RigVMRuntimeDataRegistry.h"
#include "Algo/TopologicalSort.h"
#include "CrashReporter/CrashReporterHandler.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/ModuleTaskContext.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Module/ModuleEvents.h"
#include "Module/SystemReference.h"
#include "Module/UAFWeakSystemReference.h"
#include "TraitCore/TraitEventList.h"

namespace UE::UAF
{
	namespace Private
	{
		// When we run a module tick event, this is the currently executing tick function
		static thread_local FModuleEventTickFunction* CurrentTickFunction = nullptr;

		static constexpr int32 NoBatching = 0;
		static constexpr int32 ModuleBatching = 1;
		static constexpr int32 TaskSyncBatching = 2;

		static int32 GameThreadEventBatchMode = ModuleBatching;
		static FAutoConsoleVariableRef CVarGameThreadEventBatchMode(
			TEXT("a.AnimNext.GameThreadEventBatchMode"),
			GameThreadEventBatchMode,
			TEXT("Batching mode for game thread events, 0 = no batching, 1 = simple event queue, 2 = task sync manager if available"));
	}

void FModuleEventTickFunction::Initialize(float InDeltaTime)
{
	UAF_CRASH_REPORTER_SCOPE(ModuleInstance->GetObject(), ModuleInstance->GetSystem(), Initialize);
	ModuleInstance->RunScriptEvent(InitializeEvent, InDeltaTime);
}

void FModuleEventTickFunction::ExecuteBindings_WT(float InDeltaTime)
{
	UAF_CRASH_REPORTER_SCOPE(ModuleInstance->GetObject(), ModuleInstance->GetSystem(), ExecuteBindings_WT);
	ModuleInstance->RunScriptEvent(ExecuteBindings_WTEvent, InDeltaTime);
}

void FModuleEventTickFunction::EndTick(float DeltaTime)
{
	SCOPED_NAMED_EVENT(UAF_Module_EndTick, FColor::Orange);
	UAF_CRASH_REPORTER_SCOPE(ModuleInstance->GetObject(), ModuleInstance->GetSystem(), EndTick);

	// Give module a chance to finish up processing
	ModuleInstance->EndExecution(DeltaTime);

	// Decrement the remaining lifetime of the input events we processed and queue up any remaining events
	DecrementLifetimeAndPurgeExpired(ModuleInstance->InputEventList, ModuleInstance->OutputEventList);

	// Filter out our schedule action events, we'll hand them off to the main thread to execute
	FTraitEventList MainThreadActionEventList;
	if (!ModuleInstance->OutputEventList.IsEmpty())
	{
		for (FAnimNextTraitEventPtr& OutputEvent : ModuleInstance->OutputEventList)
		{
			if (!OutputEvent->IsValid())
			{
				continue;
			}

			if (FAnimNextModule_ActionEvent* ActionEvent = OutputEvent->AsType<FAnimNextModule_ActionEvent>())
			{
				if (ActionEvent->IsThreadSafe())
				{
					// Execute this action now
					ActionEvent->Execute();
				}
				else
				{
					// Defer this action and execute it on the main thread
					MainThreadActionEventList.Push(OutputEvent);
				}
			}
		}

		// Reset our list of output events, we don't retain any
		ModuleInstance->OutputEventList.Reset();
	}

	if(ModuleInstance->InitState == FAnimNextModuleInstance::EInitState::FirstUpdate)
	{
		ModuleInstance->TransitionToInitState(FAnimNextModuleInstance::EInitState::Initialized);

		if(ModuleInstance->InitMethod == EAnimNextModuleInitMethod::InitializeAndPause
#if WITH_EDITOR
			|| (ModuleInstance->InitMethod == EAnimNextModuleInitMethod::InitializeAndPauseInEditor && ModuleInstance->WorldType == EWorldType::Editor)
#endif
		)
		{
			// Disable ourselves (deferred)
			FSystemReference(*ModuleInstance).SetEnabled(false);
		}
	}

	if (!MainThreadActionEventList.IsEmpty())
	{
		RunTaskOnGameThread([MainThreadActionEventList = MoveTemp(MainThreadActionEventList)]()
			{
				SCOPED_NAMED_EVENT(UAF_Module_EndTick_GameThread, FColor::Orange);
				check(IsInGameThread());
				for (const FAnimNextTraitEventPtr& Event : MainThreadActionEventList)
				{
					FAnimNextModule_ActionEvent* ActionEvent = Event->AsType<FAnimNextModule_ActionEvent>();
					ActionEvent->Execute();
				}
			});
	}

#if UE_ENABLE_DEBUG_DRAWING
	// Perform any debug drawing
	ModuleInstance->DebugDraw->Draw();
#endif
}

void FModuleEventTickFunction::ExecuteTick(float DeltaTime, ELevelTick TickType, ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	LLM_SCOPE_BYNAME(TEXT("UAF"));
	UAF_CRASH_REPORTER_SCOPE(ModuleInstance->GetObject(), ModuleInstance->GetSystem(), ExecuteTick);
	
	check(PostExecuteGameThreadTaskList.IsEmpty());	// Should always be empty since we move it to the game thread task

	if (BatchedWorkHandle.IsValid() && GetActualEndTickGroup() != TickGroup)
	{
		// This spans multiple tick groups so cannot use the tick group queue
		BatchedWorkHandle.Reset();
	}

	{
		check(Private::CurrentTickFunction == nullptr);
		Private::CurrentTickFunction = this;

		Run(DeltaTime);

		Private::CurrentTickFunction = nullptr;
	}

	// If we have a task sync queue, flush it and extend this task
	if (BatchedWorkHandle.IsValid())
	{
		FGraphEventRef ThisEvent = GetCompletionHandle();
		BatchedWorkHandle.SendQueuedWork(&ThisEvent);
	}

	// Dispatch any post-execute game thread tasks we might have queued up
	if (!PostExecuteGameThreadTaskList.IsEmpty())
	{
		auto PostExecuteGameThreadTask = [PostExecuteGameThreadTaskList = MoveTemp(PostExecuteGameThreadTaskList)]()
			{
				SCOPED_NAMED_EVENT(UAF_Module_PostEventTick, FColor::Orange);
				check(IsInGameThread());

				for (auto& Function : PostExecuteGameThreadTaskList)
				{
					Function();
				}
			};

		if (IsInGameThread())
		{
			// We are already on the main thread, we've batched up all our work, execute it in bulk now
			PostExecuteGameThreadTask();
		}
		else
		{
			// Dispatch our main thread work as a single task
			FGraphEventRef PostExecuteCompletionEvent = FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(PostExecuteGameThreadTask), TStatId(), nullptr, ENamedThreads::GameThread);

			if (IsCompletionHandleValid())
			{
				// Update our completion event to make sure anyone that depends on us also depends on any game thread tasks we generate
				GetCompletionHandle()->DontCompleteUntil(PostExecuteCompletionEvent);
			}
		}
	}
}

void FModuleEventTickFunction::Run(float InDeltaTime)
{
	SCOPED_NAMED_EVENT(UAF_Module_EventTick, FColor::Orange);
	UAF_CRASH_REPORTER_SCOPE(ModuleInstance->GetObject(), ModuleInstance->GetSystem(), Run);
	
	{
		TUniqueFunction<void(const FModuleTaskContext&)> Function;
		while (PreExecuteTasks.Dequeue(Function))
		{
			Function(FModuleTaskContext(*ModuleInstance));
		}
	}

	if (bFirstUserEvent)
	{
#if UAF_TRACE_ENABLED
		ModuleInstance->bTracedThisFrame = false;
#endif

		// Execute any WT bindings, if the event is valid
		ExecuteBindings_WT(InDeltaTime);

		// Run the pending initialize if required
		if (ModuleInstance->InitState == FAnimNextModuleInstance::EInitState::PendingInitializeEvent)
		{
			Initialize(InDeltaTime);
			ModuleInstance->TransitionToInitState(FAnimNextModuleInstance::EInitState::FirstUpdate);
		}

		// Call module component's extension points
		ModuleInstance->BeginExecution(InDeltaTime);
	}

	ModuleInstance->RaiseTraitEvents(ModuleInstance->InputEventList);

	OnPreModuleEvent.Broadcast(FModuleTaskContext(*ModuleInstance));

	ModuleInstance->RunScriptEvent(Event, InDeltaTime);

	ModuleInstance->RaiseTraitEvents(ModuleInstance->OutputEventList);

	{
		TUniqueFunction<void(const FModuleTaskContext&)> Function;
		while (PostExecuteTasks.Dequeue(Function))
		{
			Function(FModuleTaskContext(*ModuleInstance));
		}
	}

	if (bLastUserEvent)
	{
		EndTick(InDeltaTime);
	}
}

#if WITH_EDITOR

void FModuleEventTickFunction::InitializeAndRunModule(FAnimNextModuleInstance& InModuleInstance)
{
	// Run sorted tick functions
	for(FModuleEventTickFunction& TickFunction : InModuleInstance.TickFunctions)
	{
		TickFunction.Run(0.0f);
	}
}

#endif

void FModuleEventTickFunction::AddSubsequent(UObject* InObject, FTickFunction& InTickFunction)
{
	auto FindExistingSubsequent = [InObject, &InTickFunction](const FTickPrerequisite& InSubsequent)
	{
		return InSubsequent.PrerequisiteObject == InObject && InSubsequent.PrerequisiteTickFunction == &InTickFunction;
	};

	InTickFunction.AddPrerequisite(ModuleInstance->GetObject(), *this);
	if(!ExternalSubsequents.ContainsByPredicate(FindExistingSubsequent))
	{
		ExternalSubsequents.Emplace(InObject, InTickFunction);
	}
}

void FModuleEventTickFunction::RemoveSubsequent(UObject* InObject, FTickFunction& InTickFunction)
{
	auto FindExistingSubsequent = [InObject, &InTickFunction](const FTickPrerequisite& InSubsequent)
	{
		return InSubsequent.PrerequisiteObject == InObject && InSubsequent.PrerequisiteTickFunction == &InTickFunction;
	};

	InTickFunction.RemovePrerequisite(ModuleInstance->GetObject(), *this);
	ExternalSubsequents.RemoveAllSwap(FindExistingSubsequent, EAllowShrinking::No);
}

void FModuleEventTickFunction::RemoveAllExternalSubsequents()
{
	for(FTickPrerequisite& Subsequent : ExternalSubsequents)
	{
		if(FTickFunction* TickFunction = Subsequent.Get())
		{
			TickFunction->RemovePrerequisite(ModuleInstance->GetObject(), *this);
		}
	}

	ExternalSubsequents.Reset();
}

void FModuleEventTickFunction::InitializeBatchedWork(UWorld* World)
{
	UE::Tick::FTaskSyncManager* SyncManager = UE::Tick::FTaskSyncManager::Get();

	if (Private::GameThreadEventBatchMode == Private::TaskSyncBatching && SyncManager)
	{
		SyncManager->RegisterTickGroupWorkHandle(World, TickGroup, BatchedWorkHandle);
	}
}

FString FModuleEventTickFunction::DiagnosticMessage()
{
	TStringBuilder<256> Builder;
	Builder.Append(TEXT("UAF: "));
	Event.GetEventName().AppendString(Builder);
	return Builder.ToString();
}

void FModuleEventTickFunction::RunTaskOnGameThread(TUniqueFunction<void(void)>&& InFunction)
{
	if (!InFunction)
	{
		UE_LOGF(LogAnimation, Warning, "ModuleEventTickFunction::RunTaskOnGameThread: Could not run task - InFunction is not valid.");
		return;
	}

	if (FModuleEventTickFunction* TickFunction = Private::CurrentTickFunction)
	{
		if (TickFunction->BatchedWorkHandle.IsValid())
		{
			// Use task sync manager if it was set up
			TickFunction->BatchedWorkHandle.QueueWorkFunction(MoveTemp(InFunction));
			return;
		}
		else if (Private::GameThreadEventBatchMode != Private::NoBatching)
		{
			// Use normal batching if enabled
			TickFunction->PostExecuteGameThreadTaskList.Add(MoveTemp(InFunction));
			return;
		}
	}

	// Fallback to old behavior if no tick function or mode is 0
	if (IsInGameThread())
	{
		InFunction();
	}
	else
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(InFunction), TStatId(), nullptr, ENamedThreads::GameThread);
	}
}

}
