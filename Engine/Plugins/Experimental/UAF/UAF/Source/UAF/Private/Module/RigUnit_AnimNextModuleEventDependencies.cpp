// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/RigUnit_AnimNextModuleEventDependencies.h"

#include "Module/AnimNextModuleInstance.h"
#include "Module/RigVMTrait_ModuleEventDependency.h"
#if WITH_EDITOR
#include "RigVMModel/RigVMController.h"
#include "StructViewerModule.h"
#include "ToolMenus.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "StructViewerFilter.h"
#include "Widgets/Layout/SBox.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextModuleEventDependencies)

namespace UE::UAF
{

// Helper function used to add/remove dependencies
template<typename PredicateType>
static void ApplyDependenciesHelper(const FAnimNextExecuteContext& InContext, const PredicateType& InPredicate)
{
	FAnimNextModuleInstance& ModuleInstance = InContext.GetContextData<FAnimNextModuleContextData>().GetModuleInstance();
	TArray<TInstancedStruct<FRigVMTrait_ModuleEventDependency>> Dependencies;
	Dependencies.Reserve(InContext.GetTraits().Num());
	for(const FRigVMTraitScope& TraitScope : InContext.GetTraits())
	{
		const FRigVMTrait_ModuleEventDependency* DependencyTrait = TraitScope.GetTrait<FRigVMTrait_ModuleEventDependency>();
		if(DependencyTrait == nullptr)
		{
			continue;
		}

		// Copy the state as the memory is re-used by RigVM and we might get called again with a different value for the
		// same memory location
		TInstancedStruct<FRigVMTrait_ModuleEventDependency> Dependency;
		Dependency.InitializeAsScriptStruct(TraitScope.GetScriptStruct(), reinterpret_cast<const uint8*>(DependencyTrait));

		Dependencies.Add(MoveTemp(Dependency));
	}

	if(Dependencies.Num() == 0)
	{
		UE_LOGF(LogAnimation, Warning, "ApplyDependenciesHelper: Could not apply dependencies - No dependencies found.");
		return;
	}

	TWeakObjectPtr<UObject> WeakObject = ModuleInstance.GetObject();
	TArray<FTickFunction*> TickFunctions;
	TickFunctions.Reserve(Dependencies.Num());
	bool bTickFunctionFound = false; 
	for(const TInstancedStruct<FRigVMTrait_ModuleEventDependency>& Dependency : Dependencies)
	{
		FTickFunction* TickFunction = ModuleInstance.FindTickFunctionByName(Dependency.Get<FRigVMTrait_ModuleEventDependency>().EventName);
		TickFunctions.Add(TickFunction);
		bTickFunctionFound |= (TickFunction != nullptr);
	}

	if(!bTickFunctionFound)
	{
		UE_LOGF(LogAnimation, Warning, "ApplyDependenciesHelper: Could not apply dependencies - No valid tick function found.");
		return;
	}

	// Tick function prerequisites can only be updated on the game thread
	FAnimNextModuleInstance::RunTaskOnGameThread([WeakObject, TickFunctions = MoveTemp(TickFunctions), Dependencies = MoveTemp(Dependencies), InPredicate]()
	{
		UObject* Object = WeakObject.Get();
		if(Object == nullptr)
		{
			UE_LOGF(LogAnimation, Warning, "ApplyDependenciesHelper: Could not apply dependencies - Module Object is not valid.");
			return;
		}

		check(TickFunctions.Num() == Dependencies.Num());

		for(int32 DependencyIndex = 0; DependencyIndex < Dependencies.Num(); ++DependencyIndex)
		{
			FTickFunction* TickFunction = TickFunctions[DependencyIndex];
			if(TickFunction == nullptr)
			{
				continue;
			}
			
			FModuleDependencyContext Context(Object, *TickFunction);
			const FRigVMTrait_ModuleEventDependency& Dependency = Dependencies[DependencyIndex].Get<FRigVMTrait_ModuleEventDependency>();
			InPredicate(Context, Dependency);

			// For Before-ordered dependencies, also apply to module-owned prerequisites
			// (e.g. ExecuteBindings_GT chained before the user event tick function),
			// so that external prerequisites like a pawn primary tick also gate bindings execution.
			if(Dependency.Ordering == EAnimNextModuleEventDependencyOrdering::Before)
			{
				for(FTickPrerequisite& Prereq : TickFunction->GetPrerequisites())
				{
					if(Prereq.PrerequisiteObject == Object)
					{
						if(FTickFunction* PrereqTF = Prereq.Get())
						{
							// Only propagate to game-thread prerequisites. Worker-thread tick functions
							// run in parallel with the game thread, so adding a game-thread actor tick
							// as a prerequisite would incorrectly serialize parallel work.
							if(!PrereqTF->bRunOnAnyThread)
							{
								FModuleDependencyContext PrereqContext(Object, *PrereqTF);
								InPredicate(PrereqContext, Dependency);
							}
						}
					}
				}
			}
		}
	});
}

}

TArray<FRigVMUserWorkflow> FRigUnit_AnimNextModuleDependenciesBase::GetSupportedWorkflows(const UObject* InSubject) const
{
	TArray<FRigVMUserWorkflow> Workflows = Super::GetSupportedWorkflows(InSubject);

#if WITH_EDITOR
	Workflows.Emplace(
		TEXT("Add"),
		TEXT("Adds a module event dependency to this node"),
		ERigVMUserWorkflowType::NodeContextButton,
		FRigVMPerformUserWorkflowDelegate::CreateLambda([](const URigVMUserWorkflowOptions* InOptions, UObject* InController)
		{
			URigVMController* Controller = CastChecked<URigVMController>(InController);
			if(Controller == nullptr)
			{
				return false;
			}

			URigVMNode* Node = InOptions->GetSubject<URigVMNode>();
			if(Node == nullptr)
			{
				return false;
			}

			static const FName MenuName = "UAFModuleDependenciesAddMenu";
			if(!UToolMenus::Get()->IsMenuRegistered(MenuName))
			{
				UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
				Menu->AddDynamicSection("DependencyTraits", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					UAnimNextAddDependencyMenuContext* ContextObject = InMenu->FindContext<UAnimNextAddDependencyMenuContext>();
					if(ContextObject == nullptr)
					{
						return;
					}

					auto OnTraitPicked = [Controller = ContextObject->Controller, Node = ContextObject->Node](const UScriptStruct* InStruct)
					{
						FSlateApplication::Get().DismissAllMenus();

						FScopedTransaction Transaction(NSLOCTEXT("UAFModuleDependencies", "AddDependencyTraitTransaction", "Add dependency trait"));
						Controller->AddTrait(Node, const_cast<UScriptStruct*>(InStruct), InStruct->GetFName(), TEXT(""));
					};

					class FStructFilter : public IStructViewerFilter
					{
					public:
						virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
						{
							if (InStruct->HasMetaData(TEXT("Hidden")))
							{
								return false;
							}

							return InStruct->IsChildOf(FRigVMTrait_ModuleEventDependency::StaticStruct()) && InStruct != FRigVMTrait_ModuleEventDependency::StaticStruct();
						}

						virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<class FStructViewerFilterFuncs> InFilterFuncs)
						{
							return false;
						};
					};
					
					FToolMenuSection& Section = InMenu->AddSection("DependencyTraits");
					FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");
					FStructViewerInitializationOptions InitOptions;
					InitOptions.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
					InitOptions.StructFilter = MakeShared<FStructFilter>();
					Section.AddEntry(
						FToolMenuEntry::InitWidget(
							"Traits",
							SNew(SBox)
							.WidthOverride(300.0f)
							.HeightOverride(400.0f)
							[
								StructViewerModule.CreateStructViewer(InitOptions, FOnStructPicked::CreateLambda(OnTraitPicked))
							],
							FText::GetEmpty(),
							true,
							false,
							true));
				}));
			}

			UAnimNextAddDependencyMenuContext* ContextObject = NewObject<UAnimNextAddDependencyMenuContext>();
			ContextObject->Controller = Controller;
			ContextObject->Node = Node;

			FSlateApplication::Get().PushMenu(
				FSlateApplication::Get().GetInteractiveTopLevelWindows()[0],
				FWidgetPath(),
				UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(ContextObject)),
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);

			return true;
		}),
		URigVMUserWorkflowOptions::StaticClass());

	Workflows.Emplace(
		TEXT("Remove Dependency"),
		TEXT("Removes this module event dependency from this node"),
		ERigVMUserWorkflowType::PinContext,
		FRigVMPerformUserWorkflowDelegate::CreateLambda([](const URigVMUserWorkflowOptions* InOptions, UObject* InController)
		{
			URigVMController* Controller = CastChecked<URigVMController>(InController);
			if(Controller == nullptr)
			{
				return false;
			}

			URigVMPin* Pin = InOptions->GetSubject<URigVMPin>();
			if(Pin == nullptr)
			{
				return false;
			}

			if(!Pin->IsTraitPin() || Pin->GetTraitScriptStruct() == nullptr || !Pin->GetTraitScriptStruct()->IsChildOf(FRigVMTrait_ModuleEventDependency::StaticStruct()))
			{
				return false;
			}

			URigVMPin* TraitPin = Pin->GetNode()->FindTrait(Pin);
			if(TraitPin == nullptr)
			{
				return false;
			}

			FScopedTransaction Transaction(NSLOCTEXT("UAFModuleDependencies", "RemoveDependencyTraitTransaction", "Remove dependency trait"));
			Controller->RemoveTrait(Pin->GetNode(), TraitPin->GetFName());

			return true;
		}),
		URigVMUserWorkflowOptions::StaticClass());
#endif

	return Workflows;
}

FRigVMTrait_ModuleEventDependency::FRigVMTrait_ModuleEventDependency()
	: Ordering(EAnimNextModuleEventDependencyOrdering::Before)
	, EventName(FRigUnit_AnimNextPrePhysicsEvent::DefaultEventName)
{
}

FRigUnit_AnimNextModuleAddDependencies_Execute()
{
	UE::UAF::ApplyDependenciesHelper(ExecuteContext, [](const UE::UAF::FModuleDependencyContext& InContext, const FRigVMTrait_ModuleEventDependency& InDependency)
	{
		InDependency.OnAddDependency(InContext);
	});
}

FRigUnit_AnimNextModuleRemoveDependencies_Execute()
{
	UE::UAF::ApplyDependenciesHelper(ExecuteContext, [](const UE::UAF::FModuleDependencyContext& InContext, const FRigVMTrait_ModuleEventDependency& InDependency)
	{
		InDependency.OnRemoveDependency(InContext);
	});
}
