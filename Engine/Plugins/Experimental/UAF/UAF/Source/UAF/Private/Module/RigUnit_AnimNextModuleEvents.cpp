// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/RigUnit_AnimNextModuleEvents.h"

#include "Engine/World.h"
#include "Module/AnimNextModuleInstance.h"
#include "Script/ScriptEventTickFunctionBindings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_AnimNextModuleEvents)

namespace UE::UAF::CVars
{
	// Expose control over whether or not the AnimNext binding execution should happen before others in the tick group
	// This can allow them to dispatch animation jobs earlier, helping hide their latency
	static TAutoConsoleVariable<bool> CVarHighPriorityAnimNextExecuteBindingsTick(
		TEXT("a.AnimNext.HighPriorityAnimNextExecuteBindingsTick"),
		false,
		TEXT("If true, then schedule the AnimNext binding execution in a high priority tick group before other ticks."));
}

FRigUnit_AnimNextExecuteBindings_GT_Execute()
{
}

UE::UAF::FTickFunctionBindingFunction FRigUnit_AnimNextExecuteBindings_GT::GetBindingFunction() const
{
	return [](const UE::UAF::FTickFunctionBindingContext& InContext, FTickFunction& InTickFunction)
	{
		const bool bHighPriority = UE::UAF::CVars::CVarHighPriorityAnimNextExecuteBindingsTick.GetValueOnGameThread();
		if (InTickFunction.bHighPriority != bHighPriority)
		{
			InTickFunction.SetPriorityIncludingPrerequisites(bHighPriority);
		}
	};
}

FRigUnit_AnimNextExecuteBindings_WT_Execute()
{
}

UE::UAF::FTickFunctionBindingFunction FRigUnit_AnimNextExecuteBindings_WT::GetBindingFunction() const
{
	return [](const UE::UAF::FTickFunctionBindingContext& InContext, FTickFunction& InTickFunction)
	{
		const bool bHighPriority = UE::UAF::CVars::CVarHighPriorityAnimNextExecuteBindingsTick.GetValueOnGameThread();
		if (InTickFunction.bHighPriority != bHighPriority)
		{
			InTickFunction.SetPriorityIncludingPrerequisites(bHighPriority);
		}
	};
}

FRigUnit_AnimNextInitializeEvent_Execute()
{
}

FString FRigUnit_AnimNextUserEvent::GetUnitSubTitle() const
{
	FString Subtitle = UEnum::GetDisplayValueAsText<ETickingGroup>(TickGroup).ToString();
	if (SortOrder != 0)
	{
		Subtitle.Appendf(TEXT(" (%d)"), SortOrder);
	}

	return Subtitle;
}

UE::UAF::FTickFunctionBindingFunction FRigUnit_AnimNextUserEvent::GetBindingFunction() const
{
	return [TickGroup = TickGroup](const UE::UAF::FTickFunctionBindingContext& InContext, FTickFunction& InTickFunction)
	{
		InTickFunction.TickGroup = TickGroup;
	};
}

FRigUnit_AnimNextPrePhysicsEvent_Execute()
{
}

FRigUnit_AnimNextPostPhysicsEvent_Execute()
{
}
