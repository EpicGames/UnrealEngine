// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction)

#if WITH_EDITOR
FString FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction::GetDisplayName() const
{
	return StaticStruct()->GetDisplayNameText().ToString();
}
#endif

void FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction::OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
	if(AnimNextComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction: Could not add dependency for component of class [%ls] - UAF Component is not valid.", *GetNameSafe(ComponentClass));
		return;
	}

	AActor* Actor = AnimNextComponent->GetOwner();
	if(Actor == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction: Could not add dependency for component of class [%ls] - UAF Component has no valid owner.", *GetNameSafe(ComponentClass));
		return;
	}

	UActorComponent* TargetComponent = Actor->GetComponentByClass(ComponentClass);
	if(TargetComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Log, "FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction: Could not add dependency for component of class [%ls] - [%ls] does not have that component.",
			*GetNameSafe(ComponentClass), *Actor->GetName());
		return;
	}

	if(Ordering == EAnimNextModuleEventDependencyOrdering::Before)
	{
		InContext.TickFunction.AddPrerequisite(TargetComponent, TargetComponent->PrimaryComponentTick);
	}
	else
	{
		TargetComponent->PrimaryComponentTick.AddPrerequisite(AnimNextComponent, InContext.TickFunction);
	}
}

void FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction::OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
	if(AnimNextComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction: Could not remove dependency for component of class [%ls] - UAF Component is not valid.", *GetNameSafe(ComponentClass));
		return;
	}

	AActor* Actor = AnimNextComponent->GetOwner();
	if(Actor == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction: Could not remove dependency for component of class [%ls] - UAF Component has no valid owner.", *GetNameSafe(ComponentClass));
		return;
	}

	UActorComponent* TargetComponent = Actor->GetComponentByClass(ComponentClass);
	if(TargetComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Log, "FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction: Could not remove dependency for component of class [%ls] - [%ls] does not have that component.",
			*GetNameSafe(ComponentClass), *Actor->GetName());
		return;
	}

	if(Ordering == EAnimNextModuleEventDependencyOrdering::Before)
	{
		InContext.TickFunction.RemovePrerequisite(TargetComponent, TargetComponent->PrimaryComponentTick);
	}
	else
	{
		TargetComponent->PrimaryComponentTick.RemovePrerequisite(AnimNextComponent, InContext.TickFunction);
	}
}
