// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Module/RigVMTrait_ModuleEventDependency.h"
#include "Templates/SubclassOf.h"
#include "RigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction.generated.h"

class UActorComponent;

// A dependency on the primary tick function of the first-found actor component of the specified class
USTRUCT(DisplayName="Actor Component by Class", meta=(ShowTooltip=true))
struct FRigVMTrait_ModuleEventDependency_ActorComponentClassPrimaryTickFunction : public FRigVMTrait_ModuleEventDependency
{
	GENERATED_BODY()

	// FRigVMTrait interface
#if WITH_EDITOR
	virtual FString GetDisplayName() const override;
#endif

	// FAnimNextModuleEventDependency interface
	virtual void OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;
	virtual void OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;

	// The component class to look for when establishing the dependency
	UPROPERTY(EditAnywhere, Category = "Dependency")
	TSubclassOf<UActorComponent> ComponentClass;
};