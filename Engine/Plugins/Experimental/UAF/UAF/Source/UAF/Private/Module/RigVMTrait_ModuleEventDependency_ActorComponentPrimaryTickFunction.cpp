// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction)

#if WITH_EDITOR
FString FRigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction::GetDisplayName() const
{
	return StaticStruct()->GetDisplayNameText().ToString();
}
#endif

void FRigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction::OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
	if(AnimNextComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction: Could not add dependency for component [%ls] - UAF Component is not valid.", *GetNameSafe(Component));
		return;
	}

	if(Component == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction: Could not add dependency - Component is not valid.");
		return;
	}

	if(Ordering == EAnimNextModuleEventDependencyOrdering::Before)
	{
		InContext.TickFunction.AddPrerequisite(Component, Component->PrimaryComponentTick);
	}
	else
	{
		Component->PrimaryComponentTick.AddPrerequisite(AnimNextComponent, InContext.TickFunction);
	}
}

void FRigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction::OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
	if(AnimNextComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction: Could not remove dependency for component [%ls] - UAF Component is not valid.", *GetNameSafe(Component));
		return;
	}
	
	if(Component == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction: Could not remove dependency - Component is not valid.");
		return;
	}

	if(Ordering == EAnimNextModuleEventDependencyOrdering::Before)
	{
		InContext.TickFunction.RemovePrerequisite(Component, Component->PrimaryComponentTick);
	}
	else
	{
		Component->PrimaryComponentTick.RemovePrerequisite(AnimNextComponent, InContext.TickFunction);
	}
}
