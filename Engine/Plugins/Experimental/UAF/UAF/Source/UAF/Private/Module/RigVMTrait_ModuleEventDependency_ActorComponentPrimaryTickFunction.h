// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Module/RigVMTrait_ModuleEventDependency.h"
#include "RigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction.generated.h"

class UActorComponent;

// A dependency on an actor component's primary tick function
USTRUCT(DisplayName="Actor Component", meta=(ShowTooltip=true))
struct FRigVMTrait_ModuleEventDependency_ActorComponentPrimaryTickFunction : public FRigVMTrait_ModuleEventDependency
{
	GENERATED_BODY()

	// FRigVMTrait interface
#if WITH_EDITOR
	virtual FString GetDisplayName() const override;
#endif

	// FAnimNextModuleEventDependency interface
	virtual void OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;
	virtual void OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;

	// The component on whose primary tick function we will depend
	UPROPERTY(EditAnywhere, Category = "Dependency")
	TObjectPtr<UActorComponent> Component;
};