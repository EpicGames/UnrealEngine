// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction)

#if WITH_EDITOR
FString FRigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction::GetDisplayName() const
{
	return StaticStruct()->GetDisplayNameText().ToString();
}
#endif

void FRigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction::OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
	if(AnimNextComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction: Could not add dependency for Actor - UAF Component is not valid.");
		return;
	}

	AActor* Actor = AnimNextComponent->GetOwner();
	if(Actor == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction: Could not add dependency for Actor - UAF Component has no valid owner.");
		return;
	}

	if(Ordering == EAnimNextModuleEventDependencyOrdering::Before)
	{
		InContext.TickFunction.AddPrerequisite(Actor, Actor->PrimaryActorTick);
	}
	else
	{
		Actor->PrimaryActorTick.AddPrerequisite(AnimNextComponent, InContext.TickFunction);
	}
}

void FRigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction::OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const
{
	UActorComponent* AnimNextComponent = Cast<UActorComponent>(InContext.Object);
	if(AnimNextComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction: Could not remove dependency for Actor - UAF Component is not valid.");
		return;
	}

	AActor* Actor = AnimNextComponent->GetOwner();
	if(Actor == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FRigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction: Could not remove dependency for Actor");
		return;
	}

	if(Ordering == EAnimNextModuleEventDependencyOrdering::Before)
	{
		InContext.TickFunction.RemovePrerequisite(Actor, Actor->PrimaryActorTick);
	}
	else
	{
		Actor->PrimaryActorTick.RemovePrerequisite(AnimNextComponent, InContext.TickFunction);
	}
}
