// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Module/RigVMTrait_ModuleEventDependency.h"
#include "RigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction.generated.h"

class UActor;

// A dependency on the primary tick function of the host actor
USTRUCT(DisplayName="Actor", meta=(ShowTooltip=true))
struct FRigVMTrait_ModuleEventDependency_ActorPrimaryTickFunction : public FRigVMTrait_ModuleEventDependency
{
	GENERATED_BODY()

	// FRigVMTrait interface
#if WITH_EDITOR
	virtual FString GetDisplayName() const override;
#endif

	// FAnimNextModuleEventDependency interface
	virtual void OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;
	virtual void OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const override;
};
