// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/SystemReference.h"

#include "AnimNextDebugDraw.h"
#include "AnimNextPool.h"
#include "AnimNextPoolHandle.h"
#include "AnimNextRigVMAsset.h"
#include "Engine/World.h"
#include "Factory/SystemFactory.h"
#include "Factory/UAFSystemFactoryParams.h"
#include "Logging/StructuredLog.h"
#include "Misc/CoreDelegates.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/ModuleTaskContext.h"
#include "Module/SystemDependency.h"
#include "Module/UAFWeakSystemReference.h"
#include "Containers/MpscQueue.h"
#include "Containers/TransactionallySafeSpscQueue.h"
#include "Module/UAFSystemAssetData.h"
#include "UAF/UAFAssetData.h"

namespace UE::UAF::Private
{
	struct FSystemPool : public FGCObject
	{
		// Global pool of system instances
		TPool<FAnimNextModuleInstance> SystemInstances;

		// Lock for concurrent allocations/resolves
		FTransactionallySafeRWLock SystemInstancesLock;
		
		// Currently dont support a transactionally safe TMpscQueue, so we need to just use Spsc for now for actions that might run in transactions
		// like deletes
		TTransactionallySafeSpscQueue<TUniqueFunction<void()>> PendingDeletes;

		// Actions to perform once world tick is completed each frame
		TMpscQueue<TUniqueFunction<void()>> PendingActions;

		// Deferred cleanup of handles that are released this frame
		void FlushPendingActions()
		{
			using namespace UE::UAF;

			TUniqueFunction<void()> PendingAction;
			while (PendingDeletes.Dequeue(PendingAction))
			{
				PendingAction();
			}

			while (PendingActions.Dequeue(PendingAction))
			{
				PendingAction();
			}
		}

		// FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override
		{
			UE::TReadScopeLock InstancesLockScope(SystemInstancesLock);

			for (FAnimNextModuleInstance& Instance : SystemInstances)
			{
				Collector.AddPropertyReferencesWithStructARO(FAnimNextModuleInstance::StaticStruct(), &Instance);
			}
		}

		virtual FString GetReferencerName() const override
		{
			return TEXT("UAFSystemPool");
		}
	};

	static FSystemPool* GPool = nullptr;

	static FSystemPool& GetPool()
	{
		checkf(GPool, TEXT("UAF system pool is not initialized"));
		return *GPool;
	}

	static FDelegateHandle GSystemPoolPreExitHandle;

	static void SystemPoolOnPreExit()
	{
		GetPool().FlushPendingActions();

		FCoreDelegates::OnEnginePreExit.Remove(GSystemPoolPreExitHandle);
	}
}

namespace UE::UAF
{

struct FCustomSystemDeleter
{
	explicit FCustomSystemDeleter(TPoolHandle<FAnimNextModuleInstance> InHandle)
		: Handle(InHandle)
	{
	}

	void operator()(FAnimNextModuleInstance* InInstance) const
	{
		using namespace UE::UAF;
		check(IsInGameThread());
		if (InInstance)
		{
			check(InInstance == &Private::GetPool().SystemInstances.Get(Handle));
#if WITH_EDITOR 
			InInstance->bHasBeenQueuedForRelease = true;
#endif // WITH_EDITOR

			Private::GetPool().PendingDeletes.Enqueue([Handle = Handle]()
			{
				UE::TWriteScopeLock InstancesLockScope(Private::GetPool().SystemInstancesLock);
				Private::GetPool().SystemInstances.Release(Handle);
			});
		}
	}

	// Handle used for pooled allocation
	TPoolHandle<FAnimNextModuleInstance> Handle;
};

FSystemReference::FSystemReference(FAnimNextModuleInstance& InInstance)
	: Ptr(InInstance.AsShared())
{
}

FSystemReference::FSystemReference(TConstStructView<FUAFSystemFactoryAsset> AssetData, UObject* InObject, EAnimNextModuleInitMethod InitMethod)
{
	check(IsInGameThread() || IsInParallelGameThread());
	{
		LLM_SCOPE_BYNAME(TEXT("UAF"));

		check(AssetData.Get().Validate());

		FUAFSystemFactoryParams FactoryParams;
		const UUAFSystem* System = nullptr;
		
		// Special case for system assets, no need to create a new system
		if (AssetData.GetScriptStruct()->IsChildOf<FUAFSystemFactoryAsset_System>())
		{
			System = AssetData.Get<FUAFSystemFactoryAsset_System>().System.Get();
		}
		else
		{
			FactoryParams = FSystemFactory::GetDefaultParamsForAsset(AssetData);
			System = FSystemFactory::BuildSystem(FactoryParams.GetBuilder());
		}

		if (System == nullptr)
		{
			return;
		}

		FAnimNextModuleInstance* Instance = nullptr;
		TPoolHandle<FAnimNextModuleInstance> Handle;
		{
			UE::TWriteScopeLock InstancesLockScope(Private::GetPool().SystemInstancesLock);
			Handle = Private::GetPool().SystemInstances.Emplace(System, InObject, InitMethod);
			Instance = &Private::GetPool().SystemInstances.Get(Handle);
		}
		Ptr = TSharedPtr<FAnimNextModuleInstance>(Instance, FCustomSystemDeleter(Handle));
		Instance->Initialize(FactoryParams);
	}
}

FSystemReference::FSystemReference(FSystemReference&& InOther)
{
	Ptr = MoveTemp(InOther.Ptr);
}

FSystemReference& FSystemReference::operator=(FSystemReference&& InOther)
{
	Ptr = MoveTemp(InOther.Ptr);
	return *this;
}

void FSystemReference::Init()
{
	checkf(Private::GPool == nullptr, TEXT("UAF: System pool has already been created"));
	Private::GPool = new Private::FSystemPool();

	Private::GSystemPoolPreExitHandle = FCoreDelegates::OnEnginePreExit.AddStatic(&Private::SystemPoolOnPreExit);
}

void FSystemReference::Destroy()
{
	checkf(Private::GPool != nullptr, TEXT("UAF: System pool has already been destroyed"));
	checkf(Private::GPool->PendingActions.IsEmpty(), TEXT("UAF: System pool has pending actions that were added after Engine PreExit"));
	checkf(Private::GPool->PendingDeletes.IsEmpty(), TEXT("UAF: System pool has pending deletes that were added after Engine PreExit"));
	checkf(Private::GPool->SystemInstances.Num() == 0, TEXT("UAF: System pool is being destroyed with %d allocated entries"), Private::GPool->SystemInstances.Num());

	delete Private::GPool;
	Private::GPool = nullptr;
}

void FSystemReference::Reset()
{
	check(IsInGameThread());

	if (Ptr)
	{
#if UE_ENABLE_DEBUG_DRAWING
		// Remove debug drawing immediately as the renderer will need to know about this before EOF
		Ptr->DebugDraw->RemovePrimitive();
#endif

		// Remove all tick dependencies immediately, as once the handle has been invalidated there is no way for external systems to remove their dependencies
		Ptr->RemoveAllTickDependencies();
		Ptr.Reset();
	}
}

bool FSystemReference::IsValid() const
{
	return Ptr.IsValid();
}

const UUAFSystem* FSystemReference::GetSystem() const
{
	if (IsValid())
	{
		return Ptr->GetSystem();
	}
	return nullptr;
}

void FSystemReference::SetEnabled(bool bInEnabled) const
{
	using namespace UE::UAF;
	if (IsValid())
	{
		Private::GetPool().PendingActions.Enqueue([WeakPtr = TWeakPtr<FAnimNextModuleInstance>(Ptr), bInEnabled]()
		{
			if (TSharedPtr<FAnimNextModuleInstance> PinnedPtr = WeakPtr.Pin())
			{
				PinnedPtr->SetEnabled(bInEnabled);
			}
		});
	}
}

#if UE_ENABLE_DEBUG_DRAWING
void FSystemReference::ShowDebugDrawing(bool bInShowDebugDrawing) const
{
	using namespace UE::UAF;
	if (IsValid())
	{
		Private::GetPool().PendingActions.Enqueue([WeakPtr = TWeakPtr<FAnimNextModuleInstance>(Ptr), bInShowDebugDrawing]()
		{
			if (TSharedPtr<FAnimNextModuleInstance> PinnedPtr = WeakPtr.Pin())
			{
				PinnedPtr->ShowDebugDrawing(bInShowDebugDrawing);
			}
		});
	}
}
#endif

void FSystemReference::QueueTask(FName InSystemEventName, TUniqueFunction<void(const UE::UAF::FModuleTaskContext&)>&& InTaskFunction, UE::UAF::ETaskRunLocation InLocation) const
{
	using namespace UE::UAF;
	if (IsValid())
	{
		Ptr->QueueTask(InSystemEventName, MoveTemp(InTaskFunction), InLocation);
	}
}

void FSystemReference::QueueInputTraitEvent(FAnimNextTraitEventPtr Event) const
{
	using namespace UE::UAF;
	QueueTask(NAME_None, [Event = MoveTemp(Event)](const FModuleTaskContext& InContext)
	{
		InContext.QueueInputTraitEvent(Event);
	},
	ETaskRunLocation::Before);
}

const FTickFunction* FSystemReference::FindTickFunction(FName InEventName) const
{
	using namespace UE::UAF;
	check(IsInGameThread() || IsInParallelGameThread());
	if (!IsValid())
	{
		UE_LOGFMT(LogAnimation, Warning, "FindTickFunction: Invalid system handle");
		return nullptr;
	}

	const FModuleEventTickFunction* TickFunction = Ptr->FindTickFunctionByName(InEventName);
	if (TickFunction == nullptr)
	{
		UE_LOGFMT(LogAnimation, Warning, "FindTickFunction: Could not find event '{EventName}' in module '{ModuleName}'", InEventName, Ptr->GetAssetName());
		return nullptr;
	}

	if (TickFunction->bUserEvent)
	{
		return TickFunction;
	}

	UE_LOGFMT(LogAnimation, Warning, "FindTickFunction: Event '{EventName}' in module '{ModuleName}' is not a bUserEvent, therefore cannot be exposed", InEventName, Ptr->GetAssetName());
	return nullptr;
}

FName FSystemReference::GetFirstUserEventName() const
{
	check(IsInGameThread());
	if (!IsValid())
	{
		UE_LOGFMT(LogAnimation, Warning, "GetFirstUserEventName: Invalid system handle");
		return FName();
	}

	const FModuleEventTickFunction* TickFunction = Ptr->FindFirstUserTickFunction();
	return TickFunction != nullptr ? TickFunction->Event.GetEventName() : FName();
}

FName FSystemReference::GetLastUserEventName() const
{
	check(IsInGameThread());
	if (!IsValid())
	{
		UE_LOGFMT(LogAnimation, Warning, "GetLastUserEventName: Invalid system handle");
		return FName();
	}

	const FModuleEventTickFunction* TickFunction = Ptr->FindLastUserTickFunction();
	return TickFunction != nullptr ? TickFunction->Event.GetEventName() : FName();
}

void FSystemReference::AddDependency(UObject* InObject, FTickFunction& InTickFunction, FName InEventName, UE::UAF::ESystemDependency InDependency) const
{
	using namespace UE::UAF;
	check(IsInGameThread());
	if (!IsValid())
	{
		UE_LOGFMT(LogAnimation, Warning, "AddDependency: Invalid system handle");
		return;
	}

	FModuleEventTickFunction* TickFunction = Ptr->FindTickFunctionByName(InEventName);
	if (TickFunction == nullptr)
	{
		UE_LOGFMT(LogAnimation, Warning, "AddDependency: Could not find event '{EventName}' in module '{ModuleName}'", InEventName, Ptr->GetAssetName());
		return;
	}

	if (InDependency == ESystemDependency::Prerequisite)
	{
		TickFunction->AddPrerequisite(InObject, InTickFunction);
	}
	else
	{
		TickFunction->AddSubsequent(InObject, InTickFunction);
	}
}

void FSystemReference::RemoveDependency(UObject* InObject, FTickFunction& InTickFunction, FName InEventName, UE::UAF::ESystemDependency InDependency) const
{
	using namespace UE::UAF;
	check(IsInGameThread());
	if (!IsValid())
	{
		UE_LOGFMT(LogAnimation, Warning, "RemoveDependency: Invalid system handle");
		return;
	}

	FModuleEventTickFunction* TickFunction = Ptr->FindTickFunctionByName(InEventName);
	if (TickFunction == nullptr)
	{
		UE_LOGFMT(LogAnimation, Warning, "RemoveDependency: Could not find event '{EventName}' in module '{ModuleName}'", InEventName, Ptr->GetAssetName());
		return;
	}

	if (InDependency == ESystemDependency::Prerequisite)
	{
		TickFunction->RemovePrerequisite(InObject, InTickFunction);
	}
	else
	{
		TickFunction->RemoveSubsequent(InObject, InTickFunction);
	}
}

void FSystemReference::AddSystemEventDependency(FName InSystemEventName, FUAFWeakSystemReference InOtherSystem, FName InOtherSystemEventName, UE::UAF::ESystemDependency InDependency) const
{
	using namespace UE::UAF;
	check(IsInGameThread());
	if (!IsValid())
	{
		UE_LOGFMT(LogAnimation, Warning, "AddDependency: Invalid system handle");
		return;
	}

	TSharedPtr<FAnimNextModuleInstance> OtherInstance = InOtherSystem.WeakPtr.Pin();
	if (!OtherInstance.IsValid())
	{
		UE_LOGFMT(LogAnimation, Warning, "AddDependency: Invalid other system reference");
		return;
	}

	if(Ptr == OtherInstance)
	{
		UE_LOGFMT(LogAnimation, Warning, "AddDependency: Cannot add a dependency on ourselves");
		return;
	}

	FModuleEventTickFunction* TickFunction = OtherInstance->FindTickFunctionByName(InOtherSystemEventName);
	if (TickFunction == nullptr)
	{
		UE_LOGFMT(LogAnimation, Warning, "AddDependency: Could not find event '{EventName}' in module '{ModuleName}'", InOtherSystemEventName, OtherInstance->GetAssetName());
		return;
	}

	AddDependency(OtherInstance->GetObject(), *TickFunction, InSystemEventName, InDependency);
}

void FSystemReference::RemoveSystemEventDependency(FName InSystemEventName, FUAFWeakSystemReference InOtherSystem, FName InOtherSystemEventName, UE::UAF::ESystemDependency InDependency) const
{
	using namespace UE::UAF;
	check(IsInGameThread());
	if (!IsValid())
	{
		return;
	}

	TSharedPtr<FAnimNextModuleInstance> OtherInstance = InOtherSystem.WeakPtr.Pin();
	if (!OtherInstance.IsValid())
	{
		UE_LOGFMT(LogAnimation, Warning, "RemoveDependency: Invalid other system reference");
		return;
	}

	if(Ptr == OtherInstance)
	{
		UE_LOGFMT(LogAnimation, Warning, "RemoveDependency: Cannot remove a dependency on ourselves");
		return;
	}

	FModuleEventTickFunction* TickFunction = OtherInstance->FindTickFunctionByName(InOtherSystemEventName);
	if (TickFunction == nullptr)
	{
		UE_LOGFMT(LogAnimation, Warning, "RemoveDependency: Could not find event '{EventName}' in module '{ModuleName}'", InOtherSystemEventName, OtherInstance->GetAssetName());
		return;
	}

	RemoveDependency(OtherInstance->GetObject(), *TickFunction, InSystemEventName, InDependency);
}

EPropertyBagResult FSystemReference::SetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InData) const
{
	using namespace UE::UAF;
	if (IsValid())
	{
		return Ptr->SetProxyVariable(InVariable, InType, InData);
	}
	return EPropertyBagResult::PropertyNotFound;
}
		
EPropertyBagResult FSystemReference::WriteVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>)> InFunction) const
{
	using namespace UE::UAF;
	if (IsValid())
	{
		return Ptr->WriteProxyVariable(InVariable, InType, InFunction);
	}
	return EPropertyBagResult::PropertyNotFound;
}

EPropertyBagResult FSystemReference::ReadVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TConstArrayView<uint8>)> InFunction) const
{
	if (IsValid())
	{
		return Ptr->ReadVariableFromExternal(InVariable, InType, InFunction);
	}
	return EPropertyBagResult::PropertyNotFound;
}
	
void FSystemReference::RunEvent(FName InEventName, float InDeltaTime) const
{
	using namespace UE::UAF;
	check(IsInGameThread() || IsInParallelGameThread());
	if (!IsValid())
	{
		UE_LOGFMT(LogAnimation, Warning, "RunEvent: Invalid system handle");
		return;
	}

	FModuleEventTickFunction* TickFunction = Ptr->FindTickFunctionByName(InEventName);
	if (TickFunction == nullptr)
	{
		UE_LOGFMT(LogAnimation, Warning, "RunEvent: Could not find event '{EventName}' in module '{ModuleName}'", InEventName, Ptr->GetAssetName());
		return;
	}

	TickFunction->Run(InDeltaTime);
}

#if WITH_EDITOR
void FSystemReference::RunInitialTickInEditor() const
{
	if (IsValid())
	{
		// In editor worlds we run a linearized 'initial tick' to ensure we generate an initial output pose, as these worlds dont always tick
		if( Ptr->GetWorldType() == EWorldType::Editor ||
			Ptr->GetWorldType() == EWorldType::EditorPreview)
		{
			FModuleEventTickFunction::InitializeAndRunModule(*Ptr.Get());
		}
	}
}
#endif

bool FSystemReference::ReadComponent(UScriptStruct* InComponentType, TFunctionRef<void(FConstStructView InComponentStruct)> InFunction) const
{
	if (IsValid())
	{
		return Ptr->ReadComponentFromExternal(InComponentType, InFunction);
	}
	return false;
}

} //end namespace UE::UAF

UUAFEngineSubsystem::UUAFEngineSubsystem()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		OnCompileJobFinishedHandle = UUAFRigVMAsset::OnCompileJobFinished().AddUObject(this, &UUAFEngineSubsystem::OnCompileJobFinished);
#endif

		OnPreGarbageCollectHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddLambda([]()
		{
			// Ensure we flush any pending releases before we perform GC, as we could
			// end up with spurious references to already-released systems
			UE::UAF::Private::GetPool().FlushPendingActions();
		});

		OnWorldPreActorTickHandle = FWorldDelegates::OnWorldPreActorTick.AddLambda([this](UWorld* InWorld, ELevelTick InTickType, float InDeltaSeconds)
		{
			if (InTickType == LEVELTICK_All || InTickType == LEVELTICK_ViewportsOnly)
			{
				UE::UAF::Private::GetPool().FlushPendingActions();
			}
		});
	}
}

void UUAFEngineSubsystem::BeginDestroy()
{
	Super::BeginDestroy();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		UUAFRigVMAsset::OnCompileJobFinished().Remove(OnCompileJobFinishedHandle);
#endif

		FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(OnPreGarbageCollectHandle);
		FWorldDelegates::OnWorldPreActorTick.Remove(OnWorldPreActorTickHandle);
	}
}

#if WITH_EDITOR
void UUAFEngineSubsystem::OnCompileJobFinished(UUAFRigVMAsset* InAsset)
{
	using namespace UE::UAF;

	// Flush any pending (queued up) instance release/destroy actions 
	Private::GetPool().FlushPendingActions();

	check(IsInGameThread());
	UE::TReadScopeLock InstancesLockScope(Private::GetPool().SystemInstancesLock);

	for (FAnimNextModuleInstance& Instance : Private::GetPool().SystemInstances)
	{
		if (Instance.GetSystem() == InAsset && ensure(Instance.bHasBeenQueuedForRelease == false))
		{
			Instance.OnCompileJobFinished();
		}
	}
}
#endif
