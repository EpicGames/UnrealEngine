// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/UAFModuleInstanceComponent.h"
#include "Module/AnimNextModuleInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFModuleInstanceComponent)

FAnimNextModuleInstance* FUAFModuleInstanceComponent::GetModuleInstancePtr()
{
	return static_cast<FAnimNextModuleInstance*>(GetAssetInstancePtr());
}

FAnimNextModuleInstance& FUAFModuleInstanceComponent::GetModuleInstance()
{
	return static_cast<FAnimNextModuleInstance&>(GetAssetInstance());
}

const FAnimNextModuleInstance& FUAFModuleInstanceComponent::GetModuleInstance() const
{
	return static_cast<const FAnimNextModuleInstance&>(GetAssetInstance());
}
