// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/UAFNotifyDispatcherComponent.h"

#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/AnimNextSkeletalMeshComponentReferenceComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFNotifyDispatcherComponent)

void FUAFNotifyDispatcherComponent::OnBindToInstance()
{
	// Default to the current host's skeletal mesh component, if any
	FAnimNextModuleInstance& ModuleInstance = GetModuleInstance();
	const FAnimNextSkeletalMeshComponentReferenceComponent& ComponentReference = ModuleInstance.GetOrAddComponent<FAnimNextSkeletalMeshComponentReferenceComponent>();
	SkeletalMeshComponent = ComponentReference.GetComponent();
	NotifyQueue.PredictedLODLevel = SkeletalMeshComponent ? SkeletalMeshComponent->GetPredictedLODLevel() : 0;
}

void FUAFNotifyDispatcherComponent::OnTraitEvent(FAnimNextTraitEvent& Event)
{
	if (UE::UAF::FNotifyDispatchEvent* NotifyDispatchEvent = Event.AsType<UE::UAF::FNotifyDispatchEvent>())
	{
		NotifyQueue.AddAnimNotifies(NotifyDispatchEvent->Notifies, NotifyDispatchEvent->Weight);
		NotifyDispatchEvent->MarkConsumed();
	}
}

void FUAFNotifyDispatcherComponent::OnEndExecution(float InDeltaTime)
{
	using namespace UE::UAF;

	// Early out if we dont need to do any work
	if (NotifyQueue.AnimNotifies.Num() == 0 && ActiveAnimNotifyState.Num() == 0)
	{
		return;
	}

	FAnimNextModuleInstance& ModuleInstance = GetModuleInstance();

	TSharedPtr<FNotifyQueueDispatchEvent> NotifyQueueDispatchEvent;
	if (SkeletalMeshComponent != nullptr)
	{
		NotifyQueueDispatchEvent = MakeTraitEvent<FNotifyQueueDispatchEvent>();
		NotifyQueueDispatchEvent->WeakSkeletalMeshComponent = SkeletalMeshComponent;
		NotifyQueueDispatchEvent->DeltaSeconds = InDeltaTime;
	}

	NotifyQueue.PredictedLODLevel = SkeletalMeshComponent ? SkeletalMeshComponent->GetPredictedLODLevel() : 0;

	// Logic from here largely mimics that of UAnimInstance::TriggerAnimNotifies 

	// Array that will replace the 'ActiveAnimNotifyState' at the end of this function.
	TArray<const FAnimNotifyEvent*> NewActiveAnimNotifyState;
	NewActiveAnimNotifyState.Reserve(NotifyQueue.AnimNotifies.Num());

	TArray<FAnimNotifyEventReference> NewActiveAnimNotifyEventReference;
	NewActiveAnimNotifyEventReference.Reserve(NotifyQueue.AnimNotifies.Num());

	// AnimNotifyState freshly added that need their 'NotifyBegin' event called.
	TArray<const FAnimNotifyEvent*> NotifyStateBeginEvent;
	TArray<const FAnimNotifyEventReference*> NotifyStateBeginEventReference;

	for (int32 Index = 0; Index < NotifyQueue.AnimNotifies.Num(); Index++)
	{
		if (const FAnimNotifyEvent* AnimNotifyEvent = NotifyQueue.AnimNotifies[Index].GetNotify())
		{
			// AnimNotifyState
			if (AnimNotifyEvent->NotifyStateClass)
			{
				int32 ExistingItemIndex = ActiveAnimNotifyState.IndexOfByPredicate([AnimNotifyEvent](const FAnimNotifyEvent* InOther)
					{
						return *AnimNotifyEvent == *InOther;
					});

				if (ExistingItemIndex != INDEX_NONE)
				{
					check(ActiveAnimNotifyState.Num() == ActiveAnimNotifyEventReference.Num());
					ActiveAnimNotifyState.RemoveAtSwap(ExistingItemIndex, EAllowShrinking::No);
					ActiveAnimNotifyEventReference.RemoveAtSwap(ExistingItemIndex, EAllowShrinking::No);
				}
				else
				{
					NotifyStateBeginEvent.Add(AnimNotifyEvent);
					NotifyStateBeginEventReference.Add(&NotifyQueue.AnimNotifies[Index]);
				}

				NewActiveAnimNotifyState.Add(AnimNotifyEvent);
				FAnimNotifyEventReference& EventRef = NewActiveAnimNotifyEventReference.Add_GetRef(NotifyQueue.AnimNotifies[Index]);
				EventRef.SetNotify(AnimNotifyEvent);
				continue;
			}

			// Trigger non 'state' AnimNotifies
			TriggerSingleAnimNotify(InDeltaTime, NotifyQueueDispatchEvent.Get(), NotifyQueue.AnimNotifies[Index]);
		}
	}

	if (NotifyQueueDispatchEvent.IsValid())
	{
		// Send end notification to AnimNotifyState not active anymore.
		for (int32 Index = 0; Index < ActiveAnimNotifyState.Num(); ++Index)
		{
			const FAnimNotifyEvent* AnimNotifyEvent = ActiveAnimNotifyState[Index];
			const FAnimNotifyEventReference& EventReference = ActiveAnimNotifyEventReference[Index];
			if (AnimNotifyEvent->NotifyStateClass)
			{
#if WITH_EDITOR
				// Prevent firing notifies in animation editors if requested
				if (ModuleInstance.GetWorldType() != EWorldType::EditorPreview || AnimNotifyEvent->NotifyStateClass->ShouldFireInEditor())
#endif
				{
					NotifyQueueDispatchEvent->EventsToEnd.Add(EventReference);
				}
			}
		}

		check(NotifyStateBeginEventReference.Num() == NotifyStateBeginEvent.Num());
		for (int32 Index = 0; Index < NotifyStateBeginEvent.Num(); Index++)
		{
			const FAnimNotifyEvent* AnimNotifyEvent = NotifyStateBeginEvent[Index];
			const FAnimNotifyEventReference* EventReference = NotifyStateBeginEventReference[Index];
			if (AnimNotifyEvent->NotifyStateClass)
			{
#if WITH_EDITOR
				// Prevent firing notifies in animation editors if requested 
				if (ModuleInstance.GetWorldType() != EWorldType::EditorPreview || AnimNotifyEvent->NotifyStateClass->ShouldFireInEditor())
#endif
				{
					NotifyQueueDispatchEvent->EventsToBegin.Add(*EventReference);
				}
			}
		}
	}

	// Switch our arrays.
	ActiveAnimNotifyState = MoveTemp(NewActiveAnimNotifyState);
	ActiveAnimNotifyEventReference = MoveTemp(NewActiveAnimNotifyEventReference);

	if (NotifyQueueDispatchEvent.IsValid())
	{
		// Tick currently active AnimNotifyState
		for (int32 Index = 0; Index < ActiveAnimNotifyState.Num(); Index++)
		{
			const FAnimNotifyEvent* AnimNotifyEvent = ActiveAnimNotifyState[Index];
			const FAnimNotifyEventReference& EventReference = ActiveAnimNotifyEventReference[Index];
			if (AnimNotifyEvent->NotifyStateClass)
			{
#if WITH_EDITOR
				// Prevent firing notifies in animation editors if requested 
				if (ModuleInstance.GetWorldType() != EWorldType::EditorPreview || AnimNotifyEvent->NotifyStateClass->ShouldFireInEditor())
#endif
				{
					NotifyQueueDispatchEvent->EventsToTick.Add(EventReference);
				}
			}
		}

		GetModuleInstance().QueueOutputTraitEvent(NotifyQueueDispatchEvent);
	}
	NotifyQueue.AnimNotifies.Reset();
}

void FUAFNotifyDispatcherComponent::TriggerSingleAnimNotify(float InDeltaTime, UE::UAF::FNotifyQueueDispatchEvent* InDispatcher, const FAnimNotifyEventReference& EventReference)
{
	// This is for non 'state' anim notifies.
	const FAnimNotifyEvent* AnimNotifyEvent = EventReference.GetNotify();
	if (AnimNotifyEvent && (AnimNotifyEvent->NotifyStateClass == nullptr))
	{
		FAnimNextModuleInstance& ModuleInstance = GetModuleInstance();
		if (InDispatcher != nullptr && AnimNotifyEvent->Notify != nullptr)
		{
#if WITH_EDITOR
			// Prevent firing notifies in animation editors if requested 
			if (ModuleInstance.GetWorldType() != EWorldType::EditorPreview || AnimNotifyEvent->Notify->ShouldFireInEditor())
#endif
			{
				InDispatcher->EventsToNotify.Add(EventReference);
			}
		}
		else if (AnimNotifyEvent->NotifyName != NAME_None)
		{
			// Named notifies can be handled by a custom module event on our worker thread, no need to queue
			ModuleInstance.RunScriptEventByName(AnimNotifyEvent->NotifyName, InDeltaTime);
		}
	}
}

namespace UE::UAF
{
	template<typename TriggerFunc>
	static void TriggerEventFunc(const TArray<FAnimNotifyEventReference>& InEvents, TriggerFunc InTriggerFunc)
	{
		for (const FAnimNotifyEventReference& EventReference : InEvents)
		{
			const FAnimNotifyEvent* AnimNotifyEvent = EventReference.GetNotify();
			if (AnimNotifyEvent == nullptr)
			{
				continue;
			}

			InTriggerFunc(AnimNotifyEvent, EventReference);
		}
	}

	void FNotifyQueueDispatchEvent::Execute() const
	{
		USkeletalMeshComponent* SkeletalMeshComponent = WeakSkeletalMeshComponent.Get();
		if (SkeletalMeshComponent == nullptr)
		{
			return;
		}

		// Order of dispatch here mimics that of UAnimInstance::TriggerAnimNotifies

		// Triggers...
		TriggerEventFunc(EventsToNotify, [SkeletalMeshComponent](const FAnimNotifyEvent* InAnimNotifyEvent, const FAnimNotifyEventReference& InEventReference)
			{
				check(InAnimNotifyEvent->Notify);
				InAnimNotifyEvent->Notify->Notify(SkeletalMeshComponent, Cast<UAnimSequenceBase>(InAnimNotifyEvent->Notify->GetOuter()), InEventReference);
			});

		// Ends...
		TriggerEventFunc(EventsToEnd, [SkeletalMeshComponent](const FAnimNotifyEvent* InAnimNotifyEvent, const FAnimNotifyEventReference& InEventReference)
			{
				check(InAnimNotifyEvent->NotifyStateClass);
				InAnimNotifyEvent->NotifyStateClass->NotifyEnd(SkeletalMeshComponent, Cast<UAnimSequenceBase>(InAnimNotifyEvent->NotifyStateClass->GetOuter()), InEventReference);
			});

		// Begins...
		TriggerEventFunc(EventsToBegin, [SkeletalMeshComponent](const FAnimNotifyEvent* InAnimNotifyEvent, const FAnimNotifyEventReference& InEventReference)
			{
				check(InAnimNotifyEvent->NotifyStateClass);
				InAnimNotifyEvent->NotifyStateClass->NotifyBegin(SkeletalMeshComponent, Cast<UAnimSequenceBase>(InAnimNotifyEvent->NotifyStateClass->GetOuter()), InAnimNotifyEvent->GetDuration(), InEventReference);
			});

		// Ticks...
		TriggerEventFunc(EventsToTick, [SkeletalMeshComponent, DeltaSeconds = DeltaSeconds](const FAnimNotifyEvent* InAnimNotifyEvent, const FAnimNotifyEventReference& InEventReference)
			{
				check(InAnimNotifyEvent->NotifyStateClass);
				InAnimNotifyEvent->NotifyStateClass->NotifyTick(SkeletalMeshComponent, Cast<UAnimSequenceBase>(InAnimNotifyEvent->NotifyStateClass->GetOuter()), DeltaSeconds, InEventReference);
			});
	}
}
