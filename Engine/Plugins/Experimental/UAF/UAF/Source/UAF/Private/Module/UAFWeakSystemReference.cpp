// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/UAFWeakSystemReference.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/SystemReference.h"

FUAFWeakSystemReference::FUAFWeakSystemReference(const UE::UAF::FSystemReference& InHandle)
	: WeakPtr(InHandle.Ptr)
{
}

void FUAFWeakSystemReference::QueueTask(FName InSystemEventName, TUniqueFunction<void(const UE::UAF::FModuleTaskContext&)>&& InTaskFunction, UE::UAF::ETaskRunLocation InLocation) const
{
	TSharedPtr<FAnimNextModuleInstance> Instance = WeakPtr.Pin();
	if (ensure(Instance.IsValid()))
	{
		Instance->QueueTask(InSystemEventName, MoveTemp(InTaskFunction), InLocation);
	}
}

void FUAFWeakSystemReference::QueueInputTraitEvent(TSharedPtr<FAnimNextTraitEvent> InEvent) const
{
	TSharedPtr<FAnimNextModuleInstance> Instance = WeakPtr.Pin();
	if (ensure(Instance.IsValid()))
	{
		Instance->QueueInputTraitEvent(InEvent);
	}
}

EPropertyBagResult FUAFWeakSystemReference::SetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InData) const
{
	TSharedPtr<FAnimNextModuleInstance> Instance = WeakPtr.Pin();
	if (ensure(Instance.IsValid()))
	{
		return Instance->SetProxyVariable(InVariable, InType, InData);
	}
	return EPropertyBagResult::PropertyNotFound;
}

EPropertyBagResult FUAFWeakSystemReference::WriteVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>)> InFunction) const
{
	TSharedPtr<FAnimNextModuleInstance> Instance = WeakPtr.Pin();
	if (ensure(Instance.IsValid()))
	{
		return Instance->WriteProxyVariable(InVariable, InType, InFunction);
	}
	return EPropertyBagResult::PropertyNotFound;
}

EPropertyBagResult FUAFWeakSystemReference::ReadVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TConstArrayView<uint8>)> InFunction) const
{
	TSharedPtr<FAnimNextModuleInstance> Instance = WeakPtr.Pin();
	if (ensure(Instance.IsValid()))
	{
		return Instance->ReadVariableFromExternal(InVariable, InType, InFunction);
	}
	return EPropertyBagResult::PropertyNotFound;
}

bool FUAFWeakSystemReference::ReadComponent(UScriptStruct* InComponentType, TFunctionRef<void(FConstStructView InComponentStruct)> InFunction) const
{
	TSharedPtr<FAnimNextModuleInstance> Instance = WeakPtr.Pin();
	if (ensure(Instance.IsValid()))
	{
		return Instance->ReadComponentFromExternal(InComponentType, InFunction);
	}
	return false;
}

const UUAFSystem* FUAFWeakSystemReference::GetSystem() const
{
	TSharedPtr<FAnimNextModuleInstance> Instance = WeakPtr.Pin();
	if (ensure(Instance.IsValid()))
	{
		return Instance->GetSystem();
	}
	return nullptr;
}