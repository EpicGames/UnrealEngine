// Copyright Epic Games, Inc. All Rights Reserved.

#include "Param/AnimNextActorLocatorFragment.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorInitializeResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextActorLocatorFragment)

UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextActorLocatorFragment> FAnimNextActorLocatorFragment::FragmentType;

UE::UniversalObjectLocator::FResolveResult FAnimNextActorLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& InParams) const
{
	using namespace UE::UniversalObjectLocator;

	UObject* Result = nullptr;

	if(InParams.Context)
	{
		// Called on a component context - return the actor
		if(UActorComponent* Component = Cast<UActorComponent>(InParams.Context))
		{
			Result = Component->GetOwner();
		}
	}

	return FResolveResultData(Result);
}

void FAnimNextActorLocatorFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
}

UE::UniversalObjectLocator::FParseStringResult FAnimNextActorLocatorFragment::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

UE::UniversalObjectLocator::FInitializeResult FAnimNextActorLocatorFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	if(InParams.Context == nullptr)
	{
		return FInitializeResult::Failure();
	}

	return FInitializeResult::Absolute();
}

uint32 FAnimNextActorLocatorFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// We can't use this at all unless explicitly added by code
	return 0;
}
