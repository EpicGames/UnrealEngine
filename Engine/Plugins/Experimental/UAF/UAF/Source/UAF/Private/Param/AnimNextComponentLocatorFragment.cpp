// Copyright Epic Games, Inc. All Rights Reserved.

#include "Param/AnimNextComponentLocatorFragment.h"
#include "Component/AnimNextComponent.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorInitializeResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextComponentLocatorFragment)

UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextComponentLocatorFragment> FAnimNextComponentLocatorFragment::FragmentType;

FAnimNextComponentLocatorFragment::FAnimNextComponentLocatorFragment(TSubclassOf<UActorComponent> InClass)
	: Path(InClass)
{
}

UE::UniversalObjectLocator::FResolveResult FAnimNextComponentLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& InParams) const
{
	using namespace UE::UniversalObjectLocator;

	UObject* Result = nullptr;

	if (UActorComponent* Component = Cast<UActorComponent>(InParams.Context))
	{
		if(AActor* Actor = Component->GetOwner())
		{
			if (UClass* ComponentClass = Cast<UClass>(Path.ResolveObject()))
			{
				Result = Actor->FindComponentByClass(ComponentClass);
			}
		}
	}
	else if(AActor* Actor = Cast<AActor>(InParams.Context))
	{
		if(UClass* ComponentClass = Cast<UClass>(Path.ResolveObject()))
		{
			Result = Actor->FindComponentByClass(ComponentClass);
		}
	}

	return FResolveResultData(Result);
}

void FAnimNextComponentLocatorFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
	Path.AppendString(OutStringBuilder);
}

UE::UniversalObjectLocator::FParseStringResult FAnimNextComponentLocatorFragment::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	Path = InString;
	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

UE::UniversalObjectLocator::FInitializeResult FAnimNextComponentLocatorFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	if(InParams.Object == nullptr || !InParams.Object->GetClass()->IsChildOf(UActorComponent::StaticClass()))
	{
		return FInitializeResult::Failure();
	}
	
	return FInitializeResult::Relative(InParams.Context);
}

uint32 FAnimNextComponentLocatorFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// We can't use this at all unless explicitly added by code
	return 0;
}
