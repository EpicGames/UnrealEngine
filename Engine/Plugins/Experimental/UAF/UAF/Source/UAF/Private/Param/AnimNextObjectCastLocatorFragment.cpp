// Copyright Epic Games, Inc. All Rights Reserved.

#include "Param/AnimNextObjectCastLocatorFragment.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextObjectCastLocatorFragment)

UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextObjectCastLocatorFragment> FAnimNextObjectCastLocatorFragment::FragmentType;

FAnimNextObjectCastLocatorFragment::FAnimNextObjectCastLocatorFragment(UClass* InClass)
{
	Path = InClass;
}

UE::UniversalObjectLocator::FResolveResult FAnimNextObjectCastLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	UObject* Result = nullptr;

	if(UClass* Class = Cast<UClass>(Path.ResolveObject()))
	{
		if(Params.Context && Params.Context->GetClass()->IsChildOf(Class))
		{
			Result = Params.Context;
		}
	}

	return FResolveResultData(Result);
}

void FAnimNextObjectCastLocatorFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
	Path.AppendString(OutStringBuilder);
}

UE::UniversalObjectLocator::FParseStringResult FAnimNextObjectCastLocatorFragment::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	Path = InString;
	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

UE::UniversalObjectLocator::FInitializeResult FAnimNextObjectCastLocatorFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	if(const UClass* Class = Cast<UClass>(InParams.Object))
	{
		Path = Class;
	}

	return FInitializeResult::Relative(InParams.Context);
}

uint32 FAnimNextObjectCastLocatorFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// We can't use this at all unless explicitly added by code
	return 0;
}
