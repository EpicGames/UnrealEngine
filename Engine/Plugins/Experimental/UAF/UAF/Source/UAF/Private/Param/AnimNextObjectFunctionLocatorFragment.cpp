// Copyright Epic Games, Inc. All Rights Reserved.

#include "Param/AnimNextObjectFunctionLocatorFragment.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "Param/ParamUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextObjectFunctionLocatorFragment)

UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextObjectFunctionLocatorFragment> FAnimNextObjectFunctionLocatorFragment::FragmentType;

FAnimNextObjectFunctionLocatorFragment::FAnimNextObjectFunctionLocatorFragment(UFunction* InFunction)
{
	Path = InFunction;
}

UE::UniversalObjectLocator::FResolveResult FAnimNextObjectFunctionLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	UObject* Result = nullptr;

	if(UFunction* Function = Cast<UFunction>(Path.ResolveObject()))
	{
		if(Params.Context && UE::UAF::FParamUtils::CanUseFunction(Function, nullptr))
		{
			if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Function->GetReturnProperty()))
			{
				if (Function->NumParms == 1)
				{
					check(Params.Context->GetClass()->IsChildOf(Function->GetOuterUClass()));
					FFrame Stack(Params.Context, Function, nullptr, nullptr, Function->ChildProperties);
					Function->Invoke(Params.Context, Stack, &Result);
				}
				else if (Function->NumParms == 2)
				{
					UObject* CDO = Function->GetOuterUClass()->GetDefaultObject();
					UObject* HoistedObject = Params.Context;
					FFrame Stack(CDO, Function, &HoistedObject, nullptr, Function->ChildProperties);
					Function->Invoke(CDO, Stack, &Result);
				}
			}
		}
	}

	return FResolveResultData(Result);
}

void FAnimNextObjectFunctionLocatorFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
	Path.AppendString(OutStringBuilder);
}

UE::UniversalObjectLocator::FParseStringResult FAnimNextObjectFunctionLocatorFragment::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	Path = InString;
	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

UE::UniversalObjectLocator::FInitializeResult FAnimNextObjectFunctionLocatorFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	using namespace UE::UniversalObjectLocator;

	if(const UFunction* Function = Cast<UFunction>(InParams.Object))
	{
		if(UE::UAF::FParamUtils::CanUseFunction(Function, nullptr))
		{
			if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Function->GetReturnProperty()))
			{
				Path = Function;
			}
		}
	}

	return FInitializeResult::Relative(InParams.Context);
}

uint32 FAnimNextObjectFunctionLocatorFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// We can't use this at all unless explicitly added by code
	return 0;
}
