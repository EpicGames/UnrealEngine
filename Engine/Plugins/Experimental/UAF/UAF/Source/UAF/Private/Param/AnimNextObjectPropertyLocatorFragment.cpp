// Copyright Epic Games, Inc. All Rights Reserved.

#include "Param/AnimNextObjectPropertyLocatorFragment.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "UniversalObjectLocatorStringParams.h"
#include "UniversalObjectLocatorInitializeParams.h"
#include "UniversalObjectLocatorInitializeResult.h"
#include "String/ParseTokens.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextObjectPropertyLocatorFragment)

#define LOCTEXT_NAMESPACE "UAFObjectPropertyLocatorFragment"

UE::UniversalObjectLocator::TFragmentTypeHandle<FAnimNextObjectPropertyLocatorFragment> FAnimNextObjectPropertyLocatorFragment::FragmentType;

FAnimNextObjectPropertyLocatorFragment::FAnimNextObjectPropertyLocatorFragment(TArrayView<FProperty*> InPropertyPath)
{
	Path = InPropertyPath;
}

UE::UniversalObjectLocator::FResolveResult FAnimNextObjectPropertyLocatorFragment::Resolve(const UE::UniversalObjectLocator::FResolveParams& Params) const
{
	using namespace UE::UniversalObjectLocator;

	UObject* Result = nullptr;

	if(Params.Context)
	{
		UClass* RootClass = GetRootClass();
		if(RootClass && Params.Context->GetClass()->IsChildOf(RootClass))
		{
			void* CurrentContainer = Params.Context;
			for(int32 SegmentIndex = 0; SegmentIndex < Path.Num(); ++SegmentIndex)
			{
				FProperty* Property = Path[SegmentIndex].Get();
				if(Property)
				{
					if(SegmentIndex == Path.Num() - 1)
					{
						if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
						{
							Result = ObjectProperty->GetObjectPropertyValue_InContainer(CurrentContainer);
							break;
						}
					}
					else
					{
						CurrentContainer = Property->ContainerPtrToValuePtr<void>(CurrentContainer);
					}
				}
			}
		}
	}

	return FResolveResultData(Result);
}

void FAnimNextObjectPropertyLocatorFragment::ToString(FStringBuilderBase& OutStringBuilder) const
{
	if(Path.Num() > 0)
	{
		OutStringBuilder.Append(Path[0].ToString());
		for(int32 SegmentIndex = 1; SegmentIndex < Path.Num(); ++SegmentIndex)
		{
			OutStringBuilder.Append(TEXT("."));
			Path[SegmentIndex]->GetFName().AppendString(OutStringBuilder);
		}
	}
}

UE::UniversalObjectLocator::FParseStringResult FAnimNextObjectPropertyLocatorFragment::TryParseString(FStringView InString, const UE::UniversalObjectLocator::FParseStringParams& Params)
{
	Path.Reset();

	if(InString.Len() == 0)
	{
		return UE::UniversalObjectLocator::FParseStringResult().Success();
	}
	
	FString String;
	String.Reserve(InString.Len());
	// Append the fields before the leaf (ie. left of the last last ":")
	int32 LastSubobjectDelimiterIndex = INDEX_NONE;
	if(!InString.FindLastChar(TEXT(':'), LastSubobjectDelimiterIndex))
	{
		UE::UniversalObjectLocator::FParseStringResult().Failure(LOCTEXT("NoFieldDelimiter", "No field delimiter found"));
	}

	String.Append(InString.Left(LastSubobjectDelimiterIndex));
	String.Append(TEXT(":"));

	FStringView LeafFieldsString = InString.RightChop(LastSubobjectDelimiterIndex + 1);
	int32 SubFieldDelimiterIndex = INDEX_NONE;
	if(LeafFieldsString.FindChar(TEXT('.'), SubFieldDelimiterIndex))
	{
		String.Append(LeafFieldsString.Left(SubFieldDelimiterIndex));
		LeafFieldsString = LeafFieldsString.RightChop(SubFieldDelimiterIndex + 1);
	}
	else
	{
		String.Append(LeafFieldsString);
		LeafFieldsString = FStringView();
	}

	TFieldPath<FProperty>& NewRoot = Path.AddDefaulted_GetRef();
	NewRoot.Generate(*String);

	if(FProperty* CurrentProperty = NewRoot.Get())
	{
		// Now run thru the remaining subfields, if any
		UE::String::ParseTokens(LeafFieldsString, TEXT('.'), [this, &CurrentProperty](FStringView InToken)
		{
			if(FStructProperty* StructProperty = CastField<FStructProperty>(CurrentProperty))
			{
				CurrentProperty = StructProperty->Struct->FindPropertyByName(FName(InToken));
				if(CurrentProperty != nullptr)
				{
					Path.Add(CurrentProperty);
				}
			}
			else
			{
				CurrentProperty = nullptr;
			}
		});
	}

	return UE::UniversalObjectLocator::FParseStringResult().Success();
}

UE::UniversalObjectLocator::FInitializeResult FAnimNextObjectPropertyLocatorFragment::Initialize(const UE::UniversalObjectLocator::FInitializeParams& InParams)
{
	// Not implemented - API does not support FFields
	return UE::UniversalObjectLocator::FInitializeResult::Relative(InParams.Context);
}

uint32 FAnimNextObjectPropertyLocatorFragment::ComputePriority(const UObject* ObjectToReference, const UObject* Context)
{
	// We can't use this at all unless explicitly added by code
	return 0;
}

FObjectProperty* FAnimNextObjectPropertyLocatorFragment::GetLeafProperty() const
{
	if(Path.Num() > 0)
	{
		return CastField<FObjectProperty>(Path.Last().Get());
	}

	return nullptr;
}

UClass* FAnimNextObjectPropertyLocatorFragment::GetRootClass() const
{
	if(Path.Num() > 0)
	{
		return Cast<UClass>(Path[0].Get()->GetOwnerUObject());
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
