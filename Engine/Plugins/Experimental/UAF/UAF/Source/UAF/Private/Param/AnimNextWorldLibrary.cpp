// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextWorldLibrary.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextWorldLibrary)

double UAnimNextWorldLibrary::GetTimeSeconds(UWorld* InWorld)
{
	return InWorld->GetTimeSeconds();
}

double UAnimNextWorldLibrary::GetUnpausedTimeSeconds(UWorld* InWorld)
{
	return InWorld->GetUnpausedTimeSeconds();
}

double UAnimNextWorldLibrary::GetRealTimeSeconds(UWorld* InWorld)
{
	return InWorld->GetRealTimeSeconds();
}

float UAnimNextWorldLibrary::GetDeltaSeconds(UWorld* InWorld)
{
	return InWorld->GetDeltaSeconds();
}
