// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AnimNextWorldLibrary.generated.h"

class UWorld;
struct FAnimNextTickFunctionBinding;

// Access to non-UProperty/UFunction data on UWorld
UCLASS()
class UAnimNextWorldLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Returns time in seconds since world was brought up for play, IS stopped when game pauses, IS dilated/clamped
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	static double GetTimeSeconds(UWorld* InWorld);

	// Returns time in seconds since world was brought up for play, IS NOT stopped when game pauses, IS dilated/clamped
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	static double GetUnpausedTimeSeconds(UWorld* InWorld);

	// Returns time in seconds since world was brought up for play, does NOT stop when game pauses, NOT dilated/clamped 
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	static double GetRealTimeSeconds(UWorld* InWorld);

	// Returns the frame delta time in seconds adjusted by e.g. time dilation.
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	static float GetDeltaSeconds(UWorld* InWorld);
};
