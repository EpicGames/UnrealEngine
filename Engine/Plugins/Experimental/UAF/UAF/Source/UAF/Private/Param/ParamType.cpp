// Copyright Epic Games, Inc. All Rights Reserved.

#include "Param/ParamType.h"

#include "Misc/StringBuilder.h"
#include "UObject/Class.h"
#include "Templates/SubclassOf.h"
#include "RigVMCore/RigVMTemplate.h"
#include "UObject/TextProperty.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParamType)

FRigVMTemplateArgumentType FAnimNextParamType::ToRigVMTemplateArgument() const
{
	auto GetTypeInternal = [this]()
	{
		switch(ValueType)
		{
		case EValueType::None:
			return FRigVMTemplateArgumentType();
		case EValueType::Bool:
			return FRigVMTemplateArgumentType(RigVMTypeUtils::BoolTypeName);
		case EValueType::Byte:
			return FRigVMTemplateArgumentType(RigVMTypeUtils::UInt8Type);
		case EValueType::Int32:
			return FRigVMTemplateArgumentType(RigVMTypeUtils::Int32Type);
		case EValueType::Int64:
			return FRigVMTemplateArgumentType(RigVMTypeUtils::Int64Type);
		case EValueType::Float:
			return FRigVMTemplateArgumentType(RigVMTypeUtils::FloatType);
		case EValueType::Double:
			return FRigVMTemplateArgumentType(RigVMTypeUtils::DoubleType);
		case EValueType::Name:
			return FRigVMTemplateArgumentType(RigVMTypeUtils::FNameType);
		case EValueType::String:
			return FRigVMTemplateArgumentType(RigVMTypeUtils::FStringType);
		case EValueType::Text:
			return FRigVMTemplateArgumentType(RigVMTypeUtils::FTextType);
		case EValueType::Enum:
			if(const UEnum* Enum = Cast<UEnum>(ValueTypeObject.Get()))
			{
				return FRigVMTemplateArgumentType(const_cast<UEnum*>(Enum));
			}
		case EValueType::Struct:
			if(const UScriptStruct* Struct = Cast<UScriptStruct>(ValueTypeObject.Get()))
			{
				return FRigVMTemplateArgumentType(const_cast<UScriptStruct*>(Struct));
			}
			break;
		case EValueType::Object:
			if(const UClass* Class = Cast<UClass>(ValueTypeObject.Get()))
			{
				return FRigVMTemplateArgumentType(const_cast<UClass*>(Class));
			}
			break;
		case EValueType::SoftObject:
			return FRigVMTemplateArgumentType();
		case EValueType::Class:
			if(const UClass* Class = Cast<UClass>(ValueTypeObject.Get()))
			{
				return FRigVMTemplateArgumentType(const_cast<UClass*>(Class), RigVMTypeUtils::EClassArgType::AsClass);
			}
			break;
		case EValueType::SoftClass:
			return FRigVMTemplateArgumentType();
		default:
			break;
		}

		return FRigVMTemplateArgumentType();
	};

	switch(ContainerType)
	{
	case EContainerType::None:
		return GetTypeInternal();
	case EContainerType::Array:
		return GetTypeInternal().ConvertToArray();
	default:
		return FRigVMTemplateArgumentType();
	}
}

FAnimNextParamType FAnimNextParamType::FromRigVMTemplateArgument(const FRigVMTemplateArgumentType& RigVMType)
{
	FAnimNextParamType Type;
	if (RigVMType.IsArray())
	{
		Type.ContainerType = EPropertyBagContainerType::Array;
	}
	
	static const FName IntTypeName(TEXT("int")); // type used by some engine tests
	static const FName Int64TypeName(TEXT("Int64"));
	static const FName UInt64TypeName(TEXT("UInt64"));

	const FName CPPType = *RigVMType.GetBaseCPPType();
	
	if (CPPType == RigVMTypeUtils::BoolTypeName)
	{
		Type.ValueType = EPropertyBagPropertyType::Bool;
	}
	else if (CPPType == RigVMTypeUtils::UInt8Type)
	{
		Type.ValueType = EPropertyBagPropertyType::Byte;
	}
	else if (CPPType == RigVMTypeUtils::Int32TypeName || CPPType == IntTypeName)
	{
		Type.ValueType = EPropertyBagPropertyType::Int32;
	}
	else if (CPPType == RigVMTypeUtils::UInt32TypeName)
	{
		Type.ValueType = EPropertyBagPropertyType::UInt32;
	}
	else if (CPPType == Int64TypeName)
	{
		Type.ValueType = EPropertyBagPropertyType::Int64;
	}
	else if (CPPType == UInt64TypeName)
	{
		Type.ValueType = EPropertyBagPropertyType::UInt64;
	}
	else if (CPPType == RigVMTypeUtils::FloatTypeName)
	{
		Type.ValueType = EPropertyBagPropertyType::Float;
	}
	else if (CPPType == RigVMTypeUtils::DoubleTypeName)
	{
		Type.ValueType = EPropertyBagPropertyType::Double;
	}
	else if (CPPType == RigVMTypeUtils::FNameTypeName)
	{
		Type.ValueType = EPropertyBagPropertyType::Name;
	}
	else if (CPPType == RigVMTypeUtils::FStringTypeName)
	{
		Type.ValueType = EPropertyBagPropertyType::String;
	}
	else if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(RigVMType.CPPTypeObject))
	{
		Type.ValueType = EPropertyBagPropertyType::Struct;
		Type.ValueTypeObject = ScriptStruct;
	}
	else if (UEnum* Enum = Cast<UEnum>(RigVMType.CPPTypeObject))
	{
		Type.ValueType = EPropertyBagPropertyType::Enum;
		Type.ValueTypeObject = Enum;
	}
	else if (UObject* Object = Cast<UObject>(RigVMType.CPPTypeObject))
	{
		Type.ValueType = EPropertyBagPropertyType::Object;	
		Type.ValueTypeObject = Object;
	}
	else
	{
		ensureMsgf(false, TEXT("Unsupported type : %s"), *RigVMType.CPPType.ToString());
		Type.ValueType = EPropertyBagPropertyType::None;
	}

	return Type;
}

FAnimNextParamType FAnimNextParamType::FromProperty(const FProperty* InProperty)
{
	FAnimNextParamType Type;

	if(InProperty == nullptr)
	{
		return Type;
	}

	if (InProperty->IsA<FArrayProperty>())
	{
		Type.ContainerType = EPropertyBagContainerType::Array;
		InProperty = CastField<FArrayProperty>(InProperty)->Inner;
	}

	if (InProperty->IsA<FBoolProperty>())
	{
		Type.ValueType = EPropertyBagPropertyType::Bool;
	}
	else if (const FByteProperty* ByteProperty = CastField<const FByteProperty>(InProperty))
	{
		if (ByteProperty->Enum != nullptr)
		{
			Type.ValueType = EPropertyBagPropertyType::Enum;
			Type.ValueTypeObject = ByteProperty->Enum;
		}
		else
		{
			Type.ValueType = EPropertyBagPropertyType::Byte;
		}
	}
	else if (InProperty->IsA<FIntProperty>())
	{
		Type.ValueType = EPropertyBagPropertyType::Int32;
	}
	else if (InProperty->IsA<FInt64Property>())
	{
		Type.ValueType = EPropertyBagPropertyType::Int64;
	}
	else if (InProperty->IsA<FUInt32Property>())
	{
		Type.ValueType = EPropertyBagPropertyType::UInt32;
	}
	else if (InProperty->IsA<FUInt64Property>())
	{
		Type.ValueType = EPropertyBagPropertyType::UInt64;
	}
	else if (InProperty->IsA<FFloatProperty>())
	{
		Type.ValueType = EPropertyBagPropertyType::Float;
	}
	else if (InProperty->IsA<FDoubleProperty>())
	{
		Type.ValueType = EPropertyBagPropertyType::Double;
	}
	else if (InProperty->IsA<FNameProperty>())
	{
		Type.ValueType = EPropertyBagPropertyType::Name;
	}
	else if (InProperty->IsA<FStrProperty>())
	{
		Type.ValueType = EPropertyBagPropertyType::String;
	}
	else if (InProperty->IsA<FTextProperty>())
	{
		Type.ValueType = EPropertyBagPropertyType::Text;
	}
	else if (InProperty->IsA<FStructProperty>())
	{
		Type.ValueType = EPropertyBagPropertyType::Struct;
		Type.ValueTypeObject = CastField<FStructProperty>(InProperty)->Struct;
	}
	else if(InProperty->IsA<FObjectPropertyBase>())
	{
		UClass* Class = CastField<FObjectPropertyBase>(InProperty)->PropertyClass;
		if (InProperty->IsA<FClassProperty>())
		{
			Type.ValueType = EPropertyBagPropertyType::Class;
			Type.ValueTypeObject = CastField<FClassProperty>(InProperty)->MetaClass;
		}
		else if (InProperty->IsA<FObjectProperty>())
		{
			if(Class == UClass::StaticClass())
			{
				Type.ValueType = EPropertyBagPropertyType::Class;
				Type.ValueTypeObject = UObject::StaticClass();
			}
			else
			{
				Type.ValueType = EPropertyBagPropertyType::Object;
				Type.ValueTypeObject = Class;
			}
		}
		else if (InProperty->IsA<FSoftClassProperty>())
		{
			Type.ValueType = EPropertyBagPropertyType::SoftClass;
			Type.ValueTypeObject = CastField<FSoftClassProperty>(InProperty)->MetaClass;
		}
		else if (InProperty->IsA<FSoftObjectProperty>())
		{
			Type.ValueType = EPropertyBagPropertyType::SoftObject;
			Type.ValueTypeObject = Class;
		}
	}
	else if(InProperty->IsA<FEnumProperty>())
	{
		Type.ValueType = EPropertyBagPropertyType::Enum;
		Type.ValueTypeObject = CastField<FEnumProperty>(InProperty)->GetEnum();
	}

	return Type;
}


bool FAnimNextParamType::IsValidObject() const
{
	switch(ValueType)
	{
	default:
	case EValueType::None:
	case EValueType::Bool:
	case EValueType::Byte:
	case EValueType::Int32:
	case EValueType::Int64:
	case EValueType::Float:
	case EValueType::Double:
	case EValueType::Name:
	case EValueType::String:
	case EValueType::Text:
		return false;
	case EValueType::Enum:
		{
			const UObject* ResolvedObject = ValueTypeObject.Get();
			return ResolvedObject && ResolvedObject->IsA(UEnum::StaticClass());
		}
	case EValueType::Struct:
		{
			const UObject* ResolvedObject = ValueTypeObject.Get();
			return ResolvedObject && ResolvedObject->IsA(UScriptStruct::StaticClass());
		}
	case EValueType::Object:
	case EValueType::SoftObject:
	case EValueType::Class:
	case EValueType::SoftClass:
		{
			const UObject* ResolvedObject = ValueTypeObject.Get();
			return ResolvedObject && ResolvedObject->IsA(UClass::StaticClass());
		}
	case EValueType::UInt32:
	case EValueType::UInt64:
		return false;
	}
}

size_t FAnimNextParamType::GetSize() const
{
	switch(ContainerType)
	{
	case EContainerType::None:
		return GetValueTypeSize();
	case EContainerType::Array:
		return sizeof(TArray<uint8>);
	default:
		break;
	}

	checkf(false, TEXT("Error: FParameterType::GetSize: Unknown Type Container %d, Value %d"), EnumToUnderlyingType(ContainerType), EnumToUnderlyingType(ValueType));
	return 0;
}

size_t FAnimNextParamType::GetValueTypeSize() const
{
	switch(ValueType)
	{
	case EValueType::None:
		return 0;
	case EValueType::Bool:
		return sizeof(bool);
	case EValueType::Byte:
		return sizeof(uint8);
	case EValueType::Int32:
		return sizeof(int32);
	case EValueType::Int64:
		return sizeof(int64);
	case EValueType::Float:
		return sizeof(float);
	case EValueType::Double:
		return sizeof(double);
	case EValueType::Name:
		return sizeof(FName);
	case EValueType::String:
		return sizeof(FString);
	case EValueType::Text:
		return sizeof(FText);
	case EValueType::Enum:
		// TODO: seems to be no way to find the size of a UEnum?
		return sizeof(uint8);
	case EValueType::Struct:
		if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(ValueTypeObject.Get()))
		{
			return ScriptStruct->GetStructureSize();
		}
		else
		{
			checkf(false, TEXT("Error: FParameterType::GetValueTypeSize: Unknown Struct Type"));
			return 0;
		}
	case EValueType::Object:
		return sizeof(TObjectPtr<UObject>);
	case EValueType::SoftObject:
		return sizeof(TSoftObjectPtr<UObject>);
	case EValueType::Class:
		return sizeof(TSubclassOf<UObject>);
	case EValueType::SoftClass:
		return sizeof(TSoftClassPtr<UObject>);
	case EValueType::UInt32:
		return sizeof(uint32);
	case EValueType::UInt64:
		return sizeof(uint64);
	default:
		break;
	}

	checkf(false, TEXT("Error: FParameterType::GetValueTypeSize: Unknown Type Container %d, Value %d"), EnumToUnderlyingType(ContainerType), EnumToUnderlyingType(ValueType));
	return 0;
}

size_t FAnimNextParamType::GetAlignment() const
{
	switch(ContainerType)
	{
	case EContainerType::None:
		return GetValueTypeAlignment();
	case EContainerType::Array:
		return sizeof(TArray<uint8>);
	default:
		break;
	}

	checkf(false, TEXT("Error: FParameterType::GetAlignment: Unknown Type: Container %d, Value %d"), EnumToUnderlyingType(ContainerType), EnumToUnderlyingType(ValueType));
	return 0;
}

size_t FAnimNextParamType::GetValueTypeAlignment() const
{
	switch(ValueType)
	{
	case EValueType::None:
		return 0;
	case EValueType::Bool:
		return alignof(bool);
	case EValueType::Byte:
		return alignof(uint8);
	case EValueType::Int32:
		return alignof(int32);
	case EValueType::Int64:
		return alignof(int64);
	case EValueType::Float:
		return alignof(float);
	case EValueType::Double:
		return alignof(double);
	case EValueType::Name:
		return alignof(FName);
	case EValueType::String:
		return alignof(FString);
	case EValueType::Text:
		return alignof(FText);
	case EValueType::Enum:
		// TODO: seems to be no way to find the alignment of a UEnum?
		return alignof(uint8);
	case EValueType::Struct:
		if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(ValueTypeObject.Get()))
		{
			return ScriptStruct->GetMinAlignment();
		}
		else
		{
			checkf(false, TEXT("Error: FParameterType::GetValueTypeAlignment: Unknown Struct Type"));
			return 0;
		}
	case EValueType::Object:
		return alignof(TObjectPtr<UObject>);
	case EValueType::SoftObject:
		return alignof(TSoftObjectPtr<UObject>);
	case EValueType::Class:
		return alignof(TSubclassOf<UObject>);
	case EValueType::SoftClass:
		return alignof(TSoftClassPtr<UObject>);
	case EValueType::UInt32:
		return alignof(uint32);
	case EValueType::UInt64:
		return alignof(uint64);
	default:
		break;
	}

	checkf(false, TEXT("Error: FParameterType::GetValueTypeAlignment: Unknown Type: Container %d, Value %d"), EnumToUnderlyingType(ContainerType), EnumToUnderlyingType(ValueType));
	return 0;
}

void FAnimNextParamType::ToString(FStringBuilderBase& InStringBuilder) const
{
	auto GetTypeString = [this, &InStringBuilder]()
	{
		switch(ValueType)
		{
		case EValueType::None:
			InStringBuilder.Append(TEXT("None"));
			break;
		case EValueType::Bool:
			InStringBuilder.Append(TEXT("bool"));
			break;
		case EValueType::Byte:
			InStringBuilder.Append(TEXT("uint8"));
			break;
		case EValueType::Int32:
			InStringBuilder.Append(TEXT("int32"));
			break;
		case EValueType::Int64:
			InStringBuilder.Append(TEXT("int64"));
			break;
		case EValueType::Float:
			InStringBuilder.Append(TEXT("float"));
			break;
		case EValueType::Double:
			InStringBuilder.Append(TEXT("double"));
			break;
		case EValueType::Name:
			InStringBuilder.Append(TEXT("FName"));
			break;
		case EValueType::String:
			InStringBuilder.Append(TEXT("FString"));
			break;
		case EValueType::Text:
			InStringBuilder.Append(TEXT("FText"));
			break;
		case EValueType::Enum:
			if(const UEnum* Enum = Cast<UEnum>(ValueTypeObject.Get()))
			{
				InStringBuilder.Append(TEXT("E"));
				InStringBuilder.Append(Enum->GetName());
			}
			else
			{
				InStringBuilder.Append(TEXT("Error: Unknown Enum"));
			}
			break;
		case EValueType::Struct:
			if(const UScriptStruct* Struct = Cast<UScriptStruct>(ValueTypeObject.Get()))
			{
				InStringBuilder.Append(TEXT("F"));
				InStringBuilder.Append(Struct->GetName());
			}
			else
			{
				InStringBuilder.Append(TEXT("Error: Unknown Struct"));
			}
			break;
		case EValueType::Object:
			if(const UClass* Class = Cast<UClass>(ValueTypeObject.Get()))
			{
				InStringBuilder.Append(TEXT("TObjectPtr<U"));
				InStringBuilder.Append(Class->GetName());
				InStringBuilder.Append(TEXT(">"));
			}
			else
			{
				InStringBuilder.Append(TEXT("Error: TObjectPtr of Unknown Class"));
			}
			break;
		case EValueType::SoftObject:
			if(const UClass* Class = Cast<UClass>(ValueTypeObject.Get()))
			{
				InStringBuilder.Append(TEXT("TSoftObjectPtr<U"));
				InStringBuilder.Append(Class->GetName());
				InStringBuilder.Append(TEXT(">"));
			}
			else
			{
				InStringBuilder.Append(TEXT("Error: TSoftObjectPtr of Unknown Class"));
			}
			break;
		case EValueType::Class:
			if(const UClass* Class = Cast<UClass>(ValueTypeObject.Get()))
			{
				InStringBuilder.Append(TEXT("TSubClassOf<U"));
				InStringBuilder.Append(Class->GetName());
				InStringBuilder.Append(TEXT(">"));
			}
			else
			{
				InStringBuilder.Append(TEXT("Error: TSubClassOf of Unknown Class"));
			}
			break;
		case EValueType::SoftClass:
			if(const UClass* Class = Cast<UClass>(ValueTypeObject.Get()))
			{
				InStringBuilder.Append(TEXT("TSoftClassPtr<U"));
				InStringBuilder.Append(Class->GetName());
				InStringBuilder.Append(TEXT(">"));
			}
			else
			{
				InStringBuilder.Append(TEXT("Error: TSoftClassPtr of Unknown Class"));
			}
			break;
		case EValueType::UInt32:
			InStringBuilder.Append(TEXT("uint32"));
			break;
		case EValueType::UInt64:
			InStringBuilder.Append(TEXT("uint64"));
			break;
		default:
			InStringBuilder.Append(TEXT("Error: Unknown value type"));
			break;
		}
	};

	switch(ContainerType)
	{
	case EContainerType::None:
		GetTypeString();
		break;
	case EContainerType::Array:
		{
			InStringBuilder.Append(TEXT("TArray<"));
			GetTypeString();
			InStringBuilder.Append(TEXT(">"));
		}
		break;
	default:
		InStringBuilder.Append(TEXT("Error: Unknown container type"));
		break;
	}
}

FString FAnimNextParamType::ToString() const
{
	TStringBuilder<128> StringBuilder;
	ToString(StringBuilder);
	return StringBuilder.ToString();
}

FAnimNextParamType FAnimNextParamType::FromString(const FString& InString)
{
	auto GetInnerType = [](const FString& InTypeString, FAnimNextParamType& OutType)
	{
		static TMap<FString, FAnimNextParamType> BasicTypes =
		{
			{ TEXT("bool"),		GetType<bool>() },
			{ TEXT("uint8"),		GetType<uint8>() },
			{ TEXT("int32"),		GetType<int32>() },
			{ TEXT("int64"),		GetType<int64>() },
			{ TEXT("float"),		GetType<float>() },
			{ TEXT("double"),		GetType<double>() },
			{ TEXT("FName"),		GetType<FName>() },
			{ TEXT("FString"),		GetType<FString>() },
			{ TEXT("FText"),		GetType<FText>() },
			{ TEXT("uint32"),		GetType<uint32>() },
			{ TEXT("uint64"),		GetType<uint64>() },
		};

		if(const FAnimNextParamType* BasicType = BasicTypes.Find(InTypeString))
		{
			OutType = *BasicType;
			return true;
		}

		// Check for object/struct/enum
		EValueType ObjectValueType = EValueType::None;
		FString ObjectInnerString;
		if(InTypeString.StartsWith(TEXT("U"), ESearchCase::CaseSensitive))
		{
			ObjectInnerString = InTypeString.RightChop(1).TrimStartAndEnd();
			ObjectValueType = EValueType::Object;
		}
		else if(InTypeString.StartsWith(TEXT("TObjectPtr<U"), ESearchCase::CaseSensitive))
		{
			ObjectInnerString = InTypeString.RightChop(12).LeftChop(1).TrimStartAndEnd();
			ObjectValueType = EValueType::Object;
		}
		else if(InTypeString.StartsWith(TEXT("TSubClassOf<U"), ESearchCase::CaseSensitive))
		{
			ObjectInnerString = InTypeString.RightChop(13).LeftChop(1).TrimStartAndEnd();
			ObjectValueType = EValueType::Class;
		}
		else if(InTypeString.StartsWith(TEXT("F"), ESearchCase::CaseSensitive))
		{
			ObjectInnerString = InTypeString.RightChop(1).TrimStartAndEnd();
			ObjectValueType = EValueType::Struct;
		}
		else if(InTypeString.StartsWith(TEXT("E"), ESearchCase::CaseSensitive))
		{
			ObjectInnerString = InTypeString.RightChop(1).TrimStartAndEnd();
			ObjectValueType = EValueType::Enum;
		}
		else if(InTypeString.StartsWith(TEXT("TEnumAsByte<U"), ESearchCase::CaseSensitive))
		{
			ObjectInnerString = InTypeString.RightChop(13).LeftChop(1).TrimStartAndEnd();
			ObjectValueType = EValueType::Enum;
		}

		if(UObject* ObjectType = FindFirstObjectSafe<UObject>(*ObjectInnerString, EFindFirstObjectOptions::NativeFirst))
		{
			OutType.ValueType = ObjectValueType;
			OutType.ValueTypeObject = nullptr;

			switch(ObjectValueType)
			{
			case EPropertyBagPropertyType::Enum:
				OutType.ValueTypeObject = Cast<UEnum>(ObjectType);
				break;
			case EPropertyBagPropertyType::Struct:
				OutType.ValueTypeObject = Cast<UScriptStruct>(ObjectType);
				break;
			case EPropertyBagPropertyType::Object:
				OutType.ValueTypeObject = Cast<UClass>(ObjectType);
				break;
			case EPropertyBagPropertyType::Class:
				OutType.ValueTypeObject = Cast<UClass>(ObjectType);
				break;
			default:
				break;
			}

			return OutType.ValueTypeObject != nullptr;
		}

		return false;
	};

	{
		FAnimNextParamType Type;
		if(GetInnerType(InString, Type))
		{
			return Type;
		}
	}
	
	if(InString.StartsWith(TEXT("TArray<"), ESearchCase::CaseSensitive))
	{
		const FString InnerTypeString = InString.RightChop(7).LeftChop(1).TrimStartAndEnd();
		FAnimNextParamType Type;
		if(GetInnerType(InnerTypeString, Type))
		{
			Type.ContainerType = EContainerType::Array;
			return Type;
		}
	}

	return FAnimNextParamType();
}

bool FAnimNextParamType::IsObjectType() const
{
	switch (ValueType)
	{
	case EValueType::Object:
	case EValueType::SoftObject:
	case EValueType::Class:
	case EValueType::SoftClass:
		return true;
	default:
		return false;
	}
}
