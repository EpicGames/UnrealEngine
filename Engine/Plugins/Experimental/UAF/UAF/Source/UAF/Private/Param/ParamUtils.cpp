// Copyright Epic Games, Inc. All Rights Reserved.

#include "Param/ParamUtils.h"

#include "UniversalObjectLocator.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Animation/AnimSequence.h"
#include "Param/ParamType.h"
#include "Param/ParamCompatibility.h"

namespace UE::UAF
{

FParamCompatibility FParamUtils::GetCompatibility(const FAnimNextParamType& InLHS, const FAnimNextParamType& InRHS)
{
	switch (InRHS.GetValueType())
	{
	case EPropertyBagPropertyType::Bool:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Bool:
			return EParamCompatibility::Compatible_Equal;
		}
		break;
	case EPropertyBagPropertyType::Byte:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Bool:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Byte:
			return EParamCompatibility::Compatible_Equal;
		case EPropertyBagPropertyType::Int32:
			return EParamCompatibility::Compatible_Promotion;
		case EPropertyBagPropertyType::Int64:
			return EParamCompatibility::Compatible_Promotion;
		case EPropertyBagPropertyType::Float:
			return EParamCompatibility::Compatible_Promotion;
		case EPropertyBagPropertyType::Double:
			return EParamCompatibility::Compatible_Promotion;
		}
		break;
	case EPropertyBagPropertyType::Int32:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Bool:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Byte:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Int32:
			return EParamCompatibility::Compatible_Equal;
		case EPropertyBagPropertyType::Int64:
			return EParamCompatibility::Compatible_Promotion;
		case EPropertyBagPropertyType::Float:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Double:
			return EParamCompatibility::Compatible_Promotion;
		}
		break;
	case EPropertyBagPropertyType::Int64:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Bool:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Byte:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Int32:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Int64:
			return EParamCompatibility::Compatible_Equal;
		case EPropertyBagPropertyType::Float:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Double:
			return EParamCompatibility::Incompatible_DataLoss;
		}
		break;
	case EPropertyBagPropertyType::Float:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Bool:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Byte:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Int32:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Int64:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Float:
			return EParamCompatibility::Compatible_Equal;
		case EPropertyBagPropertyType::Double:
			return EParamCompatibility::Compatible_Promotion;
		}
		break;
	case EPropertyBagPropertyType::Double:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Bool:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Byte:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Int32:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Int64:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Float:
			return EParamCompatibility::Incompatible_DataLoss;
		case EPropertyBagPropertyType::Double:
			return EParamCompatibility::Compatible_Equal;
		}
		break;
	case EPropertyBagPropertyType::Name:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Name:
			return EParamCompatibility::Compatible_Equal;
		}
		break;
	case EPropertyBagPropertyType::String:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::String:
			return EParamCompatibility::Compatible_Equal;
		}
		break;
	case EPropertyBagPropertyType::Text:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Text:
			return EParamCompatibility::Compatible_Equal;
		}
		break;
	case EPropertyBagPropertyType::Struct:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Struct:
			if(InLHS.GetValueTypeObject() == InRHS.GetValueTypeObject())
			{
				return EParamCompatibility::Compatible_Equal;
			}
			else if(InLHS.GetValueTypeObject() && InRHS.GetValueTypeObject() && CastChecked<UScriptStruct>(InRHS.GetValueTypeObject())->IsChildOf(CastChecked<UScriptStruct>(InLHS.GetValueTypeObject())))
			{
				return EParamCompatibility::Compatible_Cast;
			}
		}
		break;
	case EPropertyBagPropertyType::Object:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Object:
			if(InLHS.GetValueTypeObject() == InRHS.GetValueTypeObject())
			{
				return EParamCompatibility::Compatible_Equal;
			}
			if(InLHS.GetValueTypeObject() && InRHS.GetValueTypeObject() && CastChecked<UClass>(InRHS.GetValueTypeObject())->IsChildOf(CastChecked<UClass>(InLHS.GetValueTypeObject())))
			{
				return EParamCompatibility::Compatible_Cast;
			}
		}
		break;
	case EPropertyBagPropertyType::SoftObject:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::SoftObject:
			if(InLHS.GetValueTypeObject() == InRHS.GetValueTypeObject())
			{
				return EParamCompatibility::Compatible_Equal;
			}
			if(InLHS.GetValueTypeObject() && InRHS.GetValueTypeObject() && CastChecked<UClass>(InRHS.GetValueTypeObject())->IsChildOf(CastChecked<UClass>(InLHS.GetValueTypeObject())))
			{
				return EParamCompatibility::Compatible_Cast;
			}
		}
		break;
	case EPropertyBagPropertyType::Class:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::Class:
			if(InLHS.GetValueTypeObject() == InRHS.GetValueTypeObject())
			{
				return EParamCompatibility::Compatible_Equal;
			}
			if(InLHS.GetValueTypeObject() && InRHS.GetValueTypeObject() && CastChecked<UClass>(InRHS.GetValueTypeObject())->IsChildOf(CastChecked<UClass>(InLHS.GetValueTypeObject())))
			{
				return EParamCompatibility::Compatible_Cast;
			}
		}
		break;
	case EPropertyBagPropertyType::SoftClass:
		switch (InLHS.GetValueType())
		{
		case EPropertyBagPropertyType::SoftClass:
			if(InLHS.GetValueTypeObject() == InRHS.GetValueTypeObject())
			{
				return EParamCompatibility::Compatible_Equal;
			}
			if(InLHS.GetValueTypeObject() && InRHS.GetValueTypeObject() && CastChecked<UClass>(InRHS.GetValueTypeObject())->IsChildOf(CastChecked<UClass>(InLHS.GetValueTypeObject())))
			{
				return EParamCompatibility::Compatible_Cast;
			}
		}
		break;
	case EPropertyBagPropertyType::Enum:
		switch (InLHS.GetValueType())
		{
			case EPropertyBagPropertyType::Enum:
			if(InLHS.GetValueTypeObject() == InRHS.GetValueTypeObject())
			{
				return EParamCompatibility::Compatible_Equal;
			}
			break;
		}
		break;
	}

	return EParamCompatibility::Incompatible;
}

static bool CanUseFunctionInternal(const UFunction* InFunction, const UClass* InExpectedClass, FProperty*& OutReturnProperty)
{
	UClass* FunctionClass = InFunction->GetOuterUClass();
	if(FunctionClass->IsChildOf(UBlueprintFunctionLibrary::StaticClass()))
	{
		// Check 'hoisted' functions on BPFLs
		if(!InFunction->HasAllFunctionFlags(FUNC_BlueprintCallable | FUNC_Static | FUNC_Public))
		{
			return false;
		}

		const bool bValidNative = InFunction->HasAllFunctionFlags(FUNC_Native) && InFunction->NumParms == 2;
		const bool bValidNonNative = !InFunction->HasAllFunctionFlags(FUNC_Native) && InFunction->NumParms == 3;
		if(!(bValidNative || bValidNonNative))
		{
			return false;
		}
		
		int32 ParamIndex = 0;
		for(TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & CPF_Parm); ++It, ++ParamIndex)
		{
			// Check first parameter is an object of the expected class
			if(ParamIndex == 0)
			{
				FObjectProperty* ObjectProperty = CastField<FObjectProperty>(*It);
				if(ObjectProperty == nullptr)
				{ 
					return false;
				}

				// TODO: Class checks have to be editor only right now until Verse moves to using UHT (and UHT can understand verse classes)
				// For now we need to use metadata to distinguish types
#if WITH_EDITORONLY_DATA
				if(InExpectedClass != nullptr)
				{
					// It its just a UObject, check the metadata
					if(ObjectProperty->PropertyClass == UObject::StaticClass())
					{
						const FString& AllowedClassMeta = ObjectProperty->GetMetaData("AllowedClass");
						if(AllowedClassMeta.Len() == 0)
						{
							return false;
						}

						const UClass* AllowedClass = FindObject<UClass>(nullptr, *AllowedClassMeta);
						if(AllowedClass == nullptr || !InExpectedClass->IsChildOf(AllowedClass))
						{
							return false;
						}
					}
					else if(!InExpectedClass->IsChildOf(ObjectProperty->PropertyClass))
					{
						return false;
					}
				}
#endif
			}

			if(bValidNative)
			{
				// Check return value
				if(ParamIndex == 1 && !It->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					return false;
				}
			}
			else
			{
				// Check world context
				FObjectProperty* ObjectProperty = CastField<FObjectProperty>(*It);
				if(ParamIndex == 1 && ObjectProperty == nullptr)
				{
					return false;
				}

				// Check return value
				if(ParamIndex == 2 && !It->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					return false;
				}
			}

			OutReturnProperty = *It;
		}
	}
	else
	{
		// We add only 'accessor' functions (no params apart from the return value) that have valid return types
		OutReturnProperty = InFunction->GetReturnProperty();
		if(OutReturnProperty == nullptr || InFunction->NumParms != 1 || !InFunction->HasAnyFunctionFlags(FUNC_BlueprintCallable))
		{
			return false;
		}
	}

	return true;
}

bool FParamUtils::CanUseFunction(const UFunction* InFunction, const UClass* InExpectedClass)
{
	FProperty* ReturnProperty = nullptr;
	return CanUseFunctionInternal(InFunction, InExpectedClass, ReturnProperty);
}

bool FParamUtils::CanUseFunction(const UFunction* InFunction, const UClass* InExpectedClass, FAnimNextParamType& OutType)
{
	FProperty* ReturnProperty = nullptr;
	if(!CanUseFunctionInternal(InFunction, InExpectedClass, ReturnProperty))
	{
		return false;
	}

	check(ReturnProperty);
	OutType = FAnimNextParamType::FromProperty(ReturnProperty);
	if(!OutType.IsValid())
	{
		return false;
	}

	return true;
}

bool FParamUtils::CanUseProperty(const FProperty* InProperty)
{
	if(!InProperty->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst | CPF_BlueprintVisible) || InProperty->HasAnyPropertyFlags(CPF_Deprecated | CPF_EditorOnly))
	{
		return false;
	}
	return true;
}

bool FParamUtils::CanUseProperty(const FProperty* InProperty, FAnimNextParamType& OutType)
{
	if(!CanUseProperty(InProperty))
	{
		return false;
	}

	OutType = FAnimNextParamType::FromProperty(InProperty);
	if(!OutType.IsValid())
	{
		return false;
	}

	return true;
}

FName FParamUtils::LocatorToName(const FUniversalObjectLocator& InLocator)
{
	// By default the string representation of an empty UOL is "uobj://none", so we shortcut here for FName consistency 
	if(InLocator.IsEmpty())
	{
		return NAME_None; 
	}

	TStringBuilder<1024> StringBuilder;
	InLocator.ToString(StringBuilder);
	ensure(StringBuilder.Len() < NAME_SIZE);
	return FName(StringBuilder.ToView());
}


}