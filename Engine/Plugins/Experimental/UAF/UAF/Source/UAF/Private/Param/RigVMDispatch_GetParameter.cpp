// Copyright Epic Games, Inc. All Rights Reserved.

#include "Param/RigVMDispatch_GetParameter.h"
#include "RigVMCore/RigVMStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_GetParameter)

const FName FRigVMDispatch_GetParameter::ValueName = TEXT("Value");
const FName FRigVMDispatch_GetParameter::TypeHandleName = TEXT("Type");
const FName FRigVMDispatch_GetParameter::ParameterName = TEXT("Parameter");
const FName FRigVMDispatch_GetParameter::ParameterIdName = TEXT("ParameterId");

FRigVMDispatch_GetParameter::FRigVMDispatch_GetParameter()
{
	FactoryScriptStruct = StaticStruct();
}

FName FRigVMDispatch_GetParameter::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FName ArgumentNames[] =
	{
		ParameterName,
		ValueName,
		ParameterIdName,
		TypeHandleName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

#if WITH_EDITOR
FString FRigVMDispatch_GetParameter::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if ((InArgumentName == TypeHandleName || InArgumentName == ParameterIdName) &&
		InMetaDataKey == FRigVMStruct::SingletonMetaName)
	{
		return TEXT("True");
	}
	else if(InArgumentName == ParameterName && InMetaDataKey == FRigVMStruct::CustomWidgetMetaName)
	{
		return TEXT("ParamName");
	}

	return Super::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}
#endif

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_GetParameter::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories =
		{
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};

		CachedArgumentInfos.Emplace(ParameterName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FName);
		CachedArgumentInfos.Emplace(ValueName, ERigVMPinDirection::Output, ValueCategories);
		CachedArgumentInfos.Emplace(ParameterIdName, ERigVMPinDirection::Hidden, RigVMTypeUtils::TypeIndex::UInt32);
		CachedArgumentInfos.Emplace(TypeHandleName, ERigVMPinDirection::Hidden, RigVMTypeUtils::TypeIndex::UInt32);
	}

	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_GetParameter::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(ParameterName, InTypeIndex);
	Types.Add(ValueName, InTypeIndex);
	Types.Add(ValueName, RigVMTypeUtils::TypeIndex::UInt32);
	Types.Add(ValueName, RigVMTypeUtils::TypeIndex::UInt32);
	return Types;
}

void FRigVMDispatch_GetParameter::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	// Deprecated stub
}

