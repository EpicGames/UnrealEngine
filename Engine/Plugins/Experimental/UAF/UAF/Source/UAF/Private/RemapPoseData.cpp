// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemapPoseData.h"
#include "AnimNextStats.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemapPoseData)

DEFINE_STAT(STAT_AnimNext_RemapPose_Mesh2Mesh);

bool FRemapPoseData::ShouldReinit(const UE::UAF::FReferencePose& InSourceRefPose,
	const UE::UAF::FReferencePose& InTargetRefPose) const
{
	if (&InSourceRefPose != SourceRefPose || &InTargetRefPose != TargetRefPose)
	{
		// First time use or dangling pointers.
		return true;
	}

	if (InSourceRefPose.SkeletalMesh != SourceRefPose->SkeletalMesh)
	{
		// Source mesh changed.
		return true;
	}

	if (InTargetRefPose.SkeletalMesh != TargetRefPose->SkeletalMesh)
	{
		// Target mesh changed.
		return true;
	}

	return false;
}

void FRemapPoseData::Reinit(const UE::UAF::FReferencePose& InSourceRefPose,
	const UE::UAF::FReferencePose& InTargetRefPose)
{
	SourceRefPose = &InSourceRefPose;
	TargetRefPose = &InTargetRefPose;

	const USkeletalMesh* SourceSkeletalMesh = InSourceRefPose.SkeletalMesh.Get();
	const USkeleton* SourceSkeleton = SourceSkeletalMesh->GetSkeleton();
	const FReferenceSkeleton& SourceRefSkeleton = SourceSkeleton->GetReferenceSkeleton();
	const int32 SourceNumLODLevels = SourceSkeletalMesh->GetLODNum();

	const USkeletalMesh* TargetSkeletalMesh = InTargetRefPose.SkeletalMesh.Get();
	const FReferenceSkeleton& TargetRefSkeleton = TargetSkeletalMesh->GetRefSkeleton();
	const int32 TargetNumLODLevels = TargetSkeletalMesh->GetLODNum();

	// First index is the source LOD
	SourceToTargetBoneIndexMapPerLOD.Empty();
	SourceToTargetBoneIndexMapPerLOD.AddDefaulted(SourceNumLODLevels);

	for (int32 SourceLODLevel = 0; SourceLODLevel < SourceNumLODLevels; ++SourceLODLevel)
	{
		const TArrayView<const FBoneIndexType> SourceSkeletonIndexToPoseIndex = InSourceRefPose.GetSkeletonBoneIndexToLODBoneIndexMap(); // skeleton -> pose
		const int32 SourceNumBonesInLOD = SourceRefPose->GetNumBonesForLOD(SourceLODLevel);

		// Second index is the target LOD
		SourceToTargetBoneIndexMapPerLOD[SourceLODLevel].AddDefaulted(TargetNumLODLevels);

		for (int32 TargetLODLevel = 0; TargetLODLevel < TargetNumLODLevels; ++TargetLODLevel)
		{
			const TArrayView<const FBoneIndexType> TargetPoseToMeshBoneIndexMap = InTargetRefPose.GetLODBoneIndexToMeshBoneIndexMap(TargetLODLevel); // pose -> mesh
			FBoneRemapping& SourceToTargetBoneIndexMap = SourceToTargetBoneIndexMapPerLOD[SourceLODLevel][TargetLODLevel];

			// Iterate over the bones present in the given LOD on the target skeleton.
			// These bones are candidates for our mapping table, only the bones from the target skeleton that are available in the source skeleton are of interest.
			for (FBoneIndexType BoneIndex = 0; BoneIndex < TargetPoseToMeshBoneIndexMap.Num(); ++BoneIndex)
			{
				const FName BoneName = TargetRefSkeleton.GetBoneName(TargetPoseToMeshBoneIndexMap[BoneIndex]);
				const int32 SourceSkeletonBoneIndex = SourceRefSkeleton.FindBoneIndex(BoneName);

				// Is the current bone on the target skeleton part of the source skeleton as well? Only add bones that are present on the target.
				if (SourceSkeletonBoneIndex != INDEX_NONE)
				{
					const FBoneIndexType RemappedIndex = SourceSkeletonIndexToPoseIndex[SourceSkeletonBoneIndex];

					// There are cases where the given bone are present in the skeleton, but not part of the actual pose/skeletal mesh,
					// like e.g. when they are disabled for the given LOD on the target while they are not on the source. Skip these bones.
					if (RemappedIndex != INDEX_NONE && RemappedIndex < SourceNumBonesInLOD)
					{
						FRemappedBone RemappedBone;
						RemappedBone.SourceBoneIndex = RemappedIndex;
						RemappedBone.TargetBoneIndex = BoneIndex;

						SourceToTargetBoneIndexMap.BoneIndexMap.Add(RemappedBone);
					}
				}
			}

			for (int32 MappingElementIndex = 0; MappingElementIndex < SourceToTargetBoneIndexMap.BoneIndexMap.Num(); ++MappingElementIndex)
			{
				const int32 SourceBoneIndex = SourceToTargetBoneIndexMap.BoneIndexMap[MappingElementIndex].SourceBoneIndex;
				const int32 TargetBoneIndex = SourceToTargetBoneIndexMap.BoneIndexMap[MappingElementIndex].TargetBoneIndex;

				if (TargetBoneIndex == UE::UAF::FReferencePose::RootBoneIndex)
				{
					SourceToTargetBoneIndexMap.TargetRootToSourceBoneIndex = SourceBoneIndex;
					break;
				}
			}
		}
	}
}

FTransform FRemapPoseData::RecursiveCalcModelspaceTransform(const UE::UAF::FLODPose& Pose, FBoneIndexType BoneIndex) const
{
	if (BoneIndex == (uint16)INDEX_NONE)
	{
		return FTransform::Identity;
	}

	const FTransform LocalTransform = Pose.LocalTransformsView[BoneIndex];

	const TArrayView<const FBoneIndexType> BoneToParentBoneIndexMap = Pose.GetLODBoneIndexToParentLODBoneIndexMap();
	FBoneIndexType ParentBoneIndex = BoneToParentBoneIndexMap[BoneIndex];
	if (ParentBoneIndex == (uint16)INDEX_NONE)
	{
		return LocalTransform;
	}

	const FTransform ParentModelTransform = RecursiveCalcModelspaceTransform(Pose, ParentBoneIndex);
	return LocalTransform * ParentModelTransform;
}

void FRemapPoseData::RemapPose(const UE::UAF::FLODPoseHeap& SourcePose, UE::UAF::FLODPoseHeap& OutTargetPose) const
{
	const int32 TargetLODLevel = TargetRefPose->GetSourceLODLevel();
	const bool bIsAdditive = SourcePose.IsAdditive();

	// Always prepare for LOD as we may not be copying in all of the target bones and it is just cheaper to copy them all in
	OutTargetPose.PrepareForLOD(*TargetRefPose, TargetLODLevel, /*bSetRefPose=*/true, bIsAdditive);
	OutTargetPose.Flags = SourcePose.Flags;

	RemapBones(SourcePose, OutTargetPose);
}

void FRemapPoseData::RemapBones(const UE::UAF::FLODPose& SourcePose, UE::UAF::FLODPose& OutTargetPose) const
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_RemapPose_Mesh2Mesh);

	const int32 SourceLODLevel = SourcePose.LODLevel;
	const int32 TargetLODLevel = OutTargetPose.LODLevel;

	const TArray<FRemappedBone>& SourceToTargetBoneIndexMap = SourceToTargetBoneIndexMapPerLOD[SourceLODLevel][TargetLODLevel].BoneIndexMap;
	const int32 NumToRemap = SourceToTargetBoneIndexMap.Num();

#if DEFAULT_SOA && DEFAULT_SOA_VIEW
	const double* RESTRICT SrcRotationPtr = reinterpret_cast<const double*>(SourcePose.LocalTransformsView.Rotations.GetData());
	double* RESTRICT DstRotationPtr = reinterpret_cast<double*>(OutTargetPose.LocalTransformsView.Rotations.GetData());

	// Our SoA buffer is contiguous
	// Because the translations/scales have the same size (FVector), each entry is a fixed offset apart and we can use a single ptr/offset pair
	const double* RESTRICT SrcTranslationPtr = reinterpret_cast<const double*>(SourcePose.LocalTransformsView.Translations.GetData());
	double* RESTRICT DstTranslationPtr = reinterpret_cast<double*>(OutTargetPose.LocalTransformsView.Translations.GetData());

	const int64 SrcScaleOffset = reinterpret_cast<const double*>(SourcePose.LocalTransformsView.Scales3D.GetData()) - SrcTranslationPtr;
	const int64 DstScaleOffset = reinterpret_cast<double*>(OutTargetPose.LocalTransformsView.Scales3D.GetData()) - DstTranslationPtr;

	const FRemappedBone* RESTRICT RemappedBonePtr = SourceToTargetBoneIndexMap.GetData();
	const FRemappedBone* RESTRICT RemappedBoneEndPtr = RemappedBonePtr + NumToRemap;

	while (RemappedBonePtr < RemappedBoneEndPtr)
	{
		const int32 SourceBoneIndex = RemappedBonePtr->SourceBoneIndex;
		const int32 TargetBoneIndex = RemappedBonePtr->TargetBoneIndex;

		SourcePose.LocalTransformsView.Rotations.RangeCheck(SourceBoneIndex);
		OutTargetPose.LocalTransformsView.Rotations.RangeCheck(TargetBoneIndex);

		VectorRegister4Double Rotation = VectorLoadAligned(SrcRotationPtr + SourceBoneIndex * 4);
		VectorRegister4Double Translation = VectorLoad(SrcTranslationPtr + SourceBoneIndex * 3);
		VectorRegister4Double Scale = VectorLoad(SrcTranslationPtr + SourceBoneIndex * 3 + SrcScaleOffset);

		VectorStoreAligned(Rotation, DstRotationPtr + TargetBoneIndex * 4);
		VectorStoreFloat3(Translation, DstTranslationPtr + TargetBoneIndex * 3);
		VectorStoreFloat3(Scale, DstTranslationPtr + TargetBoneIndex * 3 + DstScaleOffset);

		RemappedBonePtr++;
	}
#else
	for (int32 MappingElementIndex = 0; MappingElementIndex < NumToRemap; ++MappingElementIndex)
	{
		const int32 SourceBoneIndex = SourceToTargetBoneIndexMap[MappingElementIndex].SourceBoneIndex;
		const int32 TargetBoneIndex = SourceToTargetBoneIndexMap[MappingElementIndex].TargetBoneIndex;

		OutTargetPose.LocalTransformsView[TargetBoneIndex] = SourcePose.LocalTransformsView[SourceBoneIndex];
	}
#endif

	// For the cases where the target skeleton does not share the same root bone with the source, try to find the corresponding
	// bone on the source skeleton for the target's root bone and sync the skeletons up from there by calculating the delta transform
	// between the two and move it along with the source. This allows us to remap and attach skeletal meshes that only contain the
	// bones it needs, like e.g. only skinned bones.
	const int32 RootBoneOnSourceMapped = SourceToTargetBoneIndexMapPerLOD[SourceLODLevel][TargetLODLevel].TargetRootToSourceBoneIndex;
	if (RootBoneOnSourceMapped != INDEX_NONE)
	{
		// TODO: Replace with the generic method once we have access to the modelspace transform via the LOD pose.
		const FTransform RootModelTransform = RecursiveCalcModelspaceTransform(SourcePose, RootBoneOnSourceMapped);
		OutTargetPose.LocalTransformsView[UE::UAF::FReferencePose::RootBoneIndex] = RootModelTransform;
	}
}

template<typename OutContainerType>
void FRemapPoseData::RemapAttributes(const UE::UAF::FLODPose& SourceLODPose,
	const UE::Anim::FHeapAttributeContainer& InAttributes,
	const UE::UAF::FLODPose& TargetLODPose,
	OutContainerType& OutAttributes) const
{
	const USkeletalMesh* SourceSkeletalMesh = SourceRefPose->SkeletalMesh.Get();
	if (!SourceSkeletalMesh)
	{
		ensureMsgf(false, TEXT("FRemapPoseData::RemapAttributes(): Source skeletal mesh is not valid anymore."));
		return;
	}
	const FReferenceSkeleton& SourceRefSkeleton = SourceSkeletalMesh->GetRefSkeleton();
	const TArrayView<const FBoneIndexType> SourceLODBoneIndexToSkeletonBoneIndexMap = SourceLODPose.GetLODBoneIndexToSkeletonBoneIndexMap();

	const USkeletalMesh* TargetSkeletalMesh = TargetRefPose->SkeletalMesh.Get();
	if (!TargetSkeletalMesh)
	{
		ensureMsgf(false, TEXT("FRemapPoseData::RemapAttributes(): Target skeletal mesh is not valid anymore."));
		return;
	}
	const FReferenceSkeleton& TargetRefSkeleton = TargetSkeletalMesh->GetRefSkeleton();
	const TArrayView<const FBoneIndexType> TargetSkeletonBoneToLODBoneIndexMap = TargetLODPose.GetSkeletonBoneIndexToLODBoneIndexMap();

	for (const TWeakObjectPtr<UScriptStruct> WeakScriptStruct : InAttributes.GetUniqueTypes())
	{
		const UScriptStruct* ScriptStruct = WeakScriptStruct.Get();
		const int32 TypeIndex = InAttributes.FindTypeIndex(ScriptStruct);
		if (TypeIndex != INDEX_NONE)
		{
			const TArray<UE::Anim::TWrappedAttribute<FDefaultAllocator>, FDefaultAllocator>& SourceValues = InAttributes.GetValues(TypeIndex);
			const TArray<UE::Anim::FAttributeId, FDefaultAllocator>& AttributeIds = InAttributes.GetKeys(TypeIndex);

			// Try and remap all the source attributes to their respective new bone indices
			for (int32 EntryIndex = 0; EntryIndex < AttributeIds.Num(); ++EntryIndex)
			{
				const UE::Anim::FAttributeId& AttributeId = AttributeIds[EntryIndex];

				// Remap the source bone from LOD pose bone index to a skeleton bone index and get the bone name.
				const int32 SourceLODBoneIndex = AttributeId.GetIndex();
				const int32 SourceSkeletonBoneIndex = SourceLODBoneIndexToSkeletonBoneIndexMap[SourceLODBoneIndex];
				const FName BoneName = SourceRefSkeleton.GetBoneName(SourceSkeletonBoneIndex);
				
				// Find the given bone inside the target skeleton, if it is in there, remap the attribute.
				const int32 TargetSkeletonBoneIndex = TargetRefSkeleton.FindBoneIndex(BoneName);
				if (TargetSkeletonBoneIndex != INDEX_NONE)
				{
					const int32 TargetLODBoneIndex = TargetSkeletonBoneToLODBoneIndexMap[TargetSkeletonBoneIndex];

					const UE::Anim::FAttributeId NewInfo(AttributeId.GetName(), TargetLODBoneIndex, AttributeId.GetNamespace());
					uint8* NewAttribute = OutAttributes.FindOrAdd(ScriptStruct, NewInfo);
					ScriptStruct->CopyScriptStruct(NewAttribute, SourceValues[EntryIndex].GetPtr<void>());
				}
			}
		}
	}
}

// Explicit instantiations
template UAF_API void FRemapPoseData::RemapAttributes(const UE::UAF::FLODPose&, const UE::Anim::FHeapAttributeContainer&, const UE::UAF::FLODPose&, UE::Anim::FHeapAttributeContainer&) const;
template UAF_API void FRemapPoseData::RemapAttributes(const UE::UAF::FLODPose&, const UE::Anim::FHeapAttributeContainer&, const UE::UAF::FLODPose&, UE::Anim::FStackAttributeContainer&) const;
