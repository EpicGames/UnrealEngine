// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemapPoseDataPool.h"
#include "ReferencePose.h"

namespace UE::UAF
{

FRemapPoseDataPool& FRemapPoseDataPool::Get()
{
	static FRemapPoseDataPool Instance;
	return Instance;
}

const FRemapPoseData& FRemapPoseDataPool::GetRemapData(const FReferencePose& InSourceRefPose, const FReferencePose& InTargetRefPose)
{
	const FRefPosePair Key{ &InSourceRefPose, &InTargetRefPose };

	// Fast path: Read lock for lookup (concurrent, no contention).
	{
		FRWScopeLock ReadLock(PoolLock, SLT_ReadOnly);
		if (TUniquePtr<FRemapPoseData>* Found = Pool.Find(Key))
		{
			FRemapPoseData& Entry = **Found;
			if (!Entry.ShouldReinit(InSourceRefPose, InTargetRefPose))
			{
				return Entry;
			}
			// Needs reinit, fall through to write lock.
		}
	}

	// Slow path: Write lock for creation or reinit.
	// If multiple threads race here with the same key, the first one creates/reinits
	// and subsequent ones find the existing entry with ShouldReinit() == false.
	{
		FRWScopeLock WriteLock(PoolLock, SLT_Write);
		TUniquePtr<FRemapPoseData>& Entry = Pool.FindOrAdd(Key);
		if (!Entry)
		{
			Entry = MakeUnique<FRemapPoseData>();
		}

		if (Entry->ShouldReinit(InSourceRefPose, InTargetRefPose))
		{
			Entry->Reinit(InSourceRefPose, InTargetRefPose);
		}

		return *Entry;
	}
}

void FRemapPoseDataPool::GarbageCollect()
{
	FRWScopeLock WriteLock(PoolLock, SLT_Write);
	for (auto It = Pool.CreateIterator(); It; ++It)
	{
		const FRefPosePair& Key = It.Key();
		if (!Key.Source->SkeletalMesh.IsValid() || !Key.Target->SkeletalMesh.IsValid())
		{
			It.RemoveCurrent();
		}
	}
}

} // namespace UE::UAF
