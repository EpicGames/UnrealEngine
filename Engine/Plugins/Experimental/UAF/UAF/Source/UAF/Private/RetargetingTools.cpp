// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetingTools.h"
#include "AnimationRuntime.h"
#include "Animation/SkeletonRemappingRegistry.h"
#include "Animation/SkeletonRemapping.h"

namespace UE::UAF
{
	void FRetargetingTools::BuildRetargetingPairs(
	    const UE::UAF::FLODPose& SourcePose,
	    const UE::UAF::FLODPose& TargetPose,
	    const FRemapPoseData& RemapPoseData,
	    const USkeleton* SourceSkeleton,
	    const USkeleton* TargetSkeleton,
	    bool bUseSourceRetargetModes,
	    bool bDisableRetargeting,
		BoneTrackArray& OutSkeletonPairs,
	    BoneTrackArray& OutAnimScalePairs,
	    BoneTrackArray& OutAnimRelativePairs,
	    BoneTrackArray& OutOrientAndScalePairs)
	{
		OutSkeletonPairs.Reset();
	    OutAnimScalePairs.Reset();
	    OutAnimRelativePairs.Reset();
	    OutOrientAndScalePairs.Reset();

		const int32 SourceLODLevel = SourcePose.LODLevel;
		const int32 TargetLODLevel = TargetPose.LODLevel;

	    const TArrayView<const FBoneIndexType> SourcePoseToSkeletonBoneIndexMap = SourcePose.GetLODBoneIndexToSkeletonBoneIndexMap();
	    const TArrayView<const FBoneIndexType> TargetPoseToSkeletonBoneIndexMap = TargetPose.GetLODBoneIndexToSkeletonBoneIndexMap();
		const TArray<FRemappedBone>& SourceToTargetPoseBoneIndexMap = RemapPoseData.SourceToTargetBoneIndexMapPerLOD[SourceLODLevel][TargetLODLevel].BoneIndexMap; // Source LODPose -> target LODPose

		// Iterate through all bones available in the source as well as the target skeleton/pose.
	    for (const FRemappedBone& RemappedBone : SourceToTargetPoseBoneIndexMap)
	    {
	        const int32 SourcePoseBoneIndex = RemappedBone.SourceBoneIndex;
	        const int32 TargetPoseBoneIndex = RemappedBone.TargetBoneIndex;
	        if (!SourcePoseToSkeletonBoneIndexMap.IsValidIndex(SourcePoseBoneIndex) || !TargetPoseToSkeletonBoneIndexMap.IsValidIndex(TargetPoseBoneIndex))
	        {
	            continue;
	        }

	        const int32 SourceSkeletonBoneIndex = SourcePoseToSkeletonBoneIndexMap[SourcePoseBoneIndex];
	        const int32 TargetSkeletonBoneIndex = TargetPoseToSkeletonBoneIndexMap[TargetPoseBoneIndex];
	        if (SourceSkeletonBoneIndex == INDEX_NONE || TargetSkeletonBoneIndex == INDEX_NONE)
	        {
		        continue;
	        }

	    	const EBoneTranslationRetargetingMode::Type Mode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
				bUseSourceRetargetModes,
				SourceSkeletonBoneIndex,
				TargetSkeletonBoneIndex,
				SourceSkeleton,
				TargetSkeleton,
				bDisableRetargeting);

	        switch (Mode)
	        {
				case EBoneTranslationRetargetingMode::Skeleton:
				{
					// (AtomIndex = Target LODPose, TrackIndex = Source Skeleton)
					OutSkeletonPairs.Add(BoneTrackPair(TargetPoseBoneIndex, SourceSkeletonBoneIndex));
					break;
				}

				case EBoneTranslationRetargetingMode::AnimationScaled:
	        	{
					// (AtomIndex = Target LODPose, TrackIndex = Source Skeleton)
	        		OutAnimScalePairs.Add(BoneTrackPair(TargetPoseBoneIndex, SourceSkeletonBoneIndex));
	        		break;
	        	}

				case EBoneTranslationRetargetingMode::AnimationRelative:
	        	{
	        		if (!SourcePose.IsAdditive())
	        		{
	        			OutAnimRelativePairs.Add(BoneTrackPair(TargetPoseBoneIndex, SourceSkeletonBoneIndex));
	        		}
	        		break;
				}

	            case EBoneTranslationRetargetingMode::OrientAndScale:
		        {
			        if (!SourcePose.IsAdditive())
			        {
				        OutOrientAndScalePairs.Add(BoneTrackPair(TargetPoseBoneIndex, SourceSkeletonBoneIndex));
			        }
	        		break;
		        }

	            default:
		        {
			        break;
		        }
	        }
	    }
	}

	void FRetargetingTools::ReferencePoseRetarget(
		const USkeleton* SourceSkeleton,
		const USkeleton* TargetSkeleton,
		const FSkeletonRemapping& SkeletonRemapping,
		bool bUseSourceRetargetModes,
		bool bDisableRetargeting,
		UE::UAF::FLODPose& OutPose)
	{
		if (!SkeletonRemapping.RequiresReferencePoseRetarget())
		{
			return;
		}
	    
		const TArrayView<const FBoneIndexType> PoseToSkeletonBoneIndexMap = OutPose.GetLODBoneIndexToSkeletonBoneIndexMap();
		const int32 NumLODBones = PoseToSkeletonBoneIndexMap.Num();
	    
		for (int32 LODBoneIndex = 0; LODBoneIndex < NumLODBones; ++LODBoneIndex)
		{
			const int32 TargetSkeletonBoneIndex = PoseToSkeletonBoneIndexMap[LODBoneIndex];
			if (TargetSkeletonBoneIndex == INDEX_NONE)
			{
				continue;
			}
    
			OutPose.LocalTransformsView[LODBoneIndex].SetRotation(
				SkeletonRemapping.RetargetBoneRotationToTargetSkeleton(
					TargetSkeletonBoneIndex,
					OutPose.LocalTransformsView[LODBoneIndex].GetRotation()));
    
			TranslationRetargetReferencePoseBone<false>(
				SourceSkeleton,
				TargetSkeleton,
				SkeletonRemapping,
				bUseSourceRetargetModes,
				bDisableRetargeting,
				LODBoneIndex,
				TargetSkeletonBoneIndex,
				OutPose);
		}
	}
	
	void FRetargetingTools::RetargetBoneTransform(
			const FReferencePose& ReferencePose,
			const USkeleton* SourceSkeleton,
			const USkeleton* TargetSkeleton,
			const FName& SourceName,
			const TArray<FTransform>& RetargetTransforms,
			FTransform& BoneTransform,
			const int32 SkeletonBoneIndex,
			const int32 LODBoneIndex,
			const bool bIsBakedAdditive,
			const bool bDisableRetargeting)
	{
		check(!RetargetTransforms.IsEmpty());
		if (!SourceSkeleton)
		{
			return;
		}

		const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(SourceSkeleton, TargetSkeleton);
		const int32 TargetSkeletonBoneIndex = ReferencePose.GetSkeletonBoneIndexFromLODBoneIndex(LODBoneIndex);
		const int32 SourceSkeletonBoneIndex = SkeletonRemapping.IsValid() ? SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex) : SkeletonBoneIndex;

		const bool UseSourceRetargetModes = TargetSkeleton->GetUseRetargetModesFromCompatibleSkeleton();
		const EBoneTranslationRetargetingMode::Type RetargetMode = UseSourceRetargetModes 
			? SourceSkeleton->GetBoneTranslationRetargetingMode(SourceSkeletonBoneIndex, bDisableRetargeting) 
			: TargetSkeleton->GetBoneTranslationRetargetingMode(TargetSkeletonBoneIndex, bDisableRetargeting);

		switch (RetargetMode)
		{
		case EBoneTranslationRetargetingMode::AnimationScaled:
			{
				// @todo - precache that in FBoneContainer when we have SkeletonIndex->TrackIndex mapping. So we can just apply scale right away.
				if (RetargetTransforms.IsValidIndex(SourceSkeletonBoneIndex))
				{
					const TArray<FTransform>& SkeletonRefPoseArray = RetargetTransforms;
					const float SourceTranslationLength = SkeletonRefPoseArray[SourceSkeletonBoneIndex].GetTranslation().Size();
					if (SourceTranslationLength > UE_KINDA_SMALL_NUMBER)
					{
						const float TargetTranslationLength = ReferencePose.GetRefPoseTranslation(LODBoneIndex).Size();
						BoneTransform.ScaleTranslation(TargetTranslationLength / SourceTranslationLength);
					}
				}
				break;
			}

		case EBoneTranslationRetargetingMode::Skeleton:
			{
				BoneTransform.SetTranslation(bIsBakedAdditive ? FVector::ZeroVector : ReferencePose.GetRefPoseTranslation(LODBoneIndex));
				break;
			}

		case EBoneTranslationRetargetingMode::AnimationRelative:
			{
				// With baked additive animations, Animation Relative delta gets canceled out, so we can skip it.
				// (A1 + Rel) - (A2 + Rel) = A1 - A2.
				if (!bIsBakedAdditive)
				{
					const TArray<FTransform>& AuthoredOnRefSkeleton = RetargetTransforms;

					// Remap the base pose onto the target skeleton so that we are working entirely in target space
					if (AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex))
					{
						FTransform BaseTransform = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex];
						RetargetBone_AnimRelative(LODBoneIndex,
							TargetSkeletonBoneIndex,
							ReferencePose,
							SkeletonRemapping,
							BaseTransform,
							BoneTransform);
					}
				}
				break;
			}

		case EBoneTranslationRetargetingMode::OrientAndScale:
			{
				if (!bIsBakedAdditive)
				{
					const TArray<FTransform>& AuthoredOnRefSkeleton = RetargetTransforms;
					if(AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex) && TargetSkeleton && TargetSkeleton->GetReferenceSkeleton().IsValidIndex(TargetSkeletonBoneIndex))
					{
						const FVector SourceSkelTrans = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex].GetTranslation();
						const FVector TargetSkelTrans = TargetSkeleton->GetReferenceSkeleton().GetRefBonePose()[TargetSkeletonBoneIndex].GetTranslation();

						// If translations are identical, we don't need to do any retargeting
						if (!SourceSkelTrans.Equals(TargetSkelTrans, BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION))
						{
							const float SourceSkelTransLength = SourceSkelTrans.Size();
							const float TargetSkelTransLength = TargetSkelTrans.Size();

							// this only works on non zero vectors.
							if (!FMath::IsNearlyZero(SourceSkelTransLength * TargetSkelTransLength))
							{
								const FVector SourceSkelTransDir = SourceSkelTrans / SourceSkelTransLength;
								const FVector TargetSkelTransDir = TargetSkelTrans / TargetSkelTransLength;

								const FQuat DeltaRotation = FQuat::FindBetweenNormals(SourceSkelTransDir, TargetSkelTransDir);
								const float Scale = TargetSkelTransLength / SourceSkelTransLength;

								const FVector AnimatedTranslation = BoneTransform.GetTranslation();
								// If Translation is not animated, we can just copy the TargetTranslation. No retargeting needs to be done.
								const FVector NewTranslation = (AnimatedTranslation - SourceSkelTrans).IsNearlyZero(BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION) ?
									TargetSkelTrans :
									DeltaRotation.RotateVector(AnimatedTranslation) * Scale;

								BoneTransform.SetTranslation(NewTranslation);
							}
						}
					}
				}
				break;
			}
		}
	}

	void FRetargetingTools::RetargetBone_AnimRelative(
		const int32 TargetPoseBoneIndex,
		const int32 TargetSkeletonBoneIndex,
		const FReferencePose& TargetReferencePose,
		const FSkeletonRemapping& SourceToTargetRemapping,
		FTransform BaseTransform,
		FTransform& InOutBoneTransform)
	{
		const FTransform& RefPoseTransform = TargetReferencePose.GetRefPoseTransform(TargetPoseBoneIndex);

		// Remap the base pose onto the target skeleton so that we are working entirely in target space
		if (SourceToTargetRemapping.RequiresReferencePoseRetarget())
		{
			if (TargetSkeletonBoneIndex != INDEX_NONE)
			{
				BaseTransform = SourceToTargetRemapping.RetargetBoneTransformToTargetSkeleton(TargetSkeletonBoneIndex, BaseTransform);
			}
		}

		// Apply the retargeting as if it were an additive difference between the current skeleton and the retarget skeleton. 
		InOutBoneTransform.SetRotation(InOutBoneTransform.GetRotation() * BaseTransform.GetRotation().Inverse() * RefPoseTransform.GetRotation());
		InOutBoneTransform.SetTranslation(InOutBoneTransform.GetTranslation() + (RefPoseTransform.GetTranslation() - BaseTransform.GetTranslation()));
		InOutBoneTransform.SetScale3D(InOutBoneTransform.GetScale3D() * (RefPoseTransform.GetScale3D() * BaseTransform.GetSafeScaleReciprocal(BaseTransform.GetScale3D())));
		InOutBoneTransform.NormalizeRotation();
	}

	void FRetargetingTools::RetargetPose(
		const FReferencePose& TargetReferencePose,
		const FSkeletonRemapping& SourceToTargetRemapping,
		BoneTrackArray& SkeletonRetargetingPairs,
		BoneTrackArray& AnimScaleRetargetingPairs,
		BoneTrackArray& AnimRelativeRetargetingPairs,
		BoneTrackArray& OrientAndScaleRetargetingPairs,
		const TArray<FTransform>& SourceReferencePoseTransforms,
		FLODPose& OutPose)
	{
		// Anim Scale Retargeting
		int32 const NumBonesToScaleRetarget = AnimScaleRetargetingPairs.Num();
		if (NumBonesToScaleRetarget > 0)
		{
			TArray<FTransform> const& AuthoredOnRefSkeleton = SourceReferencePoseTransforms;

			for (const BoneTrackPair& BonePair : AnimScaleRetargetingPairs)
			{
				const int32 LODBoneIndex(BonePair.AtomIndex);
				const int32 SourceSkeletonBoneIndex = BonePair.TrackIndex;
				if (!AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex))
				{
					continue;
				}

				// @todo - precache that in FBoneContainer when we have SkeletonIndex->TrackIndex mapping. So we can just apply scale right away.
				float const SourceTranslationLength = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex].GetTranslation().Size();
				if (SourceTranslationLength > UE_KINDA_SMALL_NUMBER)
				{
					float const TargetTranslationLength = TargetReferencePose.GetRefPoseTranslation(LODBoneIndex).Size();
					OutPose.LocalTransformsView[LODBoneIndex].ScaleTranslation(TargetTranslationLength / SourceTranslationLength);
				}
			}
		}

		// Anim Relative Retargeting
		int32 const NumBonesToRelativeRetarget = AnimRelativeRetargetingPairs.Num();
		if (NumBonesToRelativeRetarget > 0)
		{
			TArray<FTransform> const& AuthoredOnRefSkeleton = SourceReferencePoseTransforms;

			for (const BoneTrackPair& BonePair : AnimRelativeRetargetingPairs)
			{
				const int32 LODBoneIndex(BonePair.AtomIndex);
				const int32 SourceSkeletonBoneIndex = BonePair.TrackIndex;
				if (!AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex))
				{
					continue;
				}

				const FTransform& BaseTransform = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex];
				FTransform RetargetTransform = OutPose.LocalTransformsView[LODBoneIndex];
				const int32 TargetSkeletonBoneIndex = SourceToTargetRemapping.GetTargetSkeletonBoneIndex(SourceSkeletonBoneIndex);

				RetargetBone_AnimRelative(LODBoneIndex,
					TargetSkeletonBoneIndex,
					TargetReferencePose,
					SourceToTargetRemapping,
					BaseTransform,
					RetargetTransform);

				OutPose.LocalTransformsView[LODBoneIndex] = RetargetTransform;
			}
		}

		if (!SkeletonRetargetingPairs.IsEmpty())
		{
			for (const BoneTrackPair& BonePair : SkeletonRetargetingPairs)
			{
				const int32 LODBoneIndex(BonePair.AtomIndex);
				const int32 SourceSkeletonBoneIndex = BonePair.TrackIndex;
				OutPose.LocalTransformsView[LODBoneIndex].SetTranslation(OutPose.GetRefPose().GetRefPoseTransform(LODBoneIndex).GetTranslation());
			}
		}

		// TODO : Have to recreate GetRetargetSourceCachedData for optimization
		const int32 NumBonesToOrientAndScaleRetarget = OrientAndScaleRetargetingPairs.Num();
		if (NumBonesToOrientAndScaleRetarget > 0)
		{
			const TArray<FTransform>& AuthoredOnRefSkeleton = SourceReferencePoseTransforms;
			const bool bValidSkeletonRemapping = SourceToTargetRemapping.IsValid();
			const TArray<FTransform>* TargetSkeletonRefPose = bValidSkeletonRemapping ? &SourceToTargetRemapping.GetTargetSkeleton()->GetReferenceSkeleton().GetRefBonePose() : nullptr;
			
			for (const BoneTrackPair& BonePair : OrientAndScaleRetargetingPairs)
			{
				const int32 LODBoneIndex(BonePair.AtomIndex);
				const int32 SourceSkeletonBoneIndex = BonePair.TrackIndex;
				const int32 TargetSkeletonBoneIndex = SourceToTargetRemapping.IsValid() ? SourceToTargetRemapping.GetTargetSkeletonBoneIndex(SourceSkeletonBoneIndex) : SourceSkeletonBoneIndex;
				
				const bool bValidTargetSkeletonIndex = bValidSkeletonRemapping ? TargetSkeletonRefPose->IsValidIndex(TargetSkeletonBoneIndex) : (LODBoneIndex <TargetReferencePose.GetNumBonesForLOD(0) && LODBoneIndex >= 0);
				if(AuthoredOnRefSkeleton.IsValidIndex(SourceSkeletonBoneIndex) && bValidTargetSkeletonIndex)
				{
					const FVector SourceSkelTrans = AuthoredOnRefSkeleton[SourceSkeletonBoneIndex].GetTranslation(); 

					// Use Skeleton ref pose in case of valid remapping, otherwise use incoming reference pose (populated with either a SkeletalMesh or Skeleton ref-skeleton)
					const FVector TargetSkelTrans = bValidSkeletonRemapping ? (*TargetSkeletonRefPose)[TargetSkeletonBoneIndex].GetTranslation() : TargetReferencePose.GetRefPoseTranslation(LODBoneIndex);
			
					// If translations are identical, we don't need to do any retargeting
					if (!SourceSkelTrans.Equals(TargetSkelTrans, BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION))
					{
						const float SourceSkelTransLength = SourceSkelTrans.Size();
						const float TargetSkelTransLength = TargetSkelTrans.Size();

						// this only works on non zero vectors.
						if (!FMath::IsNearlyZero(SourceSkelTransLength * TargetSkelTransLength))
						{
							const FVector SourceSkelTransDir = SourceSkelTrans / SourceSkelTransLength;
							const FVector TargetSkelTransDir = TargetSkelTrans / TargetSkelTransLength;

							const FQuat DeltaRotation = FQuat::FindBetweenNormals(SourceSkelTransDir, TargetSkelTransDir);
							const float Scale = TargetSkelTransLength / SourceSkelTransLength;

							const FVector AnimatedTranslation = OutPose.LocalTransformsView[LODBoneIndex].GetTranslation();
							// If Translation is not animated, we can just copy the TargetTranslation. No retargeting needs to be done.
							const FVector NewTranslation = (AnimatedTranslation - SourceSkelTrans).IsNearlyZero(BONE_TRANS_RT_ORIENT_AND_SCALE_PRECISION) ?
								TargetSkelTrans :
								DeltaRotation.RotateVector(AnimatedTranslation) * Scale;

							OutPose.LocalTransformsView[LODBoneIndex].SetTranslation(NewTranslation);
						}
					}
				}
			}
		}
	}
} // end namespace UE::UAF
