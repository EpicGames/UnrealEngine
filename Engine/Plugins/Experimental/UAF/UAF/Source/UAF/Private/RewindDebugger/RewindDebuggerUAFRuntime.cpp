// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerUAFRuntime.h"

#if UAF_TRACE_ENABLED

namespace UE::UAF
{
	
void FRewindDebuggerUAFRuntime::RecordingStarted()
{
	UE::Trace::ToggleChannel(TEXT("UAF"), true);
}

void FRewindDebuggerUAFRuntime::RecordingStopped()
{
	UE::Trace::ToggleChannel(TEXT("UAF"), false);
}

void FRewindDebuggerUAFRuntime::Clear()
{
	FUAFTrace::Reset();
}

}

#endif
	
