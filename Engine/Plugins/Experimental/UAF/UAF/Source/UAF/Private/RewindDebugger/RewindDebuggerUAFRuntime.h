// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"
#include "RewindDebugger/UAFTrace.h"

#if UAF_TRACE_ENABLED
// Rewind debugger extension for Chooser support

namespace UE::UAF
{

class FRewindDebuggerUAFRuntime : public IRewindDebuggerRuntimeExtension
{
public:
	virtual void RecordingStarted() override;
	virtual void RecordingStopped() override;
	
	virtual void Clear() override;
};

}

#endif