// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebugger/UAFTrace.h"

#include "ObjectTrace.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "UAFAssetInstance.h"
#include "Misc/HashBuilder.h"
#include "Module/AnimNextModuleInstance.h"
#include "Serialization/MemoryWriter.h"
#include "ObjectAsTraceIdProxyArchive.h"
#include "Serialization/ObjectWriter.h"
#include "Variables/VariableOverrides.h"
#include "Variables/StructDataCache.h"

#if UAF_TRACE_ENABLED


UE_TRACE_EVENT_BEGIN(UAF, Instance)
	UE_TRACE_EVENT_FIELD(uint64, InstanceId)
	UE_TRACE_EVENT_FIELD(uint64, HostInstanceId)
	UE_TRACE_EVENT_FIELD(uint64, OuterObjectId)
	UE_TRACE_EVENT_FIELD(uint64, AssetId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(UAF, InstanceVariables)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, InstanceId)
	UE_TRACE_EVENT_FIELD(uint32, VariableDescriptionHash)
	UE_TRACE_EVENT_FIELD(uint8[], VariableData)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(UAF, InstanceVariablesStruct)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(uint64, InstanceId)
	UE_TRACE_EVENT_FIELD(uint8[], VariableData)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(UAF, InstanceVariableDescriptions)
	UE_TRACE_EVENT_FIELD(uint32, VariableDescriptionHash)
	UE_TRACE_EVENT_FIELD(uint8[], VariableDescriptionData)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_CHANNEL_DEFINE(UAFChannel);

namespace UE::UAF
{

FRWLock GTracedInstancesRWLock;
TSet<uint64> GTracedInstances;

FRWLock GTracedPropertiesRWLock;
TMap<uint32, const UPropertyBag*> GTracedPropertyDescs;

const FGuid FUAFTrace::CustomVersionGUID = FGuid(0x83E9BE13, 0xB1C845DC, 0x86C4D3E5, 0xE66CBE91);

namespace FUAFTraceCustomVersion
{
enum CustomVersionIndex
{
	// Before any version changes were made in the plugin
	FirstVersion = 0,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};
}

FCustomVersionRegistration GPropertyBagCustomVersion(FUAFTrace::CustomVersionGUID, FUAFTraceCustomVersion::FirstVersion, TEXT("FUAFTraceCustomVersion"));

void FUAFTrace::Reset()
{
	GTracedInstances.Empty();
	GTracedPropertyDescs.Empty();
}

void FUAFTrace::OutputUAFInstance(const FUAFAssetInstance* AnimNextInstance, const UObject* OuterObject)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(UAFChannel);
	if (!bChannelEnabled || AnimNextInstance == nullptr)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(OuterObject))
	{
		return;
	}
	
	uint64 InstanceId = AnimNextInstance->GetUniqueId();

	bool bTrace = false;

	{
		FRWScopeLock RWScopeLock(GTracedInstancesRWLock, SLT_ReadOnly);
		
		if (!GTracedInstances.Contains(InstanceId))
		{
			RWScopeLock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();

			// Note that we might double add here due to 2x people writing at the same time, but it doesn't matter
			GTracedInstances.Add(InstanceId);
			bTrace = true;
		}
	}

	if (bTrace)
	{
		if (AnimNextInstance->GetHost())
		{
			OutputUAFInstance(AnimNextInstance->GetHost(), OuterObject);
		}

		const FUAFAssetInstance* HostInstance = AnimNextInstance->GetHost();
		uint64 HostInstanceId = HostInstance ? HostInstance->GetUniqueId() : FObjectTrace::GetObjectId(OuterObject);

		TRACE_OBJECT(OuterObject);
		TRACE_OBJECT(AnimNextInstance->GetAsset<UUAFRigVMAsset>());
		TRACE_INSTANCE(OuterObject, AnimNextInstance->GetUniqueId(), HostInstanceId, FUAFAssetInstance::StaticStruct(), AnimNextInstance->GetDebugName().ToString());

		UE_TRACE_LOG(UAF, Instance, UAFChannel)
			<< Instance.InstanceId(InstanceId)
			<< Instance.OuterObjectId(FObjectTrace::GetObjectId(OuterObject))
			<< Instance.HostInstanceId(HostInstanceId)
			<< Instance.AssetId(FObjectTrace::GetObjectId(AnimNextInstance->GetAsset<UUAFRigVMAsset>()));
	}
}

uint32 GetPropertyDescHash(const TConstArrayView<FPropertyBagPropertyDesc>& Descs)
{
	FHashBuilder HashBuilder;
	for (const FPropertyBagPropertyDesc& Desc : Descs)
	{
		HashBuilder.Append(Desc.Name);
		HashBuilder.Append(Desc.ContainerTypes);
		HashBuilder.Append(Desc.ID);
		HashBuilder.Append(Desc.PropertyFlags);
		HashBuilder.Append(Desc.ValueType);

		// Q: should we care about metadata?
	}
	return HashBuilder.GetHash();
}

void FUAFTrace::OutputUAFVariables(const FUAFAssetInstance* Instance, const UObject* OuterObject)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(UAFChannel);
	if (!bChannelEnabled || Instance == nullptr || OuterObject == nullptr)
	{
		return;
	}

	uint64 InstanceId = Instance->GetUniqueId();

	if (CANNOT_TRACE_OBJECT(OuterObject))
	{
		return;
	}

	OutputUAFInstance(Instance, OuterObject);

	bool bVariablesOutput = false;
	if (Instance->Variables.OwnedVariableContainers.Num() > 0)
	{
		for (const TWeakPtr<FUAFInstanceVariableContainer>& OwnedVariableSet : Instance->Variables.AllSharedVariableContainers)
		{
			if (TSharedPtr<FUAFInstanceVariableContainer> PinnedContainer = OwnedVariableSet.Pin())
			{
				if (OutputUAFVariableSet(PinnedContainer.ToSharedRef(), InstanceId, OuterObject))
				{
					bVariablesOutput = true;
				}
			}
		}
	}

	if (!bVariablesOutput)
	{
		// trace an empty InstanceVariables message, as it's used for lifetime tracking currently
		UE_TRACE_LOG(UAF, InstanceVariables, UAFChannel)
				<< InstanceVariables.Cycle(FPlatformTime::Cycles64())
				<< InstanceVariables.RecordingTime(FObjectTrace::GetWorldElapsedTime(OuterObject->GetWorld()))
				<< InstanceVariables.InstanceId(InstanceId);
	}
}

bool FUAFTrace::OutputUAFVariableSet(const TSharedRef<FUAFInstanceVariableContainer>& InVariableSet, uint64 InstanceId, const UObject* OuterObject)
{
	bool bOutputAnything = false;
	switch (InVariableSet->VariablesContainer.GetIndex())
	{
	case FUAFInstanceVariableContainer::FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			// Copy property bag data as it will be modified
			FInstancedPropertyBag InstancedPropertyBag = InVariableSet->VariablesContainer.Get<FInstancedPropertyBag>();
			if (InstancedPropertyBag.GetNumPropertiesInBag() == 0)
			{
				break;
			}

			TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs = InstancedPropertyBag.GetPropertyBagStruct()->GetPropertyDescs();
			uint32 PropertyDescHash = GetPropertyDescHash(PropertyDescs);
			{
				FRWScopeLock RWScopeLock(GTracedPropertiesRWLock, SLT_ReadOnly);
				TArray<uint8> ArchiveData;
				ArchiveData.Reserve(1024 * 10);
				
				if (!GTracedPropertyDescs.Contains(PropertyDescHash))
				{
					// Upgrade to write lock
					RWScopeLock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();

					// Check in case someone just serialized before we aquired the write lock
					if (!GTracedPropertyDescs.Contains(PropertyDescHash))
					{
						GTracedPropertyDescs.Add(PropertyDescHash, InstancedPropertyBag.GetPropertyBagStruct());
						
						// Serialize prop descs
						TArray<FPropertyBagPropertyDesc, TInlineAllocator<64>> PropertyDescriptions;
						PropertyDescriptions.Reserve(PropertyDescs.Num());
						PropertyDescriptions = PropertyDescs;	// need to copy since we can't serialize an array view

						FMemoryWriter WriterArchive(ArchiveData);
						FObjectAsTraceIdProxyArchive Archive(WriterArchive);
						Archive.UsingCustomVersion(CustomVersionGUID);
						Archive.UsingCustomVersion(FPropertyBagCustomVersion::GUID);
						Archive << PropertyDescriptions;

						FString NameStr = InVariableSet->GetFName().ToString();

						UE_TRACE_LOG(UAF, InstanceVariableDescriptions, UAFChannel)
							<< InstanceVariableDescriptions.VariableDescriptionHash(PropertyDescHash)
							<< InstanceVariableDescriptions.VariableDescriptionData(ArchiveData.GetData(), ArchiveData.Num())
							<< InstanceVariableDescriptions.Name(TCHAR_TO_ANSI(*NameStr));
					}
				}

				ArchiveData.SetNum(0, EAllowShrinking::No);
				FMemoryWriter WriterArchive(ArchiveData);
				FObjectAsTraceIdProxyArchive Archive(WriterArchive);
				UPropertyBag* PropertyBag = const_cast<UPropertyBag*>(InstancedPropertyBag.GetPropertyBagStruct());
				if (PropertyBag != nullptr)
				{
					// Apply overrides
					if (!InVariableSet->ResolvedOverrides.IsEmpty())
					{
						const int32 VariablesNum = PropertyDescs.Num();
						for (int32 VariableIndex = 0; VariableIndex < VariablesNum; ++VariableIndex)
						{
							if (InVariableSet->ResolvedOverrides[VariableIndex] != nullptr)
							{
								const FPropertyBagPropertyDesc& Desc = PropertyDescs[VariableIndex];
								uint8* Target = Desc.CachedProperty->ContainerPtrToValuePtr<uint8>(InstancedPropertyBag.GetMutableValue().GetMemory());
								uint8* Source = InVariableSet->ResolvedOverrides[VariableIndex];
								
								Desc.CachedProperty->CopyCompleteValue(Target, Source);
							}
						}
					}
				
					PropertyBag->SerializeItem(Archive, InstancedPropertyBag.GetMutableValue().GetMemory(), nullptr);
				}

				bOutputAnything = true;
				UE_TRACE_LOG(UAF, InstanceVariables, UAFChannel)
						<< InstanceVariables.Cycle(FPlatformTime::Cycles64())
						<< InstanceVariables.RecordingTime(FObjectTrace::GetWorldElapsedTime(OuterObject->GetWorld()))
						<< InstanceVariables.InstanceId(InstanceId)
						<< InstanceVariables.VariableDescriptionHash(PropertyDescHash)
						<< InstanceVariables.VariableData(ArchiveData.GetData(), ArchiveData.Num());
			}
			break;
		}
		case FUAFInstanceVariableContainer::FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			TArray<uint8> ArchiveData;
			ArchiveData.Reserve(1024 * 10);

			FMemoryWriter WriterArchive(ArchiveData);
			FObjectAsTraceIdProxyArchive Archive(WriterArchive);

			// Copy property bag data as it will be modified
			FInstancedStruct InstancedStruct = InVariableSet->VariablesContainer.Get<FInstancedStruct>();

			// Apply overrides
			if (!InVariableSet->ResolvedOverrides.IsEmpty())
			{
				for (int32 VariableIndex = 0; VariableIndex < InVariableSet->NumVariables; ++VariableIndex)
				{
					const FStructDataCache::FPropertyInfo& PropertyInfo = InVariableSet->StructDataCache->GetProperties()[VariableIndex];
					const FProperty* Property = PropertyInfo.Property;
					
					uint8* Target = Property->ContainerPtrToValuePtr<uint8>(InstancedStruct.GetMutableMemory());
                    uint8* Source = InVariableSet->ResolvedOverrides[VariableIndex];
					
					Property->CopyCompleteValue(Target, Source);
				}
			}
			
			if (InstancedStruct.IsValid())
			{
				FInstancedStruct::StaticStruct()->SerializeItem(Archive, &InstancedStruct, nullptr);
			}

			bOutputAnything = true;
			UE_TRACE_LOG(UAF, InstanceVariablesStruct, UAFChannel)
					<< InstanceVariablesStruct.Cycle(FPlatformTime::Cycles64())
					<< InstanceVariablesStruct.RecordingTime(FObjectTrace::GetWorldElapsedTime(OuterObject->GetWorld()))
					<< InstanceVariablesStruct.InstanceId(InstanceId)
					<< InstanceVariablesStruct.VariableData(ArchiveData.GetData(), ArchiveData.Num());
			break;
		}
	default:
		checkNoEntry();
		break;
	}

	return bOutputAnything;
}

}

#endif
