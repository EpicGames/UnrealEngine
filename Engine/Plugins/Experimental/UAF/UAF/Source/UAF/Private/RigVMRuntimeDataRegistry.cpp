// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMRuntimeDataRegistry.h"
#include "Misc/ScopeRWLock.h"
#include "RigVMCore/RigVM.h"

namespace UE::UAF
{

namespace Private
{

static bool bInitialized = false;
static FDelegateHandle PostGarbageCollectHandle;

static std::atomic<uint32> GCCycle = 0; // Main thread GC counter, incremented on each main thread GC cycle

static FRWLock GlobalRuntimeDataStorageLock;
static TMultiMap<FRigVMRuntimeDataID, TSharedPtr<FRigVMRuntimeData>> GlobalRuntimeDataStorage;


static thread_local uint32 LocalGCCycle = 0;	// Local thread GC counter, used to compare with main and trigger compaction if different
static thread_local TMap<FRigVMRuntimeDataID, TWeakPtr<FRigVMRuntimeData>> LocalRuntimeDataStorage;


} // end namespace Private


/*static*/ void FRigVMRuntimeDataRegistry::Init()
{
	if (!Private::bInitialized)
	{
		Private::PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(&FRigVMRuntimeDataRegistry::HandlePostGarbageCollect);

		Private::bInitialized = true;
	}
}

/*static*/ void FRigVMRuntimeDataRegistry::Destroy()
{
	if (Private::bInitialized)
	{
		Private::bInitialized = false;

		FCoreUObjectDelegates::GetPostGarbageCollect().Remove(Private::PostGarbageCollectHandle);
		Private::GlobalRuntimeDataStorage.Empty();
	}
}

/*static*/ TWeakPtr<FRigVMRuntimeData> FRigVMRuntimeDataRegistry::FindOrAddLocalRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID, const FRigVMExtendedExecuteContext& ReferenceContext)
{
	check(Private::bInitialized);

	TWeakPtr<FRigVMRuntimeData> WeakRigVMRuntimeData = FindLocalRuntimeData(RigVMRuntimeDataID);
	if (TSharedPtr<FRigVMRuntimeData> RigVMRuntimeData = WeakRigVMRuntimeData.Pin())
	{
		if (RigVMRuntimeData->Context.VMHash != ReferenceContext.VMHash)
		{
			RigVMRuntimeData->Context = ReferenceContext;
			if (URigVM* VM = RigVMRuntimeDataID.ResolveObjectPtr())
			{
				VM->InitializeInstance(RigVMRuntimeData->Context, false);
			}
		}
	}
	else
	{
		WeakRigVMRuntimeData = AddRuntimeData(RigVMRuntimeDataID, ReferenceContext);
	}

	return WeakRigVMRuntimeData;
}

/*static*/ TWeakPtr<FRigVMRuntimeData> FRigVMRuntimeDataRegistry::FindLocalRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID)
{
	check(Private::bInitialized);

	const uint32 CurrentCycle = Private::GCCycle;
	if (CurrentCycle != Private::LocalGCCycle)
	{
		PerformLocalStorageCompaction();

		Private::LocalGCCycle = CurrentCycle;
	}

	return Private::LocalRuntimeDataStorage.FindRef(RigVMRuntimeDataID);
}

/*static*/ TWeakPtr<FRigVMRuntimeData> FRigVMRuntimeDataRegistry::AddRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID, const FRigVMExtendedExecuteContext& ReferenceContext)
{
	check(Private::bInitialized);

	check(Private::LocalRuntimeDataStorage.FindRef(RigVMRuntimeDataID).IsValid() == false);

	TSharedPtr<FRigVMRuntimeData> RigVMRuntimeData = AddGlobalRuntimeData(RigVMRuntimeDataID, ReferenceContext);
	Private::LocalRuntimeDataStorage.Add(RigVMRuntimeDataID, RigVMRuntimeData);
	if (URigVM* VM = RigVMRuntimeDataID.ResolveObjectPtr())
	{
		VM->InitializeInstance(RigVMRuntimeData->Context, false);
	}

	return RigVMRuntimeData;
}

/*static*/ void FRigVMRuntimeDataRegistry::ReleaseAllVMRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID)
{
	check(Private::bInitialized);

	ReleaseAllGlobalRuntimeData(RigVMRuntimeDataID);
}

/*static*/ TSharedPtr<FRigVMRuntimeData> FRigVMRuntimeDataRegistry::AddGlobalRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID, const FRigVMExtendedExecuteContext& ReferenceContext)
{
	TSharedPtr<FRigVMRuntimeData> RigVMRuntimeData = nullptr;

	{
		FRWScopeLock Lock(Private::GlobalRuntimeDataStorageLock, SLT_Write);
		RigVMRuntimeData = Private::GlobalRuntimeDataStorage.Add(RigVMRuntimeDataID, MakeShared<FRigVMRuntimeData>());
		RigVMRuntimeData->Context = ReferenceContext;
	}

	return RigVMRuntimeData;
}

/*static*/ void FRigVMRuntimeDataRegistry::ReleaseAllGlobalRuntimeData(const FRigVMRuntimeDataID& RigVMRuntimeDataID)
{
	FRWScopeLock Lock(Private::GlobalRuntimeDataStorageLock, SLT_Write);
	Private::GlobalRuntimeDataStorage.Remove(RigVMRuntimeDataID);
}

/*static*/ void FRigVMRuntimeDataRegistry::HandlePostGarbageCollect()
{
	Private::GCCycle++;
	Private::LocalGCCycle = Private::GCCycle; // Avoid additional compaction on main thread

	PerformLocalStorageCompaction();
	PerformGlobalStorageCompaction();
}

/*static*/ void FRigVMRuntimeDataRegistry::PerformGlobalStorageCompaction()
{
	for (auto Iter = Private::GlobalRuntimeDataStorage.CreateIterator(); Iter; ++Iter)
	{
		const FRigVMRuntimeDataID& RuntimeDataID = Iter.Key();
		if (RuntimeDataID.ResolveObjectPtr() == nullptr)
		{
			Iter.RemoveCurrent();
		}
	}
}

/*static*/ void FRigVMRuntimeDataRegistry::PerformLocalStorageCompaction()
{
	for (auto Iter = Private::LocalRuntimeDataStorage.CreateIterator(); Iter; ++Iter)
	{
		const FRigVMRuntimeDataID& RuntimeDataID = Iter.Key();
		if (RuntimeDataID.ResolveObjectPtr() == nullptr)
		{
			Iter.RemoveCurrent();
		}
	}
}

} // end namespace UE::UAF
