// Copyright Epic Games, Inc. All Rights Reserved.

#include "Script/UAFRigVMComponent.h"

#include "AnimNextExecuteContext.h"
#include "AnimNextRigVMAsset.h"
#include "UAFAssetInstance.h"
#include "Module/AnimNextModuleInstance.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Module/UAFModuleInstanceComponent.h"
#include "Variables/UAFInstanceVariableContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFRigVMComponent)

void FUAFRigVMComponent::OnBindToInstance()
{
	FUAFAssetInstance& AssetInstance = GetAssetInstance();
	const UUAFRigVMAsset* Asset = AssetInstance.GetAsset<UUAFRigVMAsset>();
	VM = Asset->RigVM;
	if (VM == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FUAFRigVMComponent::OnBindToInstance: VM is not valid.");
		return;
	}

	// Initialize the RigVM context
	ExtendedExecuteContext = Asset->GetRigVMExtendedExecuteContext();

	// Hookup the runtime data ptrs
	const TArray<FRigVMExternalVariableDef>& ExternalVariableDefs = VM->GetExternalVariableDefs();
	const int32 NumExternalVariables = ExternalVariableDefs.Num();

	if (!ensure(AssetInstance.Variables.NumInternalVariables == NumExternalVariables))
	{
		return;
	}

	if(NumExternalVariables > 0)
	{
		TArray<FRigVMExternalVariableRuntimeData> ExternalVariableRuntimeData;
		ExternalVariableRuntimeData.Reserve(AssetInstance.Variables.NumInternalVariables);
		const int32 NumInternalSharedVariableContainers = AssetInstance.Variables.InternalSharedVariableContainers.Num();
		int32 CurrentBaseIndex = 0;
		for (int32 SharedVariableSetIndex = 0; SharedVariableSetIndex < NumInternalSharedVariableContainers; ++SharedVariableSetIndex)
		{
			const TSharedRef<FUAFInstanceVariableContainer>& SharedVariableSet = AssetInstance.Variables.InternalSharedVariableContainers[SharedVariableSetIndex].Pin().ToSharedRef();
			TConstArrayView<FRigVMExternalVariableDef> DefsForSet(ExternalVariableDefs.GetData() + CurrentBaseIndex, SharedVariableSet->NumVariables);
			SharedVariableSet->GetRigVMMemoryForVariables(DefsForSet, ExternalVariableRuntimeData);
			CurrentBaseIndex += SharedVariableSet->NumVariables;
		}
		ExtendedExecuteContext.ExternalVariableRuntimeData = MoveTemp(ExternalVariableRuntimeData);
	}

	// Now initialize the 'instance', cache memory handles etc. in the context
	VM->InitializeInstance(ExtendedExecuteContext);
	ExtendedExecuteContext.SetVM(VM);

#if DO_CHECK
	bIsInitialized = true;
#endif
}

void FUAFRigVMComponent::StaticCallEvent(TStructView<FUAFScriptComponent> InScriptComponent, TConstStructView<FUAFScriptContextData> InContextData)
{
	FUAFRigVMComponent& This = InScriptComponent.Get<FUAFRigVMComponent>();
#if DO_CHECK
	if (!ensure(This.bIsInitialized))
	{
		return;
	}
#endif

	if(This.VM == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "FUAFRigVMComponent::StaticCallEvent: Could not call event - RigVM is not valid.");
		return;
	}

	FName EventName = InContextData.Get<FUAFScriptContextData>().GetEventName();
	if(!This.VM->ContainsEntry(EventName))
	{
		UE_LOGF(LogAnimation, Warning, "FUAFRigVMComponent::StaticCallEvent: Could not call event - VM does not contain event [%ls]", *EventName.ToString());
		return;
	}

	check(This.ExtendedExecuteContext.VMHash == This.VM->GetVMHash());

	FAnimNextExecuteContext& AnimNextContext = This.ExtendedExecuteContext.GetPublicDataSafe<FAnimNextExecuteContext>();
	if (InContextData.GetScriptStruct()->IsChildOf(FUAFAssetContextData::StaticStruct()))
	{
		const FUAFAssetContextData& AssetContextData = InContextData.Get<FUAFAssetContextData>();
		AnimNextContext.SetDeltaTime(AssetContextData.GetDeltaTime());
		if (FAnimNextModuleInstance* ModuleInstance = AssetContextData.GetInstance().GetRootInstance())
		{
			AnimNextContext.SetOwningObject(ModuleInstance->GetObject());
#if UE_ENABLE_DEBUG_DRAWING
			AnimNextContext.SetDrawInterface(ModuleInstance->GetDebugDrawInterface());
#endif
		}
	}

	// Insert our context data for the scope of execution
	UE::UAF::FScopedExecuteContextData ContextDataScope(AnimNextContext, InContextData);

	// Run the VM for this event
	This.VM->ExecuteVM(This.ExtendedExecuteContext, EventName);
}

TConstArrayView<UE::UAF::FScriptEventInfo> FUAFRigVMComponent::GetScriptEvents()
{
	if (ImplementedEvents.Num() > 0)
	{
		return ImplementedEvents;
	}

	FUAFAssetInstance& AssetInstance = GetAssetInstance();
	const UUAFRigVMAsset* Asset = AssetInstance.GetAsset<UUAFRigVMAsset>();
	VM = Asset->RigVM;
	if (VM == nullptr)
	{
		return TConstArrayView<UE::UAF::FScriptEventInfo>();
	}

	// Get all the module events from the VM entry points, sorted by ordering in the frame
	ImplementedEvents.Reset();

	const FRigVMByteCode& ByteCode = VM->GetByteCode();
	const TArray<const FRigVMFunction*>& Functions = VM->GetFunctions();
	FRigVMInstructionArray Instructions = ByteCode.GetInstructions();
	for (int32 EntryIndex = 0; EntryIndex < ByteCode.NumEntries(); EntryIndex++)
	{
		const FRigVMByteCodeEntry& Entry = ByteCode.GetEntry(EntryIndex);
		const FRigVMInstruction& Instruction = Instructions[Entry.InstructionIndex];
		const FRigVMExecuteOp& Op = ByteCode.GetOpAt<FRigVMExecuteOp>(Instruction);
		const FRigVMFunction* Function = Functions[Op.CallableIndex];
		check(Function != nullptr);

		if (Function->Struct->IsChildOf(FRigUnit_AnimNextModuleEventBase::StaticStruct()))
		{
			TInstancedStruct<FRigUnit_AnimNextModuleEventBase> StructInstance;
			StructInstance.InitializeAsScriptStruct(Function->Struct);
			const FRigUnit_AnimNextModuleEventBase& Event = StructInstance.Get();
			UE::UAF::FScriptEventInfo& NewEvent = ImplementedEvents.AddDefaulted_GetRef();
			NewEvent.Struct = Function->Struct;
			NewEvent.Binding = Event.GetBindingFunction();
			NewEvent.Phase = Event.GetEventPhase();
			NewEvent.TickGroup = Event.GetTickGroup();
			NewEvent.EndTickGroup = Event.GetEndTickGroup();
			NewEvent.SortOrder = Event.GetSortOrder();
			NewEvent.bUserEvent = Event.IsUserEvent();
			NewEvent.bIsTask = Event.IsTask();
			NewEvent.bIsGameThreadTask = Event.IsGameThreadTask();

			FName EventName = Event.GetEventName();

			// User events can override their event name etc. via parameters 
			if (Function->Struct->IsChildOf(FRigUnit_AnimNextUserEvent::StaticStruct()))
			{
				// Pull the values out of the literal memory
				FRigVMOperandArray Operands = ByteCode.GetOperandsForCallableOp(Instruction);
				check(Function->ArgumentNames.Num() == Operands.Num());
				int32 NumOperands = Operands.Num();
				for (int32 OperandIndex = 0; OperandIndex < NumOperands; ++OperandIndex)
				{
					const FRigVMOperand& Operand = Operands[OperandIndex];
					FName OperandName = Function->ArgumentNames[OperandIndex];
					if (OperandName == GET_MEMBER_NAME_CHECKED(FRigUnit_AnimNextUserEvent, Name))
					{
						check(Operand.GetMemoryType() == ERigVMMemoryType::Literal);
						EventName = *VM->LiteralMemoryStorage.GetData<FName>(Operand.GetRegisterIndex());
					}
					else if (OperandName == GET_MEMBER_NAME_CHECKED(FRigUnit_AnimNextUserEvent, SortOrder))
					{
						check(Operand.GetMemoryType() == ERigVMMemoryType::Literal);
						NewEvent.SortOrder = *VM->LiteralMemoryStorage.GetData<int32>(Operand.GetRegisterIndex());
					}
					else if (OperandName == GET_MEMBER_NAME_CHECKED(FRigUnit_AnimNextUserEvent, EndTickGroup))
					{
						check(Operand.GetMemoryType() == ERigVMMemoryType::Literal);
						NewEvent.EndTickGroup = *VM->LiteralMemoryStorage.GetData<
							TEnumAsByte<enum ETickingGroup>>(Operand.GetRegisterIndex());
					}
				}
			}

			NewEvent.Event = UE::UAF::FScriptEvent(&FUAFRigVMComponent::StaticCallEvent, EventName);
		}
	}

	Algo::Sort(ImplementedEvents, [](const UE::UAF::FScriptEventInfo& InA, const UE::UAF::FScriptEventInfo& InB)
	{
		if (InA.Phase != InB.Phase)
		{
			return InA.Phase < InB.Phase;
		}
		else if (InA.TickGroup != InB.TickGroup)
		{
			return InA.TickGroup < InB.TickGroup;
		}
		else if (InA.SortOrder != InB.SortOrder)
		{
			return InA.SortOrder < InB.SortOrder;
		}

		// Tie-break sorting on event name for determinism
		return InA.Event.GetEventName().Compare(InB.Event.GetEventName()) < 0;
	});

	return ImplementedEvents;
}