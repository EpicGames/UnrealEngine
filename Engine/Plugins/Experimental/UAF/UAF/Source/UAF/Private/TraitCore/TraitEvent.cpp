// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TraitEvent)

bool FAnimNextTraitEvent::DecrementLifetime(UE::UAF::FTraitEventList& OutputEventList)
{
	const bool bExpired = Lifetime.Decrement();
	if (bExpired)
	{
		OnExpired(OutputEventList);
	}

	return bExpired;
}
