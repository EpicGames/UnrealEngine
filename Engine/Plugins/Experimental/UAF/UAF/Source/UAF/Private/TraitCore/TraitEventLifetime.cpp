// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitEventLifetime.h"

namespace UE::UAF
{
	bool FTraitEventLifetime::Decrement()
	{
		if (IsInfinite())
		{
			// Event has infinite duration, we never expire
			return false;
		}
		else if (IsExpired())
		{
			// Event has already expired
			return true;
		}

		LifetimeCount--;
		return LifetimeCount == 0;
	}
}
