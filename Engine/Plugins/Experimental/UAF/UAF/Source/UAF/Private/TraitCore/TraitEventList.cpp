// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraitCore/TraitEventList.h"

namespace UE::UAF
{
	void DecrementLifetimeAndPurgeExpired(FTraitEventList& EventList, FTraitEventList& OutputEventList)
	{
		// We remove expired events and modify the array in place to avoid heap churn
		const int32 NumInputEvents = EventList.Num();
		int32 InputEventWriteIndex = 0;

		for (int32 InputEventReadIndex = 0; InputEventReadIndex < NumInputEvents; ++InputEventReadIndex)
		{
			FAnimNextTraitEventPtr& Event = EventList[InputEventReadIndex];
			if (!Event->IsConsumed() && !Event->DecrementLifetime(OutputEventList))
			{
				// Event has not expired, swap it to keep it around
				Swap(EventList[InputEventWriteIndex++], Event);
			}
		}

		EventList.SetNum(InputEventWriteIndex, EAllowShrinking::Yes);
	}
}
