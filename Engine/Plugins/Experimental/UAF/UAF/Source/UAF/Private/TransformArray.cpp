// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformArray.h"

namespace UE::UAF
{

const FTransform TransformAdditiveIdentity = FTransform(FQuat::Identity, FVector::ZeroVector, FVector::ZeroVector);

} // namespace UE::UAF
