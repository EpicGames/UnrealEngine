// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformArrayOperations.h"
#include "TransformArray.h"

namespace UE::UAF
{
	void SetIdentity(const FTransformArrayAoSView& Dest, bool bIsAdditive)
	{
		const int32 NumTransforms = Dest.Num();
		SetIdentity(Dest, bIsAdditive, /*StartIndex=*/0, NumTransforms);
	}

	void SetIdentity(const FTransformArrayAoSView& Dest, bool bIsAdditive, int32 StartIndex, int32 NumTransformsToSet)
	{
		const FTransform Identity = bIsAdditive ? TransformAdditiveIdentity : FTransform::Identity;

		const int EndIndex = StartIndex + NumTransformsToSet;
		check((StartIndex + NumTransformsToSet) <= Dest.Num());
		check(StartIndex >= 0 && StartIndex <= Dest.Num());
		check(EndIndex <= Dest.Num());

		for (int32 TransformIndex = StartIndex; TransformIndex < EndIndex; ++TransformIndex)
		{
			Dest[TransformIndex] = Identity;
		}
	}

	void CopyTransforms(const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source, int32 StartIndex, int32 NumToCopy)
	{
		const int32 NumTransforms = Dest.Num();
		const int32 EndIndex = NumToCopy >= 0 ? (StartIndex + NumToCopy) : NumTransforms;

		check(Source.Num() >= NumTransforms);
		check(StartIndex >= 0 && StartIndex < NumTransforms);
		check(EndIndex <= NumTransforms);

		for (int32 TransformIndex = StartIndex; TransformIndex < EndIndex; ++TransformIndex)
		{
			Dest[TransformIndex] = Source[TransformIndex];
		}
	}

	void NormalizeRotations(const FTransformArrayAoSView& Input)
	{
		const int32 NumTransforms = Input.Num();

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			Input[TransformIndex].NormalizeRotation();
		}
	}

	void ConvertPoseLocalToMeshRotation(const FTransformArrayAoSView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap)
	{
		// @TODO: This method is untested as AoS doesn't seem to compile? 
		const int32 NumLODBoneIndexes = LODBoneIndexToParentLODBoneIndexMap.Num();

		for (int32 LODBoneIndex = 1; LODBoneIndex < NumLODBoneIndexes; ++LODBoneIndex)
		{
			const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
			const FQuat MeshSpaceRotation = Input[ParentLODBoneIndex].GetRotation() * Input[LODBoneIndex].GetRotation();
			Input[LODBoneIndex].SetRotation(MeshSpaceRotation);
		}
	}

	void ConvertPoseMeshToLocalRotation(const FTransformArrayAoSView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap)
	{
		// @TODO: This method is untested as AoS doesn't seem to compile? 
		const int32 NumLODBoneIndexes = LODBoneIndexToParentLODBoneIndexMap.Num();

		for (int32 LODBoneIndex = NumLODBoneIndexes - 1; LODBoneIndex > 0; --LODBoneIndex)
		{
			const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
			const FQuat LocalSpaceRotation = Input[ParentLODBoneIndex].GetRotation().Inverse() * Input[LODBoneIndex].GetRotation();
			Input[LODBoneIndex].SetRotation(LocalSpaceRotation);
		}
	}

	void ConvertPoseLocalToMeshRotationTranslation(const FTransformArrayAoSView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap)
	{
		// @TODO: This method is untested as AoS doesn't seem to compile? 
		const int32 NumLODBoneIndexes = LODBoneIndexToParentLODBoneIndexMap.Num();

		for (int32 LODBoneIndex = 1; LODBoneIndex < NumLODBoneIndexes; ++LODBoneIndex)
		{
			const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
			const FQuat MeshSpaceRotation = Input[ParentLODBoneIndex].GetRotation() * Input[LODBoneIndex].GetRotation();
			const FVector MeshSpaceTranslation = Input[ParentLODBoneIndex].GetTranslation() + Input[ParentLODBoneIndex].GetRotation().RotateVector(Input[LODBoneIndex].GetTranslation());
			Input[LODBoneIndex].SetRotation(MeshSpaceRotation);
			Input[LODBoneIndex].SetTranslation(MeshSpaceTranslation);
		}
	}

	void ConvertPoseMeshToLocalRotationTranslation(const FTransformArrayAoSView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap)
	{
		// @TODO: This method is untested as AoS doesn't seem to compile? 
		const int32 NumLODBoneIndexes = LODBoneIndexToParentLODBoneIndexMap.Num();

		for (int32 LODBoneIndex = NumLODBoneIndexes - 1; LODBoneIndex > 0; --LODBoneIndex)
		{
			const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
			const FQuat LocalSpaceRotation = Input[ParentLODBoneIndex].GetRotation().Inverse() * Input[LODBoneIndex].GetRotation();
			FVector LocalSpaceTranslation = Input[LODBoneIndex].GetTranslation() - Input[ParentLODBoneIndex].GetTranslation();
			LocalSpaceTranslation = Input[ParentLODBoneIndex].GetRotation().UnrotateVector(LocalSpaceTranslation);
			Input[LODBoneIndex].SetRotation(LocalSpaceRotation);
			Input[LODBoneIndex].SetTranslation(LocalSpaceTranslation);
		}
	}

	void BlendWithIdentityAndAccumulate(const FTransformArrayAoSView& Base, const FTransformArrayAoSConstView& Additive, const float BlendWeight)
	{
		const ScalarRegister VBlendWeight(BlendWeight);
		const int32 NumTransforms = Base.Num();

		check(Additive.Num() >= NumTransforms);

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			FTransform::BlendFromIdentityAndAccumulate(Base[TransformIndex], Additive[TransformIndex], VBlendWeight);
		}
	}

	void BlendWithIdentityAndAccumulateMesh(
		const FTransformArrayAoSView& Base, const FTransformArrayAoSConstView& Additive, 
		const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap, const float BlendWeight)
	{
		// @TODO: This method is untested as AoS doesn't seem to compile? 
		ConvertPoseLocalToMeshRotation(Base, LODBoneIndexToParentLODBoneIndexMap);

		BlendWithIdentityAndAccumulate(Base, Additive, BlendWeight);

		ConvertPoseMeshToLocalRotation(Base, LODBoneIndexToParentLODBoneIndexMap);
	}

	void BlendWithIdentityAndAccumulatePerBoneWeight(const FTransformArrayAoSView& Base, const FTransformArrayAoSConstView& Additive, 
		const TArrayView<const int32>& TransformIndexToWeightIndexMap, const TArrayView<float> BoneWeights, const float DefaultBlendWeight)
	{
		const int32 NumTransforms = Base.Num();
		const ScalarRegister VDefaultBlendWeight(DefaultBlendWeight);

		check(Additive.Num() >= NumTransforms);
		check(TransformIndexToWeightIndexMap.Num() >= NumTransforms);

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			const int32 PerTransformIndex = TransformIndexToWeightIndexMap[TransformIndex];
			const ScalarRegister VBlendWeight = BoneWeights.IsValidIndex(PerTransformIndex) ? ScalarRegister(BoneWeights[PerTransformIndex]) : VDefaultBlendWeight;
			FTransform::BlendFromIdentityAndAccumulate(Base[TransformIndex], Additive[TransformIndex], VBlendWeight);
		}
	}

	void BlendWithIdentityAndAccumulatePerBoneWeightMesh(
		const FTransformArrayAoSView& Base, const FTransformArrayAoSConstView& Additive, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap, 
		const TArrayView<const int32>& TransformIndexToWeightIndexMap, const TArrayView<float> BoneWeights, const float DefaultBlendWeight)
	{
		ConvertPoseLocalToMeshRotation(Base, LODBoneIndexToParentLODBoneIndexMap);

		BlendWithIdentityAndAccumulatePerBoneWeight(Base, Additive, TransformIndexToWeightIndexMap, BoneWeights, DefaultBlendWeight);

		ConvertPoseMeshToLocalRotation(Base, LODBoneIndexToParentLODBoneIndexMap);
	}

	void BlendOverwriteWithScale(const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source, const float ScaleWeight)
	{
		const ScalarRegister VScaleWeight(ScaleWeight);
		const int32 NumTransforms = Source.Num();

		check(Dest.Num() >= NumTransforms);

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			Dest[TransformIndex] = Source[TransformIndex] * VScaleWeight;
		}
	}

	void BlendAddWithScale(const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source, const float ScaleWeight)
	{
		const ScalarRegister VScaleWeight(ScaleWeight);
		const int32 NumTransforms = Source.Num();

		check(Dest.Num() >= NumTransforms);

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			Dest[TransformIndex].AccumulateWithShortestRotation(Source[TransformIndex], VScaleWeight);
		}
	}

	void BlendOverwritePerBoneWithScale(
		const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source,
		const TArrayView<const int32>& LODBoneIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultScaleWeight, const bool bInvert)
	{
		const ScalarRegister VDefaultScaleWeight(DefaultScaleWeight);
		const int32 NumTransforms = Source.Num();

		check(Dest.Num() >= NumTransforms);
		
		if (bInvert)
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumTransforms; ++LODBoneIndex)
			{
				const int32 PerBoneIndex = LODBoneIndexToWeightIndexMap[LODBoneIndex];
				const ScalarRegister VScaleWeight = BoneWeights.IsValidIndex(PerBoneIndex) ? ScalarRegister(1.0f - BoneWeights[PerBoneIndex]) : VDefaultScaleWeight;
				Dest[LODBoneIndex] = Source[LODBoneIndex] * VScaleWeight;
			}
		}
		else
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumTransforms; ++LODBoneIndex)
			{
				const int32 PerBoneIndex = LODBoneIndexToWeightIndexMap[LODBoneIndex];
				const ScalarRegister VScaleWeight = BoneWeights.IsValidIndex(PerBoneIndex) ? ScalarRegister(BoneWeights[PerBoneIndex]) : VDefaultScaleWeight;
				Dest[LODBoneIndex] = Source[LODBoneIndex] * VScaleWeight;
			}
		}
	}

	void BlendAddPerBoneWithScale(
		const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source,
		const TArrayView<const int32>& LODBoneIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultScaleWeight)
	{
		const ScalarRegister VDefaultScaleWeight(DefaultScaleWeight);
		const int32 NumTransforms = Source.Num();

		check(Dest.Num() >= NumTransforms);

		for (int32 LODBoneIndex = 0; LODBoneIndex < NumTransforms; ++LODBoneIndex)
		{
			const int32 PerBoneIndex = LODBoneIndexToWeightIndexMap[LODBoneIndex];
			const ScalarRegister VScaleWeight = BoneWeights.IsValidIndex(PerBoneIndex) ? ScalarRegister(BoneWeights[PerBoneIndex]) : VDefaultScaleWeight;
			Dest[LODBoneIndex].AccumulateWithShortestRotation(Source[LODBoneIndex], VScaleWeight);
		}
	}
	
	void ConvertToAdditive(const FTransformArrayAoSConstView& Input, const FTransformArrayAoSConstView& Base, const FTransformArrayAoSView& Output)
	{
		const int32 NumTransforms = Input.Num();
		check(Base.Num() == NumTransforms);
		check(Output.Num() == NumTransforms);
		
		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			const FTransform& InputTransform = Input[TransformIndex];
			const FTransform& BaseTransform = Base[TransformIndex];
			FTransform& OutTransform = Output[TransformIndex];
			
			OutTransform.SetRotation(InputTransform.GetRotation() * BaseTransform.GetRotation().Inverse());
			OutTransform.SetTranslation(InputTransform.GetTranslation() - BaseTransform.GetTranslation());
			// additive scale considers how much it grow or lower
			// in order to support blending between different additive scale, we save [(target scale)/(source scale) - 1.f], and this can blend with 
			// other delta scale value
			// when we apply to the another scale, we apply scale * (1 + [additive scale])
			OutTransform.SetScale3D(InputTransform.GetScale3D() * FTransform::GetSafeScaleReciprocal(BaseTransform.GetScale3D()) -1.0f); 
			
			OutTransform.NormalizeRotation();
		}
	}
}
