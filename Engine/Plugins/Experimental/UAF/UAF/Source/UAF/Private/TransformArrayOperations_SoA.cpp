// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformArrayOperations.h"
#include "TransformArray.h"
#include "AnimNextStats.h"

DEFINE_STAT(STAT_AnimNext_CopyTransforms_SoA);
DEFINE_STAT(STAT_AnimNext_NormalizeRotations_SoA);
DEFINE_STAT(STAT_AnimNext_BlendOverwrite_SoA);
DEFINE_STAT(STAT_AnimNext_BlendAccumulate_SoA);

namespace UE::UAF
{
	void SetIdentity(const FTransformArraySoAView& Dest, bool bIsAdditive)
	{
		const int32 NumTransforms = Dest.Num();
		SetIdentity(Dest, bIsAdditive, /*StartIndex=*/0, NumTransforms);
	}

	// A couple of performance notes:
	//    * Types all use doubles
	//    * Because we split transforms into 3 arrays, we pay 3x the range checks
	//    * We could use pointers directly to avoid range checks, similar to what ISPC would do
	//    * FTransform uses 3x register wide types (32 bytes each)
	//    * We could do the same and use register wide types for aligned stores at the expense of
	//      using more memory
	//    * Unaligned loads aren't a big deal, perf wise, as long as we are 16 byte aligned
	//    * FVector loads require zeroing W which we could do without if we ensure we have padding
	//      on the last array element to avoid an out-of-bounds read
	//    * FVector stores end up as 2 stores which isn't too bad
	//    * Array values are very likely to be in the CPU L1 but to help hide the latency we should
	//      move loads next to one another to bundle them up up front
	//    * The biggest cost to normalizing is the square root and division which we perform on 4 elements
	//      even though there is a single value: the length. We could reduce this cost by processing
	//      4 rotations at a time and computing the square root and division for all 4 at the same time.
	//      To achieve this, we would have to swizzle the inputs/outputs in true SOA format: XXXX, YYYY, ZZZZ, WWWW.
	//      Shuffling with 4x4 floats requires 8 shuffles (after the 4 loads) and with doubles we should be
	//      able to do it with 8 loads and 8 shuffles (perhaps less with AVX).
	//      Another option might be to split loops into stages:
	//          * do dot, write it out for 8 entries
	//          * load, do sqrt, write
	//          * load, do div, write
	//          * load, cmp, mul, mask, final store
	//      Splitting into stages would help ensure that the CPU is saturated with independent instructions
	//      it can execute out-of-order. The intermediary load/stores would be from the stack if the values don't fit
	//      in registers or if we wish to re-order things cheaply (shuffle for sqrt from above).
	//      The idea being to keep loops smaller to maximize overlap of instructions while waiting for long
	//      latency instructions (cache miss in first batch, sqrt, div, etc).

	void SetIdentity(const FTransformArraySoAView& Dest, bool bIsAdditive, int32 StartIndex, int32 NumTransformsToSet)
	{
		const int EndIndex = StartIndex + NumTransformsToSet;

		check((StartIndex + NumTransformsToSet) <= Dest.Num());
		check(StartIndex >= 0 && StartIndex <= Dest.Num());
		check(EndIndex <= Dest.Num());

		const FTransform& Identity = bIsAdditive ? TransformAdditiveIdentity : FTransform::Identity;

		const FVector IdentityTranslation = Identity.GetTranslation();
		const FQuat IdentityRotation = Identity.GetRotation();
		const FVector IdentityScale3D = Identity.GetScale3D();

		for (int32 TransformIndex = StartIndex; TransformIndex < EndIndex; ++TransformIndex)
		{
			Dest.Translations[TransformIndex] = IdentityTranslation;
			Dest.Rotations[TransformIndex] = IdentityRotation;
			Dest.Scales3D[TransformIndex] = IdentityScale3D;
		}
	}

	void CopyTransforms(const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source, int32 StartIndex, int32 NumToCopy)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_CopyTransforms_SoA);
		
		const int32 NumTransforms = Dest.Num();
		const int32 EndIndex = NumToCopy >= 0 ? (StartIndex + NumToCopy) : NumTransforms;

		check((StartIndex + NumToCopy) <= NumTransforms);
		check(StartIndex >= 0 && StartIndex <= NumTransforms);
		check(EndIndex <= NumTransforms);

		const double* RESTRICT SrcRotationPtr = reinterpret_cast<const double*>(Source.Rotations.GetData()) + (StartIndex * 4);
		const double* RESTRICT SrcRotationEndPtr = reinterpret_cast<const double*>(Source.Rotations.GetData()) + (EndIndex * 4);
		double* RESTRICT DstRotationPtr = reinterpret_cast<double*>(Dest.Rotations.GetData()) + (StartIndex * 4);

		// Our SoA buffer is contiguous
		// Because the translations/scales have the same size (FVector), each entry is a fixed offset apart and we can use a single ptr/offset pair
		const double* RESTRICT SrcTranslationPtr = reinterpret_cast<const double*>(Source.Translations.GetData());
		double* RESTRICT DstTranslationPtr = reinterpret_cast<double*>(Dest.Translations.GetData());

		const int64 SrcScaleOffset = reinterpret_cast<const double*>(Source.Scales3D.GetData()) - SrcTranslationPtr;
		const int64 DstScaleOffset = reinterpret_cast<double*>(Dest.Scales3D.GetData()) - DstTranslationPtr;

		SrcTranslationPtr += StartIndex * 3;
		DstTranslationPtr += StartIndex * 3;

		// Iterate until the last element
		// Each translation/scale is 24 bytes (FVector) and everything is contiguous
		// For all of these elements, we can use aligned 32-bytes load/stores for translation/scale
		// since the next entry will simply overwrite the last 8 bytes we trashed
		// We effectively copy 24 bytes each iteration for translation/scale and we overwrite
		// the last 8 bytes of the next entry
		const double* RESTRICT SrcRotationLastPtr = SrcRotationEndPtr - 4;
		while (SrcRotationPtr < SrcRotationLastPtr)
		{
			VectorRegister4Double Rotation = VectorLoadAligned(SrcRotationPtr);
			VectorRegister4Double Translation = VectorLoad(SrcTranslationPtr);
			VectorRegister4Double Scale = VectorLoad(SrcTranslationPtr + SrcScaleOffset);

			VectorStoreAligned(Rotation, DstRotationPtr);
			VectorStore(Translation, DstTranslationPtr);
			VectorStore(Scale, DstTranslationPtr + DstScaleOffset);

			SrcRotationPtr += 4;
			SrcTranslationPtr += 3;
			DstRotationPtr += 4;
			DstTranslationPtr += 3;
		}

		// The last iteration is performed afterwards to avoid stomping past the end of our copy
		if (SrcRotationPtr < SrcRotationEndPtr)
		{
			VectorRegister4Double Rotation = VectorLoadAligned(SrcRotationPtr);
			VectorRegister4Double Translation = VectorLoad(SrcTranslationPtr);
			VectorRegister4Double Scale = VectorLoad(SrcTranslationPtr + SrcScaleOffset);

			VectorStoreAligned(Rotation, DstRotationPtr);
			VectorStoreFloat3(Translation, DstTranslationPtr);
			VectorStoreFloat3(Scale, DstTranslationPtr + DstScaleOffset);
		}
	}

	void NormalizeRotations(const FTransformArraySoAView& Input)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_NormalizeRotations_SoA);

		const int32 NumTransforms = Input.Num();

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			Input.Rotations[TransformIndex].Normalize();
		}
	}

	void ConvertPoseLocalToMeshRotation(const FTransformArraySoAView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap)
	{
		const int32 NumLODBoneIndexes = LODBoneIndexToParentLODBoneIndexMap.Num();

		for (int32 LODBoneIndex = 1; LODBoneIndex < NumLODBoneIndexes; ++LODBoneIndex)
		{
			const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
			const FQuat MeshSpaceRotation = Input[ParentLODBoneIndex].GetRotation() * Input[LODBoneIndex].GetRotation();
			Input.Rotations[LODBoneIndex] = MeshSpaceRotation;
		}
	}

	void ConvertPoseMeshToLocalRotation(const FTransformArraySoAView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap)
	{
		const int32 NumLODBoneIndexes = LODBoneIndexToParentLODBoneIndexMap.Num();

		for (int32 LODBoneIndex = NumLODBoneIndexes - 1; LODBoneIndex > 0; --LODBoneIndex)
		{
			const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
			const FQuat LocalSpaceRotation = Input[ParentLODBoneIndex].GetRotation().Inverse() * Input[LODBoneIndex].GetRotation();
			Input.Rotations[LODBoneIndex] = LocalSpaceRotation;
		}
	}

	void ConvertPoseLocalToMeshRotationTranslation(const FTransformArraySoAView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap)
	{
		const int32 NumLODBoneIndexes = LODBoneIndexToParentLODBoneIndexMap.Num();

		for (int32 LODBoneIndex = 1; LODBoneIndex < NumLODBoneIndexes; ++LODBoneIndex)
		{
			const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
			const FQuat MeshSpaceRotation = Input[ParentLODBoneIndex].GetRotation() * Input[LODBoneIndex].GetRotation();
			const FVector MeshSpaceTranslation = Input[ParentLODBoneIndex].GetTranslation() + Input[ParentLODBoneIndex].GetRotation().RotateVector(Input[LODBoneIndex].GetTranslation());
			Input.Rotations[LODBoneIndex] = MeshSpaceRotation;
			Input.Translations[LODBoneIndex] = MeshSpaceTranslation;
		}
	}

	void ConvertPoseMeshToLocalRotationTranslation(const FTransformArraySoAView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap)
	{
		const int32 NumLODBoneIndexes = LODBoneIndexToParentLODBoneIndexMap.Num();

		for (int32 LODBoneIndex = NumLODBoneIndexes - 1; LODBoneIndex > 0; --LODBoneIndex)
		{
			const FBoneIndexType ParentLODBoneIndex = LODBoneIndexToParentLODBoneIndexMap[LODBoneIndex];
			const FQuat LocalSpaceRotation = Input[ParentLODBoneIndex].GetRotation().Inverse() * Input[LODBoneIndex].GetRotation();
			Input.Rotations[LODBoneIndex] = LocalSpaceRotation;
			Input.Translations[LODBoneIndex] -= Input[ParentLODBoneIndex].GetTranslation();
			Input.Translations[LODBoneIndex] = Input[ParentLODBoneIndex].GetRotation().UnrotateVector(Input[LODBoneIndex].GetTranslation());
		}
	}

	void BlendWithIdentityAndAccumulate(const FTransformArraySoAView& Base, const FTransformArraySoAConstView& Additive, const float BlendWeight)
	{
		const int32 NumTransforms = Base.Num();

		check(Additive.Num() >= NumTransforms);

		using TransformVectorRegister = FTransform::TransformVectorRegister;

		const TransformVectorRegister VBlendWeight(ScalarRegister(BlendWeight).Value);
		const TransformVectorRegister Zero = VectorZero();
		const TransformVectorRegister Const0001 = GlobalVectorConstants::Float0001;
		const TransformVectorRegister ConstNegative0001 = VectorSubtract(Zero, Const0001);
		const TransformVectorRegister VOneMinusAlpha = VectorSubtract(VectorOne(), VBlendWeight);
		const TransformVectorRegister DefaultScale = GlobalVectorConstants::Float1110;

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			// Blend rotation
			//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
			//     const float Bias = (|A.B| >= 0 ? 1 : -1)
			//     BlendedAtom.Rotation = (B * Alpha) + (A * (Bias * (1.f - Alpha)));
			//     BlendedAtom.Rotation.QuaternionNormalize();
			//  Note: A = (0,0,0,1), which simplifies things a lot; only care about sign of B.W now, instead of doing a dot product
			const TransformVectorRegister RotationB = VectorLoadAligned(&Additive.Rotations[TransformIndex]);

			const TransformVectorRegister QuatRotationDirMask = VectorCompareGE(RotationB, Zero);
			const TransformVectorRegister BiasTimesA = VectorSelect(QuatRotationDirMask, Const0001, ConstNegative0001);
			const TransformVectorRegister RotateBTimesWeight = VectorMultiply(RotationB, VBlendWeight);
			const TransformVectorRegister UnnormalizedRotation = VectorMultiplyAdd(BiasTimesA, VOneMinusAlpha, RotateBTimesWeight);

			// Normalize blended rotation ( result = (Q.Q >= 1e-8) ? (Q / |Q|) : (0,0,0,1) )
			const TransformVectorRegister BlendedRotation = VectorNormalizeSafe(UnnormalizedRotation, Const0001);

			// FinalAtom.Rotation = BlendedAtom.Rotation * FinalAtom.Rotation;
			Base.Rotations[TransformIndex] = FQuat::MakeFromVectorRegister(VectorQuaternionMultiply2(BlendedRotation, VectorLoadAligned(&Base.Rotations[TransformIndex])));

			// Blend translation and scale
			//    BlendedAtom.Translation = Lerp(Zero, Additive.Translation, Alpha);
			//    BlendedAtom.Scale = Lerp(0, Additive.Scale, Alpha);
			const TransformVectorRegister BlendedTranslation = FMath::Lerp<TransformVectorRegister>(Zero, VectorLoadFloat3_W0(&Additive.Translations[TransformIndex]), VBlendWeight);
			const TransformVectorRegister BlendedScale3D = FMath::Lerp<TransformVectorRegister>(Zero, VectorLoadFloat3_W0(&Additive.Scales3D[TransformIndex]), VBlendWeight);

			// Apply translation and scale to final atom
			//     FinalAtom.Translation += BlendedAtom.Translation
			//     FinalAtom.Scale *= BlendedAtom.Scale
			VectorStoreFloat3(VectorAdd(VectorLoadFloat3_W0(&Base.Translations[TransformIndex]), BlendedTranslation), &Base.Translations[TransformIndex]);
			VectorStoreFloat3(VectorMultiply(VectorLoadFloat3_W0(&Base.Scales3D[TransformIndex]), VectorAdd(DefaultScale, BlendedScale3D)), &Base.Scales3D[TransformIndex]);
		}
	}

	void BlendWithIdentityAndAccumulateMesh(
		const FTransformArraySoAView& Base, const FTransformArraySoAConstView& Additive, 
		const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap, const float BlendWeight)
	{
		// The additive blend is done in component space for rotations, but local for translation and scale
		ConvertPoseLocalToMeshRotation(Base, LODBoneIndexToParentLODBoneIndexMap);

		BlendWithIdentityAndAccumulate(Base, Additive, BlendWeight);

		ConvertPoseMeshToLocalRotation(Base, LODBoneIndexToParentLODBoneIndexMap);
	}

	void BlendWithIdentityAndAccumulatePerBoneWeight(const FTransformArraySoAView& Base, const FTransformArraySoAConstView& Additive,
		const TArrayView<const int32>& TransformIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultBlendWeight)
	{
		const int32 NumTransforms = Base.Num();

		check(Additive.Num() >= NumTransforms);
		check(TransformIndexToWeightIndexMap.Num() >= NumTransforms);

		using TransformVectorRegister = FTransform::TransformVectorRegister;

		const TransformVectorRegister Zero = VectorZero();
		const TransformVectorRegister Const0001 = GlobalVectorConstants::Float0001;
		const TransformVectorRegister ConstNegative0001 = VectorSubtract(Zero, Const0001);
		const TransformVectorRegister DefaultScale = GlobalVectorConstants::Float1110;

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			const int32 PerTransformIndex = TransformIndexToWeightIndexMap[TransformIndex];
			const float ScaleWeight = BoneWeights.IsValidIndex(PerTransformIndex) ? BoneWeights[PerTransformIndex] : DefaultBlendWeight;

			const TransformVectorRegister VBlendWeight(ScalarRegister(ScaleWeight).Value);
			const TransformVectorRegister VOneMinusAlpha = VectorSubtract(VectorOne(), VBlendWeight);

			// Blend rotation
			//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
			//     const float Bias = (|A.B| >= 0 ? 1 : -1)
			//     BlendedAtom.Rotation = (B * Alpha) + (A * (Bias * (1.f - Alpha)));
			//     BlendedAtom.Rotation.QuaternionNormalize();
			//  Note: A = (0,0,0,1), which simplifies things a lot; only care about sign of B.W now, instead of doing a dot product
			const TransformVectorRegister RotationB = VectorLoadAligned(&Additive.Rotations[TransformIndex]);

			const TransformVectorRegister QuatRotationDirMask = VectorCompareGE(RotationB, Zero);
			const TransformVectorRegister BiasTimesA = VectorSelect(QuatRotationDirMask, Const0001, ConstNegative0001);
			const TransformVectorRegister RotateBTimesWeight = VectorMultiply(RotationB, VBlendWeight);
			const TransformVectorRegister UnnormalizedRotation = VectorMultiplyAdd(BiasTimesA, VOneMinusAlpha, RotateBTimesWeight);

			// Normalize blended rotation ( result = (Q.Q >= 1e-8) ? (Q / |Q|) : (0,0,0,1) )
			const TransformVectorRegister BlendedRotation = VectorNormalizeSafe(UnnormalizedRotation, Const0001);

			// FinalAtom.Rotation = BlendedAtom.Rotation * FinalAtom.Rotation;
			Base.Rotations[TransformIndex] = FQuat::MakeFromVectorRegister(VectorQuaternionMultiply2(BlendedRotation, VectorLoadAligned(&Base.Rotations[TransformIndex])));

			// Blend translation and scale
			//    BlendedAtom.Translation = Lerp(Zero, Additive.Translation, Alpha);
			//    BlendedAtom.Scale = Lerp(0, Additive.Scale, Alpha);
			const TransformVectorRegister BlendedTranslation = FMath::Lerp<TransformVectorRegister>(Zero, VectorLoadFloat3_W0(&Additive.Translations[TransformIndex]), VBlendWeight);
			const TransformVectorRegister BlendedScale3D = FMath::Lerp<TransformVectorRegister>(Zero, VectorLoadFloat3_W0(&Additive.Scales3D[TransformIndex]), VBlendWeight);

			// Apply translation and scale to final atom
			//     FinalAtom.Translation += BlendedAtom.Translation
			//     FinalAtom.Scale *= BlendedAtom.Scale
			VectorStoreFloat3(VectorAdd(VectorLoadFloat3_W0(&Base.Translations[TransformIndex]), BlendedTranslation), &Base.Translations[TransformIndex]);
			VectorStoreFloat3(VectorMultiply(VectorLoadFloat3_W0(&Base.Scales3D[TransformIndex]), VectorAdd(DefaultScale, BlendedScale3D)), &Base.Scales3D[TransformIndex]);
		}
	}

	void BlendWithIdentityAndAccumulatePerBoneWeightMesh(
		const FTransformArraySoAView& Base, const FTransformArraySoAConstView& Additive, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap, 
		const TArrayView<const int32>& TransformIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultBlendWeight)
	{
		// The additive blend is done in component space for rotations, but local for translation and scale
		ConvertPoseLocalToMeshRotation(Base, LODBoneIndexToParentLODBoneIndexMap);

		BlendWithIdentityAndAccumulatePerBoneWeight(Base, Additive, TransformIndexToWeightIndexMap, BoneWeights, DefaultBlendWeight);

		ConvertPoseMeshToLocalRotation(Base, LODBoneIndexToParentLODBoneIndexMap);
	}

	void BlendOverwriteWithScale(const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source, const float ScaleWeight)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_BlendOverwrite_SoA);

		const int32 NumTransforms = Source.Num();

		check(Dest.Num() >= NumTransforms);

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			Dest.Translations[TransformIndex] = Source.Translations[TransformIndex] * ScaleWeight;
			Dest.Rotations[TransformIndex] = Source.Rotations[TransformIndex] * ScaleWeight;
			Dest.Scales3D[TransformIndex] = Source.Scales3D[TransformIndex] * ScaleWeight;
		}
	}

	void BlendAddWithScale(const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source, const float ScaleWeight)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_BlendAccumulate_SoA);

		const int32 NumTransforms = Source.Num();

		check(Dest.Num() >= NumTransforms);

		using QuatVectorRegister = FQuat::QuatVectorRegister;
		const QuatVectorRegister Zero = VectorZero();
		const ScalarRegister VScaleWeight(ScaleWeight);

		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			const QuatVectorRegister SourceRotation = VectorLoadAligned(&Source.Rotations[TransformIndex]);
			const QuatVectorRegister DestRotation = VectorLoadAligned(&Dest.Rotations[TransformIndex]);

			const QuatVectorRegister BlendedRotation = VectorMultiply(SourceRotation, VScaleWeight);

			// Blend rotation
			//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
			//     const float Bias = (|A.B| >= 0 ? 1 : -1)
			//     return A + B * Bias;

			const QuatVectorRegister RotationDot = VectorDot4(DestRotation, BlendedRotation);
			const QuatVectorRegister QuatRotationDirMask = VectorCompareGE(RotationDot, Zero);
			const QuatVectorRegister NegativeB = VectorSubtract(Zero, BlendedRotation);
			const QuatVectorRegister BiasTimesB = VectorSelect(QuatRotationDirMask, BlendedRotation, NegativeB);

			Dest.Rotations[TransformIndex] = FQuat::MakeFromVectorRegister(VectorAdd(DestRotation, BiasTimesB));

			Dest.Translations[TransformIndex] += Source.Translations[TransformIndex] * ScaleWeight;
			Dest.Scales3D[TransformIndex] += Source.Scales3D[TransformIndex] * ScaleWeight;
		}
	}

	void BlendOverwritePerBoneWithScale(
		const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source,
		const TArrayView<const int32>& LODBoneIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultScaleWeight, const bool bInvert)
	{
		const int32 NumTransforms = Source.Num();

		check(Dest.Num() >= NumTransforms);

		if (bInvert)
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumTransforms; ++LODBoneIndex)
			{
				const int32 PerBoneIndex = LODBoneIndexToWeightIndexMap[LODBoneIndex];
				const float ScaleWeight = BoneWeights.IsValidIndex(PerBoneIndex) ? (1.0f - BoneWeights[PerBoneIndex]) : DefaultScaleWeight;

				Dest.Translations[LODBoneIndex] = Source.Translations[LODBoneIndex] * ScaleWeight;
				Dest.Rotations[LODBoneIndex] = Source.Rotations[LODBoneIndex] * ScaleWeight;
				Dest.Scales3D[LODBoneIndex] = Source.Scales3D[LODBoneIndex] * ScaleWeight;
			}
		}
		else
		{
			for (int32 LODBoneIndex = 0; LODBoneIndex < NumTransforms; ++LODBoneIndex)
			{
				const int32 PerBoneIndex = LODBoneIndexToWeightIndexMap[LODBoneIndex];
				const float ScaleWeight = BoneWeights.IsValidIndex(PerBoneIndex) ? BoneWeights[PerBoneIndex] : DefaultScaleWeight;

				Dest.Translations[LODBoneIndex] = Source.Translations[LODBoneIndex] * ScaleWeight;
				Dest.Rotations[LODBoneIndex] = Source.Rotations[LODBoneIndex] * ScaleWeight;
				Dest.Scales3D[LODBoneIndex] = Source.Scales3D[LODBoneIndex] * ScaleWeight;
			}
		}
	}

	void BlendAddPerBoneWithScale(
		const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source,
		const TArrayView<const int32>& LODBoneIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultScaleWeight)
	{
		const int32 NumTransforms = Source.Num();

		check(Dest.Num() >= NumTransforms);

		using QuatVectorRegister = FQuat::QuatVectorRegister;
		const QuatVectorRegister Zero = VectorZero();

		for (int32 LODBoneIndex = 0; LODBoneIndex < NumTransforms; ++LODBoneIndex)
		{
			const int32 PerBoneIndex = LODBoneIndexToWeightIndexMap[LODBoneIndex];
			const float ScaleWeight = BoneWeights.IsValidIndex(PerBoneIndex) ? BoneWeights[PerBoneIndex] : DefaultScaleWeight;
			const ScalarRegister VScaleWeight(ScaleWeight);

			const QuatVectorRegister SourceRotation = VectorLoadAligned(&Source.Rotations[LODBoneIndex]);
			const QuatVectorRegister DestRotation = VectorLoadAligned(&Dest.Rotations[LODBoneIndex]);

			const QuatVectorRegister BlendedRotation = VectorMultiply(SourceRotation, VScaleWeight);

			// Blend rotation
			//     To ensure the 'shortest route', we make sure the dot product between the both rotations is positive.
			//     const float Bias = (|A.B| >= 0 ? 1 : -1)
			//     return A + B * Bias;

			const QuatVectorRegister RotationDot = VectorDot4(DestRotation, BlendedRotation);
			const QuatVectorRegister QuatRotationDirMask = VectorCompareGE(RotationDot, Zero);
			const QuatVectorRegister NegativeB = VectorSubtract(Zero, BlendedRotation);
			const QuatVectorRegister BiasTimesB = VectorSelect(QuatRotationDirMask, BlendedRotation, NegativeB);

			Dest.Rotations[LODBoneIndex] = FQuat::MakeFromVectorRegister(VectorAdd(DestRotation, BiasTimesB));

			Dest.Translations[LODBoneIndex] += Source.Translations[LODBoneIndex] * ScaleWeight;
			Dest.Scales3D[LODBoneIndex] += Source.Scales3D[LODBoneIndex] * ScaleWeight;
		}
	}
	
	void ConvertToAdditive(const FTransformArraySoAConstView& Input, const FTransformArraySoAConstView& Base, const FTransformArraySoAView& Output)
	{
		const int32 NumTransforms = Input.Num();
		check(Base.Num() == NumTransforms);
		check(Output.Num() == NumTransforms);
		
		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			Output.Rotations[TransformIndex] = Input[TransformIndex].GetRotation() * Base[TransformIndex].GetRotation().Inverse();
			Output.Translations[TransformIndex] = Input[TransformIndex].GetTranslation() - Base[TransformIndex].GetTranslation();
			// additive scale considers how much it grow or lower
			// in order to support blending between different additive scale, we save [(target scale)/(source scale) - 1.f], and this can blend with 
			// other delta scale value
			// when we apply to the another scale, we apply scale * (1 + [additive scale])
			Output.Scales3D[TransformIndex] = Input[TransformIndex].GetScale3D() * FTransform::GetSafeScaleReciprocal(Base[TransformIndex].GetScale3D()) -1.0f; 
			
			Output.Rotations[TransformIndex].Normalize();
		}
	}
}
