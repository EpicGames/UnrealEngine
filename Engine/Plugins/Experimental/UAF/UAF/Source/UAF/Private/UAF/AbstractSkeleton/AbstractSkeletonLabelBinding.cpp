// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/AbstractSkeletonLabelBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbstractSkeletonLabelBinding)

#if WITH_EDITOR

bool UAbstractSkeletonLabelBinding::BindBoneToLabel(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection, const FName BoneName, const FName Label)
{
	if (!IsLabelBound(InCollection, BoneName))
	{
		FAbstractSkeleton_LabelBinding& NewBinding = LabelBindings.AddDefaulted_GetRef();
		NewBinding.BoneName = BoneName;
		NewBinding.Label = Label;
		NewBinding.SourceCollection = InCollection;

		return true;
	}
	else
	{
		return false;
	}
}

bool UAbstractSkeletonLabelBinding::UnbindBoneFromLabel(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection, const FName BoneName, const FName Label)
{
	if (IsLabelBound(InCollection, Label))
	{
		const int32 BindingIndex = LabelBindings.IndexOfByPredicate([&](const FAbstractSkeleton_LabelBinding& Binding)
		{
			return Binding.SourceCollection == InCollection && Binding.BoneName == BoneName && Binding.Label == Label;
		});

		if (ensure(BindingIndex != INDEX_NONE))
		{
			LabelBindings.RemoveAt(BindingIndex);
			return true;
		}
	}

	return false;
}

bool UAbstractSkeletonLabelBinding::AddLabelCollection(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection)
{
	if (!LabelCollections.Contains(InCollection))
	{
		LabelCollections.Add(InCollection);
		return true;
	}

	return false;
}

bool UAbstractSkeletonLabelBinding::RemoveLabelCollection(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection)
{
	LabelCollections.Remove(InCollection);

	LabelBindings.RemoveAll([InCollection](const FAbstractSkeleton_LabelBinding& InBinding)
	{
		return InBinding.SourceCollection == InCollection;
	});

	return true;
}

bool UAbstractSkeletonLabelBinding::SetSkeleton(const TObjectPtr<USkeleton> InSkeleton)
{
	if (InSkeleton != Skeleton)
	{
		Skeleton = InSkeleton;
		LabelBindings.Empty();
		
		return true;
	}

	return false;
}

#endif // WITH_EDITOR

bool UAbstractSkeletonLabelBinding::IsLabelBound(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection, const FName LabelName) const
{
	return LabelBindings.ContainsByPredicate([&](const FAbstractSkeleton_LabelBinding& Binding)
	{
		return Binding.SourceCollection == InCollection && Binding.Label == LabelName;
	});
}

FName UAbstractSkeletonLabelBinding::GetLabelBinding(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection, const FName LabelName) const
{
	const int32 BindingIndex = LabelBindings.IndexOfByPredicate([&](const FAbstractSkeleton_LabelBinding& Binding)
		{
			return Binding.SourceCollection == InCollection && Binding.Label == LabelName;
		});
	
	return BindingIndex == INDEX_NONE ? NAME_None : LabelBindings[BindingIndex].BoneName;
}

TConstArrayView<FAbstractSkeleton_LabelBinding> UAbstractSkeletonLabelBinding::GetLabelBindings() const
{
	return LabelBindings;
}

TObjectPtr<USkeleton> UAbstractSkeletonLabelBinding::GetSkeleton() const
{
	return Skeleton;
}

TConstArrayView<TObjectPtr<const UAbstractSkeletonLabelCollection>> UAbstractSkeletonLabelBinding::GetLabelCollections() const
{
	return LabelCollections;
}
