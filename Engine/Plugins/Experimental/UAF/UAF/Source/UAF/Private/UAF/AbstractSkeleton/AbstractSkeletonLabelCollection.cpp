// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/AbstractSkeletonLabelCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbstractSkeletonLabelCollection)

#if WITH_EDITOR

bool UAbstractSkeletonLabelCollection::AddLabel(const FName InLabel)
{
	if (!HasLabel(InLabel))
	{
		Labels.Add(InLabel);
		return true;
	}
	else
	{
		return false;
	}
}

bool UAbstractSkeletonLabelCollection::RenameLabel(const FName InOldLabel, const FName InNewLabel)
{
	const int32 ExistingLabelIndex = Labels.Find(InOldLabel);
	
	if (ExistingLabelIndex == INDEX_NONE)
	{
		return false;
	}

	if (HasLabel(InNewLabel))
	{
		return false;
	}

	Labels[ExistingLabelIndex] = InNewLabel;
	return true;
}

bool UAbstractSkeletonLabelCollection::RemoveLabel(const FName InLabel)
{
	if (HasLabel(InLabel))
	{
		Labels.Remove(InLabel);
		return true;
	}
	else
	{
		return false;
	}
}

TArray<FName>& UAbstractSkeletonLabelCollection::GetMutableLabels()
{
	return Labels;
}

#endif // WITH_EDITOR

bool UAbstractSkeletonLabelCollection::HasLabel(const FName InLabel) const
{
	return Labels.Contains(InLabel);
}

TConstArrayView<FName> UAbstractSkeletonLabelCollection::GetLabels() const
{
	return Labels;
}
