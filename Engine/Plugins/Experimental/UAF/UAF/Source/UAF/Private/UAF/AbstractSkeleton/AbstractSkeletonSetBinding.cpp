// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"

#include "Animation/AnimRootMotionProvider.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/Attributes/AttributeBindingDataCache.h"
#include "UAF/Attributes/EngineAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbstractSkeletonSetBinding)

void UAbstractSkeletonSetBinding::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	AddDefaultAttributes();
	FilterOrphanedBindings();
#endif

#if WITH_EDITOR
	if (Skeleton)
	{
		Skeleton->RegisterOnSkeletonHierarchyChanged(USkeleton::FOnSkeletonHierarchyChanged::CreateUObject(this, &UAbstractSkeletonSetBinding::OnSkeletonChanged));
		Skeleton->RegisterOnReferenceSkeletonChanged(USkeleton::FOnReferenceSkeletonChanged::CreateUObject(this, &UAbstractSkeletonSetBinding::OnSkeletonChanged));
	}

	if (SetCollection)
	{
		SetCollectionChangedHandle = SetCollection->RegisterOnSetsChanged(FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UAbstractSkeletonSetBinding::OnSetCollectionChanged));
	}
#endif
}

void UAbstractSkeletonSetBinding::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);

	if (Skeleton)
	{
		OutDeps.Add(Skeleton);
	}

	if (SetCollection)
	{
		OutDeps.Add(SetCollection);
	}
}

// Bone Bindings

bool UAbstractSkeletonSetBinding::AddBoneToSet(const FName BoneName, const FName SetName)
{
	if (!SetCollection)
	{
		return false;
	}

	if (!SetCollection->HasSet(SetName) && SetName != NAME_None)
	{
		return false; // Set does not exist
	}

	if (IsBoneInSet(BoneName))
	{
		return false; // Already exists in a set
	}

	Modify();

	FAbstractSkeleton_BoneBinding& Binding = BoneBindings.AddDefaulted_GetRef();
	Binding.BoneName = BoneName;
	Binding.SetName = SetName;
	OnBindingsChanged();

	return true;
}

bool UAbstractSkeletonSetBinding::RemoveBoneFromSet(const FName BoneName)
{
	const int32 BindingIndex = BoneBindings.IndexOfByPredicate([&](const FAbstractSkeleton_BoneBinding& Binding)
	{
		return Binding.BoneName == BoneName;
	});

	if (BindingIndex != INDEX_NONE)
	{
		Modify();

		BoneBindings.RemoveAtSwap(BindingIndex);
		OnBindingsChanged();
		return true;
	}
	else
	{
		return false;
	}
}

bool UAbstractSkeletonSetBinding::IsBoneInSet(const FName BoneName) const
{
	return BoneBindings.ContainsByPredicate([&](const FAbstractSkeleton_BoneBinding& Binding)
		{
			return Binding.BoneName == BoneName;
		});
}

bool UAbstractSkeletonSetBinding::IsBoneInSet(const FName BoneName, const FName SetName) const
{
	return BoneBindings.ContainsByPredicate([&](const FAbstractSkeleton_BoneBinding& Binding)
		{
			return Binding.BoneName == BoneName && Binding.SetName == SetName;
		});
}

FName UAbstractSkeletonSetBinding::GetBoneSet(const FName BoneName)
{
	const FAbstractSkeleton_BoneBinding* const Binding = BoneBindings.FindByPredicate([&](const FAbstractSkeleton_BoneBinding& Binding)
		{
			return Binding.BoneName == BoneName;
		});

	return Binding ? Binding->SetName : NAME_None;
}

const TConstArrayView<FAbstractSkeleton_BoneBinding> UAbstractSkeletonSetBinding::GetBoneBindings() const
{
	return BoneBindings;
}

// Attribute Bindings

void UAbstractSkeletonSetBinding::AddDefaultAttributes()
{
	if (!ContainsAttribute(UE::Anim::IAnimRootMotionProvider::AttributeName))
	{
		AddAttributeToSet(FAnimationAttributeIdentifier(UE::Anim::IAnimRootMotionProvider::AttributeName, INDEX_NONE, NAME_None, FTransformAnimationAttribute::StaticStruct()), NAME_None);
	}
}

bool UAbstractSkeletonSetBinding::AddAttributeToSet(const FAnimationAttributeIdentifier Attribute, const FName SetName)
{
	if (IsAttributeInSet(Attribute))
	{
		return false; // Already exists in a named set
	}

	if (SetName != NAME_None)
	{
		if (!SetCollection || !SetCollection->HasSet(SetName))
		{
			return false; // Set does not exist
		}

		if (IsAttributeInSet(Attribute, NAME_None))
		{
			// Our attribute existed in the everything set, update it
			FAbstractSkeleton_AttributeBinding* UnboundAttribute = AttributeBindings.FindByPredicate([Attribute, SetName](const FAbstractSkeleton_AttributeBinding& Binding)
				{
					return Binding.Attribute == Attribute && Binding.SetName == NAME_None;
				});

			if (ensure(UnboundAttribute))
			{
				Modify();

				UnboundAttribute->SetName = SetName;
				OnBindingsChanged();
				return true;
			}
			else
			{
				return false;
			}
		}
	}

	if (IsAttributeInSet(Attribute, NAME_None))
	{
		return false;	// Already exists in the everything set
	}

	Modify();

	FAbstractSkeleton_AttributeBinding& Binding = AttributeBindings.AddDefaulted_GetRef();
	Binding.Attribute = Attribute;
	Binding.SetName = SetName;
	OnBindingsChanged();

	return true;
}

bool UAbstractSkeletonSetBinding::RemoveAttributeFromSet(const FAnimationAttributeIdentifier Attribute)
{
	const int32 BindingIndex = AttributeBindings.IndexOfByPredicate([&](const FAbstractSkeleton_AttributeBinding& Binding)
	{
		return Binding.Attribute == Attribute;
	});

	if (BindingIndex != INDEX_NONE)
	{
		Modify();

		FAbstractSkeleton_AttributeBinding& Binding = AttributeBindings[BindingIndex];

		if (Binding.SetName.IsNone())
		{
			AttributeBindings.RemoveAtSwap(BindingIndex);
		}
		else
		{
			Binding.SetName = NAME_None;
		}

		OnBindingsChanged();
		return true;
	}
	else
	{
		return false;
	}
}

bool UAbstractSkeletonSetBinding::RemoveAllFromSet(const FName SetName)
{
	Modify();

	int32 bCountRemoved = 0;

	bCountRemoved += BoneBindings.RemoveAll([&](const FAbstractSkeleton_BoneBinding& Binding)
		{
			return Binding.SetName == SetName;	
		});

	bCountRemoved += AttributeBindings.RemoveAll([&](const FAbstractSkeleton_AttributeBinding& Binding)
		{
			return Binding.SetName == SetName;
		});

	if (bCountRemoved > 0)
	{
		OnBindingsChanged();
		return true;
	}

	return false;
}

[[nodiscard]] bool UAbstractSkeletonSetBinding::ContainsAttribute(const FName AttributeName) const
{
	return AttributeBindings.ContainsByPredicate([AttributeName](const FAbstractSkeleton_AttributeBinding& Binding)
	{
		return Binding.Attribute.GetName() == AttributeName;
	});
}

bool UAbstractSkeletonSetBinding::IsAttributeInSet(const FAnimationAttributeIdentifier Attribute) const
{
	return AttributeBindings.ContainsByPredicate([&](const FAbstractSkeleton_AttributeBinding& Binding)
		{
			return Binding.Attribute.GetName() == Attribute.GetName() && Binding.Attribute.GetBoneName() == Attribute.GetBoneName() && Binding.SetName != NAME_None;
		});
}

bool UAbstractSkeletonSetBinding::IsAttributeInSet(const FAnimationAttributeIdentifier Attribute, const FName SetName) const
{
	return AttributeBindings.ContainsByPredicate([&](const FAbstractSkeleton_AttributeBinding& Binding)
		{
			return Binding.Attribute.GetName() == Attribute.GetName() && Binding.Attribute.GetBoneName() == Attribute.GetBoneName() && Binding.SetName == SetName;
		});
}

const TConstArrayView<FAbstractSkeleton_AttributeBinding> UAbstractSkeletonSetBinding::GetAttributeBindings() const
{
	return AttributeBindings;
}

void UAbstractSkeletonSetBinding::OnBindingsChanged()
{
	RebuildBindingData();

#if WITH_EDITOR
	// Notify external listeners a change has been made
	OnBindingsChangedDelegate.Broadcast();
#endif
}

void UAbstractSkeletonSetBinding::RebuildBindingData()
{
	UE::UAF::GAttributeBindingDataCache.ResetSetBinding(this);
}

#if WITH_EDITOR

void UAbstractSkeletonSetBinding::FilterOrphanedBindings()
{
	TSet<FName> ValidSetNames;

	// While 'None' is not a valid set name, it is used internally
	// to contain known unbound attributes
	ValidSetNames.Add(NAME_None);

	if (SetCollection)
	{
		for (const FAbstractSkeletonSet& Set : SetCollection->GetSetHierarchy())
		{
			ValidSetNames.Add(Set.SetName);
		}
	}

	int32 CountRemoved = 0;

	CountRemoved += BoneBindings.RemoveAll([&](FAbstractSkeleton_BoneBinding& Binding)
		{
			return !ValidSetNames.Contains(Binding.SetName);
		});

	CountRemoved += AttributeBindings.RemoveAll([&](FAbstractSkeleton_AttributeBinding& Binding)
		{
			return !ValidSetNames.Contains(Binding.SetName);
		});

	if (CountRemoved > 0)
	{
		OnBindingsChanged();
		UE_LOGF(LogAnimation, Warning, "Removed %d bindings whilst loading '%ls' due to those sets no longer existing in Set Collection '%ls'",
			CountRemoved,
			*GetFullName(),
			SetCollection ? *SetCollection->GetFullName() : TEXT("None"));
	}
}

#endif // WITH_EDITOR

bool UAbstractSkeletonSetBinding::SetSkeleton(const TObjectPtr<USkeleton> InSkeleton)
{
	if (Skeleton == InSkeleton)
	{
		// Ignore calls to set identical data
		return true;
	}

#if WITH_EDITOR
	if (Skeleton)
	{
		Skeleton->UnregisterOnSkeletonHierarchyChanged(this);
		Skeleton->UnregisterOnReferenceSkeletonChanged(this);
	}
#endif

	Skeleton = InSkeleton;
	BoneBindings.Empty();
	AttributeBindings.Empty();

	OnBindingsChanged();

#if WITH_EDITOR
	if (InSkeleton)
	{
		InSkeleton->RegisterOnSkeletonHierarchyChanged(USkeleton::FOnSkeletonHierarchyChanged::CreateUObject(this, &UAbstractSkeletonSetBinding::OnSkeletonChanged));
		InSkeleton->RegisterOnReferenceSkeletonChanged(USkeleton::FOnReferenceSkeletonChanged::CreateUObject(this, &UAbstractSkeletonSetBinding::OnSkeletonChanged));
	}
#endif

	return true;
}

TObjectPtr<USkeleton> UAbstractSkeletonSetBinding::GetSkeleton() const
{
	return Skeleton;
}

bool UAbstractSkeletonSetBinding::SetSetCollection(const TObjectPtr<UAbstractSkeletonSetCollection> InSetCollection)
{
	if (SetCollection == InSetCollection)
	{
		// Ignore calls to set identical data
		return true;
	}

#if WITH_EDITOR
	if (SetCollection)
	{
		SetCollection->UnregisterOnSetsChanged(SetCollectionChangedHandle);
	}
#endif

	SetCollection = InSetCollection;
	BoneBindings.Empty();
	AttributeBindings.Empty();

	OnBindingsChanged();

#if WITH_EDITOR
	if (InSetCollection)
	{
		SetCollectionChangedHandle = InSetCollection->RegisterOnSetsChanged(FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UAbstractSkeletonSetBinding::OnSetCollectionChanged));
	}
#endif

	return true;
}

TObjectPtr<const UAbstractSkeletonSetCollection> UAbstractSkeletonSetBinding::GetSetCollection() const
{
	return SetCollection;
}

#if WITH_EDITOR

void UAbstractSkeletonSetBinding::OnSkeletonChanged()
{
	RebuildBindingData();
}

void UAbstractSkeletonSetBinding::OnSetCollectionChanged()
{
	FilterOrphanedBindings();
	RebuildBindingData();
}

FDelegateHandle UAbstractSkeletonSetBinding::RegisterOnBindingsChanged(FSimpleMulticastDelegate::FDelegate&& InDelegate)
{
	return OnBindingsChangedDelegate.Add(InDelegate);
}

bool UAbstractSkeletonSetBinding::UnregisterOnBindingsChanged(const FDelegateHandle InHandle)
{
	return OnBindingsChangedDelegate.Remove(InHandle);
}

#endif // WITH_EDITOR
