// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/AbstractSkeleton/Utils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbstractSkeletonSetCollection)

bool UAbstractSkeletonSetCollection::AddSet(const FName InSetName, const FName InParentSetName)
{
	if (!IsValidSetName(InSetName))
	{
		return false;
	}

	if (HasSet(InSetName))
	{
		return false; // Set with this name already exists
	}

	FAbstractSkeletonSet& NewSet = SetHierarchy.AddDefaulted_GetRef();
	NewSet.SetName = InSetName;
	NewSet.ParentSetName = InParentSetName;

	SortSetHierarchy();

#if WITH_EDITOR
	OnSetsChanged.Broadcast();
#endif

	return true;
}

bool UAbstractSkeletonSetCollection::HasSet(const FName InSetName) const
{
	return GetSetIndex(InSetName) != INDEX_NONE;
}

bool UAbstractSkeletonSetCollection::RenameSet(const FName OldSetName, const FName NewSetName)
{
	if (!IsValidSetName(NewSetName))
	{
		return false;
	}

	if (HasSet(NewSetName))
	{
		return false; // Set with this name already exists
	}

	for (FAbstractSkeletonSet& Set : SetHierarchy)
	{
		if (Set.SetName == OldSetName)
		{
			Set.SetName = NewSetName;
		}
		else if (Set.ParentSetName == OldSetName)
		{
			Set.ParentSetName = NewSetName;
		}
	}

#if WITH_EDITOR
	OnSetsChanged.Broadcast();
#endif

	return true;
}

bool UAbstractSkeletonSetCollection::RemoveSet(const FName SetName)
{
	if (SetName == NAME_None )
	{
		return false;
	}

	const int32 SetIndex = GetSetIndex(SetName);
	if (SetIndex == INDEX_NONE)
	{
		return false;
	}

	const FName SetToRemoveParentSetName = GetParentSetName(SetName);

	// Reparent all children of the set we're remove
	for (FAbstractSkeletonSet& Set : SetHierarchy)
	{
		if (Set.ParentSetName == SetName)
		{
			Set.ParentSetName = SetToRemoveParentSetName;
		}
	}

	SetHierarchy.RemoveAt(SetIndex);

#if WITH_EDITOR
	OnSetsChanged.Broadcast();
#endif

	return true;
}

bool UAbstractSkeletonSetCollection::ReparentSet(const FName SetName, const FName NewParentName)
{
	if (SetName == NewParentName || IsDescendantOf(NewParentName, SetName))
	{
		return false;
	}

	const int32 SetIndex = GetSetIndex(SetName);
	if (SetIndex != INDEX_NONE)
	{
		SetHierarchy[SetIndex].ParentSetName = NewParentName;
		SortSetHierarchy();

#if WITH_EDITOR
		OnSetsChanged.Broadcast();
#endif

		return true;
	}

	return false;
}

FName UAbstractSkeletonSetCollection::GetParentSetName(const FName SetName) const
{
	for (const FAbstractSkeletonSet& Set : SetHierarchy)
	{
		if (Set.SetName == SetName)
		{
			return Set.ParentSetName;
		}
	}

	return NAME_None;
}

const TArrayView<const FAbstractSkeletonSet> UAbstractSkeletonSetCollection::GetSetHierarchy() const
{
	return SetHierarchy;
}

bool UAbstractSkeletonSetCollection::IsValidSetName(const FName SetName) const
{
	return !SetName.IsNone();
}

int32 UAbstractSkeletonSetCollection::GetSetIndex(const FName SetName) const
{
	for (int32 SetIndex = 0; SetIndex < SetHierarchy.Num(); ++SetIndex)
	{
		const FAbstractSkeletonSet Set = SetHierarchy[SetIndex];

		if (Set.SetName == SetName)
		{
			return SetIndex;
		}
	}

	return INDEX_NONE;
}

bool UAbstractSkeletonSetCollection::IsDescendantOf(const FName SetName, const FName AncestorSetName) const
{
	FName ParentSetName = GetParentSetName(SetName);
	
	while (ParentSetName != NAME_None)
	{
		if (ParentSetName == AncestorSetName)
		{
			return true;
		}
		ParentSetName = GetParentSetName(ParentSetName);
	}

	return false;
}

void UAbstractSkeletonSetCollection::SortSetHierarchy()
{
	TFunction<FAbstractSkeletonSet* (FName)> Functor = [this](FName InAttributeName)
		{
			return &GetSetRef(InAttributeName);
		};

	UE::UAF::Util::SortByProperty<FAbstractSkeletonSet>(SetHierarchy, &FAbstractSkeletonSet::SetName, &FAbstractSkeletonSet::ParentSetName, Functor);
}

FAbstractSkeletonSet& UAbstractSkeletonSetCollection::GetSetRef(const FName SetName)
{
	const int32 SetIndex = GetSetIndex(SetName);
	check(SetIndex != INDEX_NONE)
	return SetHierarchy[SetIndex];
}

#if WITH_EDITOR

FDelegateHandle UAbstractSkeletonSetCollection::RegisterOnSetsChanged(FSimpleMulticastDelegate::FDelegate&& InDelegate)
{
	return OnSetsChanged.Add(InDelegate);
}

bool UAbstractSkeletonSetCollection::UnregisterOnSetsChanged(const FDelegateHandle InHandle)
{
	return OnSetsChanged.Remove(InHandle);
}

#endif // WITH_EDITOR