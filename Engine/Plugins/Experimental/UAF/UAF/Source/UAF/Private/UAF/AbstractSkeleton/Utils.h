// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Queue.h"

namespace UE::UAF::Util
{
	template <typename ArrayType>
	void SortByProperty(TArray<ArrayType>& Array, FName ArrayType::* NameProperty, FName ArrayType::* ParentNameProperty, TFunctionRef<ArrayType* (FName)> FindArrayItem)
	{
		struct Node
		{
			FName AttributeName;
			TArray<FName> Children;
			int32 InDegree;

			Node(const FName InAttributeName)
				: AttributeName(InAttributeName)
				, InDegree(0)
			{
			}
		};

		TMap<FName, Node> Graph;

		for (const ArrayType& ArrayItem : Array)
		{
			Graph.Add(ArrayItem.*NameProperty, Node(ArrayItem.*NameProperty));
		}

		for (const ArrayType& ArrayItem : Array)
		{
			if (ArrayItem.*ParentNameProperty != NAME_None)
			{
				Graph[ArrayItem.*ParentNameProperty].Children.Add(ArrayItem.*NameProperty);
				++Graph[ArrayItem.*NameProperty].InDegree;
			}
		}

		TQueue<FName> NodeQueue;
		for (const ArrayType& Item : Array)
		{
			const Node& ItemNode = Graph[Item.*NameProperty];
			if (ItemNode.InDegree == 0)
			{
				NodeQueue.Enqueue(ItemNode.AttributeName);
			}
		}

		TArray<ArrayType> SortedArray;

		while (!NodeQueue.IsEmpty())
		{
			FName AttributeName;
			NodeQueue.Dequeue(AttributeName);

			if (AttributeName != NAME_None)
			{
				SortedArray.Add(*FindArrayItem(AttributeName));
			}

			for (const FName& Child : Graph[AttributeName].Children)
			{
				--Graph[Child].InDegree;
				if (Graph[Child].InDegree == 0)
				{
					NodeQueue.Enqueue(Child);
				}
			}
		}

		ensureMsgf(Array.Num() == SortedArray.Num(), TEXT("Cycle found in the DAG"));
		Array = SortedArray;
	}
}