// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Attributes/AttributeBindingData.h"

#include "Animation/BuiltInAttributeTypes.h"
#include "Engine/SkeletalMesh.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/Attributes/EngineAttributes.h"

namespace UE::UAF
{
	namespace Private
	{
		// A partition of attributes for each type within our named set
		struct FNamedSetPartition
		{
			FName Name;

			TMap<UScriptStruct*, TArray<FAttributeDescription>> AttributesPerType;
		};

		// A partition for each named set at our specific LOD
		struct FLODPartition
		{
			int32 LOD = 0;

			TMap<FName, FNamedSetPartition> NamedSets;
		};

		static TArray<FName> CollectSetNames(const UAbstractSkeletonSetBinding* Binding)
		{
			TArray<FName> SetNames;

			// We always have a special set that contains everything, the everything-set has no name
			SetNames.Add(NAME_None);

			if (const UAbstractSkeletonSetCollection* SetCollection = Binding->GetSetCollection())
			{
				const int32 NumCollectionSets = SetCollection->GetSetHierarchy().Num();
				SetNames.Reserve(NumCollectionSets + 1);

				for (const FAbstractSkeletonSet& Set : SetCollection->GetSetHierarchy())
				{
					ensure(!Set.SetName.IsNone()); // The set collection should maintain this invariant
					SetNames.AddUnique(Set.SetName);
				}
			}

			return SetNames;
		}

		static TArray<UScriptStruct*> CollectAttributeTypes(const FReferenceSkeleton& RefSkeleton, const TConstArrayView<FAbstractSkeleton_AttributeBinding>& AttributeBindings)
		{
			TArray<UScriptStruct*> AttributeTypes;

			// If we have any bones, then we'll need the type for the everything set (and others)
			if (RefSkeleton.GetNum() > 0)
			{
				AttributeTypes.AddUnique(FBoneTransformAnimationAttribute::StaticStruct());
			}

			for (const FAbstractSkeleton_AttributeBinding& AttributeBinding : AttributeBindings)
			{
				AttributeTypes.AddUnique(AttributeBinding.Attribute.GetType());
			}

			return AttributeTypes;
		}

		static void AddAttribute(const FAttributeDescription& Attribute, FName SetName, const UAbstractSkeletonSetCollection* SetCollection, TArray<FLODPartition>& LODPartitions)
		{
			// TODO: We shouldn't add to every higher LOD to allow them to be disjoint meaning attributes should explicitly repeat in every LOD we want them in
			int32 LODIndex = Attribute.LOD;

			// Add our attribute to its LOD and higher LODs (e.g. something at LOD 2 is also present at LOD 0 and 1
			while (LODIndex >= 0)
			{
				FName CurrentSetName = SetName;
				FLODPartition& LODPartition = LODPartitions[LODIndex];

				// Add our attribute to its set and supersets
				do
				{
					FNamedSetPartition& SetPartition = LODPartition.NamedSets.FindChecked(CurrentSetName);

					TArray<FAttributeDescription>& Attributes = SetPartition.AttributesPerType.FindChecked(Attribute.Type);
					Attributes.Add(Attribute);

					if (SetCollection != nullptr)
					{
						// Add our attribute to the parent sets as well
						CurrentSetName = SetCollection->GetParentSetName(CurrentSetName);
					}
					else
					{
						CurrentSetName = NAME_None;
					}
				}
				while (!CurrentSetName.IsNone());

				// Move to our next LOD
				LODIndex--;
			}
		}

		// Returns a partition per LOD
		static TArray<FLODPartition> PartitionAttributes(TNonNullPtr<const UAbstractSkeletonSetBinding> Binding, const FReferenceSkeleton& RefSkeleton, const TArray<FName>& SetNames, const TArray<UScriptStruct*>& AttributeTypes)
		{
			const UAbstractSkeletonSetCollection* SetCollection = Binding->GetSetCollection();

			const TConstArrayView<FAbstractSkeleton_BoneBinding> BoneBindings = Binding->GetBoneBindings();
			const TConstArrayView<FAbstractSkeleton_AttributeBinding> AttributeBindings = Binding->GetAttributeBindings();

			// TODO: Hook up LODs
			const int32 NumLODs = 1;

			TArray<FLODPartition> LODPartitions;
			LODPartitions.AddDefaulted(NumLODs);

			// Init our partitions
			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				FLODPartition& LODPartition = LODPartitions[LODIndex];

				LODPartition.LOD = LODIndex;
				LODPartition.NamedSets.Reserve(SetNames.Num());

				for (const FName& SetName : SetNames)
				{
					FNamedSetPartition& NamedSetPartition = LODPartition.NamedSets.Add(SetName);

					NamedSetPartition.Name = SetName;
					NamedSetPartition.AttributesPerType.Reserve(AttributeTypes.Num());

					for (UScriptStruct* AttributeType : AttributeTypes)
					{
						NamedSetPartition.AttributesPerType.Add(AttributeType);
					}
				}
			}

			UScriptStruct* BoneTransformType = FBoneTransformAnimationAttribute::StaticStruct();

			for (const FAbstractSkeleton_BoneBinding& BoneBinding : BoneBindings)
			{
				const int32 SkeletonBoneIndex = RefSkeleton.FindBoneIndex(BoneBinding.BoneName);
				if (SkeletonBoneIndex == INDEX_NONE)
				{
					// Unknown bone
					continue;
				}

				const int32 ParentSkeletonBoneIndex = RefSkeleton.GetParentIndex(SkeletonBoneIndex);
				const FName ParentBoneName = ParentSkeletonBoneIndex != INDEX_NONE ? RefSkeleton.GetBoneName(ParentSkeletonBoneIndex) : NAME_None;

				FAttributeDescription Attribute;
				Attribute.Name = BoneBinding.BoneName;
				Attribute.ParentName = ParentBoneName;
				Attribute.Type = BoneTransformType;

				// TODO: Hook up LODs
				Attribute.LOD = 0;

				// Add the attribute to its set if it has one
				if (!BoneBinding.SetName.IsNone())
				{
					AddAttribute(Attribute, BoneBinding.SetName, SetCollection, LODPartitions);
				}
			}

			for (const FAbstractSkeleton_AttributeBinding& AttributeBinding : AttributeBindings)
			{
				FAttributeDescription Attribute;
				Attribute.Name = AttributeBinding.Attribute.GetName();
				Attribute.ParentName = NAME_None;
				Attribute.Type = AttributeBinding.Attribute.GetType();

				// TODO: Hook up LODs
				Attribute.LOD = 0;

				// Add the attribute to its set if it has one
				if (!AttributeBinding.SetName.IsNone())
				{
					AddAttribute(Attribute, AttributeBinding.SetName, SetCollection, LODPartitions);
				}

				// Add the attribute to the everything set
				AddAttribute(Attribute, NAME_None, SetCollection, LODPartitions);
			}

			// Add all our bones to the everything set
			const int32 NumBones = RefSkeleton.GetNum();
			for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < NumBones; ++SkeletonBoneIndex)
			{
				const int32 ParentSkeletonBoneIndex = RefSkeleton.GetParentIndex(SkeletonBoneIndex);
				const FName ParentBoneName = ParentSkeletonBoneIndex != INDEX_NONE ? RefSkeleton.GetBoneName(ParentSkeletonBoneIndex) : NAME_None;

				FAttributeDescription Attribute;
				Attribute.Name = RefSkeleton.GetBoneName(SkeletonBoneIndex);
				Attribute.ParentName = ParentBoneName;
				Attribute.Type = BoneTransformType;

				// TODO: Hook up LODs
				Attribute.LOD = 0;

				AddAttribute(Attribute, NAME_None, SetCollection, LODPartitions);
			}

			return LODPartitions;
		}
	}

	FAttributeBindingData::FAttributeBindingData(TNonNullPtr<const UAbstractSkeletonSetBinding> Binding, const USkeletalMesh* SkeletalMesh)
	{
		using namespace Private;

		// Add a reference right away, this will keep us alive while we destroy temporary pointers
		// This reference is our external one that is creating this instance
		AddRef();

		const USkeleton* Skeleton = Binding->GetSkeleton();
		if (!ensure(Skeleton)) // Considering the case where a user creates a binding for a skeleton and then the skeleton is deleted, maybe need a Binding->IsValid()? :/
		{
			return;
		}

		const FReferenceSkeleton& SkeletonRefSkeleton = Skeleton->GetReferenceSkeleton();
		const FReferenceSkeleton* SkeletalMeshRefSkeleton = SkeletalMesh ? &SkeletalMesh->GetRefSkeleton() : nullptr;

		const TConstArrayView<FAbstractSkeleton_BoneBinding> BoneBindings = Binding->GetBoneBindings();
		const TConstArrayView<FAbstractSkeleton_AttributeBinding> AttributeBindings = Binding->GetAttributeBindings();


		UScriptStruct* BoneTransformType = FBoneTransformAnimationAttribute::StaticStruct();

		// Collect every set names and attribute type
		const TArray<FName> SetNames = CollectSetNames(Binding);
		const TArray<UScriptStruct*> AttributeTypes = CollectAttributeTypes(SkeletonRefSkeleton, AttributeBindings);

		const TArray<FLODPartition> LODPartitions = PartitionAttributes(Binding, SkeletonRefSkeleton, SetNames, AttributeTypes);

		const int32 NumLODs = LODPartitions.Num();
		const int32 NumNamedSets = SetNames.Num();
		const int32 NumAttributeTypes = AttributeTypes.Num();

		NamedSets.Reserve(NumNamedSets * NumLODs);
		NamedSets.AddZeroed(NumNamedSets * NumLODs);
		TypedSets.Reserve(NumNamedSets * NumLODs * NumAttributeTypes);
		TypedSets.AddZeroed(NumNamedSets * NumLODs * NumAttributeTypes);
		NamedSetMap.Reserve(NumNamedSets);

		{
			AttributeNamesByType.Reserve(NumAttributeTypes);

			{
				const TArray<FMeshBoneInfo>& BoneInfoList = SkeletonRefSkeleton.GetRefBoneInfo();
				const int32 NumSkeletonBones = BoneInfoList.Num();

				TArray<FName>& BoneAttributeNames = AttributeNamesByType.Add(BoneTransformType);
				BoneAttributeNames.Reserve(NumSkeletonBones);
				BoneAttributeNames.AddUninitialized(NumSkeletonBones);

				for (int32 SkeletonBoneIndex = 0; SkeletonBoneIndex < NumSkeletonBones; ++SkeletonBoneIndex)
				{
					BoneAttributeNames[SkeletonBoneIndex] = BoneInfoList[SkeletonBoneIndex].Name;
				}
			}

			for (const FAbstractSkeleton_AttributeBinding& AttributeBinding : AttributeBindings)
			{
				TArray<FName>& AttributeNames = AttributeNamesByType.FindOrAdd(AttributeBinding.Attribute.GetType());
				AttributeNames.Add(AttributeBinding.Attribute.GetName());
			}

			// Sort our attribute names
			for (UScriptStruct* AttributeType : AttributeTypes)
			{
				if (AttributeType == BoneTransformType)
				{
					// Bone transform attribute names aren't sorted
					continue;
				}

				TArray<FName>& AttributeNames = AttributeNamesByType.FindChecked(AttributeType);
				AttributeNames.Sort(FNameFastLess());
				AttributeNames.Shrink();
			}
		}

		for (int32 NamedSetIndex = 0; NamedSetIndex < NumNamedSets; ++NamedSetIndex)
		{
			const FName SetName = SetNames[NamedSetIndex];

			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				// Named sets are sorted by name, then by LOD
				FAttributeNamedSet& NamedSet = NamedSets[(NamedSetIndex * NumLODs) + LODIndex];

				NamedSet.Name = SetName;
				NamedSet.LOD = LODIndex;
				NamedSet.SetNumLODs = NumLODs;
				NamedSet.Owner = this;
				NamedSet.TypedSetMap.Reserve(NumAttributeTypes);

				for (int32 AttributeTypeIndex = 0; AttributeTypeIndex < NumAttributeTypes; ++AttributeTypeIndex)
				{
					UScriptStruct* AttributeType = AttributeTypes[AttributeTypeIndex];

					// Typed sets are sorted by named set, by type, then by LOD
					FAttributeTypedSet& TypedSet = TypedSets[(NamedSetIndex * NumAttributeTypes * NumLODs) + (AttributeTypeIndex * NumLODs) + LODIndex];

					TypedSet.Type = AttributeType;
					TypedSet.Owner = this;
					TypedSet.NamedSet = &NamedSet;

					const TArray<FAttributeDescription>& Attributes = LODPartitions[LODIndex].NamedSets[SetName].AttributesPerType[AttributeType];
					const TArray<FName>& AttributeNames = AttributeNamesByType[AttributeType];
					TypedSet.Init(Attributes, AttributeNames);

					// TODO: Order is the same for these in every named set, can we optimize?
					NamedSet.TypedSetMap.Add(AttributeType, &TypedSet);
				}

				if (LODIndex == 0)
				{
					NamedSetMap.Add(SetName, &NamedSet);
				}
			}
		}

		{
			// TODO: Should we cache all the user authored attributes within the same collection per named set? e.g. translation retarget
			// This would avoid having 1 collection per authored type and reduce waste

			const TArray<FTransform>& RefPose = SkeletalMeshRefSkeleton
				? SkeletalMeshRefSkeleton->GetRefBonePose()
				: SkeletonRefSkeleton.GetRefBonePose();

			for (auto NamedSetIt = CreateNamedSetIterator(); NamedSetIt; ++NamedSetIt)
			{
				FAttributeNamedSetPtr NamedSet = *NamedSetIt;
				const FName SetName = NamedSet->GetName();

				FPoseValueBundlePtr DefaultAttributeValues = MakeShared<FPoseValueBundleHeap>(NamedSet);
				DefaultAttributeValues->SetValueSpace(FValueSpace(EValueSpaceType::Local));

				for (auto TypedSetIt = NamedSet->CreateTypedSetIterator(); TypedSetIt; ++TypedSetIt)
				{
					// TODO: This should be replaced with the default values defined per-attribute in the set binding
					// instead of hard-coding them here

					if (TypedSetIt->GetType() == FTransformAnimationAttribute::StaticStruct())
					{
						FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFrom(TypedSetIt->GetType()).To<FTransformAnimationAttribute>();
						TBoundValueMap<FTransformAnimationAttribute>* DefaultValues = DefaultAttributeValues->GetBoundValueMaps().Add<FTransformAnimationAttribute>(MappingKey);

						const int32 NumValues = DefaultValues->Num();

						for (FAttributeSetIndex ValueIndex(0); ValueIndex < NumValues; ++ValueIndex)
						{
							(*DefaultValues)[ValueIndex] = FTransformAnimationAttribute{ FTransform::Identity };
						}
					}
					else if (TypedSetIt->GetType() == FFloatAnimationAttribute::StaticStruct())
					{
						FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFrom(TypedSetIt->GetType()).To<FFloatAnimationAttribute>();
						TBoundValueMap<FFloatAnimationAttribute>* DefaultValues = DefaultAttributeValues->GetBoundValueMaps().Add<FFloatAnimationAttribute>(MappingKey);

						const int32 NumValues = DefaultValues->Num();

						for (FAttributeSetIndex ValueIndex(0); ValueIndex < NumValues; ++ValueIndex)
						{
							(*DefaultValues)[ValueIndex] = FFloatAnimationAttribute{ 0.0f };
						}
					}
					else if (TypedSetIt->GetType() == FIntegerAnimationAttribute::StaticStruct())
					{
						FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFrom(TypedSetIt->GetType()).To<FIntegerAnimationAttribute>();
						TBoundValueMap<FIntegerAnimationAttribute>* DefaultValues = DefaultAttributeValues->GetBoundValueMaps().Add<FIntegerAnimationAttribute>(MappingKey);

						const int32 NumValues = DefaultValues->Num();

						for (FAttributeSetIndex ValueIndex(0); ValueIndex < NumValues; ++ValueIndex)
						{
							(*DefaultValues)[ValueIndex] = FIntegerAnimationAttribute{ 0 };
						}
					}
					else if (TypedSetIt->GetType() == FVectorAnimationAttribute::StaticStruct())
					{
						FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFrom(TypedSetIt->GetType()).To<FVectorAnimationAttribute>();
						TBoundValueMap<FVectorAnimationAttribute>* DefaultValues = DefaultAttributeValues->GetBoundValueMaps().Add<FVectorAnimationAttribute>(MappingKey);

						const int32 NumValues = DefaultValues->Num();

						for (FAttributeSetIndex ValueIndex(0); ValueIndex < NumValues; ++ValueIndex)
						{
							(*DefaultValues)[ValueIndex] = FVectorAnimationAttribute{ FVector::ZeroVector };
						}
					}
					else if (TypedSetIt->GetType() == FQuaternionAnimationAttribute::StaticStruct())
					{
						FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFrom(TypedSetIt->GetType()).To<FQuaternionAnimationAttribute>();
						TBoundValueMap<FQuaternionAnimationAttribute>* DefaultValues = DefaultAttributeValues->GetBoundValueMaps().Add<FQuaternionAnimationAttribute>(MappingKey);

						const int32 NumValues = DefaultValues->Num();

						for (FAttributeSetIndex ValueIndex(0); ValueIndex < NumValues; ++ValueIndex)
						{
							(*DefaultValues)[ValueIndex] = FQuaternionAnimationAttribute{ FQuat::Identity };
						}
					}
					else if (TypedSetIt->GetType() == FBoneTransformAnimationAttribute::StaticStruct())
					{
						FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFrom(TypedSetIt->GetType()).To<FBoneTransformAnimationAttribute>();
						TBoundValueMap<FBoneTransformAnimationAttribute>* DefaultValues = DefaultAttributeValues->GetBoundValueMaps().Add<FBoneTransformAnimationAttribute>(MappingKey);

						const int32 NumValues = DefaultValues->Num();

						// Set every value within our map
						if (SkeletalMeshRefSkeleton)
						{
							for (FAttributeSetIndex ValueIndex(0); ValueIndex < NumValues; ++ValueIndex)
							{
								FAttributeBindingIndex BindingIndex = TypedSetIt->GetBindingIndex(ValueIndex);
								FSkeletonPoseBoneIndex SkeletonBoneIndex = BindingIndex.GetSkeletonBoneIndex();

								FName BoneName = SkeletonRefSkeleton.GetBoneName(SkeletonBoneIndex.GetInt());
								int32 MeshBoneIndex = SkeletalMeshRefSkeleton->FindBoneIndex(BoneName);

								SkeletonToMeshIndicesMap.Add(SkeletonBoneIndex, FMeshPoseBoneIndex(MeshBoneIndex));

								if (MeshBoneIndex != INDEX_NONE)
								{
									(*DefaultValues)[ValueIndex] = FBoneTransformAnimationAttribute{ RefPose[MeshBoneIndex] };
								}
							}
						}
						else
						{
							for (FAttributeSetIndex ValueIndex(0); ValueIndex < NumValues; ++ValueIndex)
							{
								FAttributeBindingIndex BindingIndex = TypedSetIt->GetBindingIndex(ValueIndex);
								FSkeletonPoseBoneIndex SkeletonBoneIndex = BindingIndex.GetSkeletonBoneIndex();

								(*DefaultValues)[ValueIndex] = FBoneTransformAnimationAttribute{ RefPose[SkeletonBoneIndex.GetInt()] };
							}
						}

					}
					else
					{
						check(false); // Using a bound attribute without a default value being defined, accessing these attributes won't work
					}

					// Our set map has a reference to a typed set
					NumInternalReferences++;
				}

				DefaultAttributeValuesByName.Add(SetName, MoveTemp(DefaultAttributeValues));

				// Our collection has a reference to a named set
				NumInternalReferences++;
			}
		}

		BindingAsset = Binding;
	}

	int32 FAttributeBindingData::NumNamedSets() const
	{
		return NamedSetMap.Num();
	}

	int32 FAttributeBindingData::NumLODs() const
	{
		return NamedSetMap.IsEmpty() ? 0 : (NamedSets.Num() / NamedSetMap.Num());
	}

	bool FAttributeBindingData::IsEmpty() const
	{
		return NamedSetMap.IsEmpty();
	}

	int32 FAttributeBindingData::NumAttributes(UScriptStruct* AttributeType) const
	{
		if (const TArray<FName>* AttributeNames = AttributeNamesByType.Find(AttributeType))
		{
			return AttributeNames->Num();
		}

		// Attribute type not found
		return 0;
	}

	FAttributeNamedSetPtr FAttributeBindingData::FindNamedSet(FName SetName) const
	{
		if (FAttributeNamedSet* const * NamedSet = NamedSetMap.Find(SetName))
		{
			return FAttributeNamedSetPtr(*NamedSet, true);
		}

		return FAttributeNamedSetPtr();
	}

	FAttributeBindingIndex FAttributeBindingData::FindBindingIndex(FName AttributeName, UScriptStruct* AttributeType) const
	{
		if (const TArray<FName>* AttributeNames = AttributeNamesByType.Find(AttributeType))
		{
			// TODO: If we aren't a bone transform, we can do a binary search
			return FAttributeBindingIndex(AttributeNames->IndexOfByKey(AttributeName));
		}

		// Attribute not found
		return FAttributeBindingIndex();
	}

	FPoseValueBundlePtr FAttributeBindingData::GetDefaultAttributeValues(FName SetName) const
	{
		if (const FPoseValueBundlePtr* AttributeCollection = DefaultAttributeValuesByName.Find(SetName))
		{
			return *AttributeCollection;
		}

		// Set not found
		return FPoseValueBundlePtr();
	}

	uint32 FAttributeBindingData::Release() const
	{
		const uint32 RefsAfterRelease = UE::Private::FQueryableRefCountedObject::Release();
		if (RefsAfterRelease == NumInternalReferences)
		{
			// Our last external reference has been released, release our internal references to free the memory
			const_cast<FAttributeBindingData*>(this)->ResetInternalReferences();
		}

		return RefsAfterRelease;
	}

	void FAttributeBindingData::ResetInternalReferences()
	{
		// Add an extra reference to keep this container alive during destruction
		UE::Private::FQueryableRefCountedObject::AddRef();

		DefaultAttributeValuesByName.Empty();

		// We are done with destruction, destroy ourselves now
		UE::Private::FQueryableRefCountedObject::Release();
	}

	FAttributeBindingDataPtr MakeAttributeBindingData(TNonNullPtr<const UAbstractSkeletonSetBinding> Binding, const USkeletalMesh* SkeletalMesh)
	{
		const USkeleton* Skeleton = Binding->GetSkeleton();

		if (Skeleton == nullptr)
		{
			return FAttributeBindingDataPtr();
		}

		FAttributeBindingData* BindingData = new FAttributeBindingData(Binding, SkeletalMesh);

		// We already added the reference within the constructor
		const bool bAddRef = false;
		return FAttributeBindingDataPtr(BindingData, bAddRef);
	}
}
