// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Attributes/AttributeBindingDataCache.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/ScopeRWLock.h"

namespace UE::UAF
{
	FAutoConsoleCommand ClearCacheCommand(
		TEXT("UAF.ClearAttributeBindingCache"),
		TEXT("A debug command to clear the UAF attribute binding data cache"),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				GAttributeBindingDataCache.Reset();
			})
	);

	namespace Private
	{
		FSetBindingAttributeBindingDataCache::FSetBindingAttributeBindingDataCache(TNonNullPtr<const UAbstractSkeletonSetBinding> InSetBinding)
			: SkeletonBindingData(MakeAttributeBindingData(InSetBinding, nullptr))
			, WeakSetBinding(InSetBinding)
		{
		}

		FAttributeBindingDataPtr FSetBindingAttributeBindingDataCache::GetOrAdd(const USkeletalMesh* SkeletalMesh)
		{		
			if (!SkeletalMesh)
			{
				return SkeletonBindingData;
			}

			const UAbstractSkeletonSetBinding* SetBinding = WeakSetBinding.Get();
			check(SetBinding);
		
			check(SkeletalMesh->GetSkeleton() == SetBinding->GetSkeleton());

			{
				UE::TReadScopeLock Lock(CacheLock);
				if (FAttributeBindingDataPtr* CachedBindingData = SkeletalMeshBindingData.Find(SkeletalMesh))
				{
					return *CachedBindingData;
				}
			}

			FAttributeBindingDataPtr NewAttributeBindingData = MakeAttributeBindingData(SetBinding, SkeletalMesh);

			{
				UE::TWriteScopeLock Lock(CacheLock);

				// Recheck for update between read-lock released and write-lock obtained
				if (FAttributeBindingDataPtr* CachedBindingData = SkeletalMeshBindingData.Find(SkeletalMesh))
				{
					return *CachedBindingData;
				}

				return SkeletalMeshBindingData.Add(SkeletalMesh, MoveTemp(NewAttributeBindingData));
			}
		}
	}
	
	FAttributeBindingDataPtr FAttributeBindingDataCache::GetOrAdd(
		TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding,
		const USkeletalMesh* SkeletalMesh)
	{
		{
			FSetBindingAttributeBindingDataCachePtr CachedBindingData = nullptr;
			{
				UE::TReadScopeLock Lock(CacheLock);
				
				if (FSetBindingAttributeBindingDataCachePtr* CachedBindingDataPtr = SetBindingToCache.Find(SetBinding))
				{
					CachedBindingData = *CachedBindingDataPtr;
				}
			}

			if (CachedBindingData)
			{
				return CachedBindingData->GetOrAdd(SkeletalMesh);
			}
		}

		FSetBindingAttributeBindingDataCachePtr SetBindingCache = MakeShared<Private::FSetBindingAttributeBindingDataCache>(SetBinding);
		
		{
			UE::TWriteScopeLock Lock(CacheLock);

			// Recheck for update between read-lock released and write-lock obtained
			if (const FSetBindingAttributeBindingDataCachePtr* CachedBindingData = SetBindingToCache.Find(SetBinding))
			{
				SetBindingCache = *CachedBindingData;
			}
			else
			{
				SetBindingToCache.Add(SetBinding, SetBindingCache);
			}
		}

		return SetBindingCache->GetOrAdd(SkeletalMesh);
	}

	void FAttributeBindingDataCache::ResetSetBinding(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding)
	{
		// Early out without acquiring write access if there's nothing to clear
		{
			UE::TReadScopeLock Lock(CacheLock);
			if (!SetBindingToCache.Contains(SetBinding))
			{
				return;
			}
		}

		{
			UE::TWriteScopeLock Lock(CacheLock);
			SetBindingToCache.Remove(SetBinding);
		}
	}

	void FAttributeBindingDataCache::Reset()
	{
		UE::TWriteScopeLock Lock(CacheLock);
		SetBindingToCache.Reset();
	}
	
	FAttributeBindingDataCache GAttributeBindingDataCache;
}