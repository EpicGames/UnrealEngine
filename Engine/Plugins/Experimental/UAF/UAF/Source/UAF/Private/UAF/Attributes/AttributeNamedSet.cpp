// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Attributes/AttributeNamedSet.h"
#include "UAF/Attributes/AttributeBindingData.h"

namespace UE::UAF
{
	FAttributeNamedSet::FAttributeNamedSet() = default;

	FAttributeBindingDataPtr FAttributeNamedSet::GetBindingData() const
	{
		return FAttributeBindingDataPtr(Owner, true);
	}

	FName FAttributeNamedSet::GetName() const
	{
		return Name;
	}

	int32 FAttributeNamedSet::GetLOD() const
	{
		return LOD;
	}

	int32 FAttributeNamedSet::NumTypedSets() const
	{
		return TypedSetMap.Num();
	}

	int32 FAttributeNamedSet::NumLODs() const
	{
		return SetNumLODs;
	}

	bool FAttributeNamedSet::IsEmpty() const
	{
		return TypedSetMap.IsEmpty();
	}

	FAttributeTypedSetPtr FAttributeNamedSet::FindTypedSet(UScriptStruct* Type) const
	{
		if (FAttributeTypedSet* const* TypedSet = TypedSetMap.Find(Type))
		{
			return FAttributeTypedSetPtr(*TypedSet, true);
		}

		return FAttributeTypedSetPtr();
	}

	FAttributeNamedSetPtr FAttributeNamedSet::AtLOD(int32 InLOD) const
	{
		if (InLOD < 0 || InLOD >= SetNumLODs)
		{
			// Invalid LOD requested
			return FAttributeNamedSetPtr();
		}

		const FAttributeNamedSet* FirstLOD = this - LOD;
		const FAttributeNamedSet* DesiredLOD = FirstLOD + InLOD;

		return FAttributeNamedSetPtr(DesiredLOD, true);
	}

	FPoseValueBundlePtr FAttributeNamedSet::GetDefaultAttributeValues() const
	{
		return Owner->GetDefaultAttributeValues(Name);
	}

	void FAttributeNamedSet::AddRef() const
	{
		Owner->AddRef();
	}

	uint32 FAttributeNamedSet::Release() const
	{
		return Owner->Release();
	}

	uint32 FAttributeNamedSet::GetRefCount() const
	{
		return Owner->GetRefCount();
	}
}
