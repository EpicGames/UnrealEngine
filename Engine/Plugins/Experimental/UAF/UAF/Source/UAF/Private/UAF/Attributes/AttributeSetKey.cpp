// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Attributes/AttributeSetKey.h"

namespace UE::UAF
{
	FAttributeSetKey::FAttributeSetKey(FAttributeBindingIndex BindingIndex, FAttributeBindingIndex ParentBindingIndex, int32 LOD)
	{
		if (!BindingIndex.IsValid())
		{
			Value = 0;
			return;
		}

		// We convert the attribute binding index from 0-based to 1-based to allow for the value of 0 to be an invalid set key
		// Otherwise the first binding index (which might not have a parent) would have a value of 0 (at LOD 0)
		// We also know that the binding index is valid and so 65535 isn't used (our invalid index on 16 bits)
		const uint64 BindingIndex64 = BindingIndex.GetInt() + 1;

		// If the parent binding index is invalid, we want this attribute to come first before others with a parent
		// and so we give it a value of 0 (we sort smaller values first)
		// If the parent binding index is valid, we convert it from 0-based to 1-based
		const uint64 ParentBindingIndex64 = ParentBindingIndex.IsValid() ? (ParentBindingIndex.GetInt() + 1) : 0;
		const uint64 LOD64 = LOD;

		Value = (LOD64 << kLODShift) | (ParentBindingIndex64 << kParentIndexShift) | BindingIndex64;
	}
}
