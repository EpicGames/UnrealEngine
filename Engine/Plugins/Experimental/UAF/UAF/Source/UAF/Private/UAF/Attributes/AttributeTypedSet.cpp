// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Attributes/AttributeTypedSet.h"

#include "Algo/BinarySearch.h"
#include "UAF/Attributes/AttributeBindingData.h"

namespace UE::UAF
{
	FAttributeTypedSet::FAttributeTypedSet() = default;

	FAttributeNamedSetPtr FAttributeTypedSet::GetNamedSet() const
	{
		return FAttributeNamedSetPtr(NamedSet, true);
	}

	FAttributeBindingDataPtr FAttributeTypedSet::GetBindingData() const
	{
		return FAttributeBindingDataPtr(Owner, true);
	}

	UScriptStruct* FAttributeTypedSet::GetType() const
	{
		return Type;
	}

	int32 FAttributeTypedSet::GetLOD() const
	{
		return NamedSet->GetLOD();
	}

	int32 FAttributeTypedSet::Num() const
	{
		return AttributeKeys.Num();
	}

	int32 FAttributeTypedSet::NumLODs() const
	{
		return NamedSet->SetNumLODs;
	}

	bool FAttributeTypedSet::IsEmpty() const
	{
		return AttributeKeys.IsEmpty();
	}

	FAttributeTypedSetPtr FAttributeTypedSet::AtLOD(int32 LOD) const
	{
		if (LOD < 0 || LOD >= NamedSet->SetNumLODs)
		{
			// Invalid LOD requested
			return FAttributeTypedSetPtr();
		}

		const FAttributeTypedSet* FirstLOD = this - NamedSet->LOD;
		const FAttributeTypedSet* DesiredLOD = FirstLOD + LOD;

		return FAttributeTypedSetPtr(DesiredLOD, true);
	}

	FAttributeSetKey FAttributeTypedSet::FindKey(FName AttributeName) const
	{
		const int32 AttributeNameIndex = Algo::LowerBoundBy(AttributeNameToIndexMap, AttributeName, [](const TTuple<FName, uint16>& Entry) { return Entry.Key; }, FNameFastLess());
		if (AttributeNameIndex >= AttributeNameToIndexMap.Num())
		{
			// Attribute not found
			return FAttributeSetKey();
		}

		const TTuple<FName, uint16>& Entry = AttributeNameToIndexMap[AttributeNameIndex];
		if (Entry.Key != AttributeName)
		{
			// Attribute not found
			return FAttributeSetKey();
		}

		const int32 AttributeIndex = Entry.Value;
		if (AttributeIndex >= AttributeKeys.Num())
		{
			// Attribute not active in this LOD
			return FAttributeSetKey();
		}

		return AttributeKeys[AttributeIndex];
	}

	FName FAttributeTypedSet::FindName(FAttributeSetKey AttributeKey) const
	{
		const int32 AttributeIndex = Algo::LowerBound(AttributeKeys, AttributeKey);
		if (AttributeIndex >= AttributeKeys.Num() || AttributeKeys[AttributeIndex] != AttributeKey)
		{
			// Attribute not found
			return NAME_None;
		}

		return AttributeNames[AttributeIndex];
	}

	FAttributeSetIndex FAttributeTypedSet::FindIndex(FName AttributeName) const
	{
		const int32 AttributeNameIndex = Algo::LowerBoundBy(AttributeNameToIndexMap, AttributeName, [](const TTuple<FName, uint16>& Entry) { return Entry.Key; }, FNameFastLess());
		if (AttributeNameIndex >= AttributeNameToIndexMap.Num())
		{
			// Attribute not found
			return FAttributeSetIndex();
		}

		const TTuple<FName, uint16>& Entry = AttributeNameToIndexMap[AttributeNameIndex];
		if (Entry.Key != AttributeName)
		{
			// Attribute not found
			return FAttributeSetIndex();
		}

		const int32 AttributeIndex = Entry.Value;
		if (AttributeIndex >= AttributeKeys.Num())
		{
			// Attribute not active in this LOD
			return FAttributeSetIndex();
		}

		return FAttributeSetIndex(AttributeIndex);
	}

	FAttributeSetIndex FAttributeTypedSet::FindIndex(FAttributeSetKey AttributeKey) const
	{
		const int32 AttributeIndex = Algo::LowerBound(AttributeKeys, AttributeKey);
		if (AttributeIndex >= AttributeKeys.Num() || AttributeKeys[AttributeIndex] != AttributeKey)
		{
			// Attribute not found
			return FAttributeSetIndex();
		}

		return FAttributeSetIndex(AttributeIndex);
	}

	FAttributeBindingIndex FAttributeTypedSet::FindBindingIndex(FName AttributeName) const
	{
		// TODO: Store an array view of the attribute names directly within each typed set
		return Owner->FindBindingIndex(AttributeName, Type);
	}

	FAttributeSetIndex FAttributeTypedSet::GetIndex(FAttributeBindingIndex BindingIndex) const
	{
		if (!BindingIndexToSetIndexMap.IsValidIndex(BindingIndex.GetInt()))
		{
			// Invalid attribute binding index
			return FAttributeSetIndex();
		}

		const uint16 AttributeIndex = BindingIndexToSetIndexMap[BindingIndex.GetInt()];
		if (AttributeIndex == uint16(INDEX_NONE) || AttributeIndex >= AttributeKeys.Num())
		{
			// Invalid attribute index
			return FAttributeSetIndex();
		}

		return FAttributeSetIndex(AttributeIndex);
	}

	FAttributeBindingIndex FAttributeTypedSet::GetBindingIndex(FAttributeSetIndex AttributeIndex) const
	{
		if (AttributeIndex.GetInt() < 0 || AttributeIndex.GetInt() >= AttributeKeys.Num())
		{
			// Invalid attribute set index
			return FAttributeBindingIndex();
		}

		const uint16 BindingIndex = BindingIndices[AttributeIndex.GetInt()];
		return FAttributeBindingIndex(BindingIndex != uint16(INDEX_NONE) ? BindingIndex : INDEX_NONE);
	}

	FAttributeSetIndex FAttributeTypedSet::GetParentIndex(FAttributeSetIndex AttributeIndex) const
	{
		if (AttributeIndex.GetInt() < 0 || AttributeIndex.GetInt() >= AttributeKeys.Num())
		{
			// Invalid attribute set index
			return FAttributeSetIndex();
		}

		const uint16 ParentIndex = ParentAttributeIndices[AttributeIndex.GetInt()];
		return FAttributeSetIndex(ParentIndex != uint16(INDEX_NONE) ? ParentIndex : INDEX_NONE);
	}

	FName FAttributeTypedSet::GetName(FAttributeSetIndex AttributeIndex) const
	{
		if (AttributeIndex.GetInt() < 0 || AttributeIndex.GetInt() >= AttributeKeys.Num())
		{
			// Invalid attribute set index
			return NAME_None;
		}

		return AttributeNames[AttributeIndex.GetInt()];
	}

	FAttributeSetKey FAttributeTypedSet::GetKey(FAttributeSetIndex AttributeIndex) const
	{
		if (AttributeIndex.GetInt() < 0 || AttributeIndex.GetInt() >= AttributeKeys.Num())
		{
			// Invalid attribute set index
			return FAttributeSetKey();
		}

		return AttributeKeys[AttributeIndex.GetInt()];
	}

	void FAttributeTypedSet::AddRef() const
	{
		Owner->AddRef();
	}

	uint32 FAttributeTypedSet::Release() const
	{
		return Owner->Release();
	}

	uint32 FAttributeTypedSet::GetRefCount() const
	{
		return Owner->GetRefCount();
	}

	void FAttributeTypedSet::Init(TConstArrayView<FAttributeDescription> SetAttributes, TConstArrayView<FName> BindingAttributes)
	{
		const int32 NumAttributes = SetAttributes.Num();
		if (NumAttributes == 0)
		{
			return;	// Nothing to do, no need to reset
		}

		Type = SetAttributes[0].Type;

		if (Type == nullptr)
		{
			// Invalid attribute type
			Reset();
			return;
		}

		// Temporary since we don't retain the source attribute list
		TArray<TTuple<FAttributeSetKey, int32>> AttributeKeyToSourceIndexMap;
		AttributeKeyToSourceIndexMap.Reserve(NumAttributes);
		AttributeKeyToSourceIndexMap.AddUninitialized(NumAttributes);

		for (int32 SourceAttributeIndex = 0; SourceAttributeIndex < NumAttributes; ++SourceAttributeIndex)
		{
			const FAttributeDescription& Attribute = SetAttributes[SourceAttributeIndex];

			if (Attribute.Type != Type)
			{
				// Inconsistent attribute types
				Reset();
				return;
			}

			if (Attribute.Name.IsNone())
			{
				// Invalid attribute name
				Reset();
				return;
			}

			if (Attribute.LOD < 0 || Attribute.LOD >= 16)
			{
				// Invalid attribute LOD
				Reset();
				return;
			}

			const FAttributeBindingIndex BindingIndex(BindingAttributes.IndexOfByKey(Attribute.Name));
			if (!BindingIndex.IsValid())
			{
				// This attribute must exist in the binding
				Reset();
				return;
			}

			FAttributeBindingIndex ParentBindingIndex;
			if (!Attribute.ParentName.IsNone())
			{
				ParentBindingIndex = FAttributeBindingIndex(BindingAttributes.IndexOfByKey(Attribute.ParentName));
				if (!ParentBindingIndex.IsValid())
				{
					// This attribute must exist in the binding
					Reset();
					return;
				}
			}

			const FAttributeSetKey AttributeKey(BindingIndex, ParentBindingIndex, Attribute.LOD);
			AttributeKeyToSourceIndexMap[SourceAttributeIndex] = MakeTuple(AttributeKey, SourceAttributeIndex);
		}

		// Sort by higher LOD first, then by parent hash, and by name hash
		AttributeKeyToSourceIndexMap.Sort([](const TTuple<FAttributeSetKey, int32>& EntryA, const TTuple<FAttributeSetKey, int32>& EntryB) { return EntryA.Key < EntryB.Key; });

		AttributeKeys.Reserve(NumAttributes);
		AttributeKeys.AddUninitialized(NumAttributes);
		AttributeNames.Reserve(NumAttributes);
		AttributeNames.AddDefaulted(NumAttributes);
		AttributeNameToIndexMap.Reserve(NumAttributes);
		AttributeNameToIndexMap.AddUninitialized(NumAttributes);
		ParentAttributeIndices.Reserve(NumAttributes);
		ParentAttributeIndices.AddUninitialized(NumAttributes);
		BindingIndices.Reserve(NumAttributes);
		BindingIndices.AddUninitialized(NumAttributes);
		BindingIndexToSetIndexMap.Reserve(BindingAttributes.Num());
		BindingIndexToSetIndexMap.Init(INDEX_NONE, BindingAttributes.Num());

		for (int32 AttributeIndex = 0; AttributeIndex < NumAttributes; ++AttributeIndex)
		{
			const TTuple<FAttributeSetKey, int32>& Entry = AttributeKeyToSourceIndexMap[AttributeIndex];
			const FName AttributeName = SetAttributes[Entry.Value].Name;

			AttributeKeys[AttributeIndex] = Entry.Key;
			AttributeNames[AttributeIndex] = AttributeName;
			AttributeNameToIndexMap[AttributeIndex] = MakeTuple(AttributeName, static_cast<uint16>(AttributeIndex));

			const int32 BindingIndex = BindingAttributes.IndexOfByKey(AttributeName);
			check(BindingIndex != INDEX_NONE);

			BindingIndexToSetIndexMap[BindingIndex] = AttributeIndex;
			BindingIndices[AttributeIndex] = BindingIndex;

			const FName ParentAttributeName = SetAttributes[Entry.Value].ParentName;
			ParentAttributeIndices[AttributeIndex] = !ParentAttributeName.IsNone() ? AttributeNames.IndexOfByKey(ParentAttributeName) : INDEX_NONE;
			checkf(ParentAttributeIndices[AttributeIndex] == uint16(INDEX_NONE) || ParentAttributeIndices[AttributeIndex] < AttributeIndex, TEXT("Attributes should be sorted parent first"));
		}

		// Sort by name
		AttributeNameToIndexMap.Sort([](const TTuple<FName, uint16>& EntryA, const TTuple<FName, uint16>& EntryB) { return EntryA.Key.FastLess(EntryB.Key); });
	}

	void FAttributeTypedSet::Reset()
	{
		AttributeNameToIndexMap.Empty();
		AttributeKeys.Empty();
		AttributeNames.Empty();
		ParentAttributeIndices.Empty();
		BindingIndices.Empty();
		BindingIndexToSetIndexMap.Empty();
	}
}
