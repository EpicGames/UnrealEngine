// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Attributes/EngineAttributes.h"

#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"
#include "UAF/ValueRuntime/Transformers/Accumulate.h"
#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"
#include "UAF/ValueRuntime/Transformers/Interpolate.h"
#include "UAF/ValueRuntime/Transformers/Layer.h"
#include "UAF/ValueRuntime/Transformers/Overwrite.h"
#include "UAF/ValueRuntime/Transformers/Sanitize.h"

namespace UE::UAF
{
	void RegisterEngineAttributes()
	{
		FValueRuntimeRegistry& RuntimeRegistry = FValueRuntimeRegistry::Get();

		RuntimeRegistry.RegisterBoundValueMapInitializer<FFloatAnimationAttribute>();
		RuntimeRegistry.RegisterBoundValueMapInitializer<FIntegerAnimationAttribute>();
		RuntimeRegistry.RegisterBoundValueMapInitializer<FStringAnimationAttribute>();
		RuntimeRegistry.RegisterBoundValueMapInitializer<FBoneTransformAnimationAttribute>();
		RuntimeRegistry.RegisterBoundValueMapInitializer<FTransformAnimationAttribute>();
		RuntimeRegistry.RegisterBoundValueMapInitializer<FVectorAnimationAttribute>();
		RuntimeRegistry.RegisterBoundValueMapInitializer<FQuaternionAnimationAttribute>();

		RuntimeRegistry.RegisterUnboundValueMapInitializer<FFloatAnimationAttribute>();
		RuntimeRegistry.RegisterUnboundValueMapInitializer<FIntegerAnimationAttribute>();
		RuntimeRegistry.RegisterUnboundValueMapInitializer<FStringAnimationAttribute>();
		RuntimeRegistry.RegisterUnboundValueMapInitializer<FBoneTransformAnimationAttribute>();
		RuntimeRegistry.RegisterUnboundValueMapInitializer<FTransformAnimationAttribute>();
		RuntimeRegistry.RegisterUnboundValueMapInitializer<FVectorAnimationAttribute>();
		RuntimeRegistry.RegisterUnboundValueMapInitializer<FQuaternionAnimationAttribute>();

		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FAccumulate_BoneTransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FAccumulate_FloatAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FAccumulate_IntegerAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FAccumulate_TransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FAccumulate_VectorAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FAccumulate_QuaternionAttribute>();
		
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FInterpolate_BoneTransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FInterpolate_FloatAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FInterpolate_IntegerAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FInterpolate_TransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FInterpolate_VectorAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FInterpolate_QuaternionAttribute>();
		
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FLayer_BoneTransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FLayer_FloatAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FLayer_IntegerAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FLayer_TransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FLayer_VectorAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FLayer_QuaternionAttribute>();
		
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FMakeAdditiveSpace_BoneTransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FMakeAdditiveSpace_FloatAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FMakeAdditiveSpace_IntegerAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FMakeAdditiveSpace_TransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FMakeAdditiveSpace_VectorAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FMakeAdditiveSpace_QuaternionAttribute>();

		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FApplyAdditiveSpace_BoneTransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FApplyAdditiveSpace_FloatAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FApplyAdditiveSpace_IntegerAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FApplyAdditiveSpace_TransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FApplyAdditiveSpace_VectorAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FApplyAdditiveSpace_QuaternionAttribute>();
		
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FOverwrite_BoneTransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FOverwrite_FloatAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FOverwrite_IntegerAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FOverwrite_TransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FOverwrite_VectorAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FOverwrite_QuaternionAttribute>();
		
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FSanitize_BoneTransformAttribute>();
#if UE_UAF_VALIDATE_MAPPED_VALUES
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FSanitize_FloatAttribute>();
#endif
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FSanitize_TransformAttribute>();
		RuntimeRegistry.RegisterValueTransformer<Transformers::Private::FSanitize_QuaternionAttribute>();
	}
}
