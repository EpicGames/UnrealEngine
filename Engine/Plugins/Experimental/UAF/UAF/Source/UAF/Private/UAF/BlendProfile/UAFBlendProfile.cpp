// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/BlendProfile/UAFBlendProfile.h"

#if WITH_EDITOR
#include "Animation/Skeleton.h"
#include "SkeletonHierarchyTableType.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectSaveContext.h"
#include "StructUtils/InstancedStruct.h"
#include "Misc/Optional.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFBlendProfile)

#if WITH_EDITOR

void UUAFBlendProfile::BeginDestroy()
{
	if (Skeleton && !IsTemplate())
	{
		Skeleton->UnregisterOnSkeletonHierarchyChanged(this);
	}

	Super::BeginDestroy();
}

void UUAFBlendProfile::HandleSkeletonHierarchyChanged()
{
	UpdateHierarchy();
	UpdateCachedData();
}

void UUAFBlendProfile::UpdateHierarchy()
{
	/* Skeleton hierarchy tables take a snapshot of the currently bound skeleton's hierarchy.
	 *  When the skeleton hierarchy changes and this function is called we do the following:
	 *		1. Make a note of all the bone table entries with overridden values.
	 *		2. Make a note of all the curves and attributes in the table and their location in the hierarchy.
	 *		3. Clear the hierarchy table contents
	 *		4. Reconstruct the bone hierarchy using the new skeleton hierarchy and insert the old blend values 
	 *		   for the old overridden bones.
	 *		5. Reconstruct the curves and attributes hierarchy.
	 */

	if (!Table)
	{
		return;
	}

	const FHierarchyTable_TableType_Skeleton& Metadata = Table->GetTableMetadata<FHierarchyTable_TableType_Skeleton>();
	SetSkeleton(Metadata.Skeleton);

	const FReferenceSkeleton& RefSkeleton = GetSkeleton()->GetReferenceSkeleton();
	const int32 BoneCount = RefSkeleton.GetNum();

	struct FCurveAttributeBlendData
	{
		bool bIsCurve;
		FName Identifier;
		FName ParentIdentifier;
		TOptional<FInstancedStruct> Payload;
	};

	TMap<FName, FInstancedStruct> BoneData;
	TArray<FCurveAttributeBlendData> CurveAndAttributeData;
	{
		for (const FHierarchyTableEntryData& TableEntry : Table->GetTableData())
		{
			const auto& EntryMetadata = TableEntry.GetMetadata<FHierarchyTable_TablePayloadType_Skeleton>();
			if (EntryMetadata.EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Bone)
			{
				if (TableEntry.IsOverridden())
				{
					BoneData.Add(TableEntry.Identifier, TableEntry.GetPayload().GetValue());
				}
			}
			if (EntryMetadata.EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Curve)
			{
				check(TableEntry.HasParent()); // Curves must be parented
				const int32 ParentIndex = TableEntry.Parent;

				const FHierarchyTableEntryData* ParentEntry = Table->GetTableEntry(ParentIndex);
				check(ParentEntry);

				CurveAndAttributeData.Add({ true, TableEntry.Identifier, ParentEntry->Identifier, TableEntry.GetPayload() });
			}
			else if (EntryMetadata.EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Attribute)
			{
				check(TableEntry.HasParent()); // Attributes must be parented
				const int32 ParentIndex = TableEntry.Parent;

				const FHierarchyTableEntryData* ParentEntry = Table->GetTableEntry(ParentIndex);
				check(ParentEntry);

				CurveAndAttributeData.Add({ false, TableEntry.Identifier, ParentEntry->Identifier, TableEntry.GetPayload() });
			}
		}
	}

	Table->EmptyTable();

	// Re-add bone data
	{
		FInstancedStruct DefaultTablePayload;
		DefaultTablePayload.InitializeAs<FHierarchyTable_TablePayloadType_Skeleton>();
		DefaultTablePayload.GetMutable<FHierarchyTable_TablePayloadType_Skeleton>().EntryType = ESkeletonHierarchyTable_TablePayloadEntryType::Bone;

		for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
		{
			const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);

			FHierarchyTableEntryData EntryData;
			EntryData.Parent = RefSkeleton.GetParentIndex(BoneIndex);
			EntryData.Identifier = BoneName;
			EntryData.TablePayload = DefaultTablePayload;
			EntryData.OwnerTable = Table;

			if (BoneData.Contains(BoneName))
			{
				EntryData.Payload = BoneData[BoneName];
			}
			else
			{
				EntryData.Payload = (BoneIndex == 0) ? Table->CreateDefaultValue() : TOptional<FInstancedStruct>();
			}

			Table->AddEntry(EntryData);
		}
	}

	// Re-add curve and attribute data
	{
		FInstancedStruct DefaultTablePayload_Curve;
		DefaultTablePayload_Curve.InitializeAs<FHierarchyTable_TablePayloadType_Skeleton>();
		DefaultTablePayload_Curve.GetMutable<FHierarchyTable_TablePayloadType_Skeleton>().EntryType = ESkeletonHierarchyTable_TablePayloadEntryType::Curve;

		FInstancedStruct DefaultTablePayload_Attribute;
		DefaultTablePayload_Attribute.InitializeAs<FHierarchyTable_TablePayloadType_Skeleton>();
		DefaultTablePayload_Attribute.GetMutable<FHierarchyTable_TablePayloadType_Skeleton>().EntryType = ESkeletonHierarchyTable_TablePayloadEntryType::Attribute;

		for (const FCurveAttributeBlendData& Entry : CurveAndAttributeData)
		{
			const FHierarchyTableEntryData* const ParentEntry = Table->GetTableEntry(Entry.ParentIdentifier);
			if (ParentEntry)
			{
				FHierarchyTableEntryData EntryData;
				EntryData.Identifier = Entry.Identifier;
				EntryData.Parent = Table->GetTableEntryIndex(ParentEntry->Identifier);
				EntryData.TablePayload = Entry.bIsCurve ? DefaultTablePayload_Curve : DefaultTablePayload_Attribute;
				EntryData.OwnerTable = Table;
				EntryData.Payload = Entry.Payload;

				Table->AddEntry(EntryData);
			}
			else
			{
				// This curve/attribute was attached to a parent that no longer exists
			}
		}
	}

	Table->RegenerateEntriesGuid();
}

void UUAFBlendProfile::UpdateCachedData()
{
	CachedBlendProfileData.Reset();

	if (Table)
	{
		CachedBlendProfileData.Init(Table, Type == EUAFBlendProfileType::WeightFactor ? EBlendProfileMode::WeightFactor : EBlendProfileMode::TimeFactor);

		const FHierarchyTable_TableType_Skeleton& Metadata = Table->GetTableMetadata<FHierarchyTable_TableType_Skeleton>();
		SetSkeleton(Metadata.Skeleton);
	}
}

#endif // WITH_EDITOR

TObjectPtr<USkeleton> UUAFBlendProfile::GetSkeleton() const
{
	return Skeleton;
}

EUAFBlendProfileType UUAFBlendProfile::GetType() const
{
	return Type;
}

EBlendProfileMode UUAFBlendProfile::GetEngineType() const
{
	return Type == EUAFBlendProfileType::TimeFactor
		? EBlendProfileMode::TimeFactor
		: EBlendProfileMode::WeightFactor;
}

void UUAFBlendProfile::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// Update the hierarchy to reflect any changes made to the skeleton
	UpdateHierarchy();
	// Cache the flattened data for runtime use
	UpdateCachedData();
#endif // WITH_EDITOR

	CachedBlendProfileData.UnpackCachedData();
}

#if WITH_EDITOR

void UUAFBlendProfile::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	UpdateCachedData();
	
	Super::PreSave(ObjectSaveContext);
}

void UUAFBlendProfile::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	using namespace UE::UAF;
	
	Context.AddTag(FAssetRegistryTag(
		AssetRegistryTag_BlendProfileType,
		Type == EUAFBlendProfileType::TimeFactor
			? AssetRegistryTag_BlendProfileType_TimeProfile
			: AssetRegistryTag_BlendProfileType_WeightProfile,
		FAssetRegistryTag::TT_Alphabetical));
}

void UUAFBlendProfile::SetSkeleton(TObjectPtr<USkeleton> InSkeleton)
{
	if (Skeleton)
	{
		Skeleton->UnregisterOnSkeletonHierarchyChanged(this);
	}
	
	if (ensure(InSkeleton))
	{
		Skeleton = InSkeleton;

		Skeleton->RegisterOnSkeletonHierarchyChanged(
			USkeleton::FOnSkeletonHierarchyChanged::CreateUObject(this, &UUAFBlendProfile::HandleSkeletonHierarchyChanged));
	}
}

#endif // WITH_EDITOR

const UUAFBlendProfile::FSkeletonBoneWeightArray& UUAFBlendProfile::GetSkeletonBoneWeights() const
{
	return CachedBlendProfileData.GetBoneBlendWeights();
}

const UUAFBlendProfile::FSkeletonCurveWeightArray& UUAFBlendProfile::GetSkeletonCurveWeights() const
{
	return CachedBlendProfileData.GetCurveBlendWeights();
}

const UUAFBlendProfile::FSkeletonAttributeWeightArray& UUAFBlendProfile::GetSkeletonAttributeWeights() const
{
	return CachedBlendProfileData.GetAttributeBlendWeights();
}

