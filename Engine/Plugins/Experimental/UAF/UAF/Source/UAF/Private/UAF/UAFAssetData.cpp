// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/UAFAssetData.h"

#if WITH_EDITOR
FString FUAFAssetData::GetName() const
{
	TArray<const UObject*> Objects;
	GetObjectReferences(Objects);
	return Objects.Num() > 0 ? GetNameSafe(Objects[0]) : FString();
}
#endif

