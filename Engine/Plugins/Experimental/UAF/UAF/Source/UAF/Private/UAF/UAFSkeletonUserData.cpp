// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/UAFSkeletonUserData.h"

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Ticker.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UObject/UObjectGlobals.h"

namespace UE::UAF::Private
{
	struct FCachedUAFSkeletonUserData
	{
		TWeakObjectPtr<USkeleton> Skeleton;
		TStrongObjectPtr<UUAFSkeletonUserData> UserData;
	};

	static TMap<TObjectKey<USkeleton>, FCachedUAFSkeletonUserData> OnDemandUserDataCache;
	static FRWLock OnDemandUserDataCacheLock;

	// A handle to our ticker when GC runs and we need to clean things up
	static FTSTicker::FDelegateHandle OnDemandUserDataCacheGCCleanupTickerHandle;

	// Array of keys to clean up post-GC
	static TArray<TObjectKey<USkeleton>> OnDemandUserDataCacheGCKeysToCleanup;

	// The current index of GC keys to process
	static int32 OnDemandUserDataCacheGCKeysProcessedIndex = 0;

	static FDelegateHandle OnDemandUserDataCachePostGarbageCollectHandle;
	static FDelegateHandle OnDemandUserDataCachePreExitHandle;
	static bool bOnDemandUserDataCacheInitialized = false;

	static float GOnDemandUserDataCacheGCCleanupTimeBudget = 100.0f / (1000.0f * 1000.0f);
	static FAutoConsoleVariableRef CVar_OnDemandUserDataCacheGCCleanupTimeBudget(TEXT("UAF.OnDemandUserDataCache.GCCleanupTimeBudget"), GOnDemandUserDataCacheGCCleanupTimeBudget, TEXT("Amount of time in seconds that we can use when reclaiming memory post-GC"));

	static bool OnDemandUserDataCache_RunGCCleanup(double TimeBudget)
	{
		SCOPED_NAMED_EVENT(UAF_DataRegistry_GCCleanup, FColor::Blue);

		if (TimeBudget <= 0.0)
		{
			// No budget specified, process everything
			TimeBudget = UE_DOUBLE_BIG_NUMBER;
		}

		const double StartTime = FPlatformTime::Seconds();

		while (OnDemandUserDataCacheGCKeysProcessedIndex < OnDemandUserDataCacheGCKeysToCleanup.Num())
		{
			const TObjectKey<USkeleton>& SkeletonKey = OnDemandUserDataCacheGCKeysToCleanup[OnDemandUserDataCacheGCKeysProcessedIndex];
			if (SkeletonKey.ResolveObjectPtr() == nullptr)
			{
				OnDemandUserDataCache.Remove(SkeletonKey);
			}

			OnDemandUserDataCacheGCKeysProcessedIndex++;

			if ((FPlatformTime::Seconds() - StartTime) > TimeBudget)
			{
				break;
			}
		}

		if (OnDemandUserDataCacheGCKeysProcessedIndex == OnDemandUserDataCacheGCKeysToCleanup.Num())
		{
			// We are done
			OnDemandUserDataCacheGCKeysToCleanup.Empty();
			OnDemandUserDataCacheGCKeysProcessedIndex = 0;
			return true;
		}

		// Not done yet
		return false;
	}

	static bool OnDemandUserDataCache_GCCleanupTicker(float DeltaTime)
	{
		bool bIsDone;
		{
			FWriteScopeLock Lock(OnDemandUserDataCacheLock);
			bIsDone = OnDemandUserDataCache_RunGCCleanup(GOnDemandUserDataCacheGCCleanupTimeBudget);
		}

		const bool bFireTickerAgainAfterDelay = !bIsDone;
		return bFireTickerAgainAfterDelay;
	}

	static void OnDemandUserDataCache_HandlePostGarbageCollect()
	{
		FWriteScopeLock Lock(OnDemandUserDataCacheLock);

		if (OnDemandUserDataCache.IsEmpty())
		{
			// Nothing to clean up
			return;
		}

		// Start our cleanup
		// If cleanup was already in progress, we purge whatever remains (shouldn't happen unless we force GC to run)
		if (!OnDemandUserDataCacheGCKeysToCleanup.IsEmpty())
		{
			OnDemandUserDataCache_RunGCCleanup(INDEX_NONE);
		}

		if (OnDemandUserDataCacheGCCleanupTickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(OnDemandUserDataCacheGCCleanupTickerHandle);
		}

		// Copy our keys into a temporary array, we'll process them in small batches over time
		OnDemandUserDataCache.GenerateKeyArray(OnDemandUserDataCacheGCKeysToCleanup);
		OnDemandUserDataCacheGCKeysProcessedIndex = 0;

		const float TickerDelay = 0.0f;
		OnDemandUserDataCacheGCCleanupTickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("UAF Skeleton User Data Post-GC Cleanup"), TickerDelay, OnDemandUserDataCache_GCCleanupTicker);
	}

	static void OnDemandUserDataCache_HandleOnPreExit()
	{
		FCoreUObjectDelegates::GetPostGarbageCollect().Remove(OnDemandUserDataCachePostGarbageCollectHandle);
		OnDemandUserDataCachePostGarbageCollectHandle.Reset();

		FCoreDelegates::OnEnginePreExit.Remove(OnDemandUserDataCachePreExitHandle);
		OnDemandUserDataCachePreExitHandle.Reset();

		if (!OnDemandUserDataCacheGCKeysToCleanup.IsEmpty())
		{
			OnDemandUserDataCache_RunGCCleanup(INDEX_NONE);

			FTSTicker::RemoveTicker(OnDemandUserDataCacheGCCleanupTickerHandle);
		}
	}

	static void InitializeGameThread()
	{
		check(IsInGameThread());

		OnDemandUserDataCachePostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddStatic(&OnDemandUserDataCache_HandlePostGarbageCollect);
		OnDemandUserDataCachePreExitHandle = FCoreDelegates::OnEnginePreExit.AddStatic(&OnDemandUserDataCache_HandleOnPreExit);
	}

	static void Initialize()
	{
		if (IsInGameThread())
		{
			InitializeGameThread();
		}
		else
		{
			FFunctionGraphTask::CreateAndDispatchWhenReady(InitializeGameThread, TStatId(), nullptr, ENamedThreads::GameThread);
		}

		// Even if we aren't on the game thread, it's fine to be considered as being initialized, the rest of the setup will occur there
		bOnDemandUserDataCacheInitialized = true;
	}
}

UUAFSkeletonUserData* UUAFSkeletonUserData::FromSkeleton(TNonNullPtr<USkeleton> Skeleton)
{
	using namespace UE::UAF::Private;

	UUAFSkeletonUserData* UserData = Cast<UUAFSkeletonUserData>(Skeleton->GetAssetUserDataOfClass(UUAFSkeletonUserData::StaticClass()));
	if (UserData != nullptr && UserData->GetSetBinding() != nullptr)
	{
		// We found our user specified data and its valid
		return UserData;
	}

	const TObjectKey<USkeleton> SkeletonKey(Skeleton);
	const uint32 SkeletonKeyHash = GetTypeHash(SkeletonKey);

	// No user specified data found on the skeleton itself
	// Look up in our cache to see if we've allocated one already
	{
		FReadScopeLock Lock(OnDemandUserDataCacheLock);
		if (const FCachedUAFSkeletonUserData* CachedData = OnDemandUserDataCache.FindByHash(SkeletonKeyHash, SkeletonKey))
		{
			// We've already generated an instance, use it
			return CachedData->UserData.Get();
		}
	}

	// We don't have an instance yet, create one now
	{
		FWriteScopeLock Lock(OnDemandUserDataCacheLock);

		if (!bOnDemandUserDataCacheInitialized)
		{
			Initialize();
		}

		if (const FCachedUAFSkeletonUserData* CachedData = OnDemandUserDataCache.FindByHash(SkeletonKeyHash, SkeletonKey))
		{
			// We've already generated an instance, use it (another thread beat us to it)
			return CachedData->UserData.Get();
		}

		FCachedUAFSkeletonUserData& CachedData = OnDemandUserDataCache.AddByHash(SkeletonKeyHash, SkeletonKey);

		{
			FGCScopeGuard GCGuard;

			UserData = NewObject<UUAFSkeletonUserData>();
			(void)UserData->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);

			UserData->SetBinding = NewObject<UAbstractSkeletonSetBinding>();
			(void)UserData->SetBinding->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);

			UserData->SetBinding->SetSkeleton(Skeleton);
			UserData->SetBinding->AddDefaultAttributes();

			CachedData.Skeleton = Skeleton;
			CachedData.UserData = TStrongObjectPtr<UUAFSkeletonUserData>(UserData);
		}

		return UserData;
	}
}

UAbstractSkeletonLabelBinding* UUAFSkeletonUserData::GetLabelBinding() const
{
	return LabelBinding;
}

UAbstractSkeletonSetBinding* UUAFSkeletonUserData::GetSetBinding() const
{
	return SetBinding;
}
