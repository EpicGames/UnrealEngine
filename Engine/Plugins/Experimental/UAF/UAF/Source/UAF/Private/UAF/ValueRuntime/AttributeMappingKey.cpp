// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/AttributeMappingKey.h"

namespace UE::UAF
{
	// Key sorting uses signed comparison, use the smallest signed value regardless of pointer size
	const FAttributeMappingKey FAttributeMappingKey::SMALLEST_VALUE = FAttributeMappingKey(nullptr, nullptr);

	// Key sorting uses signed comparison, use the largest signed value regardless of pointer size
	const FAttributeMappingKey FAttributeMappingKey::LARGEST_VALUE =
		FAttributeMappingKey(
			reinterpret_cast<UScriptStruct*>(static_cast<intptr_t>(static_cast<uintptr_t>(~0ULL) >> 1)),
			reinterpret_cast<UScriptStruct*>(static_cast<intptr_t>(static_cast<uintptr_t>(~0ULL) >> 1)));
}
