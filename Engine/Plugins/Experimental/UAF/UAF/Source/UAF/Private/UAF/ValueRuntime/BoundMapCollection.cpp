// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/BoundMapCollection.h"

#include "Algo/BinarySearch.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"

namespace UE::UAF
{
	FBoundMapCollection::FBoundMapCollection(const FBoundMapCollection& Other)
		: NamedSet(Other.NamedSet)
		, MapCount(Other.MapCount)
		, MapCapacity(Other.MapCount)
		, ReallocFun(Other.ReallocFun)
	{
		if (Other.Maps != nullptr)
		{
			const int32 BufferSize = sizeof(FMapEntry) * Other.MapCount;
			Maps = reinterpret_cast<FMapEntry*>((*Other.ReallocFun)(nullptr, 0, BufferSize));
			FMemory::Memcpy(Maps, Other.Maps, BufferSize);

			for (int32 MapIndex = 0; MapIndex < Other.MapCount; ++MapIndex)
			{
				Maps[MapIndex].Map = Other.Maps[MapIndex].Map->Duplicate(Other.ReallocFun);
			}
		}
		else
		{
			Maps = nullptr;
		}
	}

	FBoundMapCollection::FBoundMapCollection(FBoundMapCollection&& Other)
		: NamedSet(MoveTemp(Other.NamedSet))
		, Maps(Other.Maps)
		, MapCount(Other.MapCount)
		, MapCapacity(Other.MapCapacity)
		, ReallocFun(Other.ReallocFun)
	{
		Other.Maps = nullptr;
		Other.MapCount = 0;
		Other.MapCapacity = 0;
	}

	FBoundMapCollection::FBoundMapCollection(const FAttributeNamedSetPtr& InNamedSet, FReallocFun InReallocFun)
		: NamedSet(InNamedSet)
		, ReallocFun(InReallocFun)
	{
		check(InReallocFun != nullptr);
	}

	FBoundMapCollection::~FBoundMapCollection()
	{
		Empty();
	}

	FBoundMapCollection& FBoundMapCollection::operator=(const FBoundMapCollection& Other)
	{
		if (this != &Other)
		{
			checkf(ReallocFun != nullptr, TEXT("When we copy from another collection, we retain our original allocator"));

			// Free anything we might be holding
			Empty();

			NamedSet = Other.NamedSet;
			MapCount = Other.MapCount;
			MapCapacity = Other.MapCount;

			if (Other.Maps != nullptr)
			{
				const int32 BufferSize = sizeof(FMapEntry) * Other.MapCount;
				Maps = reinterpret_cast<FMapEntry*>((*ReallocFun)(nullptr, 0, BufferSize));
				FMemory::Memcpy(Maps, Other.Maps, BufferSize);

				for (int32 MapIndex = 0; MapIndex < Other.MapCount; ++MapIndex)
				{
					Maps[MapIndex].Map = Other.Maps[MapIndex].Map->Duplicate(ReallocFun);
				}
			}
			else
			{
				Maps = nullptr;
			}
		}

		return *this;
	}

	FBoundMapCollection& FBoundMapCollection::operator=(FBoundMapCollection&& Other)
	{
		if (this != &Other)
		{
			checkf(ReallocFun == Other.ReallocFun, TEXT("When we move from another collection, our allocators must match"));

			// Free anything we might be holding
			Empty();

			NamedSet = MoveTemp(Other.NamedSet);
			Maps = Other.Maps;
			MapCount = Other.MapCount;
			MapCapacity = Other.MapCapacity;

			Other.Maps = nullptr;
			Other.MapCount = 0;
			Other.MapCapacity = 0;
		}

		return *this;
	}

	bool FBoundMapCollection::IsEmpty() const
	{
		return MapCount == 0;
	}

	const FAttributeNamedSetPtr& FBoundMapCollection::GetNamedSet() const
	{
		return NamedSet;
	}

	void FBoundMapCollection::Reset(const FAttributeNamedSetPtr& InNamedSet)
	{
		// Free our maps but retain the arrays that contains them
		for (int32 MapIndex = 0; MapIndex < MapCount; ++MapIndex)
		{
			ReleaseBoundValueMap(Maps[MapIndex].Map);
		}
		MapCount = 0;

		NamedSet = InNamedSet;
	}

	void FBoundMapCollection::Empty()
	{
		if (Maps != nullptr)
		{
			for (int32 MapIndex = 0; MapIndex < MapCount; ++MapIndex)
			{
				ReleaseBoundValueMap(Maps[MapIndex].Map);
			}

			(*ReallocFun)(reinterpret_cast<uint8*>(Maps), sizeof(FMapEntry) * MapCapacity, 0);
			Maps = nullptr;
			MapCount = 0;
			MapCapacity = 0;
		}

		NamedSet = nullptr;
	}

	FReallocFun FBoundMapCollection::GetAllocator() const
	{
		return ReallocFun;
	}

	int32 FBoundMapCollection::Num() const
	{
		return MapCount;
	}

	FBoundValueMap* FBoundMapCollection::Add(const FAttributeMappingKey& MappingKey)
	{
		if (!NamedSet)
		{
			// We are not initialized
			return nullptr;
		}

		FAttributeTypedSetPtr TypedSet = NamedSet->FindTypedSet(MappingKey.GetAttributeType());
		if (!TypedSet)
		{
			// Unknown attribute type, it isn't contained within our named set
			return nullptr;
		}

		// Use default construction arguments
		const FBoundValueMap::FConstructArgs Args(TypedSet, MappingKey.GetValueType(), ReallocFun);

		// We don't know the concrete type of the map, use indirection to construct it
		constexpr bool bConstructMap = true;
		FBoundValueMap** Map = AddImpl(Args, bConstructMap);
		return Map != nullptr ? *Map : nullptr;
	}

	FBoundValueMap** FBoundMapCollection::AddImpl(const FBoundValueMap::FConstructArgs& Args, bool bConstructMap)
	{
		UScriptStruct* AttributeType = Args.TypedSet->GetType();
		UScriptStruct* ValueType = Args.ValueType;

		const FAttributeMappingKey Key(AttributeType, ValueType);

		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), Key, [](const FMapEntry& Entry) { return Entry.Key; });
		if (MapIndex < MapCount && Maps[MapIndex].Key == Key)
		{
			// A map already exists that matches the specified types
			return nullptr;
		}

		FBoundValueMap* Map = nullptr;
		if (bConstructMap)
		{
			Map = FValueRuntimeRegistry::Get().ConstructBoundValueMap(Args);

			if (Map == nullptr)
			{
				// Failed to construct a map, the type has probably not been registered
				return nullptr;
			}
		}

		if (MapCount >= MapCapacity)
		{
			// The first time we add a map, we reserve space for an entry per typed set plus some extra room
			const bool bAllowQuantize = true;
			const int32 DesiredCapacity = MapCapacity == 0 ? (NamedSet->NumTypedSets() + 4) : (MapCapacity + 1);
			const int32 NewCapacity = DefaultCalculateSlackGrow(DesiredCapacity, MapCapacity, sizeof(FMapEntry), bAllowQuantize);

			Maps = reinterpret_cast<FMapEntry*>((*ReallocFun)(
				reinterpret_cast<uint8*>(Maps),
				sizeof(FMapEntry) * MapCapacity,
				sizeof(FMapEntry) * NewCapacity));

			MapCapacity = NewCapacity;
		}

		const int32 NumToMove = MapCount - MapIndex;
		FMemory::Memmove(Maps + MapIndex + 1, Maps + MapIndex, sizeof(FMapEntry) * NumToMove);
		Maps[MapIndex].Key = Key;
		Maps[MapIndex].Map = Map;
		MapCount++;

		return &Maps[MapIndex].Map;
	}

	bool FBoundMapCollection::Add(FBoundValueMap* Map)
	{
		if (Map == nullptr)
		{
			// Invalid map
			return false;
		}
		else if (Map->GetAllocator() != ReallocFun)
		{
			// Allocator mismatch
			return false;
		}

		const FAttributeMappingKey Key = Map->GetMappingKey();

		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), Key, [](const FMapEntry& Entry) { return Entry.Key; });
		if (MapIndex < MapCount && Maps[MapIndex].Key == Key)
		{
			// A map already exists that matches the specified types
			return false;
		}

		if (MapCount >= MapCapacity)
		{
			// The first time we add a map, we reserve space for an entry per typed set plus some extra room
			const bool bAllowQuantize = true;
			const int32 DesiredCapacity = MapCapacity == 0 ? (NamedSet->NumTypedSets() + 4) : (MapCapacity + 1);
			const int32 NewCapacity = DefaultCalculateSlackGrow(DesiredCapacity, MapCapacity, sizeof(FMapEntry), bAllowQuantize);

			Maps = reinterpret_cast<FMapEntry*>((*ReallocFun)(
				reinterpret_cast<uint8*>(Maps),
				sizeof(FMapEntry) * MapCapacity,
				sizeof(FMapEntry) * NewCapacity));

			MapCapacity = NewCapacity;
		}

		const int32 NumToMove = MapCount - MapIndex;
		FMemory::Memmove(Maps + MapIndex + 1, Maps + MapIndex, sizeof(FMapEntry) * NumToMove);
		Maps[MapIndex].Key = Key;
		Maps[MapIndex].Map = Map;
		MapCount++;

		return true;
	}

	bool FBoundMapCollection::Append(FBoundValueMap* Map)
	{
		if (Map == nullptr)
		{
			// Invalid map
			return false;
		}
		else if (Map->GetAllocator() != ReallocFun)
		{
			// Allocator mismatch
			return false;
		}

		const FAttributeMappingKey Key = Map->GetMappingKey();

		if (MapCount != 0 && !(Maps[MapCount - 1].Key < Key))
		{
			// We don't belong at the end of the map list
			return false;
		}

		if (MapCount >= MapCapacity)
		{
			// The first time we add a map, we reserve space for an entry per typed set plus some extra room
			const bool bAllowQuantize = true;
			const int32 DesiredCapacity = MapCapacity == 0 ? (NamedSet->NumTypedSets() + 4) : (MapCapacity + 1);
			const int32 NewCapacity = DefaultCalculateSlackGrow(DesiredCapacity, MapCapacity, sizeof(FMapEntry), bAllowQuantize);

			Maps = reinterpret_cast<FMapEntry*>((*ReallocFun)(
				reinterpret_cast<uint8*>(Maps),
				sizeof(FMapEntry) * MapCapacity,
				sizeof(FMapEntry) * NewCapacity));

			MapCapacity = NewCapacity;
		}

		Maps[MapCount].Key = Key;
		Maps[MapCount].Map = Map;
		MapCount++;

		return true;
	}

	bool FBoundMapCollection::Remove(FBoundValueMap* Map)
	{
		if (Map == nullptr)
		{
			// Invalid map
			return false;
		}

		return Remove(Map->GetMappingKey());
	}

	bool FBoundMapCollection::Remove(const FAttributeMappingKey& MappingKey)
	{
		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), MappingKey, [](const FMapEntry& Entry) { return Entry.Key; });
		if (MapIndex >= MapCount || Maps[MapIndex].Key != MappingKey)
		{
			// No map found that matches the desired types
			return false;
		}

		// Destroy and free our map
		ReleaseBoundValueMap(Maps[MapIndex].Map);

		// Remove our entry
		const int32 NumToMove = MapCount - MapIndex - 1;
		FMemory::Memmove(Maps + MapIndex, Maps + MapIndex + 1, sizeof(FMapEntry) * NumToMove);
		MapCount--;

		return true;
	}

	FBoundValueMap* FBoundMapCollection::Find(const FAttributeMappingKey& MappingKey)
	{
		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), MappingKey, [](const FMapEntry& Entry) { return Entry.Key; });
		if (MapIndex >= MapCount || Maps[MapIndex].Key != MappingKey)
		{
			// No map found that matches the desired types
			return nullptr;
		}

		return Maps[MapIndex].Map;
	}

	const FBoundValueMap* FBoundMapCollection::Find(const FAttributeMappingKey& MappingKey) const
	{
		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), MappingKey, [](const FMapEntry& Entry) { return Entry.Key; });
		if (MapIndex >= MapCount || Maps[MapIndex].Key != MappingKey)
		{
			// No map found that matches the desired types
			return nullptr;
		}

		return Maps[MapIndex].Map;
	}

	FBoundValueMap* FBoundMapCollection::FindOrAdd(const FAttributeMappingKey& MappingKey)
	{
		if (FBoundValueMap* Map = Find(MappingKey))
		{
			return Map;
		}

		return Add(MappingKey);
	}
}
