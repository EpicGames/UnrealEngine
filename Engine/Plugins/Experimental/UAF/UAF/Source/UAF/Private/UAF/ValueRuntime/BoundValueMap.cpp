// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/BoundValueMap.h"

namespace UE::UAF
{
	FBoundValueMap::FBoundValueMap() = default;

	FBoundValueMap::FBoundValueMap(const FConstructArgs& Args, int32 InAllocatedSize)
		: TypedSet(Args.TypedSet)
		, ValueTypeStruct(Args.ValueType)
		, AllocatedSize(InAllocatedSize)
		, NumValues(0)
		, ValuesPtr(nullptr)
		, ReallocFun(Args.ReallocFun)
	{
		check(Args.TypedSet.IsValid());
		check(Args.ReallocFun != nullptr);
	}

	FBoundValueMap::~FBoundValueMap()
	{
		checkf(ValuesPtr == nullptr, TEXT("Derived types should handle destruction"));
	}

	bool FBoundValueMap::IsEmpty() const
	{
		return ValuesPtr == nullptr;
	}

	int32 FBoundValueMap::Num() const
	{
		return NumValues;
	}

	const FAttributeTypedSetPtr& FBoundValueMap::GetTypedSet() const
	{
		return TypedSet;
	}

	UScriptStruct* FBoundValueMap::GetAttributeType() const
	{
		return TypedSet ? TypedSet->GetType() : nullptr;
	}

	UScriptStruct* FBoundValueMap::GetValueType() const
	{
		return ValueTypeStruct;
	}

	FAttributeMappingKey FBoundValueMap::GetMappingKey() const
	{
		return TypedSet ? FAttributeMappingKey(TypedSet->GetType(), ValueTypeStruct) : FAttributeMappingKey();
	}

	int32 FBoundValueMap::GetAllocatedSize() const
	{
		return AllocatedSize;
	}

	FReallocFun FBoundValueMap::GetAllocator() const
	{
		return ReallocFun;
	}
}
