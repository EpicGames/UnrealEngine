// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/Transformers/AdditiveSpace.h"

#include "UAF/Attributes/AttributeBindingData.h"
#include "UAF/ValueRuntime/IteratorUtils.h"

namespace UE::UAF::Transformers
{
	FName FMakeAdditiveSpace::TransformerName = TEXT("UAF::Transformers::MakeAdditive");
	FName FApplyAdditiveSpace::TransformerName = TEXT("UAF::Transformers::ApplyAdditive");

	void FMakeAdditiveSpace::Apply(
		const FValueTransformerMapPtr& TransformerMap,
		const FValueBundle& Base,
		const FValueBundle& Input,
		FValueBundle& AdditiveOutput)
	{
		// Named sets must match as it ensures that our inputs have the same sizes/shapes
		checkf(Base.GetNamedSet() == Input.GetNamedSet(), TEXT("Base and Input collections must use the same named set"));
		checkf(&Base != &AdditiveOutput, TEXT("Cannot output to the Base collection when making an additive space collection"));
		checkf(&Input != &AdditiveOutput, TEXT("Cannot output to the Input collection when making an additive space collection"));

		const FValueTransformerList* TransformerList = TransformerMap->Find(TransformerName);
		if (TransformerList == nullptr)
		{
			// No transformer implementations have been registered
			return;
		}

		// Setup our output using the value space of our base and making it additive
		FValueSpace OutputSpace = Base.GetValueSpace();
		checkf(OutputSpace.GetType() == EValueSpaceType::Local
			|| (OutputSpace.GetType() == EValueSpaceType::Mixed && OutputSpace.GetMixedSpaceFlags() == EMixedSpaceFlags::MeshRotation),
			TEXT("We can only create local and mesh space additives"));
		checkf(!OutputSpace.IsAdditive() && !Input.GetValueSpace().IsAdditive(), TEXT("Inputs cannot be additive"));
		OutputSpace.SetIsAdditive(true);

		AdditiveOutput.Reset(Input.GetNamedSet());
		AdditiveOutput.SetValueSpace(OutputSpace);

		// Bound value maps
		{
			FBoundMapCollection& AdditiveOutputMaps = AdditiveOutput.GetBoundValueMaps();

			OuterJoinBy(
				FValueTransformerListWithBoundMapCollectionJoinOp(),
				[&AdditiveOutputMaps](FRawTransformerFunc Transformer, const FBoundValueMap* BaseMap, const FBoundValueMap* InputMap)
				{
					if (InputMap == nullptr)
					{
						// The base doesn't have a matching pair in our input, nothing to convert
						return;
					}

					if (Transformer == nullptr)
					{
						// No transformer was found for this value type, retain the input as-is
						AdditiveOutputMaps.Append(InputMap->Duplicate(AdditiveOutputMaps.GetAllocator()));
						return;
					}

					// Transform our matching pair, note that the base map might be missing
					// If it is missing, transformer specializations should use a default value
					FBoundValueMap* OutputMap = nullptr;

					FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
					(*TypedTransformer)(BaseMap, InputMap, OutputMap, AdditiveOutputMaps.GetAllocator());

					checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
					checkf(OutputMap->GetAllocator() == AdditiveOutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
					AdditiveOutputMaps.Append(OutputMap);
				},
				TransformerList->CreateBoundValueMapTransformerIterator(),
				Base.GetBoundValueMaps().CreateConstIterator(),
				Input.GetBoundValueMaps().CreateConstIterator());
		}

		// Unbound value maps
		{
			FUnboundMapCollection& AdditiveOutputMaps = AdditiveOutput.GetUnboundValueMaps();

			OuterJoinBy(
				FValueTransformerListWithUnboundMapCollectionJoinOp(),
				[&AdditiveOutputMaps](FRawTransformerFunc Transformer, const FUnboundValueMap* BaseMap, const FUnboundValueMap* InputMap)
				{
					if (InputMap == nullptr)
					{
						// The base doesn't have a matching pair in our input, nothing to convert
						return;
					}

					if (Transformer == nullptr)
					{
						// No transformer was found for this value type, retain the input as-is
						AdditiveOutputMaps.Append(InputMap->Duplicate(AdditiveOutputMaps.GetAllocator()));
						return;
					}

					// Transform our matching pair, note that the base map might be missing
					// If it is missing, transformer specializations should use a default value
					FUnboundValueMap* OutputMap = nullptr;

					FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
					(*TypedTransformer)(BaseMap, InputMap, OutputMap, AdditiveOutputMaps.GetAllocator());

					checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
					checkf(OutputMap->GetAllocator() == AdditiveOutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
					AdditiveOutputMaps.Append(OutputMap);
				},
				TransformerList->CreateUnboundValueMapTransformerIterator(),
				Base.GetUnboundValueMaps().CreateConstIterator(),
				Input.GetUnboundValueMaps().CreateConstIterator());
		}
	}

	void FApplyAdditiveSpace::Apply(
		const FValueTransformerMapPtr& TransformerMap,
		const FValueBundle& Base,
		const FValueBundle& Additive,
		float AdditiveWeight,
		FValueBundle& Output)
	{
		const bool bIsOutputBase = &Base == &Output;
		if (bIsOutputBase && !FAnimWeight::IsRelevant(AdditiveWeight))
		{
			// Our input base is our output and the additive input is not relevant, nothing to do
			return;
		}

		// Allocator is irrelevant since it will remain empty
		FValueBundleHeap EmptyCollection(Additive.GetNamedSet());
		FApplyAdditiveSpace::Apply(TransformerMap, Base, Additive, EmptyCollection, AdditiveWeight, Output);
	}

	void FApplyAdditiveSpace::Apply(
		const FValueTransformerMapPtr& TransformerMap,
		const FValueBundle& Base,
		const FValueBundle& Additive,
		const FValueBundle& PerValueWeights,
		float DefaultWeight,
		FValueBundle& Output)
	{
		// Named sets must match as it ensures that our inputs have the same sizes / shapes
		checkf(Base.GetNamedSet() == Additive.GetNamedSet(), TEXT("Base and Additive collections must use the same named set"));

		const FValueSpace BaseSpace = Base.GetValueSpace();
		const FValueSpace AdditiveSpace = Additive.GetValueSpace();
		checkf(BaseSpace.GetType() == EValueSpaceType::Local
			|| (BaseSpace.GetType() == EValueSpaceType::Mixed && !!(BaseSpace.GetMixedSpaceFlags() & EMixedSpaceFlags::MeshRotation)), TEXT("Base must be in local or mesh space to apply an additive"));
		checkf(!BaseSpace.IsAdditive(), TEXT("Base input cannot be in additive space"));
		checkf(AdditiveSpace.IsAdditive(), TEXT("Additive input must be in additive space"));
		checkf(BaseSpace.GetType() == AdditiveSpace.GetType(), TEXT("Additive value space must match the base (local or mesh space additive)"));

		const FValueTransformerList* TransformerList = TransformerMap->Find(TransformerName);
		if (TransformerList == nullptr)
		{
			// No transformer implementations have been registered
			return;
		}

		UScriptStruct* FloatValueType = FFloatAnimationAttribute::StaticStruct();

		const bool bIsOutputBase = &Base == &Output;
		const bool bIsOutputAdditive = &Additive == &Output;
		if (bIsOutputBase || bIsOutputAdditive)
		{
			// Our output collection matches one of our inputs, we'll modify it in place
			FReallocFun OutputAllocator = Output.GetAllocator();

			// Bound value maps
			{
				const FBoundMapCollection& PerValueWeightMaps = PerValueWeights.GetBoundValueMaps();

				// Keep a list of set maps that needs to be appended to our output
				TArray<FBoundValueMap*, TInlineAllocator<8>> PendingMaps;

				OuterJoinBy(
					FValueTransformerListWithBoundMapCollectionJoinOp(),
					[&PerValueWeightMaps, DefaultWeight, bIsOutputBase, FloatValueType, OutputAllocator, &PendingMaps](FRawTransformerFunc Transformer, FBoundValueMap* BaseMap, FBoundValueMap* AdditiveMap)
					{
						if (AdditiveMap == nullptr)
						{
							// The base doesn't have a matching pair in our input, nothing to apply
							return;
						}

						if (Transformer == nullptr)
						{
							// No transformer was found for this value type, retain the one with the largest weight
							if (BaseMap == nullptr)
							{
								// We have no base, duplicate the additive
								PendingMaps.Add(AdditiveMap->Duplicate(OutputAllocator));
							}
							else if (DefaultWeight >= 0.5f)
							{
								// Additive has more weight
								if (bIsOutputBase)
								{
									// Base is our output, copy it over
									AdditiveMap->CopyTo(*BaseMap);
								}
								else
								{
									// Additive is our output, nothing to do
								}
							}
							else
							{
								// Base has more weight
								if (bIsOutputBase)
								{
									// Base is our output, nothing to do
								}
								else
								{
									// Additive is our output, copy it over
									BaseMap->CopyTo(*AdditiveMap);
								}
							}
							return;
						}

						// Transform our matching pair, note that the base map might be missing
						// If it is missing, transformer specializations should use a default value
						FBoundValueMap* OutputMap = bIsOutputBase ? BaseMap : AdditiveMap;
						const bool bIsOutputMissing = OutputMap == nullptr;

						// We cannot join our weights because their value type differs
						const FAttributeMappingKey MappingKey = AdditiveMap->GetMappingKey().To(FloatValueType);
						const FBoundValueMap* PerValueWeightMap = PerValueWeightMaps.Find(MappingKey);

						FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
						(*TypedTransformer)(BaseMap, AdditiveMap, PerValueWeightMap, DefaultWeight, OutputMap, OutputAllocator);

						if (bIsOutputMissing)
						{
							// Our output was missing, we should have allocated a buffer for it
							checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
							checkf(OutputMap->GetAllocator() == OutputAllocator, TEXT("Transformer function is responsible for allocating using the output allocator"));
							PendingMaps.Add(OutputMap);
						}
					},
					TransformerList->CreateBoundValueMapTransformerIterator(),
					// Remove the const qualifier since we'll mutate one in place as our output
					const_cast<FValueBundle&>(Base).GetBoundValueMaps().CreateIterator(),
					const_cast<FValueBundle&>(Additive).GetBoundValueMaps().CreateIterator());

				// Append our pending entries
				FBoundMapCollection& OutputMaps = Output.GetBoundValueMaps();
				for (FBoundValueMap* Map : PendingMaps)
				{
					OutputMaps.Add(Map);
				}
			}

			// Unbound value maps
			{
				// Keep a list of maps that needs to be appended to our output
				TArray<FUnboundValueMap*, TInlineAllocator<8>> PendingMaps;

				OuterJoinBy(
					FValueTransformerListWithUnboundMapCollectionJoinOp(),
					[DefaultWeight, bIsOutputBase, OutputAllocator, &PendingMaps](FRawTransformerFunc Transformer, FUnboundValueMap* BaseMap, FUnboundValueMap* AdditiveMap)
					{
						if (AdditiveMap == nullptr)
						{
							// The base doesn't have a matching pair in our input, nothing to apply
							return;
						}

						if (Transformer == nullptr)
						{
							// No transformer was found for this value type, retain the one with the largest weight
							if (BaseMap == nullptr)
							{
								// We have no base
								if (bIsOutputBase)
								{
									// Duplicate the additive
									PendingMaps.Add(AdditiveMap->Duplicate(OutputAllocator));
								}
								else
								{
									// Our output is the additive, it already contains it
								}
							}
							else if (DefaultWeight >= 0.5f)
							{
								// Additive has more weight or we have no base
								if (bIsOutputBase)
								{
									// Base is our output, copy it over
									AdditiveMap->CopyTo(*BaseMap);
								}
								else
								{
									// Additive is our output, nothing to do
								}
							}
							else
							{
								// Base has more weight
								if (bIsOutputBase)
								{
									// Base is our output, nothing to do
								}
								else
								{
									// Additive is our output, copy it over
									BaseMap->CopyTo(*AdditiveMap);
								}
							}
							return;
						}

						// Transform our matching pair, note that the base map might be missing
						// If it is missing, transformer specializations should use a default value
						FUnboundValueMap* OutputMap = nullptr;

						FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
						(*TypedTransformer)(BaseMap, AdditiveMap, DefaultWeight, OutputMap, OutputAllocator);

						checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
						checkf(OutputMap->GetAllocator() == OutputAllocator, TEXT("Transformer function is responsible for allocating using the output allocator"));

						if (bIsOutputBase)
						{
							if (BaseMap != nullptr)
							{
								// Overwrite our old base
								OutputMap->MoveTo(*BaseMap);

								// Destroy our temporary output map
								ReleaseUnboundValueMap(OutputMap);
							}
							else
							{
								// Keep our new output
								PendingMaps.Add(OutputMap);
							}
						}
						else
						{
							// Overwrite our old additive
							OutputMap->MoveTo(*AdditiveMap);

							// Destroy our temporary output map
							ReleaseUnboundValueMap(OutputMap);
						}
					},
					TransformerList->CreateUnboundValueMapTransformerIterator(),
					// Remove the const qualifier since we'll mutate one in place as our output
					const_cast<FValueBundle&>(Base).GetUnboundValueMaps().CreateIterator(),
					const_cast<FValueBundle&>(Additive).GetUnboundValueMaps().CreateIterator());

				// Append our pending entries
				FUnboundMapCollection& OutputMaps = Output.GetUnboundValueMaps();
				for (FUnboundValueMap* Map : PendingMaps)
				{
					OutputMaps.Add(Map);
				}
			}
		}
		else
		{
			// Our output collection isn't either of our inputs, reset it
			Output.Reset(Base.GetNamedSet());
			Output.SetValueSpace(BaseSpace);

			// Bound value maps
			{
				const FBoundMapCollection& PerValueWeightMaps = PerValueWeights.GetBoundValueMaps();
				FBoundMapCollection& OutputMaps = Output.GetBoundValueMaps();

				OuterJoinBy(
					FValueTransformerListWithBoundMapCollectionJoinOp(),
					[&PerValueWeightMaps, DefaultWeight, FloatValueType, &OutputMaps](FRawTransformerFunc Transformer, const FBoundValueMap* BaseMap, const FBoundValueMap* AdditiveMap)
					{
						if (AdditiveMap == nullptr)
						{
							// The base doesn't have a matching pair in our input, nothing to apply
							return;
						}

						if (Transformer == nullptr)
						{
							// No transformer was found for this value type, retain the one with the largest weight
							FBoundValueMap* MapCopy;
							if (BaseMap == nullptr || DefaultWeight >= 0.5f)
							{
								// Additive map has more weight or we have no base
								MapCopy = AdditiveMap->Duplicate(OutputMaps.GetAllocator());
							}
							else
							{
								// Base map has more weight
								MapCopy = BaseMap->Duplicate(OutputMaps.GetAllocator());
							}

							OutputMaps.Append(MapCopy);
							return;
						}

						// Transform our matching pair, note that the base map might be missing
						// If it is missing, transformer specializations should use a default value
						FBoundValueMap* OutputMap = nullptr;

						// We cannot join our weights because their value type differs
						const FAttributeMappingKey MappingKey = AdditiveMap->GetMappingKey().To(FloatValueType);
						const FBoundValueMap* PerValueWeightMap = PerValueWeightMaps.Find(MappingKey);

						FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
						(*TypedTransformer)(BaseMap, AdditiveMap, PerValueWeightMap, DefaultWeight, OutputMap, OutputMaps.GetAllocator());

						checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
						checkf(OutputMap->GetAllocator() == OutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
						OutputMaps.Append(OutputMap);
					},
					TransformerList->CreateBoundValueMapTransformerIterator(),
					Base.GetBoundValueMaps().CreateConstIterator(),
					Additive.GetBoundValueMaps().CreateConstIterator());
			}

			// Unbound value maps
			{
				FUnboundMapCollection& OutputMaps = Output.GetUnboundValueMaps();

				OuterJoinBy(
					FValueTransformerListWithUnboundMapCollectionJoinOp(),
					[DefaultWeight, &OutputMaps](FRawTransformerFunc Transformer, const FUnboundValueMap* BaseMap, const FUnboundValueMap* AdditiveMap)
					{
						if (AdditiveMap == nullptr)
						{
							// The base doesn't have a matching pair in our input, nothing to apply
							return;
						}

						if (Transformer == nullptr)
						{
							// No transformer was found for this value type, retain the one with the largest weight
							FUnboundValueMap* MapCopy;
							if (BaseMap == nullptr || DefaultWeight >= 0.5f)
							{
								// Additive map has more weight or we have no base
								MapCopy = AdditiveMap->Duplicate(OutputMaps.GetAllocator());
							}
							else
							{
								// Base map has more weight
								MapCopy = BaseMap->Duplicate(OutputMaps.GetAllocator());
							}

							OutputMaps.Append(MapCopy);
							return;
						}

						// Transform our matching pair, note that the base map might be missing
						// If it is missing, transformer specializations should use a default value
						FUnboundValueMap* OutputMap = nullptr;

						FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
						(*TypedTransformer)(BaseMap, AdditiveMap, DefaultWeight, OutputMap, OutputMaps.GetAllocator());

						checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
						checkf(OutputMap->GetAllocator() == OutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
						OutputMaps.Append(OutputMap);
					},
					TransformerList->CreateUnboundValueMapTransformerIterator(),
					Base.GetUnboundValueMaps().CreateConstIterator(),
					Additive.GetUnboundValueMaps().CreateConstIterator());
			}
		}
	}

	namespace Private
	{
		//////////////////////////////////////////////////////////////////////////
		// MakeAdditiveSpace transformers

		void FMakeAdditiveSpace_BoneTransformAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the reference pose
				const FAttributeTypedSetPtr& TypedSet = Input->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Input->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Input->GetTypedSet(), Input->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FBoneTransformAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FMakeAdditiveSpace_BoneTransformAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(true);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and input should have the same shape
			check(Base->GetTypedSet() == Input->GetTypedSet());
			check(Base->Num() == Input->Num());

			const TBoundValueMap<FBoneTransformAnimationAttribute>* BaseBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Base);
			const TBoundValueMap<FBoneTransformAnimationAttribute>* InputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Input);
			TBoundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Output);

			const FBoneTransformAnimationAttribute* BaseTransformPtr = BaseBoneTransforms->GetData();
			const FBoneTransformAnimationAttribute* InputTransformPtr = InputBoneTransforms->GetData();
			FBoneTransformAnimationAttribute* OutputTransformPtr = OutputBoneTransforms->GetData();
			const FBoneTransformAnimationAttribute* BaseTransformPtrEnd = BaseTransformPtr + BaseBoneTransforms->Num();

			while (BaseTransformPtr < BaseTransformPtrEnd)
			{
				OutputTransformPtr->Value.SetRotation(InputTransformPtr->Value.GetRotation() * BaseTransformPtr->Value.GetRotation().Inverse());
				OutputTransformPtr->Value.SetTranslation(InputTransformPtr->Value.GetTranslation() - BaseTransformPtr->Value.GetTranslation());
				// additive scale considers how much it grow or lower
				// in order to support blending between different additive scale, we save [(target scale)/(source scale) - 1.f], and this can blend with 
				// other delta scale value
				// when we apply to the another scale, we apply scale * (1 + [additive scale])
				OutputTransformPtr->Value.SetScale3D(InputTransformPtr->Value.GetScale3D() * FTransform::GetSafeScaleReciprocal(BaseTransformPtr->Value.GetScale3D()) - 1.0f);
				OutputTransformPtr->Value.NormalizeRotation();

				BaseTransformPtr++;
				InputTransformPtr++;
				OutputTransformPtr++;
			}
		}

		void FMakeAdditiveSpace_BoneTransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FBoneTransformAnimationAttribute>* BaseBoneTransforms = Cast<FBoneTransformAnimationAttribute>(Base);
			const TUnboundValueMap<FBoneTransformAnimationAttribute>* InputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Input);

			TUnboundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = MakeUnboundValueMap<FBoneTransformAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[OutputBoneTransforms](FName Name, const FBoneTransformAnimationAttribute* BaseValue, const FBoneTransformAnimationAttribute* InputValue)
				{
					if (InputValue == nullptr)
					{
						// We have no input to subtract, skip it
						return;
					}

					// Missing values are set to the identity
					const FTransform& BaseTransform = BaseValue != nullptr ? BaseValue->Value : FTransform::Identity;

					FBoneTransformAnimationAttribute Output;
					Output.Value.SetRotation(InputValue->Value.GetRotation() * BaseTransform.GetRotation().Inverse());
					Output.Value.SetTranslation(InputValue->Value.GetTranslation() - BaseTransform.GetTranslation());
					// additive scale considers how much it grow or lower
					// in order to support blending between different additive scale, we save [(target scale)/(source scale) - 1.f], and this can blend with 
					// other delta scale value
					// when we apply to the another scale, we apply scale * (1 + [additive scale])
					Output.Value.SetScale3D(InputValue->Value.GetScale3D() * FTransform::GetSafeScaleReciprocal(BaseTransform.GetScale3D()) - 1.0f);
					Output.Value.NormalizeRotation();

					OutputBoneTransforms->Append(Name, Output);
				},
				BaseBoneTransforms != nullptr ? BaseBoneTransforms->CreateConstIterator() : TUnboundValueMap<FBoneTransformAnimationAttribute>::FConstIterator(),
				InputBoneTransforms->CreateConstIterator());

			Output = OutputBoneTransforms;
		}

		void FMakeAdditiveSpace_FloatAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the user authored default values
				const FAttributeTypedSetPtr& TypedSet = Input->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Input->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Input->GetTypedSet(), Input->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FFloatAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FMakeAdditiveSpace_FloatAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(true);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and input should have the same shape
			check(Base->GetTypedSet() == Input->GetTypedSet());
			check(Base->Num() == Input->Num());

			const TBoundValueMap<FFloatAnimationAttribute>* BaseFloats = CastChecked<FFloatAnimationAttribute>(Base);
			const TBoundValueMap<FFloatAnimationAttribute>* InputFloats = CastChecked<FFloatAnimationAttribute>(Input);
			TBoundValueMap<FFloatAnimationAttribute>* OutputFloats = CastChecked<FFloatAnimationAttribute>(Output);

			const FFloatAnimationAttribute* BaseFloatsPtr = BaseFloats->GetData();
			const FFloatAnimationAttribute* InputFloatsPtr = InputFloats->GetData();
			FFloatAnimationAttribute* OutputFloatsPtr = OutputFloats->GetData();
			const FFloatAnimationAttribute* BaseFloatsPtrEnd = BaseFloatsPtr + BaseFloats->Num();

			while (BaseFloatsPtr < BaseFloatsPtrEnd)
			{
				OutputFloatsPtr->Value = InputFloatsPtr->Value - BaseFloatsPtr->Value;

				BaseFloatsPtr++;
				InputFloatsPtr++;
				OutputFloatsPtr++;
			}
		}

		void FMakeAdditiveSpace_FloatAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FFloatAnimationAttribute>* BaseFloats = Cast<FFloatAnimationAttribute>(Base);
			const TUnboundValueMap<FFloatAnimationAttribute>* InputFloats = CastChecked<FFloatAnimationAttribute>(Input);

			TUnboundValueMap<FFloatAnimationAttribute>* OutputFloats = MakeUnboundValueMap<FFloatAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[OutputFloats](FName Name, const FFloatAnimationAttribute* BaseValue, const FFloatAnimationAttribute* InputValue)
				{
					if (InputValue == nullptr)
					{
						// We have no input to subtract, skip it
						return;
					}

					// Missing values are set to 0.0
					const float BaseFloat = BaseValue != nullptr ? BaseValue->Value : 0.0f;

					const FFloatAnimationAttribute Output{ InputValue->Value - BaseFloat };
					OutputFloats->Append(Name, Output);
				},
				BaseFloats != nullptr ? BaseFloats->CreateConstIterator() : TUnboundValueMap<FFloatAnimationAttribute>::FConstIterator(),
				InputFloats->CreateConstIterator());

			Output = OutputFloats;
		}

		void FMakeAdditiveSpace_IntegerAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the user authored default values
				const FAttributeTypedSetPtr& TypedSet = Input->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Input->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Input->GetTypedSet(), Input->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FIntegerAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FMakeAdditiveSpace_IntegerAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(true);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and input should have the same shape
			check(Base->GetTypedSet() == Input->GetTypedSet());
			check(Base->Num() == Input->Num());

			const TBoundValueMap<FIntegerAnimationAttribute>* BaseIntegers = CastChecked<FIntegerAnimationAttribute>(Base);
			const TBoundValueMap<FIntegerAnimationAttribute>* InputIntegers = CastChecked<FIntegerAnimationAttribute>(Input);
			TBoundValueMap<FIntegerAnimationAttribute>* OutputIntegers = CastChecked<FIntegerAnimationAttribute>(Output);

			const FIntegerAnimationAttribute* BaseIntegersPtr = BaseIntegers->GetData();
			const FIntegerAnimationAttribute* InputIntegersPtr = InputIntegers->GetData();
			FIntegerAnimationAttribute* OutputIntegersPtr = OutputIntegers->GetData();
			const FIntegerAnimationAttribute* BaseIntegersPtrEnd = BaseIntegersPtr + BaseIntegers->Num();

			while (BaseIntegersPtr < BaseIntegersPtrEnd)
			{
				OutputIntegersPtr->Value = InputIntegersPtr->Value - BaseIntegersPtr->Value;

				BaseIntegersPtr++;
				InputIntegersPtr++;
				OutputIntegersPtr++;
			}
		}

		void FMakeAdditiveSpace_IntegerAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FIntegerAnimationAttribute>* BaseIntegers = Cast<FIntegerAnimationAttribute>(Base);
			const TUnboundValueMap<FIntegerAnimationAttribute>* InputIntegers = CastChecked<FIntegerAnimationAttribute>(Input);

			TUnboundValueMap<FIntegerAnimationAttribute>* OutputInteger = MakeUnboundValueMap<FIntegerAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[OutputInteger](FName Name, const FIntegerAnimationAttribute* BaseValue, const FIntegerAnimationAttribute* InputValue)
				{
					if (InputValue == nullptr)
					{
						// We have no input to subtract, skip it
						return;
					}

					// Missing values are set to 0
					const int32 BaseInteger = BaseValue != nullptr ? BaseValue->Value : 0;

					const FIntegerAnimationAttribute Output{ InputValue->Value - BaseInteger };
					OutputInteger->Append(Name, Output);
				},
				BaseIntegers != nullptr ? BaseIntegers->CreateConstIterator() : TUnboundValueMap<FIntegerAnimationAttribute>::FConstIterator(),
				InputIntegers->CreateConstIterator());

			Output = OutputInteger;
		}

		void FMakeAdditiveSpace_TransformAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the reference pose
				const FAttributeTypedSetPtr& TypedSet = Input->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Input->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Input->GetTypedSet(), Input->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FTransformAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FMakeAdditiveSpace_TransformAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(true);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and input should have the same shape
			check(Base->GetTypedSet() == Input->GetTypedSet());
			check(Base->Num() == Input->Num());

			const TBoundValueMap<FTransformAnimationAttribute>* BaseTransforms = CastChecked<FTransformAnimationAttribute>(Base);
			const TBoundValueMap<FTransformAnimationAttribute>* InputTransforms = CastChecked<FTransformAnimationAttribute>(Input);
			TBoundValueMap<FTransformAnimationAttribute>* OutputTransforms = CastChecked<FTransformAnimationAttribute>(Output);

			const FTransformAnimationAttribute* BaseTransformPtr = BaseTransforms->GetData();
			const FTransformAnimationAttribute* InputTransformPtr = InputTransforms->GetData();
			FTransformAnimationAttribute* OutputTransformPtr = OutputTransforms->GetData();
			const FTransformAnimationAttribute* BaseTransformPtrEnd = BaseTransformPtr + BaseTransforms->Num();

			while (BaseTransformPtr < BaseTransformPtrEnd)
			{
				OutputTransformPtr->Value.SetRotation(InputTransformPtr->Value.GetRotation() * BaseTransformPtr->Value.GetRotation().Inverse());
				OutputTransformPtr->Value.SetTranslation(InputTransformPtr->Value.GetTranslation() - BaseTransformPtr->Value.GetTranslation());
				// additive scale considers how much it grow or lower
				// in order to support blending between different additive scale, we save [(target scale)/(source scale) - 1.f], and this can blend with 
				// other delta scale value
				// when we apply to the another scale, we apply scale * (1 + [additive scale])
				OutputTransformPtr->Value.SetScale3D(InputTransformPtr->Value.GetScale3D() * FTransform::GetSafeScaleReciprocal(BaseTransformPtr->Value.GetScale3D()) - 1.0f);
				OutputTransformPtr->Value.NormalizeRotation();

				BaseTransformPtr++;
				InputTransformPtr++;
				OutputTransformPtr++;
			}
		}

		void FMakeAdditiveSpace_TransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FTransformAnimationAttribute>* BaseTransforms = Cast<FTransformAnimationAttribute>(Base);
			const TUnboundValueMap<FTransformAnimationAttribute>* InputTransforms = CastChecked<FTransformAnimationAttribute>(Input);

			TUnboundValueMap<FTransformAnimationAttribute>* OutputTransforms = MakeUnboundValueMap<FTransformAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[OutputTransforms](FName Name, const FTransformAnimationAttribute* BaseValue, const FTransformAnimationAttribute* InputValue)
				{
					if (InputValue == nullptr)
					{
						// We have no input to subtract, skip it
						return;
					}

					// Missing values are set to the identity
					const FTransform& BaseTransform = BaseValue != nullptr ? BaseValue->Value : FTransform::Identity;

					FTransformAnimationAttribute Output;
					Output.Value.SetRotation(InputValue->Value.GetRotation() * BaseTransform.GetRotation().Inverse());
					Output.Value.SetTranslation(InputValue->Value.GetTranslation() - BaseTransform.GetTranslation());
					// additive scale considers how much it grow or lower
					// in order to support blending between different additive scale, we save [(target scale)/(source scale) - 1.f], and this can blend with 
					// other delta scale value
					// when we apply to the another scale, we apply scale * (1 + [additive scale])
					Output.Value.SetScale3D(InputValue->Value.GetScale3D() * FTransform::GetSafeScaleReciprocal(BaseTransform.GetScale3D()) - 1.0f);
					Output.Value.NormalizeRotation();

					OutputTransforms->Append(Name, Output);
				},
				BaseTransforms != nullptr ? BaseTransforms->CreateConstIterator() : TUnboundValueMap<FTransformAnimationAttribute>::FConstIterator(),
				InputTransforms->CreateConstIterator());

			Output = OutputTransforms;
		}

		void FMakeAdditiveSpace_VectorAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the user authored default values
				const FAttributeTypedSetPtr& TypedSet = Input->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Input->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Input->GetTypedSet(), Input->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FVectorAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FMakeAdditiveSpace_VectorAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(true);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and input should have the same shape
			check(Base->GetTypedSet() == Input->GetTypedSet());
			check(Base->Num() == Input->Num());

			const TBoundValueMap<FVectorAnimationAttribute>* BaseVectors = CastChecked<FVectorAnimationAttribute>(Base);
			const TBoundValueMap<FVectorAnimationAttribute>* InputVectors = CastChecked<FVectorAnimationAttribute>(Input);
			TBoundValueMap<FVectorAnimationAttribute>* OutputVectors = CastChecked<FVectorAnimationAttribute>(Output);

			const FVectorAnimationAttribute* BaseVectorPtr = BaseVectors->GetData();
			const FVectorAnimationAttribute* InputVectorPtr = InputVectors->GetData();
			FVectorAnimationAttribute* OutputVectorPtr = OutputVectors->GetData();
			const FVectorAnimationAttribute* BaseVectorPtrEnd = BaseVectorPtr + BaseVectors->Num();

			while (BaseVectorPtr < BaseVectorPtrEnd)
			{
				OutputVectorPtr->Value = InputVectorPtr->Value - BaseVectorPtr->Value;

				BaseVectorPtr++;
				InputVectorPtr++;
				OutputVectorPtr++;
			}
		}

		void FMakeAdditiveSpace_VectorAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FVectorAnimationAttribute>* BaseVectors = Cast<FVectorAnimationAttribute>(Base);
			const TUnboundValueMap<FVectorAnimationAttribute>* InputVectors = CastChecked<FVectorAnimationAttribute>(Input);

			TUnboundValueMap<FVectorAnimationAttribute>* OutputVectors = MakeUnboundValueMap<FVectorAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[OutputVectors](FName Name, const FVectorAnimationAttribute* BaseValue, const FVectorAnimationAttribute* InputValue)
				{
					if (InputValue == nullptr)
					{
						// We have no input to subtract, skip it
						return;
					}

					// Missing values are set to 0.0
					const FVector BaseVector = BaseValue != nullptr ? BaseValue->Value : FVector::ZeroVector;

					const FVectorAnimationAttribute Output{ InputValue->Value - BaseVector };
					OutputVectors->Append(Name, Output);
				},
				BaseVectors != nullptr ? BaseVectors->CreateConstIterator() : TUnboundValueMap<FVectorAnimationAttribute>::FConstIterator(),
				InputVectors->CreateConstIterator());

			Output = OutputVectors;
		}

		void FMakeAdditiveSpace_QuaternionAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the user authored default values
				const FAttributeTypedSetPtr& TypedSet = Input->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Input->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Input->GetTypedSet(), Input->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FQuaternionAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FMakeAdditiveSpace_QuaternionAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(true);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and input should have the same shape
			check(Base->GetTypedSet() == Input->GetTypedSet());
			check(Base->Num() == Input->Num());

			const TBoundValueMap<FQuaternionAnimationAttribute>* BaseQuaternions = CastChecked<FQuaternionAnimationAttribute>(Base);
			const TBoundValueMap<FQuaternionAnimationAttribute>* InputQuaternions = CastChecked<FQuaternionAnimationAttribute>(Input);
			TBoundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = CastChecked<FQuaternionAnimationAttribute>(Output);

			const FQuaternionAnimationAttribute* BaseQuaternionPtr = BaseQuaternions->GetData();
			const FQuaternionAnimationAttribute* InputQuaternionPtr = InputQuaternions->GetData();
			FQuaternionAnimationAttribute* OutputQuaternionPtr = OutputQuaternions->GetData();
			const FQuaternionAnimationAttribute* BaseQuaternionEndPtr = BaseQuaternionPtr + BaseQuaternions->Num();

			while (BaseQuaternionPtr < BaseQuaternionEndPtr)
			{
				OutputQuaternionPtr->Value = InputQuaternionPtr->Value * BaseQuaternionPtr->Value.Inverse();
				OutputQuaternionPtr->Value.Normalize();

				BaseQuaternionPtr++;
				InputQuaternionPtr++;
				OutputQuaternionPtr++;
			}
		}

		void FMakeAdditiveSpace_QuaternionAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FQuaternionAnimationAttribute>* BaseQuaternions = Cast<FQuaternionAnimationAttribute>(Base);
			const TUnboundValueMap<FQuaternionAnimationAttribute>* InputQuaternions = CastChecked<FQuaternionAnimationAttribute>(Input);

			TUnboundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = MakeUnboundValueMap<FQuaternionAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[OutputQuaternions](FName Name, const FQuaternionAnimationAttribute* BaseValue, const FQuaternionAnimationAttribute* InputValue)
				{
					if (InputValue == nullptr)
					{
						// We have no input to subtract, skip it
						return;
					}

					// Missing values are set to identity
					const FQuat BaseQuaternion = BaseValue != nullptr ? BaseValue->Value : FQuat::Identity;

					FQuaternionAnimationAttribute Output{ InputValue->Value * BaseQuaternion.Inverse() };
					Output.Value.Normalize();
						
					OutputQuaternions->Append(Name, Output);
				},
				BaseQuaternions != nullptr ? BaseQuaternions->CreateConstIterator() : TUnboundValueMap<FQuaternionAnimationAttribute>::FConstIterator(),
				InputQuaternions->CreateConstIterator());

			Output = OutputQuaternions;
		}
		
		//////////////////////////////////////////////////////////////////////////
		// ApplyAdditiveSpace transformers

		void FApplyAdditiveSpace_BoneTransformAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the reference pose
				const FAttributeTypedSetPtr& TypedSet = Additive->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Additive->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Additive->GetTypedSet(), Additive->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FBoneTransformAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FApplyAdditiveSpace_BoneTransformAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(false);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and additive should have the same shape
			check(Base->GetTypedSet() == Additive->GetTypedSet());
			check(Base->Num() == Additive->Num());

			const TBoundValueMap<FBoneTransformAnimationAttribute>* BaseBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Base);
			const TBoundValueMap<FBoneTransformAnimationAttribute>* AdditiveBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Additive);
			TBoundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Output);

			const FBoneTransformAnimationAttribute* BaseTransformPtr = BaseBoneTransforms->GetData();
			const FBoneTransformAnimationAttribute* AdditiveTransformPtr = AdditiveBoneTransforms->GetData();
			FBoneTransformAnimationAttribute* OutputTransformPtr = OutputBoneTransforms->GetData();
			const FBoneTransformAnimationAttribute* BaseTransformPtrEnd = BaseTransformPtr + BaseBoneTransforms->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(Base->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(Base->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (BaseTransformPtr < BaseTransformPtrEnd)
				{
					FTransform Result = BaseTransformPtr->Value;
					FTransform::BlendFromIdentityAndAccumulate(Result, AdditiveTransformPtr->Value, ScalarRegister(PerValueWeightsPtr->Value));

					OutputTransformPtr->Value = Result;

					BaseTransformPtr++;
					AdditiveTransformPtr++;
					PerValueWeightsPtr++;
					OutputTransformPtr++;
				}
			}
			else
			{
				const ScalarRegister VDefaultWeight(DefaultWeight);

				if (FAnimWeight::IsFullWeight(DefaultWeight))
				{
					// Our additive has full weight, no need to interpolate with identity
					while (BaseTransformPtr < BaseTransformPtrEnd)
					{
						// TODO: The VDefaultWeight is practically 1.0, can assume that it is?
						FTransform Result = BaseTransformPtr->Value;
						Result.AccumulateWithAdditiveScale(AdditiveTransformPtr->Value, VDefaultWeight);

						OutputTransformPtr->Value = Result;

						BaseTransformPtr++;
						AdditiveTransformPtr++;
						OutputTransformPtr++;
					}
				}
				else
				{
					// Interpolate with identity before adding
					while (BaseTransformPtr < BaseTransformPtrEnd)
					{
						FTransform Result = BaseTransformPtr->Value;
						FTransform::BlendFromIdentityAndAccumulate(Result, AdditiveTransformPtr->Value, VDefaultWeight);

						OutputTransformPtr->Value = Result;

						BaseTransformPtr++;
						AdditiveTransformPtr++;
						OutputTransformPtr++;
					}
				}
			}
		}

		void FApplyAdditiveSpace_BoneTransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FBoneTransformAnimationAttribute>* BaseBoneTransforms = Cast<FBoneTransformAnimationAttribute>(Base);
			const TUnboundValueMap<FBoneTransformAnimationAttribute>* AdditiveBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Additive);

			TUnboundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = MakeUnboundValueMap<FBoneTransformAnimationAttribute>(OutputAllocator);

			const ScalarRegister VAdditiveWeight(AdditiveWeight);

			if (FAnimWeight::IsFullWeight(AdditiveWeight))
			{
				// Our additive has full weight, no need to interpolate with identity
				OuterJoinBy(
					FUnboundValueMapJoinOp(),
					[VAdditiveWeight, OutputBoneTransforms](FName Name, const FBoneTransformAnimationAttribute* BaseValue, const FBoneTransformAnimationAttribute* AdditiveValue)
					{
						// Missing values are set to the identity
						const FTransform& BaseTransform = BaseValue != nullptr ? BaseValue->Value : FTransform::Identity;

						FBoneTransformAnimationAttribute Output{ BaseTransform };

						if (AdditiveValue != nullptr)
						{
							// TODO: The VAdditiveWeight is practically 1.0, can assume that it is?
							Output.Value.AccumulateWithAdditiveScale(AdditiveValue->Value, VAdditiveWeight);
						}

						OutputBoneTransforms->Append(Name, Output);
					},
					BaseBoneTransforms != nullptr ? BaseBoneTransforms->CreateConstIterator() : TUnboundValueMap<FBoneTransformAnimationAttribute>::FConstIterator(),
					AdditiveBoneTransforms->CreateConstIterator());
			}
			else
			{
				// Interpolate with identity before adding
				OuterJoinBy(
					FUnboundValueMapJoinOp(),
					[VAdditiveWeight, OutputBoneTransforms](FName Name, const FBoneTransformAnimationAttribute* BaseValue, const FBoneTransformAnimationAttribute* AdditiveValue)
					{
						// Missing values are set to the identity
						const FTransform& BaseTransform = BaseValue != nullptr ? BaseValue->Value : FTransform::Identity;

						FBoneTransformAnimationAttribute Output{ BaseTransform };

						if (AdditiveValue != nullptr)
						{
							FTransform::BlendFromIdentityAndAccumulate(Output.Value, AdditiveValue->Value, VAdditiveWeight);
						}

						OutputBoneTransforms->Append(Name, Output);
					},
					BaseBoneTransforms != nullptr ? BaseBoneTransforms->CreateConstIterator() : TUnboundValueMap<FBoneTransformAnimationAttribute>::FConstIterator(),
					AdditiveBoneTransforms->CreateConstIterator());
			}

			Output = OutputBoneTransforms;
		}

		void FApplyAdditiveSpace_FloatAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the user authored default values
				const FAttributeTypedSetPtr& TypedSet = Additive->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Additive->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Additive->GetTypedSet(), Additive->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FFloatAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FApplyAdditiveSpace_FloatAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(false);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and additive should have the same shape
			check(Base->GetTypedSet() == Additive->GetTypedSet());
			check(Base->Num() == Additive->Num());

			const TBoundValueMap<FFloatAnimationAttribute>* BaseFloats = CastChecked<FFloatAnimationAttribute>(Base);
			const TBoundValueMap<FFloatAnimationAttribute>* AdditiveFloats = CastChecked<FFloatAnimationAttribute>(Additive);
			TBoundValueMap<FFloatAnimationAttribute>* OutputFloats = CastChecked<FFloatAnimationAttribute>(Output);

			const FFloatAnimationAttribute* BaseFloatsPtr = BaseFloats->GetData();
			const FFloatAnimationAttribute* AdditiveFloatsPtr = AdditiveFloats->GetData();
			FFloatAnimationAttribute* OutputFloatsPtr = OutputFloats->GetData();
			const FFloatAnimationAttribute* BaseFloatsPtrEnd = BaseFloatsPtr + BaseFloats->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(Base->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(Base->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (BaseFloatsPtr < BaseFloatsPtrEnd)
				{
					OutputFloatsPtr->Value = BaseFloatsPtr->Value + (PerValueWeightsPtr->Value * AdditiveFloatsPtr->Value);

					BaseFloatsPtr++;
					AdditiveFloatsPtr++;
					PerValueWeightsPtr++;
					OutputFloatsPtr++;
				}
			}
			else
			{
				while (BaseFloatsPtr < BaseFloatsPtrEnd)
				{
					OutputFloatsPtr->Value = BaseFloatsPtr->Value + (DefaultWeight * AdditiveFloatsPtr->Value);

					BaseFloatsPtr++;
					AdditiveFloatsPtr++;
					OutputFloatsPtr++;
				}
			}
		}

		void FApplyAdditiveSpace_FloatAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FFloatAnimationAttribute>* BaseFloats = Cast<FFloatAnimationAttribute>(Base);
			const TUnboundValueMap<FFloatAnimationAttribute>* AdditiveFloats = CastChecked<FFloatAnimationAttribute>(Additive);

			TUnboundValueMap<FFloatAnimationAttribute>* OutputFloats = MakeUnboundValueMap<FFloatAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[AdditiveWeight, OutputFloats](FName Name, const FFloatAnimationAttribute* BaseValue, const FFloatAnimationAttribute* AdditiveValue)
				{
					// Missing values are set to 0.0
					const float BaseFloat = BaseValue != nullptr ? BaseValue->Value : 0.0f;
					const float AdditiveFloat = AdditiveValue != nullptr ? AdditiveValue->Value : 0.0f;

					const FFloatAnimationAttribute Output{ BaseFloat + (AdditiveWeight * AdditiveFloat) };
					OutputFloats->Append(Name, Output);
				},
				BaseFloats != nullptr ? BaseFloats->CreateConstIterator() : TUnboundValueMap<FFloatAnimationAttribute>::FConstIterator(),
				AdditiveFloats->CreateConstIterator());

			Output = OutputFloats;
		}

		void FApplyAdditiveSpace_IntegerAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the user authored default values
				const FAttributeTypedSetPtr& TypedSet = Additive->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Additive->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Additive->GetTypedSet(), Additive->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FFloatAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FApplyAdditiveSpace_IntegerAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(false);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and additive should have the same shape
			check(Base->GetTypedSet() == Additive->GetTypedSet());
			check(Base->Num() == Additive->Num());

			const TBoundValueMap<FIntegerAnimationAttribute>* BaseIntegers = CastChecked<FIntegerAnimationAttribute>(Base);
			const TBoundValueMap<FIntegerAnimationAttribute>* AdditiveIntegers = CastChecked<FIntegerAnimationAttribute>(Additive);
			TBoundValueMap<FIntegerAnimationAttribute>* OutputIntegers = CastChecked<FIntegerAnimationAttribute>(Output);

			const FIntegerAnimationAttribute* BaseIntegerPtr = BaseIntegers->GetData();
			const FIntegerAnimationAttribute* AdditiveIntegerPtr = AdditiveIntegers->GetData();
			FIntegerAnimationAttribute* OutputIntegerPtr = OutputIntegers->GetData();
			const FIntegerAnimationAttribute* BaseIntegerPtrEnd = BaseIntegerPtr + BaseIntegers->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(Base->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(Base->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (BaseIntegerPtr < BaseIntegerPtrEnd)
				{
					OutputIntegerPtr->Value = BaseIntegerPtr->Value + static_cast<int32>(PerValueWeightsPtr->Value * AdditiveIntegerPtr->Value);

					BaseIntegerPtr++;
					AdditiveIntegerPtr++;
					PerValueWeightsPtr++;
					OutputIntegerPtr++;
				}
			}
			else
			{
				while (BaseIntegerPtr < BaseIntegerPtrEnd)
				{
					OutputIntegerPtr->Value = BaseIntegerPtr->Value + static_cast<int32>(DefaultWeight * AdditiveIntegerPtr->Value);

					BaseIntegerPtr++;
					AdditiveIntegerPtr++;
					OutputIntegerPtr++;
				}
			}
		}

		void FApplyAdditiveSpace_IntegerAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FIntegerAnimationAttribute>* BaseIntegers = Cast<FIntegerAnimationAttribute>(Base);
			const TUnboundValueMap<FIntegerAnimationAttribute>* AdditiveIntegers = CastChecked<FIntegerAnimationAttribute>(Additive);

			TUnboundValueMap<FIntegerAnimationAttribute>* OutputIntegers = MakeUnboundValueMap<FIntegerAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[AdditiveWeight, OutputIntegers](FName Name, const FIntegerAnimationAttribute* BaseValue, const FIntegerAnimationAttribute* AdditiveValue)
				{
					// Missing values are set to 0
					const int32 BaseInteger = BaseValue != nullptr ? BaseValue->Value : 0;
					const int32 AdditiveInteger = AdditiveValue != nullptr ? AdditiveValue->Value : 0;

					const FIntegerAnimationAttribute Output{ BaseInteger + static_cast<int32>(AdditiveWeight * AdditiveInteger) };
					OutputIntegers->Append(Name, Output);
				},
				BaseIntegers != nullptr ? BaseIntegers->CreateConstIterator() : TUnboundValueMap<FIntegerAnimationAttribute>::FConstIterator(),
				AdditiveIntegers->CreateConstIterator());

			Output = OutputIntegers;
		}

		void FApplyAdditiveSpace_TransformAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the reference pose
				const FAttributeTypedSetPtr& TypedSet = Additive->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Additive->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Additive->GetTypedSet(), Additive->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FTransformAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FApplyAdditiveSpace_TransformAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(false);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and additive should have the same shape
			check(Base->GetTypedSet() == Additive->GetTypedSet());
			check(Base->Num() == Additive->Num());

			const TBoundValueMap<FTransformAnimationAttribute>* BaseTransforms = CastChecked<FTransformAnimationAttribute>(Base);
			const TBoundValueMap<FTransformAnimationAttribute>* AdditiveTransforms = CastChecked<FTransformAnimationAttribute>(Additive);
			TBoundValueMap<FTransformAnimationAttribute>* OutputTransforms = CastChecked<FTransformAnimationAttribute>(Output);

			const FTransformAnimationAttribute* BaseTransformPtr = BaseTransforms->GetData();
			const FTransformAnimationAttribute* AdditiveTransformPtr = AdditiveTransforms->GetData();
			FTransformAnimationAttribute* OutputTransformPtr = OutputTransforms->GetData();
			const FTransformAnimationAttribute* BaseTransformPtrEnd = BaseTransformPtr + BaseTransforms->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(Base->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(Base->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (BaseTransformPtr < BaseTransformPtrEnd)
				{
					FTransform Result = BaseTransformPtr->Value;
					FTransform::BlendFromIdentityAndAccumulate(Result, AdditiveTransformPtr->Value, ScalarRegister(PerValueWeightsPtr->Value));

					OutputTransformPtr->Value = Result;

					BaseTransformPtr++;
					AdditiveTransformPtr++;
					PerValueWeightsPtr++;
					OutputTransformPtr++;
				}
			}
			else
			{
				const ScalarRegister VDefaultWeight(DefaultWeight);

				if (FAnimWeight::IsFullWeight(DefaultWeight))
				{
					// Our additive has full weight, no need to interpolate with identity
					while (BaseTransformPtr < BaseTransformPtrEnd)
					{
						// TODO: The VDefaultWeight is practically 1.0, can assume that it is?
						FTransform Result = BaseTransformPtr->Value;
						Result.AccumulateWithAdditiveScale(AdditiveTransformPtr->Value, VDefaultWeight);

						OutputTransformPtr->Value = Result;

						BaseTransformPtr++;
						AdditiveTransformPtr++;
						OutputTransformPtr++;
					}
				}
				else
				{
					// Interpolate with identity before adding
					while (BaseTransformPtr < BaseTransformPtrEnd)
					{
						FTransform Result = BaseTransformPtr->Value;
						FTransform::BlendFromIdentityAndAccumulate(Result, AdditiveTransformPtr->Value, VDefaultWeight);

						OutputTransformPtr->Value = Result;

						BaseTransformPtr++;
						AdditiveTransformPtr++;
						OutputTransformPtr++;
					}
				}
			}
		}

		void FApplyAdditiveSpace_TransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FTransformAnimationAttribute>* BaseTransforms = Cast<FTransformAnimationAttribute>(Base);
			const TUnboundValueMap<FTransformAnimationAttribute>* AdditiveTransforms = CastChecked<FTransformAnimationAttribute>(Additive);

			TUnboundValueMap<FTransformAnimationAttribute>* OutputTransforms = MakeUnboundValueMap<FTransformAnimationAttribute>(OutputAllocator);

			const ScalarRegister VAdditiveWeight(AdditiveWeight);

			if (FAnimWeight::IsFullWeight(AdditiveWeight))
			{
				// Our additive has full weight, no need to interpolate with identity
				OuterJoinBy(
					FUnboundValueMapJoinOp(),
					[VAdditiveWeight, OutputTransforms](FName Name, const FTransformAnimationAttribute* BaseValue, const FTransformAnimationAttribute* AdditiveValue)
					{
						// Missing values are set to the identity
						const FTransform& BaseTransform = BaseValue != nullptr ? BaseValue->Value : FTransform::Identity;

						FTransformAnimationAttribute Output{ BaseTransform };

						if (AdditiveValue != nullptr)
						{
							// TODO: The VAdditiveWeight is practically 1.0, can assume that it is?
							Output.Value.AccumulateWithAdditiveScale(AdditiveValue->Value, VAdditiveWeight);
						}

						OutputTransforms->Append(Name, Output);
					},
					BaseTransforms != nullptr ? BaseTransforms->CreateConstIterator() : TUnboundValueMap<FTransformAnimationAttribute>::FConstIterator(),
					AdditiveTransforms->CreateConstIterator());
			}
			else
			{
				// Interpolate with identity before adding
				OuterJoinBy(
					FUnboundValueMapJoinOp(),
					[VAdditiveWeight, OutputTransforms](FName Name, const FTransformAnimationAttribute* BaseValue, const FTransformAnimationAttribute* AdditiveValue)
					{
						// Missing values are set to the identity
						const FTransform& BaseTransform = BaseValue != nullptr ? BaseValue->Value : FTransform::Identity;

						FTransformAnimationAttribute Output{ BaseTransform };

						if (AdditiveValue != nullptr)
						{
							FTransform::BlendFromIdentityAndAccumulate(Output.Value, AdditiveValue->Value, VAdditiveWeight);
						}

						OutputTransforms->Append(Name, Output);
					},
					BaseTransforms != nullptr ? BaseTransforms->CreateConstIterator() : TUnboundValueMap<FTransformAnimationAttribute>::FConstIterator(),
					AdditiveTransforms->CreateConstIterator());
			}

			Output = OutputTransforms;
		}

		void FApplyAdditiveSpace_VectorAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the user authored default values
				const FAttributeTypedSetPtr& TypedSet = Additive->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Additive->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Additive->GetTypedSet(), Additive->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FVectorAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FApplyAdditiveSpace_VectorAttribute::TransformBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(false);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and additive should have the same shape
			check(Base->GetTypedSet() == Additive->GetTypedSet());
			check(Base->Num() == Additive->Num());

			const TBoundValueMap<FVectorAnimationAttribute>* BaseVectors = CastChecked<FVectorAnimationAttribute>(Base);
			const TBoundValueMap<FVectorAnimationAttribute>* AdditiveVectors = CastChecked<FVectorAnimationAttribute>(Additive);
			TBoundValueMap<FVectorAnimationAttribute>* OutputVectors = CastChecked<FVectorAnimationAttribute>(Output);

			const FVectorAnimationAttribute* BaseVectorPtr = BaseVectors->GetData();
			const FVectorAnimationAttribute* AdditiveVectorPtr = AdditiveVectors->GetData();
			FVectorAnimationAttribute* OutputVectorPtr = OutputVectors->GetData();
			const FVectorAnimationAttribute* BaseVectorPtrEnd = BaseVectorPtr + BaseVectors->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(Base->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(Base->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (BaseVectorPtr < BaseVectorPtrEnd)
				{
					OutputVectorPtr->Value = BaseVectorPtr->Value + (PerValueWeightsPtr->Value * AdditiveVectorPtr->Value);

					BaseVectorPtr++;
					AdditiveVectorPtr++;
					PerValueWeightsPtr++;
					OutputVectorPtr++;
				}
			}
			else
			{
				while (BaseVectorPtr < BaseVectorPtrEnd)
				{
					OutputVectorPtr->Value = BaseVectorPtr->Value + (DefaultWeight * AdditiveVectorPtr->Value);

					BaseVectorPtr++;
					AdditiveVectorPtr++;
					OutputVectorPtr++;
				}
			}
		}

		void FApplyAdditiveSpace_VectorAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FVectorAnimationAttribute>* BaseVectors = Cast<FVectorAnimationAttribute>(Base);
			const TUnboundValueMap<FVectorAnimationAttribute>* AdditiveVectors = CastChecked<FVectorAnimationAttribute>(Additive);

			TUnboundValueMap<FVectorAnimationAttribute>* OutputVectors = MakeUnboundValueMap<FVectorAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[AdditiveWeight, OutputVectors](FName Name, const FVectorAnimationAttribute* BaseValue, const FVectorAnimationAttribute* AdditiveValue)
				{
					// Missing values are set to identity
					const FVector BaseVector = BaseValue != nullptr ? BaseValue->Value : FVector::ZeroVector;
					const FVector AdditiveVector = AdditiveValue != nullptr ? AdditiveValue->Value : FVector::ZeroVector;

					const FVectorAnimationAttribute Output{ BaseVector + (AdditiveWeight * AdditiveVector) };
					OutputVectors->Append(Name, Output);
				},
				BaseVectors != nullptr ? BaseVectors->CreateConstIterator() : TUnboundValueMap<FVectorAnimationAttribute>::FConstIterator(),
				AdditiveVectors->CreateConstIterator());

			Output = OutputVectors;
		}

		void FApplyAdditiveSpace_QuaternionAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator)
		{
			if (Base == nullptr)
			{
				// Our base is missing, use the user authored default values
				const FAttributeTypedSetPtr& TypedSet = Additive->GetTypedSet();
				const FValueBundlePtr ReferenceValues = TypedSet->GetBindingData()->GetDefaultAttributeValues(TypedSet->GetNamedSet()->GetName());
				Base = ReferenceValues->GetBoundValueMaps().Find(Additive->GetMappingKey());
			}

			if (Output == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Additive->GetTypedSet(), Additive->GetValueType(), OutputAllocator);
				Output = MakeBoundValueMap<FQuaternionAnimationAttribute>(Args);
			}

			if (Base == nullptr)
			{
				UE_LOGF(LogAnimation, Warning, "FApplyAdditiveSpace_VectorAttribute::QuaternionBoundValueMap: Could not transform bound value map - no base provided and no reference found. Setting to identity instead.");
				Output->FillWithIdentity(false);
				return;
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == Output->GetTypedSet());
			check(Base->Num() == Output->Num());

			// Our base and additive should have the same shape
			check(Base->GetTypedSet() == Additive->GetTypedSet());
			check(Base->Num() == Additive->Num());

			const TBoundValueMap<FQuaternionAnimationAttribute>* BaseQuaternions = CastChecked<FQuaternionAnimationAttribute>(Base);
			const TBoundValueMap<FQuaternionAnimationAttribute>* AdditiveQuaternions = CastChecked<FQuaternionAnimationAttribute>(Additive);
			TBoundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = CastChecked<FQuaternionAnimationAttribute>(Output);

			const FQuaternionAnimationAttribute* BaseQuaternionPtr = BaseQuaternions->GetData();
			const FQuaternionAnimationAttribute* AdditiveQuaternionPtr = AdditiveQuaternions->GetData();
			FQuaternionAnimationAttribute* OutputQuaternionPtr = OutputQuaternions->GetData();
			const FQuaternionAnimationAttribute* BaseQuaternionPtrEnd = BaseQuaternionPtr + BaseQuaternions->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(Base->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(Base->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (BaseQuaternionPtr < BaseQuaternionPtrEnd)
				{
					const FQuat WeightedRotation = FQuat::FastLerp(FQuat::Identity, AdditiveQuaternionPtr->Value, PerValueWeightsPtr->Value).GetNormalized();
					OutputQuaternionPtr->Value = BaseQuaternionPtr->Value + WeightedRotation;
					
					BaseQuaternionPtr++;
					AdditiveQuaternionPtr++;
					PerValueWeightsPtr++;
					OutputQuaternionPtr++;
				}
			}
			else
			{
				while (BaseQuaternionPtr < BaseQuaternionPtrEnd)
				{
					const FQuat WeightedRotation = FQuat::FastLerp(FQuat::Identity, AdditiveQuaternionPtr->Value, DefaultWeight).GetNormalized();
					OutputQuaternionPtr->Value = BaseQuaternionPtr->Value + WeightedRotation;

					BaseQuaternionPtr++;
					AdditiveQuaternionPtr++;
					OutputQuaternionPtr++;
				}
			}
		}

		void FApplyAdditiveSpace_QuaternionAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator)
		{
			check(Output == nullptr);

			const TUnboundValueMap<FQuaternionAnimationAttribute>* BaseQuaternions = Cast<FQuaternionAnimationAttribute>(Base);
			const TUnboundValueMap<FQuaternionAnimationAttribute>* AdditiveQuaternions = CastChecked<FQuaternionAnimationAttribute>(Additive);

			TUnboundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = MakeUnboundValueMap<FQuaternionAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[AdditiveWeight, OutputQuaternions](FName Name, const FQuaternionAnimationAttribute* BaseValue, const FQuaternionAnimationAttribute* AdditiveValue)
				{
					// Missing values are set to identity
					const FQuat BaseQuaternion = BaseValue != nullptr ? BaseValue->Value : FQuat::Identity;
					const FQuat AdditiveQuaternion = AdditiveValue != nullptr ? AdditiveValue->Value : FQuat::Identity;

					const FQuat WeightedRotation = FQuat::FastLerp(FQuat::Identity, AdditiveQuaternion, AdditiveWeight).GetNormalized();

					FQuaternionAnimationAttribute Output;
					Output.Value = BaseQuaternion + WeightedRotation;

					OutputQuaternions->Append(Name, Output);
				},
				BaseQuaternions != nullptr ? BaseQuaternions->CreateConstIterator() : TUnboundValueMap<FQuaternionAnimationAttribute>::FConstIterator(),
				AdditiveQuaternions->CreateConstIterator());

			Output = OutputQuaternions;
		}
	}
}
