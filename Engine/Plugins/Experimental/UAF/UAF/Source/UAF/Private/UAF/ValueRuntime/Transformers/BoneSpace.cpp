// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/Transformers/BoneSpace.h"

#include "UAF/ValueRuntime/PoseValueBundle.h"
#include "UAF/Attributes/EngineAttributes.h"

namespace UE::UAF::Transformers
{
	void FBoneSpace::LocalToComponent(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues)
	{
		checkf(InputValues.GetValueSpace().GetType() == EValueSpaceType::Local, TEXT("Input must be in local space"));
		checkf(!InputValues.GetValueSpace().IsAdditive(), TEXT("Input space cannot be additive"));

		if (&InputValues != &OutputValues)
		{
			// Copy our input, we'll modify the bone transforms in-place
			OutputValues = InputValues;
		}

		OutputValues.SetValueSpace(FValueSpace(EValueSpaceType::Component));

		// Modify in place
		TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = OutputValues.FindBoneTransforms();
		if (BoneTransforms == nullptr)
		{
			// No bone transforms, nothing to convert
			return;
		}

		const FAttributeTypedSetPtr& BoneSet = BoneTransforms->GetTypedSet();
		const int32 NumBones = BoneTransforms->Num();

		// Root is equivalent in both spaces, skip it
		for (FAttributeSetIndex BoneSetIndex(1); BoneSetIndex < NumBones; ++BoneSetIndex)
		{
			const FAttributeSetIndex ParentBoneSetIndex = BoneSet->GetParentIndex(BoneSetIndex);
			if (ParentBoneSetIndex.IsValid())
			{
				const FBoneTransformAnimationAttribute& ParentBoneTransform = (*BoneTransforms)[ParentBoneSetIndex];
				FBoneTransformAnimationAttribute& BoneTransform = (*BoneTransforms)[BoneSetIndex];

				FTransform::Multiply(&BoneTransform.Value, &BoneTransform.Value, &ParentBoneTransform.Value);

				BoneTransform.Value.NormalizeRotation();
			}
			else
			{
				// TODO: Use the ref pose in component space
				checkf(false, TEXT("Bones must have their parent within the same set to convert from Local to Component space"));
			}
		}
	}

	void FBoneSpace::ComponentToLocal(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues)
	{
		checkf(InputValues.GetValueSpace().GetType() == EValueSpaceType::Component, TEXT("Input must be in component space"));
		checkf(!InputValues.GetValueSpace().IsAdditive(), TEXT("Input space cannot be additive"));

		if (&InputValues != &OutputValues)
		{
			// Copy our input, we'll modify the bone transforms in-place
			OutputValues = InputValues;
		}

		OutputValues.SetValueSpace(FValueSpace(EValueSpaceType::Local));

		// Modify in place
		TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = OutputValues.FindBoneTransforms();
		if (BoneTransforms == nullptr)
		{
			// No bone transforms, nothing to convert
			return;
		}

		const FAttributeTypedSetPtr& BoneSet = BoneTransforms->GetTypedSet();
		const int32 NumBones = BoneTransforms->Num();

		// Root is equivalent in both spaces, skip it
		// Iterate backwards since we need the parent transforms intact
		for (FAttributeSetIndex BoneSetIndex(NumBones - 1); BoneSetIndex > 0; --BoneSetIndex)
		{
			const FAttributeSetIndex ParentBoneSetIndex = BoneSet->GetParentIndex(BoneSetIndex);
			if (ParentBoneSetIndex.IsValid())
			{
				const FBoneTransformAnimationAttribute& ParentBoneTransform = (*BoneTransforms)[ParentBoneSetIndex];
				FBoneTransformAnimationAttribute& BoneTransform = (*BoneTransforms)[BoneSetIndex];

				BoneTransform.Value.SetToRelativeTransform(ParentBoneTransform.Value);
				BoneTransform.Value.NormalizeRotation();
			}
			else
			{
				// TODO: Use the ref pose in component space
				checkf(false, TEXT("Bones must have their parent within the same set to convert from Component to Local space"));
			}
		}
	}

	void FBoneSpace::LocalToMeshRotation(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues)
	{
		const FValueSpace InputSpace = InputValues.GetValueSpace();
		checkf(InputSpace.GetType() == EValueSpaceType::Local || InputSpace.GetType() == EValueSpaceType::Mixed, TEXT("Input must be in local or mixed space"));
		checkf(!InputSpace.IsAdditive(), TEXT("Input space cannot be additive"));

		if (&InputValues != &OutputValues)
		{
			// Copy our input, we'll modify the bone transforms in-place
			OutputValues = InputValues;
		}

		EMixedSpaceFlags MixedSpaceFlags = InputSpace.GetMixedSpaceFlags();
		checkf(!(MixedSpaceFlags & EMixedSpaceFlags::MeshRotation), TEXT("Input is already in mesh rotation space"));
		checkf(!(MixedSpaceFlags & EMixedSpaceFlags::RootRotation), TEXT("Input is already in root rotation space"));
		EnumAddFlags(MixedSpaceFlags, EMixedSpaceFlags::MeshRotation);
		OutputValues.SetValueSpace(FValueSpace(MixedSpaceFlags));

		// Modify in place
		TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = OutputValues.FindBoneTransforms();
		if (BoneTransforms == nullptr)
		{
			// No bone transforms, nothing to convert
			return;
		}

		const FAttributeTypedSetPtr& BoneSet = BoneTransforms->GetTypedSet();
		const int32 NumBones = BoneTransforms->Num();

		// Root is equivalent in both spaces, skip it
		for (FAttributeSetIndex BoneSetIndex(1); BoneSetIndex < NumBones; ++BoneSetIndex)
		{
			const FAttributeSetIndex ParentBoneSetIndex = BoneSet->GetParentIndex(BoneSetIndex);
			if (ParentBoneSetIndex.IsValid())
			{
				const FBoneTransformAnimationAttribute& ParentBoneTransform = (*BoneTransforms)[ParentBoneSetIndex];
				FBoneTransformAnimationAttribute& BoneTransform = (*BoneTransforms)[BoneSetIndex];

				BoneTransform.Value.SetRotation(ParentBoneTransform.Value.GetRotation() * BoneTransform.Value.GetRotation());
				BoneTransform.Value.NormalizeRotation();
			}
			else
			{
				// TODO: Use the ref pose in component space
				checkf(false, TEXT("Bones must have their parent within the same set to convert from Local to Mesh Rotation space"));
			}
		}
	}

	void FBoneSpace::MeshRotationToLocal(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues)
	{
		const FValueSpace InputSpace = InputValues.GetValueSpace();
		EMixedSpaceFlags MixedSpaceFlags = InputSpace.GetMixedSpaceFlags();
		checkf(InputSpace.GetType() == EValueSpaceType::Mixed && !!(MixedSpaceFlags & EMixedSpaceFlags::MeshRotation), TEXT("Input must be in mesh rotation space"));
		checkf(!InputSpace.IsAdditive(), TEXT("Input space cannot be additive"));

		if (&InputValues != &OutputValues)
		{
			// Copy our input, we'll modify the bone transforms in-place
			OutputValues = InputValues;
		}

		EnumRemoveFlags(MixedSpaceFlags, EMixedSpaceFlags::MeshRotation);

		FValueSpace OutputSpace;
		if (MixedSpaceFlags == EMixedSpaceFlags::None)
		{
			// We aren't mixed space anymore
			OutputSpace = FValueSpace(EValueSpaceType::Local);
		}
		else
		{
			OutputSpace = FValueSpace(MixedSpaceFlags);
		}
		OutputValues.SetValueSpace(OutputSpace);

		// Modify in place
		TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = OutputValues.FindBoneTransforms();
		if (BoneTransforms == nullptr)
		{
			// No bone transforms, nothing to convert
			return;
		}

		const FAttributeTypedSetPtr& BoneSet = BoneTransforms->GetTypedSet();
		const int32 NumBones = BoneTransforms->Num();

		// Root is equivalent in both spaces, skip it
		// Iterate backwards since we need the parent transforms intact
		for (FAttributeSetIndex BoneSetIndex(NumBones - 1); BoneSetIndex > 0; --BoneSetIndex)
		{
			const FAttributeSetIndex ParentBoneSetIndex = BoneSet->GetParentIndex(BoneSetIndex);
			if (ParentBoneSetIndex.IsValid())
			{
				const FBoneTransformAnimationAttribute& ParentBoneTransform = (*BoneTransforms)[ParentBoneSetIndex];
				FBoneTransformAnimationAttribute& BoneTransform = (*BoneTransforms)[BoneSetIndex];

				BoneTransform.Value.SetRotation(ParentBoneTransform.Value.GetRotation().Inverse() * BoneTransform.Value.GetRotation());
				BoneTransform.Value.NormalizeRotation();
			}
			else
			{
				// TODO: Use the ref pose in component space
				checkf(false, TEXT("Bones must have their parent within the same set to convert from Mesh Rotation to Local space"));
			}
		}
	}

	void FBoneSpace::LocalToRootRotation(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues)
	{
		const FValueSpace InputSpace = InputValues.GetValueSpace();
		checkf(InputSpace.GetType() == EValueSpaceType::Local || InputSpace.GetType() == EValueSpaceType::Mixed, TEXT("Input must be in local or mixed space"));
		checkf(!InputSpace.IsAdditive(), TEXT("Input space cannot be additive"));

		if (&InputValues != &OutputValues)
		{
			// Copy our input, we'll modify the bone transforms in-place
			OutputValues = InputValues;
		}

		EMixedSpaceFlags MixedSpaceFlags = InputSpace.GetMixedSpaceFlags();
		checkf(!(MixedSpaceFlags & EMixedSpaceFlags::MeshRotation), TEXT("Input is already in mesh rotation space"));
		checkf(!(MixedSpaceFlags & EMixedSpaceFlags::RootRotation), TEXT("Input is already in root rotation space"));
		EnumAddFlags(MixedSpaceFlags, EMixedSpaceFlags::RootRotation);
		OutputValues.SetValueSpace(FValueSpace(MixedSpaceFlags));

		// Modify in place
		TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = OutputValues.FindBoneTransforms();
		if (BoneTransforms == nullptr)
		{
			// No bone transforms, nothing to convert
			return;
		}

		const FAttributeTypedSetPtr& BoneSet = BoneTransforms->GetTypedSet();
		const int32 NumBones = BoneTransforms->Num();

		const FBoneTransformAnimationAttribute Identity{ FTransform::Identity };

		// Root is equivalent in both spaces, skip it
		for (FAttributeSetIndex BoneSetIndex(1); BoneSetIndex < NumBones; ++BoneSetIndex)
		{
			const FAttributeSetIndex ParentBoneSetIndex = BoneSet->GetParentIndex(BoneSetIndex);
			if (ParentBoneSetIndex.IsValid())
			{
				const FBoneTransformAnimationAttribute& ParentBoneTransform = ParentBoneSetIndex.IsRootBone() ? Identity : (*BoneTransforms)[ParentBoneSetIndex];
				FBoneTransformAnimationAttribute& BoneTransform = (*BoneTransforms)[BoneSetIndex];

				BoneTransform.Value.SetRotation(ParentBoneTransform.Value.GetRotation() * BoneTransform.Value.GetRotation());
				BoneTransform.Value.NormalizeRotation();
			}
			else
			{
				// TODO: Use the ref pose in component space
				checkf(false, TEXT("Bones must have their parent within the same set to convert from Local to Root Rotation space"));
			}
		}
	}

	void FBoneSpace::RootRotationToLocal(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues)
	{
		const FValueSpace InputSpace = InputValues.GetValueSpace();
		EMixedSpaceFlags MixedSpaceFlags = InputSpace.GetMixedSpaceFlags();
		checkf(InputSpace.GetType() == EValueSpaceType::Mixed && !!(MixedSpaceFlags & EMixedSpaceFlags::RootRotation), TEXT("Input must be in root rotation space"));
		checkf(!InputSpace.IsAdditive(), TEXT("Input space cannot be additive"));

		if (&InputValues != &OutputValues)
		{
			// Copy our input, we'll modify the bone transforms in-place
			OutputValues = InputValues;
		}

		EnumRemoveFlags(MixedSpaceFlags, EMixedSpaceFlags::RootRotation);

		FValueSpace OutputSpace;
		if (MixedSpaceFlags == EMixedSpaceFlags::None)
		{
			// We aren't mixed space anymore
			OutputSpace = FValueSpace(EValueSpaceType::Local);
		}
		else
		{
			OutputSpace = FValueSpace(MixedSpaceFlags);
		}
		OutputValues.SetValueSpace(OutputSpace);

		// Modify in place
		TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = OutputValues.FindBoneTransforms();
		if (BoneTransforms == nullptr)
		{
			// No bone transforms, nothing to convert
			return;
		}

		const FAttributeTypedSetPtr& BoneSet = BoneTransforms->GetTypedSet();
		const int32 NumBones = BoneTransforms->Num();

		const FBoneTransformAnimationAttribute Identity{ FTransform::Identity };

		// Root is equivalent in both spaces, skip it
		// Iterate backwards since we need the parent transforms intact
		for (FAttributeSetIndex BoneSetIndex(NumBones - 1); BoneSetIndex > 0; --BoneSetIndex)
		{
			const FAttributeSetIndex ParentBoneSetIndex = BoneSet->GetParentIndex(BoneSetIndex);
			if (ParentBoneSetIndex.IsValid())
			{
				const FBoneTransformAnimationAttribute& ParentBoneTransform = ParentBoneSetIndex.IsRootBone() ? Identity : (*BoneTransforms)[ParentBoneSetIndex];
				FBoneTransformAnimationAttribute& BoneTransform = (*BoneTransforms)[BoneSetIndex];

				BoneTransform.Value.SetRotation(ParentBoneTransform.Value.GetRotation().Inverse() * BoneTransform.Value.GetRotation());
				BoneTransform.Value.NormalizeRotation();
			}
			else
			{
				// TODO: Use the ref pose in component space
				checkf(false, TEXT("Bones must have their parent within the same set to convert from Root Rotation to Local space"));
			}
		}
	}

	void FBoneSpace::LocalToMeshScale(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues)
	{
		const FValueSpace InputSpace = InputValues.GetValueSpace();
		checkf(InputSpace.GetType() == EValueSpaceType::Local || InputSpace.GetType() == EValueSpaceType::Mixed, TEXT("Input must be in local or mixed space"));
		checkf(!InputSpace.IsAdditive(), TEXT("Input space cannot be additive"));

		if (&InputValues != &OutputValues)
		{
			// Copy our input, we'll modify the bone transforms in-place
			OutputValues = InputValues;
		}

		EMixedSpaceFlags MixedSpaceFlags = InputSpace.GetMixedSpaceFlags();
		checkf(!(MixedSpaceFlags & EMixedSpaceFlags::MeshScale), TEXT("Input is already in mesh scale space"));
		EnumAddFlags(MixedSpaceFlags, EMixedSpaceFlags::MeshScale);
		OutputValues.SetValueSpace(FValueSpace(MixedSpaceFlags));

		// Modify in place
		TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = OutputValues.FindBoneTransforms();
		if (BoneTransforms == nullptr)
		{
			// No bone transforms, nothing to convert
			return;
		}

		const FAttributeTypedSetPtr& BoneSet = BoneTransforms->GetTypedSet();
		const int32 NumBones = BoneTransforms->Num();

		const FBoneTransformAnimationAttribute Identity{ FTransform::Identity };

		// Root is equivalent in both spaces, skip it
		for (FAttributeSetIndex BoneSetIndex(1); BoneSetIndex < NumBones; ++BoneSetIndex)
		{
			const FAttributeSetIndex ParentBoneSetIndex = BoneSet->GetParentIndex(BoneSetIndex);
			if (ParentBoneSetIndex.IsValid())
			{
				const FBoneTransformAnimationAttribute& ParentBoneTransform = ParentBoneSetIndex.IsRootBone() ? Identity : (*BoneTransforms)[ParentBoneSetIndex];
				FBoneTransformAnimationAttribute& BoneTransform = (*BoneTransforms)[BoneSetIndex];

				BoneTransform.Value.SetScale3D(ParentBoneTransform.Value.GetScale3D() * BoneTransform.Value.GetScale3D());
			}
			else
			{
				// TODO: Use the ref pose in component space
				checkf(false, TEXT("Bones must have their parent within the same set to convert from Local to Mesh Scale space"));
			}
		}
	}

	void FBoneSpace::MeshScaleToLocal(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues)
	{
		const FValueSpace InputSpace = InputValues.GetValueSpace();
		EMixedSpaceFlags MixedSpaceFlags = InputSpace.GetMixedSpaceFlags();
		checkf(InputSpace.GetType() == EValueSpaceType::Mixed && !!(MixedSpaceFlags & EMixedSpaceFlags::MeshScale), TEXT("Input must be in mesh scale space"));
		checkf(!InputSpace.IsAdditive(), TEXT("Input space cannot be additive"));

		if (&InputValues != &OutputValues)
		{
			// Copy our input, we'll modify the bone transforms in-place
			OutputValues = InputValues;
		}

		EnumRemoveFlags(MixedSpaceFlags, EMixedSpaceFlags::MeshScale);

		FValueSpace OutputSpace;
		if (MixedSpaceFlags == EMixedSpaceFlags::None)
		{
			// We aren't mixed space anymore
			OutputSpace = FValueSpace(EValueSpaceType::Local);
		}
		else
		{
			OutputSpace = FValueSpace(MixedSpaceFlags);
		}
		OutputValues.SetValueSpace(OutputSpace);

		// Modify in place
		TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = OutputValues.FindBoneTransforms();
		if (BoneTransforms == nullptr)
		{
			// No bone transforms, nothing to convert
			return;
		}

		const FAttributeTypedSetPtr& BoneSet = BoneTransforms->GetTypedSet();
		const int32 NumBones = BoneTransforms->Num();

		const FBoneTransformAnimationAttribute Identity{ FTransform::Identity };

		// Root is equivalent in both spaces, skip it
		// Iterate backwards since we need the parent transforms intact
		for (FAttributeSetIndex BoneSetIndex(NumBones - 1); BoneSetIndex > 0; --BoneSetIndex)
		{
			const FAttributeSetIndex ParentBoneSetIndex = BoneSet->GetParentIndex(BoneSetIndex);
			if (ParentBoneSetIndex.IsValid())
			{
				const FBoneTransformAnimationAttribute& ParentBoneTransform = ParentBoneSetIndex.IsRootBone() ? Identity : (*BoneTransforms)[ParentBoneSetIndex];
				FBoneTransformAnimationAttribute& BoneTransform = (*BoneTransforms)[BoneSetIndex];

				const FVector ParentScaleInv = FTransform::GetSafeScaleReciprocal(ParentBoneTransform.Value.GetScale3D(), UE_SMALL_NUMBER);
				BoneTransform.Value.SetScale3D(ParentScaleInv * BoneTransform.Value.GetScale3D());
			}
			else
			{
				// TODO: Use the ref pose in component space
				checkf(false, TEXT("Bones must have their parent within the same set to convert from Local to Mesh Scale space"));
			}
		}
	}
}
