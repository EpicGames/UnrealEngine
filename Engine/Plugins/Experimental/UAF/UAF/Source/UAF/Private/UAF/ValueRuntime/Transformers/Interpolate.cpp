// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/Transformers/Interpolate.h"

#include "UAF/ValueRuntime/IteratorUtils.h"

namespace UE::UAF::Transformers
{
	FName FInterpolate::TransformerName = TEXT("UAF::Transformers::Interpolate");

	void FInterpolate::Apply(
		const FValueTransformerMapPtr& TransformerMap,
		const FValueBundle& InputA,
		const FValueBundle& InputB,
		float WeightB,
		FValueBundle& Output)
	{
		// Named sets must match as it ensures that our inputs have the same sizes/shapes
		checkf(InputA.GetNamedSet() == InputB.GetNamedSet(), TEXT("Both input collections must use the same named set"));
		checkf(InputA.GetValueSpace() == InputB.GetValueSpace(), TEXT("Both input collections must have matching value spaces: [%s] != [%s]"), *InputA.GetValueSpace().ToString(), *InputB.GetValueSpace().ToString());

		const bool bIsOutputA = &InputA == &Output;
		if (bIsOutputA && !FAnimWeight::IsRelevant(WeightB))
		{
			// Our input A is our output and B is not relevant, nothing to do
			return;
		}

		const bool bIsOutputB = &InputB == &Output;
		if (bIsOutputB && FAnimWeight::IsFullWeight(WeightB))
		{
			// Our input B is our output and A is not relevant, nothing to do
			return;
		}

		// Allocator is irrelevant since it will remain empty
		FValueBundleHeap EmptyCollection(InputA.GetNamedSet());
		FInterpolate::Apply(TransformerMap, InputA, InputB, EmptyCollection, WeightB, Output);
	}

	void FInterpolate::Apply(
		const FValueTransformerMapPtr& TransformerMap,
		const FValueBundle& InputA,
		const FValueBundle& InputB,
		const FValueBundle& PerValueWeightsB,
		float DefaultWeightB,
		FValueBundle& Output)
	{
		// Named sets must match as it ensures that our inputs have the same sizes/shapes
		checkf(InputA.GetNamedSet() == InputB.GetNamedSet(), TEXT("Both input collections must use the same named set"));
		checkf(InputA.GetNamedSet() == PerValueWeightsB.GetNamedSet(), TEXT("Input and per-value weight collections must use the same named set"));
		checkf(InputA.GetValueSpace() == InputB.GetValueSpace(), TEXT("Both input collections must have matching value spaces: [%s] != [%s]"), *InputA.GetValueSpace().ToString(), *InputB.GetValueSpace().ToString());

		const FValueTransformerList* TransformerList = TransformerMap->Find(TransformerName);
		if (TransformerList == nullptr)
		{
			// No transformer implementations have been registered
			return;
		}

		UScriptStruct* FloatValueType = FFloatAnimationAttribute::StaticStruct();

		const bool bIsOutputA = &InputA == &Output;
		const bool bIsOutputB = &InputB == &Output;
		if (bIsOutputA || bIsOutputB)
		{
			// Our output collection matches one of our inputs, we'll modify it in place
			FReallocFun OutputAllocator = Output.GetAllocator();

			// Bound value maps
			{
				const FBoundMapCollection& PerValueWeightBMaps = PerValueWeightsB.GetBoundValueMaps();

				// Keep a list of set maps that needs to be appended to our output
				TArray<FBoundValueMap*, TInlineAllocator<8>> PendingMaps;

				OuterJoinBy(
					FValueTransformerListWithBoundMapCollectionJoinOp(),
					[&PerValueWeightBMaps, DefaultWeightB, bIsOutputA, FloatValueType, OutputAllocator, &PendingMaps](FRawTransformerFunc Transformer, FBoundValueMap* MapA, FBoundValueMap* MapB)
					{
						if (MapA == nullptr)
						{
							if (MapB == nullptr)
							{
								// No maps found for the transformer type
							}
							else
							{
								// A doesn't have a matching pair
								if (bIsOutputA)
								{
									// A is our output, duplicate it
									PendingMaps.Add(MapB->Duplicate(OutputAllocator));
								}
								else
								{
									// B is our output, nothing to do
								}
							}
						}
						else
						{
							if (MapB == nullptr)
							{
								// B doesn't have a matching pair
								if (bIsOutputA)
								{
									// A is our output, nothing to do
								}
								else
								{
									// B is our output, duplicate it
									PendingMaps.Add(MapA->Duplicate(OutputAllocator));
								}
							}
							else
							{
								if (Transformer == nullptr)
								{
									// No transformer was found for this value type, retain the one with the largest weight
									if (DefaultWeightB >= 0.5f)
									{
										// B has more weight
										if (bIsOutputA)
										{
											// A is our output, copy it over
											MapB->CopyTo(*MapA);
										}
										else
										{
											// B is our output, nothing to do
										}
									}
									else
									{
										// A has more weight
										if (bIsOutputA)
										{
											// A is our output, nothing to do
										}
										else
										{
											// B is our output, copy it over
											MapA->CopyTo(*MapB);
										}
									}
								}
								else
								{
									// Transform our matching pair in place
									FBoundValueMap* OutputMap = bIsOutputA ? MapA : MapB;

									// We cannot join our weights because their value type differs
									const FAttributeMappingKey MappingKey = MapA->GetMappingKey().To(FloatValueType);
									const FBoundValueMap* PerValueWeightMapB = PerValueWeightBMaps.Find(MappingKey);

									FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
									(*TypedTransformer)(MapA, MapB, PerValueWeightMapB, DefaultWeightB, OutputMap, nullptr);
								}
							}
						}
					},
					TransformerList->CreateBoundValueMapTransformerIterator(),
					// Remove the const qualifier since we'll mutate one in place as our output
					const_cast<FValueBundle&>(InputA).GetBoundValueMaps().CreateIterator(),
					const_cast<FValueBundle&>(InputB).GetBoundValueMaps().CreateIterator());

				// Append our pending entries
				FBoundMapCollection& OutputMaps = Output.GetBoundValueMaps();
				for (FBoundValueMap* Map : PendingMaps)
				{
					OutputMaps.Add(Map);
				}
			}

			// Name/value maps
			{
				// Keep a list of maps that needs to be appended to our output
				TArray<FUnboundValueMap*, TInlineAllocator<8>> PendingMaps;

				OuterJoinBy(
					FValueTransformerListWithUnboundMapCollectionJoinOp(),
					[DefaultWeightB, bIsOutputA, OutputAllocator, &PendingMaps](FRawTransformerFunc Transformer, FUnboundValueMap* MapA, FUnboundValueMap* MapB)
					{
						if (MapA == nullptr)
						{
							if (MapB == nullptr)
							{
								// No maps found for the transformer type
							}
							else
							{
								// A doesn't have a matching pair
								if (bIsOutputA)
								{
									// A is our output, duplicate it
									PendingMaps.Add(MapB->Duplicate(OutputAllocator));
								}
								else
								{
									// B is our output, nothing to do
								}
							}
						}
						else
						{
							if (MapB == nullptr)
							{
								// B doesn't have a matching pair
								if (bIsOutputA)
								{
									// A is our output, nothing to do
								}
								else
								{
									// B is our output, duplicate it
									PendingMaps.Add(MapA->Duplicate(OutputAllocator));
								}
							}
							else
							{
								if (Transformer == nullptr)
								{
									// No transformer was found for this value type, retain the one with the largest weight
									if (DefaultWeightB >= 0.5f)
									{
										// B has more weight
										if (bIsOutputA)
										{
											// A is our output, copy it over
											MapB->CopyTo(*MapA);
										}
										else
										{
											// B is our output, nothing to do
										}
									}
									else
									{
										// A has more weight
										if (bIsOutputA)
										{
											// A is our output, nothing to do
										}
										else
										{
											// B is our output, copy it over
											MapA->CopyTo(*MapB);
										}
									}
								}
								else
								{
									// Transform our matching pair in a new map
									FUnboundValueMap* OutputMap = nullptr;

									FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
									(*TypedTransformer)(MapA, MapB, DefaultWeightB, OutputMap, OutputAllocator);

									checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
									checkf(OutputMap->GetAllocator() == OutputAllocator, TEXT("Transformer function is responsible for allocating using the output allocator"));

									// Overwrite our old output
									OutputMap->MoveTo(bIsOutputA ? *MapA : *MapB);

									// Destroy our temporary output map
									ReleaseUnboundValueMap(OutputMap);
								}
							}
						}
					},
					TransformerList->CreateUnboundValueMapTransformerIterator(),
					// Remove the const qualifier since we'll mutate one in place as our output
					const_cast<FValueBundle&>(InputA).GetUnboundValueMaps().CreateIterator(),
					const_cast<FValueBundle&>(InputB).GetUnboundValueMaps().CreateIterator());

				// Append our pending entries
				FUnboundMapCollection& OutputMaps = Output.GetUnboundValueMaps();
				for (FUnboundValueMap* Map : PendingMaps)
				{
					OutputMaps.Add(Map);
				}
			}
		}
		else
		{
			// Our output collection isn't either of our inputs, reset it
			Output.Reset(InputA.GetNamedSet());
			Output.SetValueSpace(InputA.GetValueSpace());

			// Bound value maps
			{
				const FBoundMapCollection& PerValueWeightBMaps = PerValueWeightsB.GetBoundValueMaps();
				FBoundMapCollection& OutputMaps = Output.GetBoundValueMaps();

				OuterJoinBy(
					FValueTransformerListWithBoundMapCollectionJoinOp(),
					[&PerValueWeightBMaps, DefaultWeightB, FloatValueType, &OutputMaps](FRawTransformerFunc Transformer, const FBoundValueMap* MapA, const FBoundValueMap* MapB)
					{
						if (MapA == nullptr)
						{
							if (MapB == nullptr)
							{
								// No maps found for the transformer type
							}
							else
							{
								// This set map doesn't have a matching pair, duplicate it
								OutputMaps.Append(MapB->Duplicate(OutputMaps.GetAllocator()));
							}
						}
						else
						{
							if (MapB == nullptr)
							{
								// This set map doesn't have a matching pair, duplicate it
								OutputMaps.Append(MapA->Duplicate(OutputMaps.GetAllocator()));
							}
							else
							{
								if (Transformer == nullptr)
								{
									// No transformer was found for this value type, retain the one with the largest weight
									FBoundValueMap* MapCopy;
									if (DefaultWeightB >= 0.5f)
									{
										MapCopy = MapB->Duplicate(OutputMaps.GetAllocator());
									}
									else
									{
										MapCopy = MapA->Duplicate(OutputMaps.GetAllocator());
									}

									OutputMaps.Append(MapCopy);
								}
								else
								{
									// Transform our matching pair
									FBoundValueMap* OutputMap = nullptr;

									// We cannot join our weights because their value type differs
									const FAttributeMappingKey MappingKey = MapA->GetMappingKey().To(FloatValueType);
									const FBoundValueMap* PerValueWeightMapB = PerValueWeightBMaps.Find(MappingKey);

									FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
									(*TypedTransformer)(MapA, MapB, PerValueWeightMapB, DefaultWeightB, OutputMap, OutputMaps.GetAllocator());

									checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
									checkf(OutputMap->GetAllocator() == OutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
									OutputMaps.Append(OutputMap);
								}
							}
						}
					},
					TransformerList->CreateBoundValueMapTransformerIterator(),
					InputA.GetBoundValueMaps().CreateConstIterator(),
					InputB.GetBoundValueMaps().CreateConstIterator());
			}

			// Name/value maps
			{
				FUnboundMapCollection& OutputMaps = Output.GetUnboundValueMaps();

				OuterJoinBy(
					FValueTransformerListWithUnboundMapCollectionJoinOp(),
					[DefaultWeightB, &OutputMaps](FRawTransformerFunc Transformer, const FUnboundValueMap* MapA, const FUnboundValueMap* MapB)
					{
						if (MapA == nullptr)
						{
							if (MapB == nullptr)
							{
								// No maps found for the transformer type
							}
							else
							{
								// This map doesn't have a matching pair, duplicate it
								OutputMaps.Append(MapB->Duplicate(OutputMaps.GetAllocator()));
							}
						}
						else
						{
							if (MapB == nullptr)
							{
								// This map doesn't have a matching pair
								// Our output is A and thus already contains this
							}
							else
							{
								if (Transformer == nullptr)
								{
									// No transformer was found for this value type, retain the one with the largest weight
									FUnboundValueMap* MapCopy;
									if (DefaultWeightB >= 0.5f)
									{
										MapCopy = MapB->Duplicate(OutputMaps.GetAllocator());
									}
									else
									{
										MapCopy = MapA->Duplicate(OutputMaps.GetAllocator());
									}

									OutputMaps.Append(MapCopy);
								}
								else
								{
									// Transform our matching pair in place
									FUnboundValueMap* OutputMap = nullptr;

									FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
									(*TypedTransformer)(MapA, MapB, DefaultWeightB, OutputMap, OutputMaps.GetAllocator());

									checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
									checkf(OutputMap->GetAllocator() == OutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
									OutputMaps.Append(OutputMap);
								}
							}
						}
					},
					TransformerList->CreateUnboundValueMapTransformerIterator(),
					InputA.GetUnboundValueMaps().CreateConstIterator(),
					InputB.GetUnboundValueMaps().CreateConstIterator());
			}
		}
	}

	namespace Private
	{
		void FInterpolate_BoneTransformAttribute::TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(MapA->GetTypedSet() == MapB->GetTypedSet());
			check(MapA->Num() == MapB->Num());

			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(MapA->GetTypedSet(), MapA->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FBoneTransformAnimationAttribute>(Args);
			}

			check(MapA->GetTypedSet() == OutputMap->GetTypedSet());
			check(MapA->Num() == OutputMap->Num());

			const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransformsA = CastChecked<FBoneTransformAnimationAttribute>(MapA);
			const TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransformsB = CastChecked<FBoneTransformAnimationAttribute>(MapB);
			TBoundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(OutputMap);

			const FBoneTransformAnimationAttribute* TransformPtrA = BoneTransformsA->GetData();
			const FBoneTransformAnimationAttribute* TransformPtrB = BoneTransformsB->GetData();
			FBoneTransformAnimationAttribute* OutputTransformPtr = OutputBoneTransforms->GetData();
			const FBoneTransformAnimationAttribute* TransformPtrAEnd = TransformPtrA + BoneTransformsA->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightsB = Cast<FFloatAnimationAttribute>(PerValueWeightMapB))
			{
				check(MapA->GetTypedSet() == PerValueWeightsB->GetTypedSet());
				check(MapA->Num() == PerValueWeightsB->Num());

				const FFloatAnimationAttribute* PerValueWeightsBPtr = PerValueWeightsB->GetData();

				while (TransformPtrA < TransformPtrAEnd)
				{
					OutputTransformPtr->Value.Blend(TransformPtrA->Value, TransformPtrB->Value, PerValueWeightsBPtr->Value);

					TransformPtrA++;
					TransformPtrB++;
					PerValueWeightsBPtr++;
					OutputTransformPtr++;
				}
			}
			else
			{
				while (TransformPtrA < TransformPtrAEnd)
				{
					OutputTransformPtr->Value.Blend(TransformPtrA->Value, TransformPtrB->Value, DefaultWeightB);

					TransformPtrA++;
					TransformPtrB++;
					OutputTransformPtr++;
				}
			}
		}

		void FInterpolate_BoneTransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneTransformsA = CastChecked<FBoneTransformAnimationAttribute>(MapA);
			const TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneTransformsB = CastChecked<FBoneTransformAnimationAttribute>(MapB);
			TUnboundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = MakeUnboundValueMap<FBoneTransformAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[WeightB, OutputBoneTransforms](FName Name, const FBoneTransformAnimationAttribute* A, const FBoneTransformAnimationAttribute* B)
				{
					// Missing values are set to the identity
					const FTransform& ValueA = A != nullptr ? A->Value : FTransform::Identity;
					const FTransform& ValueB = B != nullptr ? B->Value : FTransform::Identity;

					FBoneTransformAnimationAttribute Output;
					Output.Value.Blend(ValueA, ValueB, WeightB);

					OutputBoneTransforms->Append(Name, Output);
				},
				BoneTransformsA->CreateConstIterator(),
				BoneTransformsB->CreateConstIterator());

			OutputMap = OutputBoneTransforms;
		}

		void FInterpolate_FloatAttribute::TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(MapA->GetTypedSet() == MapB->GetTypedSet());
			check(MapA->Num() == MapB->Num());

			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(MapA->GetTypedSet(), MapA->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FFloatAnimationAttribute>(Args);
			}

			check(MapA->GetTypedSet() == OutputMap->GetTypedSet());
			check(MapA->Num() == OutputMap->Num());

			const TBoundValueMap<FFloatAnimationAttribute>* FloatsA = CastChecked<FFloatAnimationAttribute>(MapA);
			const TBoundValueMap<FFloatAnimationAttribute>* FloatsB = CastChecked<FFloatAnimationAttribute>(MapB);
			TBoundValueMap<FFloatAnimationAttribute>* OutputFloats = CastChecked<FFloatAnimationAttribute>(OutputMap);

			const FFloatAnimationAttribute* FloatsPtrA = FloatsA->GetData();
			const FFloatAnimationAttribute* FloatsPtrB = FloatsB->GetData();
			FFloatAnimationAttribute* OutputFloatsPtr = OutputFloats->GetData();
			const FFloatAnimationAttribute* FloatsPtrAEnd = FloatsPtrA + FloatsA->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightsB = Cast<FFloatAnimationAttribute>(PerValueWeightMapB))
			{
				check(MapA->GetTypedSet() == PerValueWeightsB->GetTypedSet());
				check(MapA->Num() == PerValueWeightsB->Num());

				const FFloatAnimationAttribute* PerValueWeightsBPtr = PerValueWeightsB->GetData();

				while (FloatsPtrA < FloatsPtrAEnd)
				{
					OutputFloatsPtr->Value = FMath::Lerp(FloatsPtrA->Value, FloatsPtrB->Value, PerValueWeightsBPtr->Value);

					FloatsPtrA++;
					FloatsPtrB++;
					PerValueWeightsBPtr++;
					OutputFloatsPtr++;
				}
			}
			else
			{
				while (FloatsPtrA < FloatsPtrAEnd)
				{
					OutputFloatsPtr->Value = FMath::Lerp(FloatsPtrA->Value, FloatsPtrB->Value, DefaultWeightB);

					FloatsPtrA++;
					FloatsPtrB++;
					OutputFloatsPtr++;
				}
			}
		}

		void FInterpolate_FloatAttribute::TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FFloatAnimationAttribute>* FloatsA = CastChecked<FFloatAnimationAttribute>(MapA);
			const TUnboundValueMap<FFloatAnimationAttribute>* FloatsB = CastChecked<FFloatAnimationAttribute>(MapB);
			TUnboundValueMap<FFloatAnimationAttribute>* OutputFloats = MakeUnboundValueMap<FFloatAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[WeightB, OutputFloats](FName Name, const FFloatAnimationAttribute* A, const FFloatAnimationAttribute* B)
				{
					// Missing values are set to 0.0
					const float ValueA = A != nullptr ? A->Value : 0.0f;
					const float ValueB = B != nullptr ? B->Value : 0.0f;

					FFloatAnimationAttribute Output;
					Output.Value = FMath::Lerp(ValueA, ValueB, WeightB);

					OutputFloats->Append(Name, Output);
				},
				FloatsA->CreateConstIterator(),
				FloatsB->CreateConstIterator());

			OutputMap = OutputFloats;
		}

		void FInterpolate_IntegerAttribute::TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(MapA->GetTypedSet() == MapB->GetTypedSet());
			check(MapA->Num() == MapB->Num());

			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(MapA->GetTypedSet(), MapA->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FIntegerAnimationAttribute>(Args);
			}

			check(MapA->GetTypedSet() == OutputMap->GetTypedSet());
			check(MapA->Num() == OutputMap->Num());

			const TBoundValueMap<FIntegerAnimationAttribute>* IntegersA = CastChecked<FIntegerAnimationAttribute>(MapA);
			const TBoundValueMap<FIntegerAnimationAttribute>* IntegersB = CastChecked<FIntegerAnimationAttribute>(MapB);
			TBoundValueMap<FIntegerAnimationAttribute>* OutputIntegers = CastChecked<FIntegerAnimationAttribute>(OutputMap);

			const FIntegerAnimationAttribute* IntegersPtrA = IntegersA->GetData();
			const FIntegerAnimationAttribute* IntegersPtrB = IntegersB->GetData();
			FIntegerAnimationAttribute* OutputIntegersPtr = OutputIntegers->GetData();
			const FIntegerAnimationAttribute* IntegersPtrAEnd = IntegersPtrA + IntegersA->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightsB = Cast<FFloatAnimationAttribute>(PerValueWeightMapB))
			{
				check(MapA->GetTypedSet() == PerValueWeightsB->GetTypedSet());
				check(MapA->Num() == PerValueWeightsB->Num());

				const FFloatAnimationAttribute* PerValueWeightsBPtr = PerValueWeightsB->GetData();

				while (IntegersPtrA < IntegersPtrAEnd)
				{
					OutputIntegersPtr->Value = FMath::TruncToInt32(IntegersPtrA->Value * (1.0 - PerValueWeightsBPtr->Value) + (IntegersPtrB->Value * PerValueWeightsBPtr->Value));

					IntegersPtrA++;
					IntegersPtrB++;
					PerValueWeightsBPtr++;
					OutputIntegersPtr++;
				}
			}
			else
			{
				// Use doubles to reduce precision loss during weighted sum
				const double WeightBDouble = DefaultWeightB;
				const double WeightADouble = 1.0 - DefaultWeightB;
				
				while (IntegersPtrA < IntegersPtrAEnd)
				{
					OutputIntegersPtr->Value = FMath::TruncToInt32(IntegersPtrA->Value * WeightADouble + IntegersPtrB->Value * WeightBDouble);
					
					IntegersPtrA++;
					IntegersPtrB++;
					OutputIntegersPtr++;
				}
			}
		}

		void FInterpolate_IntegerAttribute::TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FIntegerAnimationAttribute>* IntegersA = CastChecked<FIntegerAnimationAttribute>(MapA);
			const TUnboundValueMap<FIntegerAnimationAttribute>* IntegersB = CastChecked<FIntegerAnimationAttribute>(MapB);
			TUnboundValueMap<FIntegerAnimationAttribute>* OutputIntegers = MakeUnboundValueMap<FIntegerAnimationAttribute>(OutputAllocator);

			const double WeightBDouble = WeightB;
			const double WeightADouble = 1.0 - WeightB;
			
			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[WeightADouble, WeightBDouble, OutputIntegers](FName Name, const FIntegerAnimationAttribute* A, const FIntegerAnimationAttribute* B)
				{
					// Missing values are set to 0
					const int32 ValueA = A != nullptr ? A->Value : 0;
					const int32 ValueB = B != nullptr ? B->Value : 0;

					FIntegerAnimationAttribute Output;
					Output.Value = FMath::TruncToInt32(ValueA * WeightADouble + ValueB * WeightBDouble);

					OutputIntegers->Append(Name, Output);
				},
				IntegersA->CreateConstIterator(),
				IntegersB->CreateConstIterator());

			OutputMap = OutputIntegers;
		}

		void FInterpolate_TransformAttribute::TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(MapA->GetTypedSet() == MapB->GetTypedSet());
			check(MapA->Num() == MapB->Num());

			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(MapA->GetTypedSet(), MapA->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FTransformAnimationAttribute>(Args);
			}

			check(MapA->GetTypedSet() == OutputMap->GetTypedSet());
			check(MapA->Num() == OutputMap->Num());

			const TBoundValueMap<FTransformAnimationAttribute>* TransformsA = CastChecked<FTransformAnimationAttribute>(MapA);
			const TBoundValueMap<FTransformAnimationAttribute>* TransformsB = CastChecked<FTransformAnimationAttribute>(MapB);
			TBoundValueMap<FTransformAnimationAttribute>* OutputTransforms = CastChecked<FTransformAnimationAttribute>(OutputMap);

			const FTransformAnimationAttribute* TransformPtrA = TransformsA->GetData();
			const FTransformAnimationAttribute* TransformPtrB = TransformsB->GetData();
			FTransformAnimationAttribute* OutputTransformPtr = OutputTransforms->GetData();
			const FTransformAnimationAttribute* TransformPtrAEnd = TransformPtrA + TransformsA->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightsB = Cast<FFloatAnimationAttribute>(PerValueWeightMapB))
			{
				check(MapA->GetTypedSet() == PerValueWeightsB->GetTypedSet());
				check(MapA->Num() == PerValueWeightsB->Num());

				const FFloatAnimationAttribute* PerValueWeightsBPtr = PerValueWeightsB->GetData();

				while (TransformPtrA < TransformPtrAEnd)
				{
					OutputTransformPtr->Value.Blend(TransformPtrA->Value, TransformPtrB->Value, PerValueWeightsBPtr->Value);

					TransformPtrA++;
					TransformPtrB++;
					PerValueWeightsBPtr++;
					OutputTransformPtr++;
				}
			}
			else
			{
				while (TransformPtrA < TransformPtrAEnd)
				{
					OutputTransformPtr->Value.Blend(TransformPtrA->Value, TransformPtrB->Value, DefaultWeightB);

					TransformPtrA++;
					TransformPtrB++;
					OutputTransformPtr++;
				}
			}
		}

		void FInterpolate_TransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FTransformAnimationAttribute>* TransformsA = CastChecked<FTransformAnimationAttribute>(MapA);
			const TUnboundValueMap<FTransformAnimationAttribute>* TransformsB = CastChecked<FTransformAnimationAttribute>(MapB);
			TUnboundValueMap<FTransformAnimationAttribute>* OutputTransforms = MakeUnboundValueMap<FTransformAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[WeightB, OutputTransforms](FName Name, const FTransformAnimationAttribute* A, const FTransformAnimationAttribute* B)
				{
					// Missing values are set to the identity
					const FTransform& ValueA = A != nullptr ? A->Value : FTransform::Identity;
					const FTransform& ValueB = B != nullptr ? B->Value : FTransform::Identity;

					FTransformAnimationAttribute Output;
					Output.Value.Blend(ValueA, ValueB, WeightB);

					OutputTransforms->Append(Name, Output);
				},
				TransformsA->CreateConstIterator(),
				TransformsB->CreateConstIterator());

			OutputMap = OutputTransforms;
		}

		void FInterpolate_VectorAttribute::TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(MapA->GetTypedSet() == MapB->GetTypedSet());
			check(MapA->Num() == MapB->Num());

			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(MapA->GetTypedSet(), MapA->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FVectorAnimationAttribute>(Args);
			}

			check(MapA->GetTypedSet() == OutputMap->GetTypedSet());
			check(MapA->Num() == OutputMap->Num());

			const TBoundValueMap<FVectorAnimationAttribute>* VectorsA = CastChecked<FVectorAnimationAttribute>(MapA);
			const TBoundValueMap<FVectorAnimationAttribute>* VectorsB = CastChecked<FVectorAnimationAttribute>(MapB);
			TBoundValueMap<FVectorAnimationAttribute>* OutputVectors = CastChecked<FVectorAnimationAttribute>(OutputMap);

			const FVectorAnimationAttribute* VectorPtrA = VectorsA->GetData();
			const FVectorAnimationAttribute* VectorPtrB = VectorsB->GetData();
			FVectorAnimationAttribute* OutputVectorPtr = OutputVectors->GetData();
			const FVectorAnimationAttribute* VectorPtrAEnd = VectorPtrA + VectorsA->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightsB = Cast<FFloatAnimationAttribute>(PerValueWeightMapB))
			{
				check(MapA->GetTypedSet() == PerValueWeightsB->GetTypedSet());
				check(MapA->Num() == PerValueWeightsB->Num());

				const FFloatAnimationAttribute* PerValueWeightsBPtr = PerValueWeightsB->GetData();

				while (VectorPtrA < VectorPtrAEnd)
				{
					OutputVectorPtr->Value = VectorPtrA->Value * (1.0f - PerValueWeightsBPtr->Value) + VectorPtrB->Value * PerValueWeightsBPtr->Value;

					VectorPtrA++;
					VectorPtrB++;
					PerValueWeightsBPtr++;
					OutputVectorPtr++;
				}
			}
			else
			{
				while (VectorPtrA < VectorPtrAEnd)
				{
					OutputVectorPtr->Value = VectorPtrA->Value * (1.0f - DefaultWeightB) + VectorPtrB->Value * DefaultWeightB;

					VectorPtrA++;
					VectorPtrB++;
					OutputVectorPtr++;
				}
			}
		}

		void FInterpolate_VectorAttribute::TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FVectorAnimationAttribute>* VectorsA = CastChecked<FVectorAnimationAttribute>(MapA);
			const TUnboundValueMap<FVectorAnimationAttribute>* VectorsB = CastChecked<FVectorAnimationAttribute>(MapB);
			TUnboundValueMap<FVectorAnimationAttribute>* OutputVectors = MakeUnboundValueMap<FVectorAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[WeightB, OutputVectors](FName Name, const FVectorAnimationAttribute* A, const FVectorAnimationAttribute* B)
				{
					// Missing values are set to the identity
					const FVector& ValueA = A != nullptr ? A->Value : FVector::ZeroVector;
					const FVector& ValueB = B != nullptr ? B->Value : FVector::ZeroVector;

					FVectorAnimationAttribute Output;
					Output.Value = ValueA * (1.0f - WeightB) + ValueB * WeightB;

					OutputVectors->Append(Name, Output);
				},
				VectorsA->CreateConstIterator(),
				VectorsB->CreateConstIterator());

			OutputMap = OutputVectors;
		}

		void FInterpolate_QuaternionAttribute::TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(MapA->GetTypedSet() == MapB->GetTypedSet());
			check(MapA->Num() == MapB->Num());

			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(MapA->GetTypedSet(), MapA->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FQuaternionAnimationAttribute>(Args);
			}

			check(MapA->GetTypedSet() == OutputMap->GetTypedSet());
			check(MapA->Num() == OutputMap->Num());

			const TBoundValueMap<FQuaternionAnimationAttribute>* QuaternionsA = CastChecked<FQuaternionAnimationAttribute>(MapA);
			const TBoundValueMap<FQuaternionAnimationAttribute>* QuaternionsB = CastChecked<FQuaternionAnimationAttribute>(MapB);
			TBoundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = CastChecked<FQuaternionAnimationAttribute>(OutputMap);

			const FQuaternionAnimationAttribute* QuaternionPtrA = QuaternionsA->GetData();
			const FQuaternionAnimationAttribute* QuaternionPtrB = QuaternionsB->GetData();
			FQuaternionAnimationAttribute* OutputQuaternionPtr = OutputQuaternions->GetData();
			const FQuaternionAnimationAttribute* QuaternionPtrAEnd = QuaternionPtrA + QuaternionsA->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightsB = Cast<FFloatAnimationAttribute>(PerValueWeightMapB))
			{
				check(MapA->GetTypedSet() == PerValueWeightsB->GetTypedSet());
				check(MapA->Num() == PerValueWeightsB->Num());

				const FFloatAnimationAttribute* PerValueWeightsBPtr = PerValueWeightsB->GetData();

				while (QuaternionPtrA < QuaternionPtrAEnd)
				{
					OutputQuaternionPtr->Value = FQuat::FastLerp(QuaternionPtrA->Value, QuaternionPtrB->Value, PerValueWeightsBPtr->Value);

					QuaternionPtrA++;
					QuaternionPtrB++;
					PerValueWeightsBPtr++;
					OutputQuaternionPtr++;
				}
			}
			else
			{
				while (QuaternionPtrA < QuaternionPtrAEnd)
				{
					OutputQuaternionPtr->Value = FQuat::FastLerp(QuaternionPtrA->Value, QuaternionPtrB->Value, DefaultWeightB);

					QuaternionPtrA++;
					QuaternionPtrB++;
					OutputQuaternionPtr++;
				}
			}
		}

		void FInterpolate_QuaternionAttribute::TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FQuaternionAnimationAttribute>* QuaternionsA = CastChecked<FQuaternionAnimationAttribute>(MapA);
			const TUnboundValueMap<FQuaternionAnimationAttribute>* QuaternionsB = CastChecked<FQuaternionAnimationAttribute>(MapB);
			TUnboundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = MakeUnboundValueMap<FQuaternionAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[WeightB, OutputQuaternions](FName Name, const FQuaternionAnimationAttribute* A, const FQuaternionAnimationAttribute* B)
				{
					// Missing values are set to the identity
					const FQuat& ValueA = A != nullptr ? A->Value : FQuat::Identity;
					const FQuat& ValueB = B != nullptr ? B->Value : FQuat::Identity;

					FQuaternionAnimationAttribute Output;
					Output.Value = FQuat::FastLerp(ValueA, ValueB, WeightB);

					OutputQuaternions->Append(Name, Output);
				},
				QuaternionsA->CreateConstIterator(),
				QuaternionsB->CreateConstIterator());

			OutputMap = OutputQuaternions;
		}
	}
}
