// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/Transformers/Layer.h"

#include "UAF/ValueRuntime/IteratorUtils.h"

namespace UE::UAF::Transformers
{
	FName FLayer::TransformerName = TEXT("UAF::Transformers::Layer");

	void FLayer::Apply(
		const FValueTransformerMapPtr& TransformerMap,
		const FValueBundle& Base,
		const FValueBundle& Layer,
		float LayerWeight,
		FValueBundle& Output)
	{
		const bool bIsOutputBase = &Base == &Output;
		if (bIsOutputBase && !FAnimWeight::IsRelevant(LayerWeight))
		{
			// Our input base is our output and the layer is not relevant, nothing to do
			return;
		}

		// Allocator is irrelevant since it will remain empty
		FValueBundleHeap EmptyCollection(Base.GetNamedSet());
		FLayer::Apply(TransformerMap, Base, Layer, EmptyCollection, LayerWeight, Output);
	}

	void FLayer::Apply(
		const FValueTransformerMapPtr& TransformerMap,
		const FValueBundle& Base,
		const FValueBundle& Layer,
		const FValueBundle& PerValueLayerWeights,
		float DefaultLayerWeight,
		FValueBundle& Output)
	{
		const FValueTransformerList* TransformerList = TransformerMap->Find(TransformerName);
		if (TransformerList == nullptr)
		{
			// No transformer implementations have been registered
			return;
		}

		const float LayerWeight = FMath::Clamp(DefaultLayerWeight, 0.0f, 1.0f);

		FValueBundleStack LayerCopy;
		const FValueBundle* LayerPtr = &Layer;

		const bool bIsOutputBase = &Base == &Output;
		bool bIsOutputLayer = &Layer == &Output;
		if (bIsOutputLayer)
		{
			// Our output is the layer, we can only modify it in place if it has the same set as our base
			if (Base.GetNamedSet() == Layer.GetNamedSet())
			{
				// We'll modify it in place
			}
			else
			{
				// Duplicate our layer
				LayerCopy = Layer;
				LayerPtr = &LayerCopy;
				bIsOutputLayer = false;
			}
		}

		UScriptStruct* FloatValueType = FFloatAnimationAttribute::StaticStruct();

		if (bIsOutputBase || bIsOutputLayer)
		{
			// Our output collection matches one of our inputs, we'll modify it in place
			FReallocFun OutputAllocator = Output.GetAllocator();

			const FBoundMapCollection& PerValueLayerWeightMaps = PerValueLayerWeights.GetBoundValueMaps();

			// Bound value maps
			OuterJoinBy(
				FValueTransformerListWithBoundMapCollectionJoinOp(),
				[&PerValueLayerWeightMaps, LayerWeight, bIsOutputBase, FloatValueType, OutputAllocator](FRawTransformerFunc Transformer, FBoundValueMap* BaseMap, FBoundValueMap* LayerMap)
				{
					if (BaseMap == nullptr)
					{
						if (LayerMap == nullptr)
						{
							// No maps found for the transformer type
						}
						else
						{
							// Base doesn't have a matching pair, there is nothing to layer on
						}
					}
					else
					{
						if (LayerMap == nullptr)
						{
							// Layer doesn't have a matching pair, there is nothing to layer with
						}
						else
						{
							if (Transformer == nullptr)
							{
								// No transformer was found for this value type, retain the one with the largest weight
								if (LayerWeight >= 0.5f)
								{
									// Layer has more weight
									if (bIsOutputBase)
									{
										// Base is our output, copy it over
										LayerMap->CopyTo(*BaseMap);
									}
									else
									{
										// Layer is our output, nothing to do
									}
								}
								else
								{
									// Base has more weight
									if (bIsOutputBase)
									{
										// Base is our output, nothing to do
									}
									else
									{
										// Layer is our output, copy it over
										BaseMap->CopyTo(*LayerMap);
									}
								}
							}
							else
							{
								// Transform our matching pair in place
								FBoundValueMap* OutputMap = bIsOutputBase ? BaseMap : LayerMap;

								// We cannot join our weights because their value type differs
								const FAttributeMappingKey MappingKey = BaseMap->GetMappingKey().To(FloatValueType);
								const FBoundValueMap* PerValueWeightMap = PerValueLayerWeightMaps.Find(MappingKey);

								FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
								(*TypedTransformer)(BaseMap, LayerMap, PerValueWeightMap, LayerWeight, OutputMap, nullptr);
							}
						}
					}
				},
				TransformerList->CreateBoundValueMapTransformerIterator(),
				// Remove the const qualifier since we'll mutate one in place as our output
				const_cast<FValueBundle&>(Base).GetBoundValueMaps().CreateIterator(),
				const_cast<FValueBundle&>(Layer).GetBoundValueMaps().CreateIterator());

			// Name/value maps
			OuterJoinBy(
				FValueTransformerListWithUnboundMapCollectionJoinOp(),
				[LayerWeight, bIsOutputBase, OutputAllocator](FRawTransformerFunc Transformer, FUnboundValueMap* BaseMap, FUnboundValueMap* LayerMap)
				{
					if (BaseMap == nullptr)
					{
						if (LayerMap == nullptr)
						{
							// No maps found for the transformer type
						}
						else
						{
							// Base doesn't have a matching pair, there is nothing to layer on
						}
					}
					else
					{
						if (LayerMap == nullptr)
						{
							// Layer doesn't have a matching pair, there is nothing to layer with
						}
						else
						{
							if (Transformer == nullptr)
							{
								// No transformer was found for this value type, retain the one with the largest weight
								if (LayerWeight >= 0.5f)
								{
									// B has more weight
									if (bIsOutputBase)
									{
										// A is our output, copy it over
										LayerMap->CopyTo(*BaseMap);
									}
									else
									{
										// B is our output, nothing to do
									}
								}
								else
								{
									// A has more weight
									if (bIsOutputBase)
									{
										// A is our output, nothing to do
									}
									else
									{
										// B is our output, copy it over
										BaseMap->CopyTo(*LayerMap);
									}
								}
							}
							else
							{
								// Transform our matching pair in a new map
								FUnboundValueMap* OutputMap = nullptr;

								FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
								(*TypedTransformer)(BaseMap, LayerMap, LayerWeight, OutputMap, OutputAllocator);

								checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
								checkf(OutputMap->GetAllocator() == OutputAllocator, TEXT("Transformer function is responsible for allocating using the output allocator"));

								// Overwrite our old output
								OutputMap->MoveTo(bIsOutputBase ? *BaseMap : *LayerMap);

								// Destroy our temporary output map
								ReleaseUnboundValueMap(OutputMap);
							}
						}
					}
				},
				TransformerList->CreateUnboundValueMapTransformerIterator(),
				// Remove the const qualifier since we'll mutate one in place as our output
				const_cast<FValueBundle&>(Base).GetUnboundValueMaps().CreateIterator(),
				const_cast<FValueBundle&>(Layer).GetUnboundValueMaps().CreateIterator());
		}
		else
		{
			// Our output collection isn't either of our inputs, reset it
			Output.Reset(Base.GetNamedSet());
			Output.SetValueSpace(Base.GetValueSpace());

			// Bound value maps
			{
				const FBoundMapCollection& PerValueLayerWeightMaps = PerValueLayerWeights.GetBoundValueMaps();
				FBoundMapCollection& OutputMaps = Output.GetBoundValueMaps();

				OuterJoinBy(
					FValueTransformerListWithBoundMapCollectionJoinOp(),
					[&PerValueLayerWeightMaps, LayerWeight, FloatValueType, &OutputMaps](FRawTransformerFunc Transformer, const FBoundValueMap* BaseMap, const FBoundValueMap* LayerMap)
					{
						if (BaseMap == nullptr)
						{
							// Nothing to layer into, nothing to do
						}
						else
						{
							if (LayerMap == nullptr)
							{
								// No layer map, nothing to do apart from duplicating unmodified base into output
								OutputMaps.Append(BaseMap->Duplicate(OutputMaps.GetAllocator()));
							}
							else
							{
								if (Transformer == nullptr)
								{
									// No transformer was found for this value type, retain the one with the largest weight
									FBoundValueMap* MapCopy;
									if (LayerWeight >= 0.5f)
									{
										MapCopy = LayerMap->Duplicate(OutputMaps.GetAllocator());
									}
									else
									{
										MapCopy = BaseMap->Duplicate(OutputMaps.GetAllocator());
									}

									OutputMaps.Append(MapCopy);
								}
								else
								{
									// Transform our matching pair
									FBoundValueMap* OutputMap = nullptr;

									// We cannot join our weights because their value type differs
									const FAttributeMappingKey MappingKey = BaseMap->GetMappingKey().To(FloatValueType);
									const FBoundValueMap* PerValueWeightMap = PerValueLayerWeightMaps.Find(MappingKey);

									FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
									(*TypedTransformer)(BaseMap, LayerMap, PerValueWeightMap, LayerWeight, OutputMap, OutputMaps.GetAllocator());

									checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
									checkf(OutputMap->GetAllocator() == OutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
									OutputMaps.Append(OutputMap);
								}
							}
						}
					},
					TransformerList->CreateBoundValueMapTransformerIterator(),
					Base.GetBoundValueMaps().CreateConstIterator(),
					LayerPtr->GetBoundValueMaps().CreateConstIterator());
			}

			// Name/value maps
			{
				FUnboundMapCollection& OutputMaps = Output.GetUnboundValueMaps();

				OuterJoinBy(
					FValueTransformerListWithUnboundMapCollectionJoinOp(),
					[LayerWeight, &OutputMaps](FRawTransformerFunc Transformer, const FUnboundValueMap* BaseMap, const FUnboundValueMap* LayerMap)
					{
						if (BaseMap == nullptr)
						{
							// Nothing to layer into, nothing to do
						}
						else
						{
							if (LayerMap == nullptr)
							{
								// No layer map, nothing to do apart from duplicating unmodified base into output
								OutputMaps.Append(BaseMap->Duplicate(OutputMaps.GetAllocator()));
							}
							else
							{
								if (Transformer == nullptr)
								{
									// No transformer was found for this value type, retain the one with the largest weight
									FUnboundValueMap* MapCopy;
									if (LayerWeight >= 0.5f)
									{
										MapCopy = LayerMap->Duplicate(OutputMaps.GetAllocator());
									}
									else
									{
										MapCopy = BaseMap->Duplicate(OutputMaps.GetAllocator());
									}

									OutputMaps.Append(MapCopy);
								}
								else
								{
									// Transform our matching pair in place
									FUnboundValueMap* OutputMap = nullptr;

									FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
									(*TypedTransformer)(BaseMap, LayerMap, LayerWeight, OutputMap, OutputMaps.GetAllocator());

									checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
									checkf(OutputMap->GetAllocator() == OutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
									OutputMaps.Append(OutputMap);
								}
							}
						}
					},
					TransformerList->CreateUnboundValueMapTransformerIterator(),
					Base.GetUnboundValueMaps().CreateConstIterator(),
					LayerPtr->GetUnboundValueMaps().CreateConstIterator());
			}
		}
	}

	namespace Private
	{
		void FLayer_BoneTransformAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Base->GetTypedSet(), Base->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FBoneTransformAnimationAttribute>(Args);
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == OutputMap->GetTypedSet());
			check(Base->Num() == OutputMap->Num());

			// Our per-value weights should have the same shape as our base
			check(PerValueWeights == nullptr || Base->GetTypedSet() == PerValueWeights->GetTypedSet());
			check(PerValueWeights == nullptr || Base->Num() == PerValueWeights->Num());

			const FAttributeTypedSetPtr& BaseTypedSet = Base->GetTypedSet();
			const FAttributeTypedSetPtr& LayerTypedSet = Layer->GetTypedSet();
			const FAttributeTypedSet* PerValueWeightTypedSet = PerValueWeights != nullptr ? PerValueWeights->GetTypedSet().GetReference() : nullptr;

			const TBoundValueMap<FBoneTransformAnimationAttribute>* BaseBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Base);
			const TBoundValueMap<FBoneTransformAnimationAttribute>* LayerBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Layer);
			const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightFloats = PerValueWeights != nullptr ? CastChecked<FFloatAnimationAttribute>(PerValueWeights) : nullptr;
			TBoundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(OutputMap);

			// Use an outer join over our base, layer, and weights
			// If a value is in the base but not the layer, use the base value
			// If a value is not in the base but in the layer, ignore it
			// If a value is in both, blend it
			//
			// Efficiency of an outer join wouldn't be great and would be difficult to port so ISPC
			// Instead we iterate over the base set and use the binding index to get layer values and weights

			const int32 NumBaseBones = BaseTypedSet->Num();
			for (FAttributeSetIndex BaseSetIndex(0); BaseSetIndex < NumBaseBones; ++BaseSetIndex)
			{
				const FAttributeBindingIndex BaseBindingIndex = BaseTypedSet->GetBindingIndex(BaseSetIndex);
				check(BaseBindingIndex.IsValid());

				FTransform Result = (*BaseBoneTransforms)[BaseSetIndex].Value;

				const FAttributeSetIndex LayerSetIndex = LayerTypedSet->GetIndex(BaseBindingIndex);
				if (LayerSetIndex.IsValid())
				{
					float ValueWeight = DefaultLayerWeight;
					if (PerValueWeightTypedSet != nullptr)
					{
						const FAttributeSetIndex PerWeightSetIndex = PerValueWeightTypedSet->GetIndex(BaseBindingIndex);
						if (PerWeightSetIndex.IsValid())
						{
							ValueWeight = (*PerValueWeightFloats)[PerWeightSetIndex].Value;
							ValueWeight = FMath::Clamp(ValueWeight, 0.0f, 1.0f);
						}
					}

					Result.BlendWith((*LayerBoneTransforms)[LayerSetIndex].Value, ValueWeight);
				}

				(*OutputBoneTransforms)[BaseSetIndex].Value = Result;
			}
		}

		void FLayer_BoneTransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FBoneTransformAnimationAttribute>* BaseBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Base);
			const TUnboundValueMap<FBoneTransformAnimationAttribute>* LayerBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Layer);

			TUnboundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = MakeUnboundValueMap<FBoneTransformAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[LayerWeight, OutputBoneTransforms](FName Name, const FBoneTransformAnimationAttribute* BaseValue, const FBoneTransformAnimationAttribute* LayerValue)
				{
					// Mismatched pairs are ignored
					if (BaseValue != nullptr && LayerValue != nullptr)
					{
						FBoneTransformAnimationAttribute Output = *BaseValue;
						Output.Value.BlendWith(LayerValue->Value, LayerWeight);

						OutputBoneTransforms->Append(Name, Output);
					}
				},
				BaseBoneTransforms->CreateConstIterator(),
				LayerBoneTransforms->CreateConstIterator());

			OutputMap = OutputBoneTransforms;
		}

		void FLayer_FloatAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Base->GetTypedSet(), Base->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FFloatAnimationAttribute>(Args);
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == OutputMap->GetTypedSet());
			check(Base->Num() == OutputMap->Num());

			// Our per-value weights should have the same shape as our base
			check(PerValueWeights == nullptr || Base->GetTypedSet() == PerValueWeights->GetTypedSet());
			check(PerValueWeights == nullptr || Base->Num() == PerValueWeights->Num());

			const FAttributeTypedSetPtr& BaseTypedSet = Base->GetTypedSet();
			const FAttributeTypedSetPtr& LayerTypedSet = Layer->GetTypedSet();
			const FAttributeTypedSet* PerValueWeightTypedSet = PerValueWeights != nullptr ? PerValueWeights->GetTypedSet().GetReference() : nullptr;

			const TBoundValueMap<FFloatAnimationAttribute>* BaseFloats = CastChecked<FFloatAnimationAttribute>(Base);
			const TBoundValueMap<FFloatAnimationAttribute>* LayerFloats = CastChecked<FFloatAnimationAttribute>(Layer);
			const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightFloats = PerValueWeights != nullptr ? CastChecked<FFloatAnimationAttribute>(PerValueWeights) : nullptr;
			TBoundValueMap<FFloatAnimationAttribute>* OutputFloats = CastChecked<FFloatAnimationAttribute>(OutputMap);

			// Use an outer join over our base, layer, and weights
			// If a value is in the base but not the layer, use the base value
			// If a value is not in the base but in the layer, ignore it
			// If a value is in both, blend it
			//
			// Efficiency of an outer join wouldn't be great and would be difficult to port so ISPC
			// Instead we iterate over the base set and use the binding index to get layer values and weights

			const int32 NumBaseBones = BaseTypedSet->Num();
			for (FAttributeSetIndex BaseSetIndex(0); BaseSetIndex < NumBaseBones; ++BaseSetIndex)
			{
				const FAttributeBindingIndex BaseBindingIndex = BaseTypedSet->GetBindingIndex(BaseSetIndex);
				check(BaseBindingIndex.IsValid());

				float Result = (*BaseFloats)[BaseSetIndex].Value;

				const FAttributeSetIndex LayerSetIndex = LayerTypedSet->GetIndex(BaseBindingIndex);
				if (LayerSetIndex.IsValid())
				{
					float ValueWeight = DefaultLayerWeight;
					if (PerValueWeightTypedSet != nullptr)
					{
						const FAttributeSetIndex PerWeightSetIndex = PerValueWeightTypedSet->GetIndex(BaseBindingIndex);
						if (PerWeightSetIndex.IsValid())
						{
							ValueWeight = (*PerValueWeightFloats)[PerWeightSetIndex].Value;
							ValueWeight = FMath::Clamp(ValueWeight, 0.0f, 1.0f);
						}
					}

					const float LayerValue = (*LayerFloats)[LayerSetIndex].Value;
					Result = FMath::Lerp(Result, LayerValue, ValueWeight);
				}

				(*OutputFloats)[BaseSetIndex].Value = Result;
			}
		}

		void FLayer_FloatAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FFloatAnimationAttribute>* BaseFloats = CastChecked<FFloatAnimationAttribute>(Base);
			const TUnboundValueMap<FFloatAnimationAttribute>* LayerFloats = CastChecked<FFloatAnimationAttribute>(Layer);

			TUnboundValueMap<FFloatAnimationAttribute>* OutputFloats = MakeUnboundValueMap<FFloatAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[LayerWeight, OutputFloats](FName Name, const FFloatAnimationAttribute* BaseValue, const FFloatAnimationAttribute* LayerValue)
				{
					// Mismatched pairs are ignored
					if (BaseValue != nullptr && LayerValue != nullptr)
					{
						FFloatAnimationAttribute Output{ FMath::Lerp(BaseValue->Value, LayerValue->Value, LayerWeight) };
						OutputFloats->Append(Name, Output);
					}
					else if (BaseValue != nullptr)
					{
						OutputFloats->Append(Name, *BaseValue);
					}
				},
				BaseFloats->CreateConstIterator(),
				LayerFloats->CreateConstIterator());

			OutputMap = OutputFloats;
		}

		void FLayer_IntegerAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Base->GetTypedSet(), Base->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FIntegerAnimationAttribute>(Args);
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == OutputMap->GetTypedSet());
			check(Base->Num() == OutputMap->Num());

			// Our per-value weights should have the same shape as our base
			check(PerValueWeights == nullptr || Base->GetTypedSet() == PerValueWeights->GetTypedSet());
			check(PerValueWeights == nullptr || Base->Num() == PerValueWeights->Num());

			const FAttributeTypedSetPtr& BaseTypedSet = Base->GetTypedSet();
			const FAttributeTypedSetPtr& LayerTypedSet = Layer->GetTypedSet();
			const FAttributeTypedSet* PerValueWeightTypedSet = PerValueWeights != nullptr ? PerValueWeights->GetTypedSet().GetReference() : nullptr;

			const TBoundValueMap<FIntegerAnimationAttribute>* BaseIntegers = CastChecked<FIntegerAnimationAttribute>(Base);
			const TBoundValueMap<FIntegerAnimationAttribute>* LayerIntegers = CastChecked<FIntegerAnimationAttribute>(Layer);
			const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightIntegers = PerValueWeights != nullptr ? CastChecked<FFloatAnimationAttribute>(PerValueWeights) : nullptr;
			TBoundValueMap<FIntegerAnimationAttribute>* OutputIntegers = CastChecked<FIntegerAnimationAttribute>(OutputMap);

			// Use an outer join over our base, layer, and weights
			// If a value is in the base but not the layer, use the base value
			// If a value is not in the base but in the layer, ignore it
			// If a value is in both, blend it
			//
			// Efficiency of an outer join wouldn't be great and would be difficult to port so ISPC
			// Instead we iterate over the base set and use the binding index to get layer values and weights

			const int32 NumBaseBones = BaseTypedSet->Num();
			for (FAttributeSetIndex BaseSetIndex(0); BaseSetIndex < NumBaseBones; ++BaseSetIndex)
			{
				const FAttributeBindingIndex BaseBindingIndex = BaseTypedSet->GetBindingIndex(BaseSetIndex);
				check(BaseBindingIndex.IsValid());

				int32 Result = (*BaseIntegers)[BaseSetIndex].Value;

				const FAttributeSetIndex LayerSetIndex = LayerTypedSet->GetIndex(BaseBindingIndex);
				if (LayerSetIndex.IsValid())
				{
					double LayerWeight = DefaultLayerWeight;
					
					if (PerValueWeightTypedSet != nullptr)
					{
						const FAttributeSetIndex PerWeightSetIndex = PerValueWeightTypedSet->GetIndex(BaseBindingIndex);
						if (PerWeightSetIndex.IsValid())
						{
							LayerWeight = (*PerValueWeightIntegers)[PerWeightSetIndex].Value;
							LayerWeight = FMath::Clamp(LayerWeight, 0.0, 1.0);
						}
					}

					double BaseWeight = 1.0 - LayerWeight;
					const int32 LayerValue = (*LayerIntegers)[LayerSetIndex].Value;

					Result = FMath::TruncToInt32(Result * BaseWeight + LayerValue * LayerWeight);
				}

				(*OutputIntegers)[BaseSetIndex].Value = Result;
			}
		}

		void FLayer_IntegerAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FIntegerAnimationAttribute>* BaseIntegers = CastChecked<FIntegerAnimationAttribute>(Base);
			const TUnboundValueMap<FIntegerAnimationAttribute>* LayerIntegers = CastChecked<FIntegerAnimationAttribute>(Layer);

			TUnboundValueMap<FIntegerAnimationAttribute>* OutputIntegers = MakeUnboundValueMap<FIntegerAnimationAttribute>(OutputAllocator);

			const double LayerWeightDouble = LayerWeight;
			const double BaseWeightDouble = 1.0 - LayerWeight;
			
			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[LayerWeightDouble, BaseWeightDouble, OutputIntegers](FName Name, const FIntegerAnimationAttribute* BaseValue, const FIntegerAnimationAttribute* LayerValue)
				{
					// Mismatched pairs are ignored
					if (BaseValue != nullptr && LayerValue != nullptr)
					{
						int32 Result = FMath::TruncToInt32(BaseValue->Value * BaseWeightDouble + LayerValue->Value * LayerWeightDouble);
						
						OutputIntegers->Append(Name, FIntegerAnimationAttribute { Result });
					}
				},
				BaseIntegers->CreateConstIterator(),
				LayerIntegers->CreateConstIterator());

			OutputMap = OutputIntegers;
		}

		void FLayer_TransformAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Base->GetTypedSet(), Base->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FTransformAnimationAttribute>(Args);
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == OutputMap->GetTypedSet());
			check(Base->Num() == OutputMap->Num());

			// Our per-value weights should have the same shape as our base
			check(PerValueWeights == nullptr || Base->GetTypedSet() == PerValueWeights->GetTypedSet());
			check(PerValueWeights == nullptr || Base->Num() == PerValueWeights->Num());

			const FAttributeTypedSetPtr& BaseTypedSet = Base->GetTypedSet();
			const FAttributeTypedSetPtr& LayerTypedSet = Layer->GetTypedSet();
			const FAttributeTypedSet* PerValueWeightTypedSet = PerValueWeights != nullptr ? PerValueWeights->GetTypedSet().GetReference() : nullptr;

			const TBoundValueMap<FTransformAnimationAttribute>* BaseTransforms = CastChecked<FTransformAnimationAttribute>(Base);
			const TBoundValueMap<FTransformAnimationAttribute>* LayerTransforms = CastChecked<FTransformAnimationAttribute>(Layer);
			const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightFloats = PerValueWeights != nullptr ? CastChecked<FFloatAnimationAttribute>(PerValueWeights) : nullptr;
			TBoundValueMap<FTransformAnimationAttribute>* OutputTransforms = CastChecked<FTransformAnimationAttribute>(OutputMap);

			// Use an outer join over our base, layer, and weights
			// If a value is in the base but not the layer, use the base value
			// If a value is not in the base but in the layer, ignore it
			// If a value is in both, blend it
			//
			// Efficiency of an outer join wouldn't be great and would be difficult to port so ISPC
			// Instead we iterate over the base set and use the binding index to get layer values and weights

			const int32 NumBaseBones = BaseTypedSet->Num();
			for (FAttributeSetIndex BaseSetIndex(0); BaseSetIndex < NumBaseBones; ++BaseSetIndex)
			{
				const FAttributeBindingIndex BaseBindingIndex = BaseTypedSet->GetBindingIndex(BaseSetIndex);
				check(BaseBindingIndex.IsValid());

				FTransform Result = (*BaseTransforms)[BaseSetIndex].Value;

				const FAttributeSetIndex LayerSetIndex = LayerTypedSet->GetIndex(BaseBindingIndex);
				if (LayerSetIndex.IsValid())
				{
					float ValueWeight = DefaultLayerWeight;
					if (PerValueWeightTypedSet != nullptr)
					{
						const FAttributeSetIndex PerWeightSetIndex = PerValueWeightTypedSet->GetIndex(BaseBindingIndex);
						if (PerWeightSetIndex.IsValid())
						{
							ValueWeight = (*PerValueWeightFloats)[PerWeightSetIndex].Value;
							ValueWeight = FMath::Clamp(ValueWeight, 0.0f, 1.0f);
						}
					}

					Result.BlendWith((*LayerTransforms)[LayerSetIndex].Value, ValueWeight);
				}

				(*OutputTransforms)[BaseSetIndex].Value = Result;
			}
		}

		void FLayer_TransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FTransformAnimationAttribute>* BaseTransforms = CastChecked<FTransformAnimationAttribute>(Base);
			const TUnboundValueMap<FTransformAnimationAttribute>* LayerTransforms = CastChecked<FTransformAnimationAttribute>(Layer);

			TUnboundValueMap<FTransformAnimationAttribute>* OutputBoneTransforms = MakeUnboundValueMap<FTransformAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[LayerWeight, OutputBoneTransforms](FName Name, const FTransformAnimationAttribute* BaseValue, const FTransformAnimationAttribute* LayerValue)
				{
					// Mismatched pairs are ignored
					if (BaseValue != nullptr && LayerValue != nullptr)
					{
						FTransformAnimationAttribute Output = *BaseValue;
						Output.Value.BlendWith(LayerValue->Value, LayerWeight);

						OutputBoneTransforms->Append(Name, Output);
					}
				},
				BaseTransforms->CreateConstIterator(),
				LayerTransforms->CreateConstIterator());

			OutputMap = OutputBoneTransforms;
		}

		void FLayer_VectorAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Base->GetTypedSet(), Base->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FVectorAnimationAttribute>(Args);
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == OutputMap->GetTypedSet());
			check(Base->Num() == OutputMap->Num());

			// Our per-value weights should have the same shape as our base
			check(PerValueWeights == nullptr || Base->GetTypedSet() == PerValueWeights->GetTypedSet());
			check(PerValueWeights == nullptr || Base->Num() == PerValueWeights->Num());

			const FAttributeTypedSetPtr& BaseTypedSet = Base->GetTypedSet();
			const FAttributeTypedSetPtr& LayerTypedSet = Layer->GetTypedSet();
			const FAttributeTypedSet* PerValueWeightTypedSet = PerValueWeights != nullptr ? PerValueWeights->GetTypedSet().GetReference() : nullptr;

			const TBoundValueMap<FVectorAnimationAttribute>* BaseVectors = CastChecked<FVectorAnimationAttribute>(Base);
			const TBoundValueMap<FVectorAnimationAttribute>* LayerVectors = CastChecked<FVectorAnimationAttribute>(Layer);
			const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightFloats = PerValueWeights != nullptr ? CastChecked<FFloatAnimationAttribute>(PerValueWeights) : nullptr;
			TBoundValueMap<FVectorAnimationAttribute>* OutputVectors = CastChecked<FVectorAnimationAttribute>(OutputMap);

			// Use an outer join over our base, layer, and weights
			// If a value is in the base but not the layer, use the base value
			// If a value is not in the base but in the layer, ignore it
			// If a value is in both, blend it
			//
			// Efficiency of an outer join wouldn't be great and would be difficult to port so ISPC
			// Instead we iterate over the base set and use the binding index to get layer values and weights

			const int32 NumBaseBones = BaseTypedSet->Num();
			for (FAttributeSetIndex BaseSetIndex(0); BaseSetIndex < NumBaseBones; ++BaseSetIndex)
			{
				const FAttributeBindingIndex BaseBindingIndex = BaseTypedSet->GetBindingIndex(BaseSetIndex);
				check(BaseBindingIndex.IsValid());

				FVector Result = (*BaseVectors)[BaseSetIndex].Value;

				const FAttributeSetIndex LayerSetIndex = LayerTypedSet->GetIndex(BaseBindingIndex);
				if (LayerSetIndex.IsValid())
				{
					float ValueWeight = DefaultLayerWeight;
					if (PerValueWeightTypedSet != nullptr)
					{
						const FAttributeSetIndex PerWeightSetIndex = PerValueWeightTypedSet->GetIndex(BaseBindingIndex);
						if (PerWeightSetIndex.IsValid())
						{
							ValueWeight = (*PerValueWeightFloats)[PerWeightSetIndex].Value;
							ValueWeight = FMath::Clamp(ValueWeight, 0.0f, 1.0f);
						}
					}

					Result *= (1.0f - ValueWeight);
					Result += (*LayerVectors)[LayerSetIndex].Value * ValueWeight;
				}

				(*OutputVectors)[BaseSetIndex].Value = Result;
			}
		}

		void FLayer_VectorAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FVectorAnimationAttribute>* BaseVectors = CastChecked<FVectorAnimationAttribute>(Base);
			const TUnboundValueMap<FVectorAnimationAttribute>* LayerVectors = CastChecked<FVectorAnimationAttribute>(Layer);

			TUnboundValueMap<FVectorAnimationAttribute>* OutputVectors = MakeUnboundValueMap<FVectorAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[LayerWeight, OutputVectors](FName Name, const FVectorAnimationAttribute* BaseValue, const FVectorAnimationAttribute* LayerValue)
				{
					// Mismatched pairs are ignored
					if (BaseValue != nullptr && LayerValue != nullptr)
					{
						FVectorAnimationAttribute Output = *BaseValue;
						Output.Value *= (1.0f - LayerWeight);
						Output.Value += LayerValue->Value * LayerWeight;

						OutputVectors->Append(Name, Output);
					}
					else if (BaseValue != nullptr)
					{
						OutputVectors->Append(Name, *BaseValue);
					}
				},
				BaseVectors->CreateConstIterator(),
				LayerVectors->CreateConstIterator());

			OutputMap = OutputVectors;
		}

		void FLayer_QuaternionAttribute::TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(Base->GetTypedSet(), Base->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FQuaternionAnimationAttribute>(Args);
			}

			// Our output should have the same shape as our base
			check(Base->GetTypedSet() == OutputMap->GetTypedSet());
			check(Base->Num() == OutputMap->Num());

			// Our per-value weights should have the same shape as our base
			check(PerValueWeights == nullptr || Base->GetTypedSet() == PerValueWeights->GetTypedSet());
			check(PerValueWeights == nullptr || Base->Num() == PerValueWeights->Num());

			const FAttributeTypedSetPtr& BaseTypedSet = Base->GetTypedSet();
			const FAttributeTypedSetPtr& LayerTypedSet = Layer->GetTypedSet();
			const FAttributeTypedSet* PerValueWeightTypedSet = PerValueWeights != nullptr ? PerValueWeights->GetTypedSet().GetReference() : nullptr;

			const TBoundValueMap<FQuaternionAnimationAttribute>* BaseQuaternions = CastChecked<FQuaternionAnimationAttribute>(Base);
			const TBoundValueMap<FQuaternionAnimationAttribute>* LayerQuaternions = CastChecked<FQuaternionAnimationAttribute>(Layer);
			const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeightFloats = PerValueWeights != nullptr ? CastChecked<FFloatAnimationAttribute>(PerValueWeights) : nullptr;
			TBoundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = CastChecked<FQuaternionAnimationAttribute>(OutputMap);

			// Use an outer join over our base, layer, and weights
			// If a value is in the base but not the layer, use the base value
			// If a value is not in the base but in the layer, ignore it
			// If a value is in both, blend it
			//
			// Efficiency of an outer join wouldn't be great and would be difficult to port so ISPC
			// Instead we iterate over the base set and use the binding index to get layer values and weights

			const int32 NumBaseBones = BaseTypedSet->Num();
			for (FAttributeSetIndex BaseSetIndex(0); BaseSetIndex < NumBaseBones; ++BaseSetIndex)
			{
				const FAttributeBindingIndex BaseBindingIndex = BaseTypedSet->GetBindingIndex(BaseSetIndex);
				check(BaseBindingIndex.IsValid());

				FQuat Result = (*BaseQuaternions)[BaseSetIndex].Value;

				const FAttributeSetIndex LayerSetIndex = LayerTypedSet->GetIndex(BaseBindingIndex);
				if (LayerSetIndex.IsValid())
				{
					float ValueWeight = DefaultLayerWeight;
					if (PerValueWeightTypedSet != nullptr)
					{
						const FAttributeSetIndex PerWeightSetIndex = PerValueWeightTypedSet->GetIndex(BaseBindingIndex);
						if (PerWeightSetIndex.IsValid())
						{
							ValueWeight = (*PerValueWeightFloats)[PerWeightSetIndex].Value;
							ValueWeight = FMath::Clamp(ValueWeight, 0.0f, 1.0f);
						}
					}

					Result = FQuat::FastLerp(Result, (*LayerQuaternions)[LayerSetIndex].Value, ValueWeight);
				}

				(*OutputQuaternions)[BaseSetIndex].Value = Result;
			}
		}

		void FLayer_QuaternionAttribute::TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			check(OutputMap == nullptr);

			const TUnboundValueMap<FQuaternionAnimationAttribute>* BaseQuaternions = CastChecked<FQuaternionAnimationAttribute>(Base);
			const TUnboundValueMap<FQuaternionAnimationAttribute>* LayerQuaternions = CastChecked<FQuaternionAnimationAttribute>(Layer);

			TUnboundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = MakeUnboundValueMap<FQuaternionAnimationAttribute>(OutputAllocator);

			OuterJoinBy(
				FUnboundValueMapJoinOp(),
				[LayerWeight, OutputQuaternions](FName Name, const FQuaternionAnimationAttribute* BaseValue, const FQuaternionAnimationAttribute* LayerValue)
				{
					// Mismatched pairs are ignored
					if (BaseValue != nullptr && LayerValue != nullptr)
					{
						FQuaternionAnimationAttribute Output = *BaseValue;
						Output.Value = FQuat::FastLerp(Output.Value, LayerValue->Value, LayerWeight);

						OutputQuaternions->Append(Name, Output);
					}
				},
				BaseQuaternions->CreateConstIterator(),
				LayerQuaternions->CreateConstIterator());

			OutputMap = OutputQuaternions;
		}
	}
}
