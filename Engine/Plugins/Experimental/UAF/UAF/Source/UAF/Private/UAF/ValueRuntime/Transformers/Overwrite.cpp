// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/Transformers/Overwrite.h"

#include "UAF/ValueRuntime/IteratorUtils.h"

namespace UE::UAF::Transformers
{
	FName FOverwrite::TransformerName = TEXT("UAF::Transformers::Overwrite");

	void FOverwrite::Apply(const FValueTransformerMapPtr& TransformerMap, const FValueBundle& Input, float Weight, FValueBundle& Output)
	{
		const bool bIsInputOutput = &Input == &Output;
		if (bIsInputOutput && FAnimWeight::IsFullWeight(Weight))
		{
			// Our input is our output and it has full weight, nothing to do
			return;
		}

		// Allocator is irrelevant since it will remain empty
		FValueBundleHeap EmptyCollection(Input.GetNamedSet());
		FOverwrite::Apply(TransformerMap, Input, EmptyCollection, Weight, Output);
	}

	void FOverwrite::Apply(const FValueTransformerMapPtr& TransformerMap, const FValueBundle& Input, const FValueBundle& PerValueWeights, float DefaultWeight, FValueBundle& Output)
	{
		checkf(Input.GetNamedSet() == PerValueWeights.GetNamedSet(), TEXT("Input and per-value weight collections must use the same named set"));

		const FValueTransformerList* TransformerList = TransformerMap->Find(TransformerName);
		if (TransformerList == nullptr)
		{
			// No transformer implementations have been registered
			return;
		}

		UScriptStruct* FloatValueType = FFloatAnimationAttribute::StaticStruct();

		const bool bIsInputOutput = &Input == &Output;
		if (bIsInputOutput)
		{
			// Our output collection matches one of our inputs, we'll modify it in place
			FReallocFun OutputAllocator = Output.GetAllocator();

			const FBoundMapCollection& PerValueWeightMaps = PerValueWeights.GetBoundValueMaps();

			// Bound value maps
			{
				OuterJoinBy(
					FValueTransformerListWithBoundMapCollectionJoinOp(),
					[&PerValueWeightMaps, DefaultWeight, FloatValueType, OutputAllocator](FRawTransformerFunc Transformer, FBoundValueMap* InputMap)
					{
						if (InputMap == nullptr)
						{
							// No maps found for the transformer type
						}
						else
						{
							if (Transformer == nullptr)
							{
								// No transformer was found for this value type, retain it as-is
							}
							else
							{
								// Transform our input in place
								FBoundValueMap* OutputMap = InputMap;

								// We cannot join our weights because their value type differs
								const FAttributeMappingKey MappingKey = InputMap->GetMappingKey().To(FloatValueType);
								const FBoundValueMap* PerValueWeightMap = PerValueWeightMaps.Find(MappingKey);

								FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
								(*TypedTransformer)(InputMap, PerValueWeightMap, DefaultWeight, OutputMap, nullptr);
							}
						}
					},
					TransformerList->CreateBoundValueMapTransformerIterator(),
					// Our input is our output
					Output.GetBoundValueMaps().CreateIterator());
			}

			// Unbound value maps
			{
				OuterJoinBy(
					FValueTransformerListWithUnboundMapCollectionJoinOp(),
					[DefaultWeight, OutputAllocator](FRawTransformerFunc Transformer, FUnboundValueMap* InputMap)
					{
						if (InputMap == nullptr)
						{
							// No maps found for the transformer type
						}
						else
						{
							if (Transformer == nullptr)
							{
								// No transformer was found for this value type, retain it as-is
							}
							else
							{
								// Transform our input in place
								FUnboundValueMap* OutputMap = InputMap;

								FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
								(*TypedTransformer)(InputMap, DefaultWeight, OutputMap, nullptr);
							}
						}
					},
					TransformerList->CreateUnboundValueMapTransformerIterator(),
					// Our input is our output
					Output.GetUnboundValueMaps().CreateIterator());
			}
		}
		else
		{
			// Our output collection isn't either of our inputs, reset it
			Output.Reset(Input.GetNamedSet());
			Output.SetValueSpace(Input.GetValueSpace());

			// Bound value maps
			{
				const FBoundMapCollection& PerValueWeightMaps = PerValueWeights.GetBoundValueMaps();
				FBoundMapCollection& OutputMaps = Output.GetBoundValueMaps();

				OuterJoinBy(
					FValueTransformerListWithBoundMapCollectionJoinOp(),
					[&PerValueWeightMaps, DefaultWeight, FloatValueType, &OutputMaps](FRawTransformerFunc Transformer, const FBoundValueMap* InputMap)
					{
						if (InputMap == nullptr)
						{
							// No maps found for the transformer type
						}
						else
						{
							if (Transformer == nullptr)
							{
								// No transformer was found for this value type, retain it as-is
								OutputMaps.Append(InputMap->Duplicate(OutputMaps.GetAllocator()));
							}
							else
							{
								// Transform our matching pair
								FBoundValueMap* OutputMap = nullptr;

								// We cannot join our weights because their value type differs
								const FAttributeMappingKey MappingKey = InputMap->GetMappingKey().To(FloatValueType);
								const FBoundValueMap* PerValueWeightMap = PerValueWeightMaps.Find(MappingKey);

								FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
								(*TypedTransformer)(InputMap, PerValueWeightMap, DefaultWeight, OutputMap, OutputMaps.GetAllocator());

								checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
								checkf(OutputMap->GetAllocator() == OutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
								OutputMaps.Append(OutputMap);
							}
						}
					},
					TransformerList->CreateBoundValueMapTransformerIterator(),
					Input.GetBoundValueMaps().CreateConstIterator());
			}

			// Unbound value maps
			{
				FUnboundMapCollection& OutputMaps = Output.GetUnboundValueMaps();

				OuterJoinBy(
					FValueTransformerListWithUnboundMapCollectionJoinOp(),
					[DefaultWeight, &OutputMaps](FRawTransformerFunc Transformer, const FUnboundValueMap* InputMap)
					{
						if (InputMap == nullptr)
						{
							// No maps found for the transformer type
						}
						else
						{
							if (Transformer == nullptr)
							{
								// No transformer was found for this value type, retain it as-is
								OutputMaps.Append(InputMap->Duplicate(OutputMaps.GetAllocator()));
							}
							else
							{
								// Transform our matching pair
								FUnboundValueMap* OutputMap = nullptr;

								FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
								(*TypedTransformer)(InputMap, DefaultWeight, OutputMap, OutputMaps.GetAllocator());

								checkf(OutputMap != nullptr, TEXT("Transformer function is responsible for allocating an output buffer"));
								checkf(OutputMap->GetAllocator() == OutputMaps.GetAllocator(), TEXT("Transformer function is responsible for allocating using the output allocator"));
								OutputMaps.Append(OutputMap);
							}
						}
					},
					TransformerList->CreateUnboundValueMapTransformerIterator(),
					Input.GetUnboundValueMaps().CreateConstIterator());
			}
		}
	}

	namespace Private
	{
		void FOverwrite_BoneTransformAttribute::TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(InputMap->GetTypedSet(), InputMap->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FBoneTransformAnimationAttribute>(Args);
			}

			check(InputMap->GetTypedSet() == OutputMap->GetTypedSet());
			check(InputMap->Num() == OutputMap->Num());

			const TBoundValueMap<FBoneTransformAnimationAttribute>* InputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(InputMap);
			TBoundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(OutputMap);

			const FBoneTransformAnimationAttribute* InputTransformPtr = InputBoneTransforms->GetData();
			FBoneTransformAnimationAttribute* OutputTransformPtr = OutputBoneTransforms->GetData();
			const FBoneTransformAnimationAttribute* InputTransformPtrEnd = InputTransformPtr + InputBoneTransforms->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(InputMap->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(InputMap->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (InputTransformPtr < InputTransformPtrEnd)
				{
					OutputTransformPtr->Value = InputTransformPtr->Value * ScalarRegister(PerValueWeightsPtr->Value);

					InputTransformPtr++;
					PerValueWeightsPtr++;
					OutputTransformPtr++;
				}
			}
			else
			{
				const ScalarRegister VDefaultWeight(DefaultWeight);

				while (InputTransformPtr < InputTransformPtrEnd)
				{
					OutputTransformPtr->Value = InputTransformPtr->Value * VDefaultWeight;

					InputTransformPtr++;
					OutputTransformPtr++;
				}
			}
		}

		void FOverwrite_BoneTransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			const bool bAllocateOutput = OutputMap == nullptr;
			if (bAllocateOutput)
			{
				OutputMap = MakeUnboundValueMap<FBoneTransformAnimationAttribute>(OutputAllocator);
			}

			const TUnboundValueMap<FBoneTransformAnimationAttribute>* InputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(InputMap);
			TUnboundValueMap<FBoneTransformAnimationAttribute>* OutputBoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(OutputMap);

			const ScalarRegister VWeight(Weight);

			if (bAllocateOutput)
			{
				// We allocated a new output, append to it
				for (auto It = InputBoneTransforms->CreateConstIterator(); It; ++It)
				{
					FBoneTransformAnimationAttribute Output{ It.GetValue()->Value * VWeight };
					OutputBoneTransforms->Append(It.GetName(), Output);
				}
			}
			else
			{
				// We overwrite in place
				check(InputBoneTransforms == OutputBoneTransforms);

				for (auto It = OutputBoneTransforms->CreateIterator(); It; ++It)
				{
					It.GetValue()->Value *= VWeight;
				}
			}
		}

		void FOverwrite_FloatAttribute::TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(InputMap->GetTypedSet(), InputMap->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FFloatAnimationAttribute>(Args);
			}

			check(InputMap->GetTypedSet() == OutputMap->GetTypedSet());
			check(InputMap->Num() == OutputMap->Num());

			const TBoundValueMap<FFloatAnimationAttribute>* InputFloats = CastChecked<FFloatAnimationAttribute>(InputMap);
			TBoundValueMap<FFloatAnimationAttribute>* OutputFloats = CastChecked<FFloatAnimationAttribute>(OutputMap);

			const FFloatAnimationAttribute* FloatsPtr = InputFloats->GetData();
			FFloatAnimationAttribute* OutputFloatsPtr = OutputFloats->GetData();
			const FFloatAnimationAttribute* FloatsPtrEnd = FloatsPtr + InputFloats->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(InputMap->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(InputMap->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (FloatsPtr < FloatsPtrEnd)
				{
					OutputFloatsPtr->Value = FloatsPtr->Value * PerValueWeightsPtr->Value;

					FloatsPtr++;
					PerValueWeightsPtr++;
					OutputFloatsPtr++;
				}
			}
			else
			{
				while (FloatsPtr < FloatsPtrEnd)
				{
					OutputFloatsPtr->Value = FloatsPtr->Value * DefaultWeight;

					FloatsPtr++;
					OutputFloatsPtr++;
				}
			}
		}

		void FOverwrite_FloatAttribute::TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			const bool bAllocateOutput = OutputMap == nullptr;
			if (bAllocateOutput)
			{
				OutputMap = MakeUnboundValueMap<FFloatAnimationAttribute>(OutputAllocator);
			}

			const TUnboundValueMap<FFloatAnimationAttribute>* InputFloats = CastChecked<FFloatAnimationAttribute>(InputMap);
			TUnboundValueMap<FFloatAnimationAttribute>* OutputFloats = CastChecked<FFloatAnimationAttribute>(OutputMap);

			if (bAllocateOutput)
			{
				// We allocated a new output, append to it
				for (auto It = InputFloats->CreateConstIterator(); It; ++It)
				{
					FFloatAnimationAttribute Output{ It.GetValue()->Value * Weight };
					OutputFloats->Append(It.GetName(), Output);
				}
			}
			else
			{
				// We overwrite in place
				check(InputFloats == OutputFloats);

				for (auto It = OutputFloats->CreateIterator(); It; ++It)
				{
					It.GetValue()->Value *= Weight;
				}
			}
		}

		void FOverwrite_IntegerAttribute::TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(InputMap->GetTypedSet(), InputMap->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FIntegerAnimationAttribute>(Args);
			}

			check(InputMap->GetTypedSet() == OutputMap->GetTypedSet());
			check(InputMap->Num() == OutputMap->Num());

			const TBoundValueMap<FIntegerAnimationAttribute>* InputIntegers = CastChecked<FIntegerAnimationAttribute>(InputMap);
			TBoundValueMap<FIntegerAnimationAttribute>* OutputIntegers = CastChecked<FIntegerAnimationAttribute>(OutputMap);

			const FIntegerAnimationAttribute* IntegersPtr = InputIntegers->GetData();
			FIntegerAnimationAttribute* OutputIntegersPtr = OutputIntegers->GetData();
			const FIntegerAnimationAttribute* IntegersPtrEnd = IntegersPtr + InputIntegers->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(InputMap->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(InputMap->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (IntegersPtr < IntegersPtrEnd)
				{
					OutputIntegersPtr->Value = static_cast<int32>(IntegersPtr->Value * PerValueWeightsPtr->Value);

					IntegersPtr++;
					PerValueWeightsPtr++;
					OutputIntegersPtr++;
				}
			}
			else
			{
				while (IntegersPtr < IntegersPtrEnd)
				{
					OutputIntegersPtr->Value = static_cast<int32>(IntegersPtr->Value * DefaultWeight);

					IntegersPtr++;
					OutputIntegersPtr++;
				}
			}
		}

		void FOverwrite_IntegerAttribute::TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			const bool bAllocateOutput = OutputMap == nullptr;
			if (bAllocateOutput)
			{
				OutputMap = MakeUnboundValueMap<FIntegerAnimationAttribute>(OutputAllocator);
			}

			const TUnboundValueMap<FIntegerAnimationAttribute>* InputIntegers = CastChecked<FIntegerAnimationAttribute>(InputMap);
			TUnboundValueMap<FIntegerAnimationAttribute>* OutputIntegers = CastChecked<FIntegerAnimationAttribute>(OutputMap);

			if (bAllocateOutput)
			{
				// We allocated a new output, append to it
				// TODO: Memcpy for weight = 1.0?
				for (auto It = InputIntegers->CreateConstIterator(); It; ++It)
				{
					FIntegerAnimationAttribute Output{ static_cast<int32>(It.GetValue()->Value * Weight) };
					OutputIntegers->Append(It.GetName(), Output);
				}
			}
			else
			{
				// We overwrite in place
				check(InputIntegers == OutputIntegers);

				for (auto It = OutputIntegers->CreateIterator(); It; ++It)
				{
					It.GetValue()->Value = static_cast<int32>(It.GetValue()->Value * Weight);
				}
			}
		}

		void FOverwrite_TransformAttribute::TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(InputMap->GetTypedSet(), InputMap->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FTransformAnimationAttribute>(Args);
			}

			check(InputMap->GetTypedSet() == OutputMap->GetTypedSet());
			check(InputMap->Num() == OutputMap->Num());

			const TBoundValueMap<FTransformAnimationAttribute>* InputTransforms = CastChecked<FTransformAnimationAttribute>(InputMap);
			TBoundValueMap<FTransformAnimationAttribute>* OutputTransforms = CastChecked<FTransformAnimationAttribute>(OutputMap);

			const FTransformAnimationAttribute* InputTransformPtr = InputTransforms->GetData();
			FTransformAnimationAttribute* OutputTransformPtr = OutputTransforms->GetData();
			const FTransformAnimationAttribute* InputTransformPtrEnd = InputTransformPtr + InputTransforms->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(InputMap->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(InputMap->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (InputTransformPtr < InputTransformPtrEnd)
				{
					OutputTransformPtr->Value = InputTransformPtr->Value * ScalarRegister(PerValueWeightsPtr->Value);

					InputTransformPtr++;
					PerValueWeightsPtr++;
					OutputTransformPtr++;
				}
			}
			else
			{
				const ScalarRegister VDefaultWeight(DefaultWeight);

				while (InputTransformPtr < InputTransformPtrEnd)
				{
					OutputTransformPtr->Value = InputTransformPtr->Value * VDefaultWeight;

					InputTransformPtr++;
					OutputTransformPtr++;
				}
			}
		}

		void FOverwrite_TransformAttribute::TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			const bool bAllocateOutput = OutputMap == nullptr;
			if (bAllocateOutput)
			{
				OutputMap = MakeUnboundValueMap<FTransformAnimationAttribute>(OutputAllocator);
			}

			const TUnboundValueMap<FTransformAnimationAttribute>* InputTransforms = CastChecked<FTransformAnimationAttribute>(InputMap);
			TUnboundValueMap<FTransformAnimationAttribute>* OutputTransforms = CastChecked<FTransformAnimationAttribute>(OutputMap);

			const ScalarRegister VWeight(Weight);

			if (bAllocateOutput)
			{
				// We allocated a new output, append to it
				for (auto It = InputTransforms->CreateConstIterator(); It; ++It)
				{
					FBoneTransformAnimationAttribute Output{ It.GetValue()->Value * VWeight };
					OutputTransforms->Append(It.GetName(), Output);
				}
			}
			else
			{
				// We overwrite in place
				check(InputTransforms == OutputTransforms);

				for (auto It = OutputTransforms->CreateIterator(); It; ++It)
				{
					It.GetValue()->Value *= VWeight;
				}
			}
		}

		void FOverwrite_VectorAttribute::TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(InputMap->GetTypedSet(), InputMap->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FTransformAnimationAttribute>(Args);
			}

			check(InputMap->GetTypedSet() == OutputMap->GetTypedSet());
			check(InputMap->Num() == OutputMap->Num());

			const TBoundValueMap<FVectorAnimationAttribute>* InputVectors = CastChecked<FVectorAnimationAttribute>(InputMap);
			TBoundValueMap<FVectorAnimationAttribute>* OutputVectors = CastChecked<FVectorAnimationAttribute>(OutputMap);
			
			const FVectorAnimationAttribute* InputVectorPtr = InputVectors->GetData();
			FVectorAnimationAttribute* OutputVectorPtr = OutputVectors->GetData();
			const FVectorAnimationAttribute* InputVectorPtrEnd = InputVectorPtr + InputVectors->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(InputMap->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(InputMap->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (InputVectorPtr < InputVectorPtrEnd)
				{
					OutputVectorPtr->Value = InputVectorPtr->Value * PerValueWeightsPtr->Value;

					InputVectorPtr++;
					PerValueWeightsPtr++;
					OutputVectorPtr++;
				}
			}
			else
			{
				while (InputVectorPtr < InputVectorPtrEnd)
				{
					OutputVectorPtr->Value = InputVectorPtr->Value * DefaultWeight;

					InputVectorPtr++;
					OutputVectorPtr++;
				}
			}
		}

		void FOverwrite_VectorAttribute::TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			const bool bAllocateOutput = OutputMap == nullptr;
			if (bAllocateOutput)
			{
				OutputMap = MakeUnboundValueMap<FVectorAnimationAttribute>(OutputAllocator);
			}

			const TUnboundValueMap<FVectorAnimationAttribute>* InputVectors = CastChecked<FVectorAnimationAttribute>(InputMap);
			TUnboundValueMap<FVectorAnimationAttribute>* OutputVectors = CastChecked<FVectorAnimationAttribute>(OutputMap);

			if (bAllocateOutput)
			{
				// We allocated a new output, append to it
				for (auto It = InputVectors->CreateConstIterator(); It; ++It)
				{
					FVectorAnimationAttribute Output{ It.GetValue()->Value * Weight };
					OutputVectors->Append(It.GetName(), Output);
				}
			}
			else
			{
				// We overwrite in place
				check(InputVectors == OutputVectors);

				for (auto It = OutputVectors->CreateIterator(); It; ++It)
				{
					It.GetValue()->Value *= Weight;
				}
			}
		}

		void FOverwrite_QuaternionAttribute::TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			if (OutputMap == nullptr)
			{
				const FBoundValueMap::FConstructArgs Args(InputMap->GetTypedSet(), InputMap->GetValueType(), OutputAllocator);
				OutputMap = MakeBoundValueMap<FQuaternionAnimationAttribute>(Args);
			}

			check(InputMap->GetTypedSet() == OutputMap->GetTypedSet());
			check(InputMap->Num() == OutputMap->Num());

			const TBoundValueMap<FQuaternionAnimationAttribute>* InputQuaternions = CastChecked<FQuaternionAnimationAttribute>(InputMap);
			TBoundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = CastChecked<FQuaternionAnimationAttribute>(OutputMap);

			const FQuaternionAnimationAttribute* InputQuaternionPtr = InputQuaternions->GetData();
			FQuaternionAnimationAttribute* OutputQuaternionPtr = OutputQuaternions->GetData();
			const FQuaternionAnimationAttribute* InputQuaternionPtrEnd = InputQuaternionPtr + InputQuaternions->Num();

			if (const TBoundValueMap<FFloatAnimationAttribute>* PerValueWeights = Cast<FFloatAnimationAttribute>(PerValueWeightMap))
			{
				check(InputMap->GetTypedSet() == PerValueWeights->GetTypedSet());
				check(InputMap->Num() == PerValueWeights->Num());

				const FFloatAnimationAttribute* PerValueWeightsPtr = PerValueWeights->GetData();

				while (InputQuaternionPtr < InputQuaternionPtrEnd)
				{
					OutputQuaternionPtr->Value = InputQuaternionPtr->Value * PerValueWeightsPtr->Value;

					InputQuaternionPtr++;
					PerValueWeightsPtr++;
					OutputQuaternionPtr++;
				}
			}
			else
			{
				while (InputQuaternionPtr < InputQuaternionPtrEnd)
				{
					OutputQuaternionPtr->Value = InputQuaternionPtr->Value * DefaultWeight;

					InputQuaternionPtr++;
					OutputQuaternionPtr++;
				}
			}
		}

		void FOverwrite_QuaternionAttribute::TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator)
		{
			const bool bAllocateOutput = OutputMap == nullptr;
			if (bAllocateOutput)
			{
				OutputMap = MakeUnboundValueMap<FQuaternionAnimationAttribute>(OutputAllocator);
			}

			const TUnboundValueMap<FQuaternionAnimationAttribute>* InputQuaternions = CastChecked<FQuaternionAnimationAttribute>(InputMap);
			TUnboundValueMap<FQuaternionAnimationAttribute>* OutputQuaternions = CastChecked<FQuaternionAnimationAttribute>(OutputMap);

			if (bAllocateOutput)
			{
				// We allocated a new output, append to it
				for (auto It = InputQuaternions->CreateConstIterator(); It; ++It)
				{
					FQuaternionAnimationAttribute Output{ It.GetValue()->Value * Weight };
					OutputQuaternions->Append(It.GetName(), Output);
				}
			}
			else
			{
				// We overwrite in place
				check(InputQuaternions == OutputQuaternions);

				for (auto It = OutputQuaternions->CreateIterator(); It; ++It)
				{
					It.GetValue()->Value *= Weight;
				}
			}
		}
	}
}
