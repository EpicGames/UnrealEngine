// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/Transformers/Sanitize.h"

#include "UAF/ValueRuntime/IteratorUtils.h"

namespace UE::UAF::Transformers
{
	FName FSanitize::TransformerName = TEXT("UAF::Transformers::Sanitize");

	void FSanitize::Apply(const FValueTransformerMapPtr& TransformerMap, FValueBundle& Collection)
	{
		const FValueTransformerList* TransformerList = TransformerMap->Find(TransformerName);
		if (TransformerList == nullptr)
		{
			// No transformer implementations have been registered
			return;
		}

		// Set maps
		InnerJoinBy(
			FValueTransformerListWithBoundMapCollectionJoinOp(),
			[](FRawTransformerFunc Transformer, FBoundValueMap* Map)
			{
				FTransformBoundValueMapFunc TypedTransformer = reinterpret_cast<FTransformBoundValueMapFunc>(Transformer);
				(*TypedTransformer)(Map);
			},
			TransformerList->CreateBoundValueMapTransformerIterator(),
			Collection.GetBoundValueMaps().CreateIterator());

		// Name/value maps
		InnerJoinBy(
			FValueTransformerListWithUnboundMapCollectionJoinOp(),
			[](FRawTransformerFunc Transformer, FUnboundValueMap* Map)
			{
				FTransformUnboundValueMapFunc TypedTransformer = reinterpret_cast<FTransformUnboundValueMapFunc>(Transformer);
				(*TypedTransformer)(Map);
			},
			TransformerList->CreateUnboundValueMapTransformerIterator(),
			Collection.GetUnboundValueMaps().CreateIterator());
	}

	namespace Private
	{
#if UE_UAF_VALIDATE_MAPPED_VALUES
		
		// Don't inline this function to keep the stack usage down
		template <typename TransformAttributeType>
		static FORCENOINLINE void IsTransformMapValid(const TBoundValueMap<TransformAttributeType>* Transforms)
		{
			const TransformAttributeType* TransformPtr = Transforms->GetData();
			const TransformAttributeType* EndTransformPtr = TransformPtr + Transforms->Num();

			while (TransformPtr < EndTransformPtr)
			{
				if (TransformPtr->Value.ContainsNaN())
				{
					ensureMsgf(false, TEXT("Transform (%s) contains NaN with Value:[%s]"),
						*Transforms->GetTypedSet()->GetName(FAttributeSetIndex(TransformPtr - Transforms->GetData())).ToString(),
						*TransformPtr->Value.ToString());
				}
				else if (!TransformPtr->Value.IsRotationNormalized())
				{
					ensureMsgf(false, TEXT("Transform (%s) rotation not normalized with Rotation:[%s]"),
						*Transforms->GetTypedSet()->GetName(FAttributeSetIndex(TransformPtr - Transforms->GetData())).ToString(),
						*TransformPtr->Value.GetRotation().ToString());
				}

				TransformPtr++;
			}
		}

		// Don't inline this function to keep the stack usage down
		template <typename TransformAttributeType>
		static FORCENOINLINE void IsTransformMapValid(const TUnboundValueMap<TransformAttributeType>* Transforms)
		{
			for (auto It = Transforms->CreateConstIterator(); It; ++It)
			{
				if (It.GetValue()->Value.ContainsNaN())
				{
					ensureMsgf(false, TEXT("Transform attribute (%s) contains NaN with Value:[%s]"),
						*It.GetName().ToString(),
						*It.GetValue()->Value.ToString());
				}
				else if (!It.GetValue()->Value.IsRotationNormalized())
				{
					ensureMsgf(false, TEXT("Transform attribute (%s) rotation not normalized with Rotation:[%s]"),
						*It.GetName().ToString(),
						*It.GetValue()->Value.GetRotation().ToString());
				}
			}
		}
#endif

		void FSanitize_BoneTransformAttribute::TransformBoundValueMap(FBoundValueMap* Map)
		{
			TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Map);

			FBoneTransformAnimationAttribute* TransformPtr = BoneTransforms->GetData();
			const FBoneTransformAnimationAttribute* TransformPtrEnd = TransformPtr + BoneTransforms->Num();

			while (TransformPtr < TransformPtrEnd)
			{
				TransformPtr->Value.NormalizeRotation();

				TransformPtr++;
			}

#if UE_UAF_VALIDATE_MAPPED_VALUES
			IsTransformMapValid(BoneTransforms);
#endif
		}

		void FSanitize_BoneTransformAttribute::TransformUnboundValueMap(FUnboundValueMap* Map)
		{
			TUnboundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = CastChecked<FBoneTransformAnimationAttribute>(Map);

			for (auto It = BoneTransforms->CreateIterator(); It; ++It)
			{
				It.GetValue()->Value.NormalizeRotation();
			}

#if UE_UAF_VALIDATE_MAPPED_VALUES
			IsTransformMapValid(BoneTransforms);
#endif
		}

#if UE_UAF_VALIDATE_MAPPED_VALUES
		void FSanitize_FloatAttribute::TransformBoundValueMap(FBoundValueMap* Map)
		{
			const TBoundValueMap<FFloatAnimationAttribute>* Floats = CastChecked<FFloatAnimationAttribute>(Map);

			const FFloatAnimationAttribute* FloatsPtr = Floats->GetData();
			const FFloatAnimationAttribute* FloatsPtrEnd = FloatsPtr + Floats->Num();

			while (FloatsPtr < FloatsPtrEnd)
			{
				ensureMsgf(!FMath::IsNaN(FloatsPtr->Value), TEXT("Float attribute (%s) contains NaN with Value:[%3.3f]"),
					*Floats->GetTypedSet()->GetName(FAttributeSetIndex(FloatsPtr - Floats->GetData())).ToString(),
					FloatsPtr->Value);

				FloatsPtr++;
			}
		}

		void FSanitize_FloatAttribute::TransformUnboundValueMap(FUnboundValueMap* Map)
		{
			const TUnboundValueMap<FFloatAnimationAttribute>* Floats = CastChecked<FFloatAnimationAttribute>(Map);

			for (auto It = Floats->CreateConstIterator(); It; ++It)
			{
				ensureMsgf(!FMath::IsNaN(It.GetValue()->Value), TEXT("Float attribute (%s) contains NaN with Value:[%3.3f]"),
					*It.GetName().ToString(),
					It.GetValue()->Value);
			}
		}
#endif

		void FSanitize_TransformAttribute::TransformBoundValueMap(FBoundValueMap* Map)
		{
			TBoundValueMap<FTransformAnimationAttribute>* Transforms = CastChecked<FTransformAnimationAttribute>(Map);

			FTransformAnimationAttribute* TransformPtr = Transforms->GetData();
			const FTransformAnimationAttribute* TransformPtrEnd = TransformPtr + Transforms->Num();

			while (TransformPtr < TransformPtrEnd)
			{
				TransformPtr->Value.NormalizeRotation();

				TransformPtr++;
			}

#if UE_UAF_VALIDATE_MAPPED_VALUES
			IsTransformMapValid(Transforms);
#endif
		}

		void FSanitize_TransformAttribute::TransformUnboundValueMap(FUnboundValueMap* Map)
		{
			TUnboundValueMap<FTransformAnimationAttribute>* Transforms = CastChecked<FTransformAnimationAttribute>(Map);

			for (auto It = Transforms->CreateIterator(); It; ++It)
			{
				It.GetValue()->Value.NormalizeRotation();
			}

#if UE_UAF_VALIDATE_MAPPED_VALUES
			IsTransformMapValid(Transforms);
#endif
		}

		void FSanitize_QuaternionAttribute::TransformBoundValueMap(FBoundValueMap* Map)
		{
			TBoundValueMap<FQuaternionAnimationAttribute>* Quaternions = CastChecked<FQuaternionAnimationAttribute>(Map);

			FQuaternionAnimationAttribute* QuaternionPtr = Quaternions->GetData();
			const FQuaternionAnimationAttribute* QuaternionPtrEnd = QuaternionPtr + Quaternions->Num();

			while (QuaternionPtr < QuaternionPtrEnd)
			{
				QuaternionPtr->Value.Normalize();

				QuaternionPtr++;
			}
		}

		void FSanitize_QuaternionAttribute::TransformUnboundValueMap(FUnboundValueMap* Map)
		{
			TUnboundValueMap<FQuaternionAnimationAttribute>* Quaternions = CastChecked<FQuaternionAnimationAttribute>(Map);

			for (auto It = Quaternions->CreateIterator(); It; ++It)
			{
				It.GetValue()->Value.Normalize();
			}
		}
	}
}
