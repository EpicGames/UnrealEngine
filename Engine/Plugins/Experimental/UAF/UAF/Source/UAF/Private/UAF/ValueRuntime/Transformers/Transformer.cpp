// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/Transformers/Transformer.h"

namespace UE::UAF
{
	FValueTransformerList::FValueTransformerList() = default;

	FValueTransformerList::FValueTransformerList(FName InTransformerName)
		: TransformerName(InTransformerName)
	{
	}

	FName FValueTransformerList::GetTransformerName() const
	{
		return TransformerName;
	}

	bool FValueTransformerList::AddBoundValueMapTransformer(UScriptStruct* ValueType, FRawTransformerFunc TransformerFunc)
	{
		const int32 EntryIndex = Algo::LowerBound(MakeArrayView(ValueTypes.GetData(), NumBoundValueMapTransformers()), ValueType);
		if (EntryIndex < ValueTypes.Num() && ValueTypes[EntryIndex] == ValueType)
		{
			// A transformer for this value type has already been registered
			return false;
		}

		ValueTypes.Insert(ValueType, EntryIndex);
		Transformers.Insert(TransformerFunc, EntryIndex);
		BoundValueMapTransformerCount++;

		return true;
	}

	bool FValueTransformerList::AddUnboundValueMapTransformer(UScriptStruct* ValueType, FRawTransformerFunc TransformerFunc)
	{
		const int32 NumBoundValueMapEntries = NumBoundValueMapTransformers();
		const int32 NumUnboundValueMapEntries = NumUnboundValueMapTransformers();

		const int32 PerContainerEntryIndex = Algo::LowerBound(MakeArrayView(ValueTypes.GetData() + NumBoundValueMapEntries, NumUnboundValueMapEntries), ValueType);

		// Fixup our index
		const int32 EntryIndex = PerContainerEntryIndex + NumBoundValueMapEntries;

		if (EntryIndex < ValueTypes.Num() && ValueTypes[EntryIndex] == ValueType)
		{
			// A transformer for this value type has already been registered
			return false;
		}

		ValueTypes.Insert(ValueType, EntryIndex);
		Transformers.Insert(TransformerFunc, EntryIndex);

		return true;
	}
}
