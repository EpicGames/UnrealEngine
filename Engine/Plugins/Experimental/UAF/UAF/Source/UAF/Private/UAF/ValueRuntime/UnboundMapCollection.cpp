// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/UnboundMapCollection.h"

#include "Algo/BinarySearch.h"
#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"

namespace UE::UAF
{
	FUnboundMapCollection::FUnboundMapCollection(const FUnboundMapCollection& Other)
		: MapCount(Other.MapCount)
		, MapCapacity(Other.MapCount)
		, ReallocFun(Other.ReallocFun)
	{
		if (Other.Maps != nullptr)
		{
			const int32 BufferSize = sizeof(FMapEntry) * Other.MapCount;
			Maps = reinterpret_cast<FMapEntry*>((*Other.ReallocFun)(nullptr, 0, BufferSize));
			FMemory::Memcpy(Maps, Other.Maps, BufferSize);

			for (int32 MapIndex = 0; MapIndex < Other.MapCount; ++MapIndex)
			{
				Maps[MapIndex].Map = Other.Maps[MapIndex].Map->Duplicate(Other.ReallocFun);
			}
		}
		else
		{
			Maps = nullptr;
		}
	}

	FUnboundMapCollection::FUnboundMapCollection(FUnboundMapCollection&& Other)
		: Maps(Other.Maps)
		, MapCount(Other.MapCount)
		, MapCapacity(Other.MapCapacity)
		, ReallocFun(Other.ReallocFun)
	{
		Other.Maps = nullptr;
		Other.MapCount = 0;
		Other.MapCapacity = 0;
	}

	FUnboundMapCollection::FUnboundMapCollection(FReallocFun InReallocFun)
		: ReallocFun(InReallocFun)
	{
		check(InReallocFun != nullptr);
	}

	FUnboundMapCollection::~FUnboundMapCollection()
	{
		Empty();
	}

	FUnboundMapCollection& FUnboundMapCollection::operator=(const FUnboundMapCollection& Other)
	{
		if (this != &Other)
		{
			checkf(ReallocFun != nullptr, TEXT("When we copy from another collection, we retain our original allocator"));

			// Free anything we might be holding
			Empty();

			MapCount = Other.MapCount;
			MapCapacity = Other.MapCount;

			if (Other.Maps != nullptr)
			{
				const int32 BufferSize = sizeof(FMapEntry) * Other.MapCount;
				Maps = reinterpret_cast<FMapEntry*>((*ReallocFun)(nullptr, 0, BufferSize));
				FMemory::Memcpy(Maps, Other.Maps, BufferSize);

				for (int32 MapIndex = 0; MapIndex < Other.MapCount; ++MapIndex)
				{
					Maps[MapIndex].Map = Other.Maps[MapIndex].Map->Duplicate(ReallocFun);
				}
			}
			else
			{
				Maps = nullptr;
			}
		}

		return *this;
	}

	FUnboundMapCollection& FUnboundMapCollection::operator=(FUnboundMapCollection&& Other)
	{
		if (this != &Other)
		{
			checkf(ReallocFun == Other.ReallocFun, TEXT("When we move from another collection, our allocators must match"));

			// Free anything we might be holding
			Empty();

			Maps = Other.Maps;
			MapCount = Other.MapCount;
			MapCapacity = Other.MapCapacity;

			Other.Maps = nullptr;
			Other.MapCount = 0;
			Other.MapCapacity = 0;
		}

		return *this;
	}

	bool FUnboundMapCollection::IsEmpty() const
	{
		return MapCount == 0;
	}

	void FUnboundMapCollection::Reset()
	{
		// Free our maps but retain the arrays that contains them
		for (int32 MapIndex = 0; MapIndex < MapCount; ++MapIndex)
		{
			ReleaseUnboundValueMap(Maps[MapIndex].Map);
		}
		MapCount = 0;
	}

	void FUnboundMapCollection::Empty()
	{
		if (Maps != nullptr)
		{
			for (int32 MapIndex = 0; MapIndex < MapCount; ++MapIndex)
			{
				ReleaseUnboundValueMap(Maps[MapIndex].Map);
			}

			(*ReallocFun)(reinterpret_cast<uint8*>(Maps), sizeof(FMapEntry) * MapCapacity, 0);
			Maps = nullptr;
			MapCount = 0;
			MapCapacity = 0;
		}
	}

	FReallocFun FUnboundMapCollection::GetAllocator() const
	{
		return ReallocFun;
	}

	int32 FUnboundMapCollection::Num() const
	{
		return MapCount;
	}

	FUnboundValueMap* FUnboundMapCollection::Add(UScriptStruct* ValueType)
	{
		// We don't know the concrete type of the map, use indirection to construct it
		constexpr bool bConstructMap = true;
		FUnboundValueMap** Map = AddImpl(ValueType, bConstructMap);
		return Map != nullptr ? *Map : nullptr;
	}

	FUnboundValueMap** FUnboundMapCollection::AddImpl(UScriptStruct* ValueType, bool bConstructMap)
	{
		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), ValueType, [](const FMapEntry& Entry) { return Entry.ValueType; });
		if (MapIndex < MapCount && Maps[MapIndex].ValueType == ValueType)
		{
			// A map already exists that matches the specified type
			return nullptr;
		}

		FUnboundValueMap* Map = nullptr;
		if (bConstructMap)
		{
			Map = FValueRuntimeRegistry::Get().ConstructUnboundValueMap(ValueType, ReallocFun);

			if (Map == nullptr)
			{
				// Failed to construct a map, the type has probably not been registered
				return nullptr;
			}
		}

		if (MapCount >= MapCapacity)
		{
			const bool bAllowQuantize = true;
			const int32 DesiredCapacity = MapCapacity + 1;
			const int32 NewCapacity = DefaultCalculateSlackGrow(DesiredCapacity, MapCapacity, sizeof(FMapEntry), bAllowQuantize);

			Maps = reinterpret_cast<FMapEntry*>((*ReallocFun)(
				reinterpret_cast<uint8*>(Maps),
				sizeof(FMapEntry) * MapCapacity,
				sizeof(FMapEntry) * NewCapacity));

			MapCapacity = NewCapacity;
		}

		const int32 NumToMove = MapCount - MapIndex;
		FMemory::Memmove(Maps + MapIndex + 1, Maps + MapIndex, sizeof(FMapEntry) * NumToMove);
		Maps[MapIndex].ValueType = ValueType;
		Maps[MapIndex].Map = Map;
		MapCount++;

		return &Maps[MapIndex].Map;
	}

	bool FUnboundMapCollection::Add(FUnboundValueMap* Map)
	{
		if (Map == nullptr)
		{
			// Invalid map
			return false;
		}
		else if (Map->GetAllocator() != ReallocFun)
		{
			// Allocator mismatch
			return false;
		}

		UScriptStruct* ValueType = Map->GetValueType();

		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), ValueType, [](const FMapEntry& Entry) { return Entry.ValueType; });
		if (MapIndex < MapCount && Maps[MapIndex].ValueType == ValueType)
		{
			// A map already exists that matches the specified types
			return false;
		}

		if (MapCount >= MapCapacity)
		{
			const bool bAllowQuantize = true;
			const int32 DesiredCapacity = MapCapacity + 1;
			const int32 NewCapacity = DefaultCalculateSlackGrow(DesiredCapacity, MapCapacity, sizeof(FMapEntry), bAllowQuantize);

			Maps = reinterpret_cast<FMapEntry*>((*ReallocFun)(
				reinterpret_cast<uint8*>(Maps),
				sizeof(FMapEntry) * MapCapacity,
				sizeof(FMapEntry) * NewCapacity));

			MapCapacity = NewCapacity;
		}

		const int32 NumToMove = MapCount - MapIndex;
		FMemory::Memmove(Maps + MapIndex + 1, Maps + MapIndex, sizeof(FMapEntry) * NumToMove);
		Maps[MapIndex].ValueType = ValueType;
		Maps[MapIndex].Map = Map;
		MapCount++;

		return true;
	}

	bool FUnboundMapCollection::Append(FUnboundValueMap* Map)
	{
		if (Map == nullptr)
		{
			// Invalid map
			return false;
		}
		else if (Map->GetAllocator() != ReallocFun)
		{
			// Allocator mismatch
			return false;
		}

		UScriptStruct* ValueType = Map->GetValueType();

		if (MapCount != 0 && !(Maps[MapCount - 1].ValueType < ValueType))
		{
			// We don't belong at the end of the map list
			return false;
		}

		if (MapCount >= MapCapacity)
		{
			const bool bAllowQuantize = true;
			const int32 DesiredCapacity = MapCapacity + 1;
			const int32 NewCapacity = DefaultCalculateSlackGrow(DesiredCapacity, MapCapacity, sizeof(FMapEntry), bAllowQuantize);

			Maps = reinterpret_cast<FMapEntry*>((*ReallocFun)(
				reinterpret_cast<uint8*>(Maps),
				sizeof(FMapEntry) * MapCapacity,
				sizeof(FMapEntry) * NewCapacity));

			MapCapacity = NewCapacity;
		}

		Maps[MapCount].ValueType = ValueType;
		Maps[MapCount].Map = Map;
		MapCount++;

		return true;
	}

	bool FUnboundMapCollection::Remove(FUnboundValueMap* Map)
	{
		if (Map == nullptr)
		{
			// Invalid map
			return false;
		}

		return Remove(Map->GetValueType());
	}

	bool FUnboundMapCollection::Remove(UScriptStruct* ValueType)
	{
		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), ValueType, [](const FMapEntry& Entry) { return Entry.ValueType; });
		if (MapIndex >= MapCount || Maps[MapIndex].ValueType != ValueType)
		{
			// No map found that matches the desired types
			return false;
		}

		// Destroy and free our map
		ReleaseUnboundValueMap(Maps[MapIndex].Map);

		// Remove our entry
		const int32 NumToMove = MapCount - MapIndex - 1;
		FMemory::Memmove(Maps + MapIndex, Maps + MapIndex + 1, sizeof(FMapEntry) * NumToMove);
		MapCount--;

		return true;
	}

	FUnboundValueMap* FUnboundMapCollection::Find(UScriptStruct* ValueType)
	{
		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), ValueType, [](const FMapEntry& Entry) { return Entry.ValueType; });
		if (MapIndex >= MapCount || Maps[MapIndex].ValueType != ValueType)
		{
			// No map found that matches the desired types
			return nullptr;
		}

		return Maps[MapIndex].Map;
	}

	const FUnboundValueMap* FUnboundMapCollection::Find(UScriptStruct* ValueType) const
	{
		const int32 MapIndex = Algo::LowerBoundBy(TArrayView<const FMapEntry>(Maps, MapCount), ValueType, [](const FMapEntry& Entry) { return Entry.ValueType; });
		if (MapIndex >= MapCount || Maps[MapIndex].ValueType != ValueType)
		{
			// No map found that matches the desired types
			return nullptr;
		}

		return Maps[MapIndex].Map;
	}

	FUnboundValueMap* FUnboundMapCollection::FindOrAdd(UScriptStruct* ValueType)
	{
		if (FUnboundValueMap* Map = Find(ValueType))
		{
			return Map;
		}

		return Add(ValueType);
	}
}
