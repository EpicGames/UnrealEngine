// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/UnboundValueMap.h"

#include "UObject/Class.h"

namespace UE::UAF
{
	FUnboundValueMap::FUnboundValueMap() = default;

	FUnboundValueMap::FUnboundValueMap(UScriptStruct* InValueType, FReallocFun InReallocFun, int32 InContainerSize)
		: ValueTypeStruct(InValueType)
		, NumValues(0)
		, MaxValues(0)
		, DataPtr(nullptr)
		, ReallocFun(InReallocFun)
		, ContainerSize(InContainerSize)
	{
		check(InReallocFun != nullptr);
	}

	FUnboundValueMap::~FUnboundValueMap()
	{
		checkf(DataPtr == nullptr, TEXT("Derived types should handle destruction"));
	}

	bool FUnboundValueMap::IsEmpty() const
	{
		return NumValues == 0;
	}

	int32 FUnboundValueMap::Num() const
	{
		return NumValues;
	}

	int32 FUnboundValueMap::Max() const
	{
		return MaxValues;
	}

	UScriptStruct* FUnboundValueMap::GetValueType() const
	{
		return ValueTypeStruct;
	}

	int32 FUnboundValueMap::GetAllocatedSize() const
	{
		const int32 ValueTypeSize = ValueTypeStruct != nullptr ? ValueTypeStruct->GetStructureSize() : 0;
		const int32 ValueTypeAlignment = ValueTypeSize >= 16 ? 16 : 8;
		return ContainerSize + Align(sizeof(FName) * MaxValues, ValueTypeAlignment) + (ValueTypeSize * MaxValues);
	}

	FReallocFun FUnboundValueMap::GetAllocator() const
	{
		return ReallocFun;
	}

	void FUnboundValueMap::MoveTo(FUnboundValueMap& Other)
	{
		if (!ensureMsgf(ValueTypeStruct == Other.GetValueType(), TEXT("Value types must match to allow moving")))
		{
			return;
		}

		Swap(NumValues, Other.NumValues);
		Swap(MaxValues, Other.MaxValues);
		Swap(DataPtr, Other.DataPtr);
		Swap(ReallocFun, Other.ReallocFun);
		Swap(ContainerSize, Other.ContainerSize);
	}
}
