// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/ValueBundle.h"
#include "UAF/ValueRuntime/PoseValueBundle.h"

namespace UE::UAF
{
	FValueBundle::FValueBundle(const FValueBundle& Other)
		: NamedSet(Other.NamedSet)
		, BoundMaps(Other.BoundMaps)
		, UnboundMaps(Other.UnboundMaps)
		, ValueSpace(Other.ValueSpace)
	{
	}

	FValueBundle::FValueBundle(FValueBundle&& Other)
		: NamedSet(MoveTemp(Other.NamedSet))
		, BoundMaps(MoveTemp(Other.BoundMaps))
		, UnboundMaps(MoveTemp(Other.UnboundMaps))
		, ValueSpace(Other.ValueSpace)
		, ReallocFun(Other.ReallocFun)
	{
	}

	FValueBundle::FValueBundle(const FAttributeNamedSetPtr& InNamedSet, FReallocFun InReallocFun)
		: NamedSet(InNamedSet)
		, BoundMaps(InNamedSet, InReallocFun)
		, UnboundMaps(InReallocFun)
		, ReallocFun(InReallocFun)
	{
		check(InReallocFun != nullptr);
	}

	FValueBundle::~FValueBundle()
	{
		Empty();
	}

	FValueBundle& FValueBundle::operator=(const FValueBundle& Other)
	{
		if (this != &Other)
		{
			checkf(ReallocFun != nullptr, TEXT("When we copy from another collection, we retain our original allocator"));

			// Free anything we might be holding
			Empty();

			NamedSet = Other.NamedSet;
			BoundMaps = Other.BoundMaps;
			UnboundMaps = Other.UnboundMaps;
			ValueSpace = Other.ValueSpace;
		}

		return *this;
	}

	FValueBundle& FValueBundle::operator=(FValueBundle&& Other)
	{
		if (this != &Other)
		{
			checkf(ReallocFun == Other.ReallocFun, TEXT("When we move from another collection, our allocators must match"));

			// Free anything we might be holding
			Empty();

			NamedSet = MoveTemp(Other.NamedSet);
			BoundMaps = MoveTemp(Other.BoundMaps);
			UnboundMaps = MoveTemp(Other.UnboundMaps);
			ValueSpace = Other.ValueSpace;
		}

		return *this;
	}

	bool FValueBundle::IsEmpty() const
	{
		return BoundMaps.IsEmpty() && UnboundMaps.IsEmpty();
	}

	const FAttributeNamedSetPtr& FValueBundle::GetNamedSet() const
	{
		return NamedSet;
	}

	FValueSpace FValueBundle::GetValueSpace() const
	{
		return ValueSpace;
	}

	void FValueBundle::SetValueSpace(FValueSpace InValueSpace)
	{
		ValueSpace = InValueSpace;
	}

	void FValueBundle::InitWithValueSpace(FValueSpace InValueSpace)
	{
		if (!NamedSet)
		{
			// Nothing to initialize if we have no named set
			return;
		}

		// Empty the collection
		Reset(NamedSet);

		const bool bIsAdditive = InValueSpace.IsAdditive();

		bool bIsInitialized = false;
		if (!bIsAdditive)
		{
			const EValueSpaceType SpaceType = InValueSpace.GetType();
			switch (SpaceType)
			{
			case EValueSpaceType::Local:
				// From local space reference pose
				*this = *NamedSet->GetDefaultAttributeValues();
				bIsInitialized = true;
				break;
			case EValueSpaceType::Component:
				// From component space reference pose
				checkf(false, TEXT("Not yet supported"));
				break;
			}
		}

		if (!bIsInitialized)
		{
			for (auto It = NamedSet->CreateTypedSetIterator(); It; ++It)
			{
				const FAttributeTypedSetPtr TypedSet = *It;

				// By default, attributes map to values of their type
				const FAttributeMappingKey MappingKey = FAttributeMappingKey::MakeFromTo(TypedSet->GetType());

				FBoundValueMap* Copy = BoundMaps.Add(MappingKey);
				Copy->FillWithIdentity(bIsAdditive);
			}
		}

		ValueSpace = InValueSpace;
	}

	bool FValueBundle::IsAdditive() const
	{
		return ValueSpace.IsAdditive();
	}

	void FValueBundle::Reset(const FAttributeNamedSetPtr& InNamedSet)
	{
		BoundMaps.Reset(InNamedSet);
		UnboundMaps.Reset();

		NamedSet = InNamedSet;
		ValueSpace = FValueSpace();
	}

	void FValueBundle::Empty()
	{
		BoundMaps.Empty();
		UnboundMaps.Empty();

		NamedSet = nullptr;
		ValueSpace = FValueSpace();
	}

	FReallocFun FValueBundle::GetAllocator() const
	{
		return ReallocFun;
	}

	FBoundMapCollection& FValueBundle::GetBoundValueMaps()
	{
		return BoundMaps;
	}

	const FBoundMapCollection& FValueBundle::GetBoundValueMaps() const
	{
		return BoundMaps;
	}

	FUnboundMapCollection& FValueBundle::GetUnboundValueMaps()
	{
		return UnboundMaps;
	}

	const FUnboundMapCollection& FValueBundle::GetUnboundValueMaps() const
	{
		return UnboundMaps;
	}
}
