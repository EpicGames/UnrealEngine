// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/ValueRuntimeRegistry.h"

#include "Algo/BinarySearch.h"

namespace UE::UAF
{
	namespace Private
	{
		static FValueRuntimeRegistry* GValueRuntimeRegistry = nullptr;
	}

	void FValueRuntimeRegistry::Init()
	{
		if (Private::GValueRuntimeRegistry == nullptr)
		{
			Private::GValueRuntimeRegistry = new FValueRuntimeRegistry();
		}
	}

	void FValueRuntimeRegistry::Destroy()
	{
		if (Private::GValueRuntimeRegistry != nullptr)
		{
			delete Private::GValueRuntimeRegistry;
			Private::GValueRuntimeRegistry = nullptr;
		}
	}

	FValueRuntimeRegistry& FValueRuntimeRegistry::Get()
	{
		checkf(Private::GValueRuntimeRegistry != nullptr, TEXT("Attribute Runtime Registry is not instanced. It is only valid to access this while the UAF module is loaded."));
		return *Private::GValueRuntimeRegistry;
	}

	FValueRuntimeRegistry::FValueRuntimeRegistry() = default;

	FBoundValueMap* FValueRuntimeRegistry::ConstructBoundValueMap(const FBoundValueMap::FConstructArgs& Args) const
	{
		FReadScopeLock Lock(BoundValueMapInitializerLock);
		if (const BoundValueMapConstructorFun* ConstructorFun = BoundValueMapInitializers.Find(Args.ValueType))
		{
			return (*ConstructorFun)(Args);
		}

		// Value type has not been registered
		return nullptr;
	}

	FUnboundValueMap* FValueRuntimeRegistry::ConstructUnboundValueMap(UScriptStruct* ValueType, FReallocFun ReallocFun) const
	{
		FReadScopeLock Lock(UnboundValueMapInitializerLock);
		if (const UnboundValueMapConstructorFun* ConstructorFun = UnboundValueMapInitializers.Find(ValueType))
		{
			return (*ConstructorFun)(ReallocFun);
		}

		// Value type has not been registered
		return nullptr;
	}

	FValueTransformerMapPtr FValueRuntimeRegistry::GetTransformerMap() const
	{
		FReadScopeLock Lock(TransformerMapLock);
		return TransformerMap;
	}

	void FValueRuntimeRegistry::RegisterBoundValueMapInitializer(UScriptStruct* ValueType, BoundValueMapConstructorFun Initializer)
	{
		FWriteScopeLock Lock(BoundValueMapInitializerLock);
		BoundValueMapInitializers.Add(ValueType, Initializer);
	}

	void FValueRuntimeRegistry::RegisterUnboundValueMapInitializer(UScriptStruct* ValueType, UnboundValueMapConstructorFun Initializer)
	{
		FWriteScopeLock Lock(UnboundValueMapInitializerLock);
		UnboundValueMapInitializers.Add(ValueType, Initializer);
	}

	bool FValueRuntimeRegistry::RegisterBouneValueMapTransformer(FName TransformerName, UScriptStruct* ValueType, FRawTransformerFunc TransformerFunc)
	{
		FWriteScopeLock Lock(TransformerMapLock);

		// If someone is already holding a reference to our named operators, we cannot mutate
		// When this occurs, we make a copy and swap it, keeping the old one alive until it is no longer needed
		FValueTransformerMapPtr MutableTransformerMap = TransformerMap;
		if (!MutableTransformerMap || MutableTransformerMap.GetSharedReferenceCount() != 2)
		{
			MutableTransformerMap = MakeShared<typename FValueTransformerMapPtr::ElementType>();
		}

		const uint32 TransformerNameHash = GetTypeHash(TransformerName);
		FValueTransformerList* TransformerList = MutableTransformerMap->FindByHash(TransformerNameHash, TransformerName);
		if (TransformerList == nullptr)
		{
			TransformerList = &MutableTransformerMap->AddByHash(TransformerNameHash, TransformerName, FValueTransformerList(TransformerName));
		}

		if (!TransformerList->AddBoundValueMapTransformer(ValueType, TransformerFunc))
		{
			// We failed to add this transformer
			return false;
		}

		if (TransformerMap != MutableTransformerMap)
		{
			TransformerMap = MutableTransformerMap;
		}

		return true;
	}

	bool FValueRuntimeRegistry::RegisterUnboundValueMapTransformer(FName TransformerName, UScriptStruct* ValueType, FRawTransformerFunc TransformerFunc)
	{
		FWriteScopeLock Lock(TransformerMapLock);

		// If someone is already holding a reference to our named operators, we cannot mutate
		// When this occurs, we make a copy and swap it, keeping the old one alive until it is no longer needed
		FValueTransformerMapPtr MutableTransformerMap = TransformerMap;
		if (!MutableTransformerMap || MutableTransformerMap.GetSharedReferenceCount() != 2)
		{
			MutableTransformerMap = MakeShared<typename FValueTransformerMapPtr::ElementType>();
		}

		const uint32 TransformerNameHash = GetTypeHash(TransformerName);
		FValueTransformerList* TransformerList = MutableTransformerMap->FindByHash(TransformerNameHash, TransformerName);
		if (TransformerList == nullptr)
		{
			TransformerList = &MutableTransformerMap->AddByHash(TransformerNameHash, TransformerName, FValueTransformerList(TransformerName));
		}

		if (!TransformerList->AddUnboundValueMapTransformer(ValueType, TransformerFunc))
		{
			// We failed to add this transformer
			return false;
		}

		if (TransformerMap != MutableTransformerMap)
		{
			TransformerMap = MutableTransformerMap;
		}

		return true;
	}
}
