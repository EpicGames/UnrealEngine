// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/ValueRuntime/ValueSpace.h"

namespace UE::UAF
{
	namespace Private
	{
		const TCHAR* ValueSpaceTypeToString(EValueSpaceType Type)
		{
			switch (Type)
			{
			default:
			case EValueSpaceType::None:			return TEXT("None");
			case EValueSpaceType::Local:		return TEXT("Local");
			case EValueSpaceType::Component:	return TEXT("Component");
			case EValueSpaceType::Mixed:		return TEXT("Mixed");
			case EValueSpaceType::AI:			return TEXT("AI");
			}
		}

		FString MixedSpaceFlagsToString(EMixedSpaceFlags Flags)
		{
			FString Result;

			if (EnumHasAnyFlags(Flags, EMixedSpaceFlags::MeshRotation))
			{
				Result = TEXT("MeshRotation");
			}

			if (EnumHasAnyFlags(Flags, EMixedSpaceFlags::MeshScale))
			{
				if (!Result.IsEmpty())
				{
					Result += TEXT(" | ");
				}

				Result += TEXT("MeshScale");
			}

			if (EnumHasAnyFlags(Flags, EMixedSpaceFlags::RootRotation))
			{
				if (!Result.IsEmpty())
				{
					Result += TEXT(" | ");
				}

				Result += TEXT("RootRotation");
			}

			if (Result.IsEmpty())
			{
				Result = TEXT("None");
			}

			return Result;
		}
	}

	FString FValueSpace::ToString() const
	{
		const TCHAR* IsAdditiveStr = IsAdditive() ? TEXT("Additive ") : TEXT("");
		const TCHAR* ValueTypeStr = Private::ValueSpaceTypeToString(GetType());

		FString Result = FString::Printf(TEXT("%s%s"), IsAdditiveStr, ValueTypeStr);

		if (GetType() == EValueSpaceType::Mixed)
		{
			Result += FString::Printf(TEXT(" (%s)"), *Private::MixedSpaceFlagsToString(GetMixedSpaceFlags()));
		}

		return Result;
	}
}
