// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualValueBundle_ComponentOutput.h"

#include "Engine/SkeletalMesh.h"
#include "Graph/UAFSystemOutputComponent.h"

namespace UE::UAF
{

void ReadPose(USkeletalMeshComponent* InputComponent, FAnimNextGraphLODPose& OutPose)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UAF_READ_SMC_Pose);

	using namespace UE::UAF;
	FTaskTagScope TaskTag(ETaskTag::EParallelGameThread);

	if(InputComponent == nullptr)
	{
		UE_LOGF(LogAnimation, Warning, "Could not read from skeletal mesh component - No valid skeletal mesh component found.");
		OutPose.LODPose.SetRefPose();
		return;
	}

	USkeletalMesh* SkeletalMesh = InputComponent->GetSkeletalMeshAsset();
	if(SkeletalMesh == nullptr)
	{
		OutPose.LODPose.SetRefPose();
		return;
	}

	FDataHandle RefPoseHandle = FDataRegistry::Get()->GetOrGenerateReferencePose(InputComponent);
	const FReferencePose& RefPose = RefPoseHandle.GetRef<UE::UAF::FReferencePose>();
	OutPose.LODPose.PrepareForLOD(RefPose, InputComponent->GetPredictedLODLevel());

	// In case animation is enabled, use the (local) bone space transforms - otherwise (if driven by UAF) use component space transforms instead
	const bool bUseLocalSpaceTransforms = InputComponent->bEnableAnimation;
	const TArrayView<const FBoneIndexType> LODBoneToMeshBone = OutPose.LODPose.GetLODBoneIndexToMeshBoneIndexMap();
	const TArray<FBoneIndexType>& MeshBoneIndices = InputComponent->FillComponentSpaceTransformsRequiredBones;
	const int32 NumBoneTransforms = MeshBoneIndices.Num();
	const int32 NumPoseTransforms = OutPose.LODPose.LocalTransformsView.Num();
	ensure(NumBoneTransforms >= NumPoseTransforms);

	if (bUseLocalSpaceTransforms)
	{
		TArrayView<const FTransform> LocalSpaceTransforms = InputComponent->GetBoneSpaceTransformsView();

		//Set root bone which is always evaluated.
		OutPose.LODPose.LocalTransforms[0] = LocalSpaceTransforms[0];
		for (int32 PoseBoneIdx = 1; PoseBoneIdx < NumPoseTransforms; ++PoseBoneIdx)
		{
			const FBoneIndexType MeshIndex = LODBoneToMeshBone[PoseBoneIdx];
			const int32 ComponentBoneIndex = MeshBoneIndices.IndexOfByKey(MeshIndex);
			if (ComponentBoneIndex != INDEX_NONE)
			{
				OutPose.LODPose.LocalTransforms[PoseBoneIdx] = LocalSpaceTransforms[MeshIndex];
			}
		}
	}
	else
	{
		const TArray<FTransform>& ComponentSpaceTransforms = InputComponent->GetComponentSpaceTransforms();

		//Set root bone which is always evaluated.
		OutPose.LODPose.LocalTransforms[0] = ComponentSpaceTransforms[0];
		const TArrayView<const FBoneIndexType> MeshBoneToParentMapping = RefPose.GetMeshBoneIndexToParentMeshBoneIndexMap();
		for (int32 PoseBoneIdx = 1; PoseBoneIdx < NumPoseTransforms; ++PoseBoneIdx)
		{
			const FBoneIndexType MeshIndex = LODBoneToMeshBone[PoseBoneIdx];
			const int32 ComponentBoneIndex = MeshBoneIndices.IndexOfByKey(MeshIndex);
			if (ComponentBoneIndex != INDEX_NONE)
			{
				const int32 ParentIndex = MeshBoneToParentMapping[MeshIndex];
				const FTransform& ParentTransform = ComponentSpaceTransforms[ParentIndex];
				const FTransform& ChildTransform = ComponentSpaceTransforms[MeshIndex];
				// Convert component to local space
				OutPose.LODPose.LocalTransforms[PoseBoneIdx] = ChildTransform.GetRelativeTransform(ParentTransform);
			}
		}
	}
}

const FAnimNextGraphLODPose* FVirtualValueBundle_ComponentOutput::GetLODPose() const
{
	ReadPose(WeakSMC.Get(), CachedPose);

	return &CachedPose;
}

const FValueBundleHeap* FVirtualValueBundle_ComponentOutput::GetValueBundle() const
{
	ReadPose(WeakSMC.Get(), CachedPose);

	// TODO: Implement this for value runtime
	return &CachedValueBundle;
}

}


