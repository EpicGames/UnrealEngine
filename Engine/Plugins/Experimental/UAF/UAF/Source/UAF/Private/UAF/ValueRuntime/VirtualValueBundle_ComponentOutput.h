// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemapPoseData.h"
#include "Graph/AnimNext_LODPose.h"
#include "Module/UAFWeakSystemReference.h"
#include "UAF/ValueRuntime/IVirtualValueBundle.h"
#include "UAF/ValueRuntime/ValueBundle.h"

class USkeletalMeshComponent;

#define UE_API UAF_API

namespace UE::UAF
{

struct FVirtualValueBundle_ComponentOutput : public IVirtualValueBundle
{
	explicit FVirtualValueBundle_ComponentOutput(TWeakObjectPtr<USkeletalMeshComponent> InWeakSMC)
		: WeakSMC(InWeakSMC)
	{}

private:
	// IVirtualValueBundle interface
	virtual const FAnimNextGraphLODPose* GetLODPose() const override;
	virtual const FValueBundleHeap* GetValueBundle() const override;

	mutable FAnimNextGraphLODPose CachedPose;
	mutable FValueBundleHeap CachedValueBundle;
	TWeakObjectPtr<USkeletalMeshComponent> WeakSMC;
};

}

#undef UE_API
