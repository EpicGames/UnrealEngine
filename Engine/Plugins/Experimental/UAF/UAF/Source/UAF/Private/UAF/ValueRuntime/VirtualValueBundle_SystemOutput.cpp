// Copyright Epic Games, Inc. All Rights Reserved.

#include "VirtualValueBundle_SystemOutput.h"

#include "Graph/UAFSystemOutputComponent.h"

namespace UE::UAF
{

const FAnimNextGraphLODPose* FVirtualValueBundle_SystemOutput::GetLODPose() const
{
	ReadSystemOutput();

	return &CachedPose;
}

const FValueBundleHeap* FVirtualValueBundle_SystemOutput::GetValueBundle() const
{
	ReadSystemOutput();

	// TODO: Implement this for value runtime
	return &CachedValueBundle;
}

void FVirtualValueBundle_SystemOutput::ReadSystemOutput() const
{
	// TODO: Implement this for value runtime
	auto PoseReader = [this](uint32 InSerialNumber, const FAnimNextGraphLODPose& InPose, const FAnimNextGraphReferencePose& InRefPose)
	{
		SerialNumber = InSerialNumber;
		CachedPose = InPose;
	};

	SystemReference.ReadComponent<FUAFSystemOutputComponent>([this, &PoseReader](TConstStructView<FUAFSystemOutputComponent> InStructView)
	{
		const FUAFSystemOutputComponent& SystemOutputComponent = InStructView.Get<FUAFSystemOutputComponent>();
		SystemOutputComponent.ReadOutput(SerialNumber, PoseReader);
	});
}

}
