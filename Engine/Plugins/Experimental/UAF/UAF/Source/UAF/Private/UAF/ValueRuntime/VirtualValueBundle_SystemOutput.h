// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Graph/AnimNext_LODPose.h"
#include "Module/UAFWeakSystemReference.h"
#include "UAF/ValueRuntime/IVirtualValueBundle.h"
#include "UAF/ValueRuntime/ValueBundle.h"

#define UE_API UAF_API

namespace UE::UAF
{

struct FVirtualValueBundle_SystemOutput : public IVirtualValueBundle
{
	explicit FVirtualValueBundle_SystemOutput(const FUAFWeakSystemReference& InSystemReference)
		: SystemReference(InSystemReference)
	{}

private:
	// IVirtualValueBundle interface
	virtual const FAnimNextGraphLODPose* GetLODPose() const override;
	virtual const FValueBundleHeap* GetValueBundle() const override;

	// Read the output value bundle of the system
	void ReadSystemOutput() const;

	mutable FAnimNextGraphLODPose CachedPose;
	mutable FValueBundleHeap CachedValueBundle;
	FUAFWeakSystemReference SystemReference;
	mutable uint32 SerialNumber = 0;
};

}

#undef UE_API