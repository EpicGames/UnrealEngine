// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAssetInstance.h"

#include "AnimNextFunctionReference.h"
#include "AnimNextFunctionHandle.h"
#include "AnimNextRigVMAsset.h"
#include "ObjectTrace.h"
#include "Logging/StructuredLog.h"
#include "Script/UAFRigVMComponent.h"
#include "Variables/VariableOverrides.h"
#include "Variables/VariableOverridesCollection.h"
#include "UAFAssetInstanceComponent.h"
#include "Module/AnimNextModuleInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFAssetInstance)

void FUAFAssetInstance::InitializeVariables(const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides)
{
	Variables.Initialize(*this, InOverrides);
}

#if WITH_EDITOR
void FUAFAssetInstance::MigrateVariables()
{
	Variables.Migrate(*this);
}
#endif

void FUAFAssetInstance::CopyDefaultComponents()
{
	Components = GetAsset<UUAFRigVMAsset>()->Components;
}

void FUAFAssetInstance::BindDefaultComponents()
{
	for (TInstancedStruct<FUAFAssetInstanceComponent>& InstancedStruct : Components)
	{
		FUAFAssetInstanceComponent& Component = InstancedStruct.GetMutable<FUAFAssetInstanceComponent>();
		Component.Instance = this;
		Component.OnBindToInstance();
	}
}

void FUAFAssetInstance::ReleaseComponents()
{
	Components.Empty();
}

bool FUAFAssetInstance::ExecuteParameterlessFunction(const FAnimNextFunctionReference& InFunction, void* OutReturnValue, int32 ReturnValueSize)
{
	// Find the instance whose asset matches the function reference's owning asset.
	// The bindable value may live on a child graph instance, but the function is
	// typically defined on a parent system asset.
	const UObject* FunctionOwnerAsset = InFunction.GetObject();
	FUAFAssetInstance* TargetInstance = this;
	if (FunctionOwnerAsset)
	{
		while (TargetInstance && TargetInstance->GetAsset<UUAFRigVMAsset>() != FunctionOwnerAsset)
		{
			TargetInstance = TargetInstance->GetHost();
		}
	}
	if (!TargetInstance)
	{
		return false;
	}

	FUAFRigVMComponent* Component = TargetInstance->TryGetComponent<FUAFRigVMComponent>();
	if (!Component)
	{
		return false;
	}

	UUAFRigVMAsset* TargetAsset = const_cast<UUAFRigVMAsset*>(TargetInstance->GetAsset<UUAFRigVMAsset>());
	if (!TargetAsset)
	{
		return false;
	}

	// Try GUID lookup first (rename-robust), fall back to EventName in editor (backward compat)
	UE::UAF::FFunctionHandle Handle = TargetAsset->GetFunctionHandle(InFunction.GetFunctionGuid());
#if WITH_EDITORONLY_DATA
	if (!Handle.IsValid())
	{
		Handle = TargetAsset->GetFunctionHandle(InFunction.GetEventName());
	}
#endif
	if (!Handle.IsValid())
	{
		return false;
	}

	return TargetAsset->ExecuteParameterlessFunction(
		Handle, Component->GetExtendedExecuteContext(), OutReturnValue, ReturnValueSize);
}



FUAFAssetInstance::FUAFAssetInstance(UScriptStruct* InScriptStruct)
	: ScriptStruct(InScriptStruct)
{
	check(ScriptStruct != nullptr);
#if UAF_TRACE_ENABLED
	UniqueId = FObjectTrace::AllocateInstanceId();
#endif
}

#if DO_CHECK
bool FUAFAssetInstance::LayoutMatches(const FInstancedPropertyBag& InPropertyBag) const
{
	const UPropertyBag* PropertyBagStruct = InPropertyBag.GetPropertyBagStruct();
	if (PropertyBagStruct == nullptr)
	{
		return Variables.NumInternalVariables == 0;
	}

	if (Variables.NumInternalVariables != InPropertyBag.GetNumPropertiesInBag())
	{
		UE_LOGF(LogAnimation, Warning, "FUAFAssetInstance::LayoutMatches: Number of internal variables does not match number of properties in PropertyBag.");
		return false;
	}

	bool bMatches = true;
	TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBagStruct->GetPropertyDescs();
	Variables.ForEachVariable([&Descs, &bMatches](FName InName, const FAnimNextParamType& InType, int32 InVariableIndex)
	{
		if (!Descs.IsValidIndex(InVariableIndex))
		{
			UE_LOGF(LogAnimation, Warning, "FUAFAssetInstance::LayoutMatches: Variable Index %d is not a valid index into the PropertyBag's description.", InVariableIndex);
			bMatches = false;
			return false;
		}

		const FPropertyBagPropertyDesc& Desc = Descs[InVariableIndex];
		if (Desc.Name != InName)
		{
			UE_LOGF(LogAnimation, Warning, "FUAFAssetInstance::LayoutMatches: Variable name [%ls] does not match the PropertyBag's entry name [%ls].", *InName.ToString(), *Desc.Name.ToString());
			bMatches = false;
			return false;
		}

		if (Desc.ValueType != InType.GetValueType() ||
			Desc.ContainerTypes.GetFirstContainerType() != InType.GetContainerType() ||
			Desc.ValueTypeObject != InType.GetValueTypeObject())
		{
			UE_LOGF(LogAnimation, Warning, "FUAFAssetInstance::LayoutMatches: Variable type does not match the type of the PropertyBag's entry.");
			bMatches = false;
			return false;
		}

		return true;
	});

	return bMatches;
}
#endif

FUAFAssetInstanceComponent* FUAFAssetInstance::TryGetComponent(const UScriptStruct* InStruct)
{
	checkf(InStruct->IsChildOf<FUAFAssetInstanceComponent>(), TEXT("ComponentType type must derive from FUAFAssetInstanceComponent"));

	for (TInstancedStruct<FUAFAssetInstanceComponent>& Component : Components)
	{
		if(Component.GetScriptStruct()->IsChildOf(InStruct))
		{
			return Component.GetMutablePtr<FUAFAssetInstanceComponent>();
		}
	}

	return nullptr;
}

FAnimNextModuleInstance* FUAFAssetInstance::GetRootInstance()
{
	for (FUAFAssetInstance* CurrentHost = this; CurrentHost != nullptr; CurrentHost = CurrentHost->HostInstance.Pin().Get())
	{
		if (!CurrentHost->HostInstance.IsValid())
		{
			return CurrentHost->AsPtr<FAnimNextModuleInstance>();
		}
	}

	return nullptr;
}
