// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/AnimNextSharedVariables.h"
#include "AnimNextExecuteContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextSharedVariables)

#if WITH_EDITOR	
#include "Engine/ExternalAssetDependencyGatherer.h"
REGISTER_ASSETDEPENDENCY_GATHERER(FExternalAssetDependencyGatherer, UUAFSharedVariables);
#endif // WITH_EDITOR

UUAFSharedVariables::UUAFSharedVariables(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExtendedExecuteContext.SetContextPublicDataStruct(FAnimNextExecuteContext::StaticStruct());
}
