// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/AnimNextVariableOverridesCollection.h"
#include "AnimNextRigVMAsset.h"
#include "VariableOverridesCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextVariableOverridesCollection)

void FAnimNextVariableOverridesCollection::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	using namespace UE::UAF;

	if (TSharedPtr<FVariableOverridesCollection> PinnedCollection = Collection.Pin())
	{
		for (FVariableOverrides& Overrides : PinnedCollection->Collection)
		{
			switch(Overrides.AssetOrStructData.GetIndex())
			{
			case FVariableOverrides::FAssetOrStructType::IndexOfType<FVariableOverrides::FAssetType>():
				Collector.AddReferencedObject(Overrides.AssetOrStructData.Get<FVariableOverrides::FAssetType>());
				break;
			case FVariableOverrides::FAssetOrStructType::IndexOfType<FVariableOverrides::FStructType>():
				Collector.AddReferencedObject(Overrides.AssetOrStructData.Get<FVariableOverrides::FStructType>());
				break;
			default:
				checkNoEntry();
				break;
			}

			for (FVariableOverrides::FOverride& Override : Overrides.Overrides)
			{
				Collector.AddPropertyReferencesWithStructARO(FAnimNextParamType::StaticStruct(), &Override.Type);
			}
		}
	}
}
