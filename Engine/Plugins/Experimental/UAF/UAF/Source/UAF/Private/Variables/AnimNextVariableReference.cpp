// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/AnimNextVariableReference.h"
#include "AnimNextRigVMAsset.h"
#include "Variables/AnimNextSoftVariableReference.h"

#if WITH_EDITOR
#include "Modules/ModuleManager.h"
#include "IAnimNextUncookedOnlyModule.h"
#endif // WITH_EDITOR

#include "RigVMRuntimeAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextVariableReference)

FAnimNextVariableReference FAnimNextVariableReference::FromProperty(const FProperty* InProperty, const UScriptStruct* InStruct)
{
	check(InProperty && InStruct);
	const FName Name = InProperty->GetFName();
	check(Name != NAME_None);
	FGuid Guid;
#if WITH_EDITOR
	ensureAlwaysMsgf(!UE::UAF::IsInternalVariableName(Name), TEXT("Attempting to create variable reference using a variable name indicative of a programmatic variable %s"), *Name.ToString());
	Guid = UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule::Get().GetVariableGuidByName(Name, InStruct);
	if (!Guid.IsValid())
	{
		UE_LOGF(LogAnimation, Warning, "Unable to retrieve Variable Guid while attempting to construct FAnimNextVariableReference for %ls in %ls, variable might not exist on struct.", *Name.ToString(), *InStruct->GetPathName());
	}
#endif // WITH_EDITOR

	return FAnimNextVariableReference(Name, Guid, InStruct);
}

FAnimNextVariableReference FAnimNextVariableReference::FromName(FName InName, const UObject* InObject)
{
	check(InObject && InName != NAME_None);
	FGuid Guid;
#if WITH_EDITOR
	ensureAlwaysMsgf(!UE::UAF::IsInternalVariableName(InName), TEXT("Attempting to create variable reference using a variable name indicative of a programmatic variable %s"), *InName.ToString());

	Guid = UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule::Get().GetVariableGuidByName(InName, InObject);
	if (!Guid.IsValid())
	{
		UE_LOGF(LogAnimation, Warning, "Unable to retrieve Variable Guid while attempting to construct FAnimNextVariableReference for %ls in %ls, variable might not exist on object.", *InName.ToString(), *InObject->GetPathName());
	}
#endif // WITH_EDITOR

	return FAnimNextVariableReference(InName, Guid, InObject);
}

FAnimNextVariableReference::FAnimNextVariableReference(FName InName, FGuid InGuid, const UObject* InObject)
	: Name(InName)
	, Object(InObject)
#if WITH_EDITORONLY_DATA
	, CachedGuid(InGuid)
#endif // WITH_EDITORONLY_DATA
{
	check(InObject != nullptr);
	
#if WITH_EDITORONLY_DATA 
	check(Name != NAME_None	|| InGuid.IsValid());
	ensureAlwaysMsgf(!UE::UAF::IsInternalVariableName(Name), TEXT("Attempting to create variable reference using a variable name indicative of a programmatic variable %s"), *Name.ToString());
#else
	check(Name != NAME_None);
#endif // WITH_EDITORONLY_DATA
}

FAnimNextVariableReference::FAnimNextVariableReference(const FAnimNextSoftVariableReference& InSoftReference)
	: Name(InSoftReference.GetName())
	, Object(InSoftReference.GetSoftObjectPath().TryLoad())
{
	check(Object == nullptr || Object->IsA<UUAFRigVMAsset>() || Object->IsA<UScriptStruct>() || Object->Implements<URigVMRuntimeAssetInterface>());

#if WITH_EDITOR
	ensureAlwaysMsgf(!UE::UAF::IsInternalVariableName(Name), TEXT("Attempting to create variable reference using a variable name indicative of a programmatic variable %s"), *Name.ToString());
	if (Object != nullptr)
	{
		CachedGuid = UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule::Get().GetVariableGuidByName(Name, Object);
		if (!CachedGuid.IsValid() && !InSoftReference.HasType())
		{
			UE_LOGF(LogAnimation, Warning, "Unable to retrieve Variable Guid while attempting to construct FAnimNextVariableReference for %ls in %ls, variable might not exist on object.", *Name.ToString(), *Object->GetPathName());
		}
	}
#endif // WITH_EDITOR
}

FAnimNextVariableReference::FAnimNextVariableReference(FName InName, const UUAFRigVMAsset* InAsset)
	: Name(InName)
	, Object(InAsset)
{
	check(InAsset != nullptr);
	ensureAlwaysMsgf(!UE::UAF::IsInternalVariableName(Name), TEXT("Attempting to create variable reference using a variable name indicative of a programmatic variable %s"), *Name.ToString());
}

FAnimNextVariableReference::FAnimNextVariableReference(FName InName, const UScriptStruct* InStruct)
	: Name(InName)
	, Object(InStruct)
{
	check(InStruct != nullptr);
	ensureAlwaysMsgf(!UE::UAF::IsInternalVariableName(Name), TEXT("Attempting to create variable reference using a variable name indicative of a programmatic variable %s"), *Name.ToString());
}

#if WITH_EDITOR
FAnimNextVariableReference FAnimNextVariableReference::FromNameAndGuid(FName InName, FGuid InGuid, const UObject* InObject)
{
	check(InObject && InName != NAME_None && InGuid.IsValid());
	return FAnimNextVariableReference(InName, InGuid, InObject);
}

bool FAnimNextVariableReference::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	return false;
}

void FAnimNextVariableReference::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::UAFVariableReferenceGUID)
		{
			if (Object)
			{
				const FGuid Guid =  UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule::Get().GetVariableGuidByName(Name, Object);
				if (!Guid.IsValid())
				{
					UE_LOGF(LogAnimation, Warning, "Could not find variable %ls in %ls referenced by %ls, invalidating reference", *Name.ToString(), *Object->GetPathName(), *Ar.GetArchiveState().GetArchiveName());
				}

				CachedGuid = Guid;
			}
		}
		else if (Name != NAME_None)
		{
			ValidateVariableNameAndGuid(Name, CachedGuid, Object, Ar.GetArchiveState().GetArchiveName());
		}
	}
}

void FAnimNextVariableReference::ValidateVariableNameAndGuid(FName& InOutName, FGuid& InOutGuid, const UObject* InSourceObject, const FString& OwnerName)
{
	if (InSourceObject)
	{
		const FName CachedName = InOutName;
		const FGuid CachedGuid = InOutGuid;

		UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule& UncookedModule = UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule::Get();
		const FName FoundName = UncookedModule.GetVariableNameByGuid(InOutGuid, InSourceObject);
		if (FoundName == NAME_None)
		{
			const FGuid FoundGuid = UncookedModule.GetVariableGuidByName(InOutName, InSourceObject);
			if (FoundGuid.IsValid())
			{
				InOutGuid = FoundGuid;
				
				const FName RedirectedName = UncookedModule.GetVariableNameByGuid(InOutGuid, InSourceObject);
				if (RedirectedName != InOutName)
				{
					InOutName = RedirectedName;
				}
				
				UE_LOGF(LogAnimation, Warning, "Out-of-date variable reference to %ls [%ls] (now %ls [%ls]) from %ls please re-save %ls", *CachedName.ToString(), *CachedGuid.ToString(), *InOutName.ToString(), *InOutGuid.ToString(), *InSourceObject->GetPathName(), *OwnerName);
			}
			else
			{
				// No guid or name, invalid reference
				InOutName = NAME_None;
				InOutGuid.Invalidate();

				UE_LOGF(LogAnimation, Warning, "Could not find variable %ls in %ls referenced by %ls, invalidating (soft)reference", *InOutName.ToString(), *InSourceObject->GetPathName(), *OwnerName);
			}
		}
		else if (FoundName != InOutName)
		{
			InOutName = FoundName;
			UE_LOGF(LogAnimation, Warning, "Out-of-date variable reference to %ls [%ls] (now %ls [%ls]) from %ls please re-save %ls", *CachedName.ToString(), *CachedGuid.ToString(), *FoundName.ToString(), *InOutGuid.ToString(), *InSourceObject->GetPathName(), *OwnerName);
		}
	}
	// Only log warnings for reference with a valid name (but invalid source object)
	else if (InOutName != NAME_None)
	{
		UE_LOGF(LogAnimation, Warning, "Could not find variable %ls referenced by %ls as source object does not exist, invalidating reference", *InOutName.ToString(), *OwnerName);

		InOutName = NAME_None;
		InOutGuid.Invalidate();
	}
}
#endif // WITH_EDITOR

bool FAnimNextVariableReference::Identical(const FAnimNextVariableReference* Other, uint32 PortFlags) const
{
	return Other != nullptr && (Name == Other->Name && Object == Other->Object);
}

bool FAnimNextVariableReference::IsValid() const
{
	// TODO: Remove legacy name-based lookup path when deprecations are all fixed up
	if (Object == nullptr)
	{
		return !Name.IsNone();
	}

	return ResolveProperty() != nullptr;
}

const FProperty* FAnimNextVariableReference::ResolveProperty() const
{
	if (CachedProperty.IsPathToFieldEmpty() || CachedProperty.IsStale())
	{
		if (const UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(Object))
		{
			if (const FPropertyBagPropertyDesc* NameDesc = Asset->GetVariableDefaults().FindPropertyDescByName(Name))
			{
				CachedProperty = NameDesc->CachedProperty;
			}
		}
		else if (const UScriptStruct* Struct = Cast<UScriptStruct>(Object))
		{
			if (const FProperty* FoundProperty = Struct->FindPropertyByName(Name))
			{
				CachedProperty = FoundProperty;
			}
		}
		else if (const IRigVMRuntimeAssetInterface* RigVMAsset = Cast<IRigVMRuntimeAssetInterface>(Object))
		{
			if (const FProperty* FoundProperty = RigVMAsset->FindGeneratedPropertyByName(Name))
			{
				CachedProperty = FoundProperty;
			}
		}
	}
	return CachedProperty.Get();
}
