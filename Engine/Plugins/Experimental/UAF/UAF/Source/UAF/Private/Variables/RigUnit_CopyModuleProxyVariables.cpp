// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/RigUnit_CopyModuleProxyVariables.h"
#include "Module/AnimNextModuleInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_CopyModuleProxyVariables)

FRigUnit_CopyModuleProxyVariables_Execute()
{
	const FAnimNextModuleContextData& ContextData = ExecuteContext.GetContextData<FAnimNextModuleContextData>();
	ContextData.GetModuleInstance().CopyProxyVariables();
}
