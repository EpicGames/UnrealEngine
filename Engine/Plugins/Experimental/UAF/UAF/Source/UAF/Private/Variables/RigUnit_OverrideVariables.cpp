// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_OverrideVariables.h"

#include "RigVMTrait_OverrideVariables.h"
#include "VariableOverridesCollection.h"

#if WITH_EDITOR
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Blueprint/BlueprintSupport.h"
#include "RigVMModel/RigVMController.h"
#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "ToolMenus.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SBox.h"
#include "Variables/AnimNextSharedVariables.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_OverrideVariables)

#define LOCTEXT_NAMESPACE "AnimNextOverrideVariables"

FRigUnit_OverrideVariables_Execute()
{
	// We create the overrides one time, then leave the overrides to live as long as the execute context
	if (!WorkData.Collection.IsValid())
	{
		WorkData.Collection = MakeShared<UE::UAF::FVariableOverridesCollection>();

		FAnimNextModuleInstance& ModuleInstance = ExecuteContext.GetContextData<FAnimNextModuleContextData>().GetModuleInstance();
		TArray<UE::UAF::FVariableOverrides> VariableOverrides;
		for (const FRigVMTraitScope& TraitScope : ExecuteContext.GetTraits())
		{
			const FRigVMTrait_OverrideVariables* OverrideTrait = TraitScope.GetTrait<FRigVMTrait_OverrideVariables>();
			if (OverrideTrait == nullptr)
			{
				continue;
			}

			OverrideTrait->GenerateOverrides(TraitScope, VariableOverrides);
		}

		if (VariableOverrides.Num() > 0)
		{
			WorkData.Collection->Collection = VariableOverrides;
		}

		Overrides.Collection = WorkData.Collection;
	}
}


TArray<FRigVMUserWorkflow> FRigUnit_OverrideVariables::GetSupportedWorkflows(const UObject* InSubject) const
{
	TArray<FRigVMUserWorkflow> Workflows = Super::GetSupportedWorkflows(InSubject);

#if WITH_EDITOR
	Workflows.Emplace(
		TEXT("Add"),
		TEXT("Adds an override for a shared variables asset or struct"),
		ERigVMUserWorkflowType::NodeContextButton,
		FRigVMPerformUserWorkflowDelegate::CreateLambda([](const URigVMUserWorkflowOptions* InOptions, UObject* InController)
		{
			URigVMController* Controller = CastChecked<URigVMController>(InController);
			if (Controller == nullptr)
			{
				return false;
			}

			URigVMNode* Node = InOptions->GetSubject<URigVMNode>();
			if (Node == nullptr)
			{
				return false;
			}

			static const FName MenuName = "AnimNextVariableOverridesAddMenu";
			if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
			{
				UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
				Menu->AddDynamicSection("DependencyTraits", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					FToolMenuSection& Section = InMenu->AddSection("DependencyTraits");
					Section.AddSubMenu(
						"AddAssetOverrides",
						LOCTEXT("AddAssetOverrideSubMenuTitle", "Asset Override"),
						LOCTEXT("AddAssetOverrideSubMenuTooltip", "Add set of overrides for an asset"),
						FNewToolMenuDelegate::CreateLambda([](UToolMenu* InSubMenu)
						{
							UAnimNextOverrideVariablesMenuContext* ContextObject = InSubMenu->FindContext<UAnimNextOverrideVariablesMenuContext>();
							if (ContextObject == nullptr)
							{
								return;
							}

							FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

							FAssetPickerConfig AssetPickerConfig;
							AssetPickerConfig.Filter.bRecursiveClasses = true;
							AssetPickerConfig.Filter.ClassPaths.Add(UUAFSharedVariables::StaticClass()->GetClassPathName());
							AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
							AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([Controller = ContextObject->Controller, Node = ContextObject->Node](const FAssetData& InAssetData)
							{
								FSlateApplication::Get().DismissAllMenus();
								
								FScopedTransaction Transaction(LOCTEXT("AddOverridenVariablesTransaction", "Add overriden variables"));

								UUAFSharedVariables* Asset = CastChecked<UUAFSharedVariables>(InAssetData.GetAsset());
								
								FRigVMTrait_OverrideVariablesForAsset Trait;
								Trait.Asset = Asset;

								const FInstancedPropertyBag& VariableDefaults = Asset->GetVariableDefaults();
								if (const UPropertyBag* PropertyBag = VariableDefaults.GetPropertyBagStruct())
								{
									const void* ContainerPtr = VariableDefaults.GetValue().GetMemory();
									for (const FPropertyBagPropertyDesc& Desc : PropertyBag->GetPropertyDescs())
									{
										if (Desc.CachedProperty->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPublic))
										{
											const void* ValuePtr = Desc.CachedProperty->ContainerPtrToValuePtr<void>(ContainerPtr);
											FString PropertyDefaultValue;
											Desc.CachedProperty->ExportText_Direct(PropertyDefaultValue, ValuePtr, ValuePtr, nullptr, PPF_None);
											Trait.Overrides.Emplace(Desc.CachedProperty->GetFName(), FAnimNextParamType::FromPropertyBagPropertyDesc(Desc), PropertyDefaultValue);
										}
									}

									FString TraitDefaultValue;
									FRigVMTrait_OverrideVariablesForAsset::StaticStruct()->ExportText(TraitDefaultValue, &Trait, &Trait, nullptr, PPF_None, nullptr);
									Controller->AddTrait(Node, FRigVMTrait_OverrideVariablesForAsset::StaticStruct(), Asset->GetFName(), TraitDefaultValue);
								}
							});

							FToolMenuSection& Section = InSubMenu->AddSection("AssetPicker");
							Section.AddEntry(
								FToolMenuEntry::InitWidget(
									"AssetPicker",
									SNew(SBox)
									.WidthOverride(300.0f)
									.HeightOverride(400.0f)
									[
										ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
									],
									FText::GetEmpty(),
									true,
									false,
									true));
						}));

					Section.AddSubMenu(
						"AddStructOverrides",
						LOCTEXT("AddStructOverrideSubMenuTitle", "Struct Override"),
						LOCTEXT("AddStructOverrideSubMenuTooltip", "Add set of overrides for a struct"),
						FNewToolMenuDelegate::CreateLambda([](UToolMenu* InSubMenu)
						{
							UAnimNextOverrideVariablesMenuContext* ContextObject = InSubMenu->FindContext<UAnimNextOverrideVariablesMenuContext>();
							if (ContextObject == nullptr)
							{
								return;
							}

							auto OnStructPicked = [Controller = ContextObject->Controller, Node = ContextObject->Node](const UScriptStruct* InStruct)
							{
								FSlateApplication::Get().DismissAllMenus();

								FScopedTransaction Transaction(LOCTEXT("AddOverridenVariablesTransaction", "Add overriden variables"));

								FRigVMTrait_OverrideVariablesForStruct Trait;
								Trait.Struct = InStruct;

								FInstancedStruct Instance(InStruct);
								const void* ContainerPtr = Instance.GetMemory();
								for (TFieldIterator<FProperty> It(InStruct); It; ++It)
								{
									const FProperty* Property = *It;
									const void* ValuePtr = Property->ContainerPtrToValuePtr<void>(ContainerPtr);
									FString PropertyDefaultValue;
									Property->ExportText_Direct(PropertyDefaultValue, ValuePtr, ValuePtr, nullptr, PPF_None);
									Trait.Overrides.Emplace(Property->GetFName(), FAnimNextParamType::FromProperty(Property), PropertyDefaultValue);
								}

								FString TraitDefaultValue;
								FRigVMTrait_OverrideVariablesForStruct::StaticStruct()->ExportText(TraitDefaultValue, &Trait, &Trait, nullptr, PPF_None, nullptr);
								Controller->AddTrait(Node, FRigVMTrait_OverrideVariablesForStruct::StaticStruct(), InStruct->GetFName(), TraitDefaultValue);
							};

							class FStructFilter : public IStructViewerFilter
							{
							public:
								virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
								{
									if (InStruct->IsA<UUserDefinedStruct>())
									{
										return false;
									}

									if (!InStruct->HasMetaData(TEXT("BlueprintType")) && InStruct->HasMetaData(TEXT("Hidden")))
									{
										return false;
									}

									return InStruct->HasMetaData(FBlueprintTags::BlueprintType);
								}

								virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<class FStructViewerFilterFuncs> InFilterFuncs)
								{
									return false;
								};
							};
							
							FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");
							FStructViewerInitializationOptions InitOptions;
							InitOptions.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
							InitOptions.StructFilter = MakeShared<FStructFilter>();
							FToolMenuSection& Section = InSubMenu->AddSection("StructPicker");
							Section.AddEntry(
								FToolMenuEntry::InitWidget(
									"StructPicker",
									SNew(SBox)
									.WidthOverride(300.0f)
									.HeightOverride(400.0f)
									[
										StructViewerModule.CreateStructViewer(InitOptions, FOnStructPicked::CreateLambda(OnStructPicked))
									],
									FText::GetEmpty(),
									true,
									false,
									true));
						}));


				}));
			}

			UAnimNextOverrideVariablesMenuContext* ContextObject = NewObject<UAnimNextOverrideVariablesMenuContext>();
			ContextObject->Controller = Controller;
			ContextObject->Node = Node;

			FSlateApplication::Get().PushMenu(
				FSlateApplication::Get().GetInteractiveTopLevelWindows()[0],
				FWidgetPath(),
				UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(ContextObject)),
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);

			return true;
		}),
		URigVMUserWorkflowOptions::StaticClass());

	Workflows.Emplace(
		TEXT("Remove Override"),
		TEXT("Removes this set of overriden variables"),
		ERigVMUserWorkflowType::PinContext,
		FRigVMPerformUserWorkflowDelegate::CreateLambda([](const URigVMUserWorkflowOptions* InOptions, UObject* InController)
		{
			URigVMController* Controller = CastChecked<URigVMController>(InController);
			if (Controller == nullptr)
			{
				return false;
			}

			URigVMPin* Pin = InOptions->GetSubject<URigVMPin>();
			if (Pin == nullptr)
			{
				return false;
			}

			if (!Pin->IsTraitPin() || Pin->GetTraitScriptStruct() == nullptr || !Pin->GetTraitScriptStruct()->IsChildOf(FRigVMTrait_OverrideVariables::StaticStruct()))
			{
				return false;
			}

			URigVMPin* TraitPin = Pin->GetNode()->FindTrait(Pin);
			if (TraitPin == nullptr)
			{
				return false;
			}

			FScopedTransaction Transaction(LOCTEXT("RemoveOverridenVariablesTransaction", "Remove overriden variables"));
			Controller->RemoveTrait(Pin->GetNode(), TraitPin->GetFName());

			return true;
		}),
		URigVMUserWorkflowOptions::StaticClass());
#endif

	return Workflows;
}

#undef LOCTEXT_NAMESPACE
