// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "AnimNextExecuteContext.h"
#include "Variables/AnimNextVariableOverridesCollection.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "RigUnit_OverrideVariables.generated.h"

#define UE_API UAF_API

class URigVMController;
class URigVMNode;

UCLASS()
class UAnimNextOverrideVariablesMenuContext : public UObject
{
	GENERATED_BODY()
public:
	TObjectPtr<URigVMController> Controller = nullptr;
	TObjectPtr<URigVMNode> Node = nullptr;
};

USTRUCT()
struct FAnimNextOverrideVariablesWorkData
{
	GENERATED_BODY()

	// Strong reference held in work data
	TSharedPtr<UE::UAF::FVariableOverridesCollection> Collection;
};

/** Hosts a number of traits that allow overrides of variables for specific assets/structs */
USTRUCT(meta = (DisplayName = "Override Variables", NodeColor="0, 1, 1", Category="Variables"))
struct FRigUnit_OverrideVariables : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API void Execute();

	// FRigVMStruct interface
	UE_API virtual TArray<FRigVMUserWorkflow> GetSupportedWorkflows(const UObject* InSubject) const override;

	// Variable overrides to be applied to subgraphs
	UPROPERTY(EditAnywhere, Category = "Variables", meta = (Output))
	FAnimNextVariableOverridesCollection Overrides;

	UPROPERTY(Transient)
	FAnimNextOverrideVariablesWorkData WorkData;
};

#undef UE_API
