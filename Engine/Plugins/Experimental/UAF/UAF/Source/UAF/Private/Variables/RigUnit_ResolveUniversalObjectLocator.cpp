// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/RigUnit_ResolveUniversalObjectLocator.h"
#include "Module/AnimNextModuleInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_ResolveUniversalObjectLocator)

FRigUnit_ResolveUniversalObjectLocator_Execute()
{
	using namespace UE::UniversalObjectLocator;

	const FAnimNextModuleContextData& ContextData = ExecuteContext.GetContextData<FAnimNextModuleContextData>();

	FResolveParams ResolveParams;
	ResolveParams.Context = ContextData.GetObject();
	FResolveResult Result = Locator.Resolve(ResolveParams);
	Object = Result.SyncGet().Object;
}
