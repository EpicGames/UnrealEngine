// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/RigVMDispatch_CallHoistedAccessorFunction.h"
#include "RigVMCore/RigVMStruct.h"
#include "Variables/AnimNextSoftFunctionPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_CallHoistedAccessorFunction)

FRigVMDispatch_CallHoistedAccessorFunctionBase::FRigVMDispatch_CallHoistedAccessorFunctionBase()
{
	FactoryScriptStruct = StaticStruct();
}

FName FRigVMDispatch_CallHoistedAccessorFunctionBase::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FName ArgumentNames[] =
	{
		ObjectName,
		FunctionName,
		ValueName,
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

void FRigVMDispatch_CallHoistedAccessorFunctionBase::RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const
{
	static UScriptStruct* const AllowedStructTypes[] =
	{
		FAnimNextSoftFunctionPtr::StaticStruct(),
	};

	InRegistry->RegisterStructTypes_NoLock(AllowedStructTypes);
	for(UScriptStruct* const ScriptStruct : AllowedStructTypes)
	{
		InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(ScriptStruct));
	}

	static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
	{
		{ UObject::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
	};

	InRegistry->RegisterObjectTypes_NoLock(AllowedObjectTypes);

	for(const TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation>& ObjectType : AllowedObjectTypes)
	{
		InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(ObjectType.Key));
	}
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_CallHoistedAccessorFunctionBase::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories =
		{
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};

		CachedArgumentInfos.Emplace(ObjectName, ERigVMPinDirection::Input, InRegistry->GetTypeIndex_NoLock<UObject>());
		CachedArgumentInfos.Emplace(FunctionName, ERigVMPinDirection::Input, InRegistry->GetTypeIndex_NoLock<FAnimNextSoftFunctionPtr>());
		CachedArgumentInfos.Emplace(ValueName, ERigVMPinDirection::Output, ValueCategories);
	}

	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_CallHoistedAccessorFunctionBase::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(ObjectName, InRegistry->GetTypeIndex_NoLock<UObject>());
	Types.Add(FunctionName, InRegistry->GetTypeIndex_NoLock<FAnimNextSoftFunctionPtr>());
	Types.Add(ValueName, InTypeIndex);
	return Types;
}

FRigVMDispatch_CallHoistedAccessorFunctionNative::FRigVMDispatch_CallHoistedAccessorFunctionNative()
{
	FactoryScriptStruct = StaticStruct();
}

void FRigVMDispatch_CallHoistedAccessorFunctionNative::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	UObject* ObjectPtr = *reinterpret_cast<UObject**>(Handles[0].GetOutputData());
	if(ObjectPtr == nullptr)
	{
		// Something failed to resolve upstream, OK to just skip this work
		return;
	}

	FAnimNextSoftFunctionPtr* SoftFunctionPtr = reinterpret_cast<FAnimNextSoftFunctionPtr*>(Handles[1].GetOutputData());
	UFunction* Function = SoftFunctionPtr->SoftObjectPtr.Get();
	if(Function == nullptr || Function->NumParms != 2)
	{
		return;
	}

	const FProperty* ReturnValueProperty = CastField<FProperty>(Function->GetReturnProperty());
	if(ReturnValueProperty == nullptr)
	{
		return;
	}

	const FProperty* TargetProperty = Handles[2].GetResolvedProperty();
	checkSlow(TargetProperty);

	if (!ensureMsgf(TargetProperty->GetClass() == ReturnValueProperty->GetClass(), TEXT("FRigVMDispatch_CallHoistedAccessorFunctionNative::Execute - Type mismatch: %s, %s")
		, *TargetProperty->GetClass()->GetFName().ToString()
		, *ReturnValueProperty->GetClass()->GetFName().ToString()))
	{
		return;
	}

	uint8* TargetAddress = Handles[2].GetOutputData();
	checkSlow(TargetAddress);

	UObject* CDO = Function->GetOuterUClass()->GetDefaultObject();
	FFrame Stack(CDO, Function, &ObjectPtr, nullptr, Function->ChildProperties);
	Function->Invoke(CDO, Stack, TargetAddress);
}

FRigVMDispatch_CallHoistedAccessorFunctionScript::FRigVMDispatch_CallHoistedAccessorFunctionScript()
{
	FactoryScriptStruct = StaticStruct();
}

void FRigVMDispatch_CallHoistedAccessorFunctionScript::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	UObject* ObjectPtr = *reinterpret_cast<UObject**>(Handles[0].GetOutputData());
	if(ObjectPtr == nullptr)
	{
		// Something failed to resolve upstream, OK to just skip this work
		return;
	}

	FAnimNextSoftFunctionPtr* SoftFunctionPtr = reinterpret_cast<FAnimNextSoftFunctionPtr*>(Handles[1].GetOutputData());
	UFunction* Function = SoftFunctionPtr->SoftObjectPtr.Get();
	if(Function == nullptr || Function->NumParms != 3)
	{
		return;
	}

	const FProperty* ReturnValueProperty = CastField<FProperty>(Function->GetReturnProperty());
	if(ReturnValueProperty == nullptr)
	{
		return;
	}
	
	const FObjectProperty* HoistedProperty = CastField<FObjectProperty>(Function->PropertyLink);
	if(HoistedProperty == nullptr)
	{
		return;
	}

	check(ObjectPtr->GetClass()->IsChildOf(HoistedProperty->PropertyClass));

	const FObjectProperty* WorldContextProperty = CastField<FObjectProperty>(HoistedProperty->Next);
	if(WorldContextProperty == nullptr)
	{
		return;
	}

	const FProperty* TargetProperty = Handles[2].GetResolvedProperty();
	checkSlow(TargetProperty);

	if (!ensureMsgf(TargetProperty->GetClass() == ReturnValueProperty->GetClass(), TEXT("FRigVMDispatch_CallHoistedAccessorFunctionScript::Execute - Type mismatch: %s, %s")
		, *TargetProperty->GetClass()->GetFName().ToString()
		, *ReturnValueProperty->GetClass()->GetFName().ToString()))
	{
		return;
	}

	uint8* TargetAddress = Handles[2].GetOutputData();
	checkSlow(TargetAddress);

	// Allocate space for function params (script calls require multiple params like the input object and world context to be in one contiguous struct)
	void* CallParams = FMemory_Alloca_Aligned(Function->GetStructureSize(), Function->GetMinAlignment());
	Function->InitializeStruct(CallParams);
	HoistedProperty->SetObjectPropertyValue_InContainer(CallParams, ObjectPtr);

	// Note: currently not setting the world context object here.
	// If it is needed we can add it, but its generally not thread safe to use

	UObject* CDO = Function->GetOuterUClass()->GetDefaultObject();
	CDO->ProcessEvent(Function, CallParams);

	// Now copy result out to target address
	void* SourceAddress = ReturnValueProperty->ContainerPtrToValuePtr<void>(CallParams);
	ReturnValueProperty->CopyCompleteValue(TargetAddress, SourceAddress);
}

