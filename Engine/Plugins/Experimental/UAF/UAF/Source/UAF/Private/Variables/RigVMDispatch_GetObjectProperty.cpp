// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/RigVMDispatch_GetObjectProperty.h"
#include "RigVMCore/RigVMStruct.h"
#include "Variables/AnimNextFieldPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_GetObjectProperty)

FRigVMDispatch_GetObjectProperty::FRigVMDispatch_GetObjectProperty()
{
	FactoryScriptStruct = StaticStruct();
}

FName FRigVMDispatch_GetObjectProperty::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FName ArgumentNames[] =
	{
		ObjectName,
		PropertyName,
		ValueName,
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

void FRigVMDispatch_GetObjectProperty::RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const
{
	static UScriptStruct* const AllowedStructTypes[] =
	{
		FAnimNextFieldPath::StaticStruct(),
	};

	InRegistry->RegisterStructTypes_NoLock(AllowedStructTypes);
	for(UScriptStruct* const ScriptStruct : AllowedStructTypes)
	{
		InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(ScriptStruct));
	}

	static TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation> const AllowedObjectTypes[] =
	{
		{ UObject::StaticClass(), FRigVMRegistry::ERegisterObjectOperation::Class },
	};

	InRegistry->RegisterObjectTypes_NoLock(AllowedObjectTypes);

	for(const TPair<UClass*, FRigVMRegistry::ERegisterObjectOperation>& ObjectType : AllowedObjectTypes)
	{
		InRegistry->FindOrAddType_NoLock(FRigVMTemplateArgumentType(ObjectType.Key));
	}
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_GetObjectProperty::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories =
		{
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};

		CachedArgumentInfos.Emplace(ObjectName, ERigVMPinDirection::Input, InRegistry->GetTypeIndex_NoLock<UObject>());
		CachedArgumentInfos.Emplace(PropertyName, ERigVMPinDirection::Input, InRegistry->GetTypeIndex_NoLock<FAnimNextFieldPath>());
		CachedArgumentInfos.Emplace(ValueName, ERigVMPinDirection::Output, ValueCategories);
	}

	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_GetObjectProperty::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(ObjectName, InRegistry->GetTypeIndex_NoLock<UObject>());
	Types.Add(PropertyName, InRegistry->GetTypeIndex_NoLock<FAnimNextFieldPath>());
	Types.Add(ValueName, InTypeIndex);
	return Types;
}

void FRigVMDispatch_GetObjectProperty::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	const UObject* ObjectPtr = *reinterpret_cast<UObject**>(Handles[0].GetOutputData());
	if(ObjectPtr == nullptr)
	{
		// Something failed to resolve upstream, OK to just skip this work
		return;
	}

	const FAnimNextFieldPath* FieldPathPtr = reinterpret_cast<const FAnimNextFieldPath*>(Handles[1].GetInputData());
	const FProperty* SourceProperty = FieldPathPtr->FieldPath.Get();
	if(SourceProperty == nullptr)
	{
		return;
	}

	const uint8* SourceAddress = SourceProperty->ContainerPtrToValuePtr<uint8>(ObjectPtr);
	checkSlow(SourceAddress);

	const FProperty* TargetProperty = Handles[2].GetResolvedProperty();
	checkSlow(TargetProperty);

	if (!ensureMsgf(TargetProperty->GetClass() == SourceProperty->GetClass(), TEXT("FRigVMDispatch_GetObjectProperty::Execute - Type mismatch: %s, %s")
		, *TargetProperty->GetClass()->GetFName().ToString()
		, *SourceProperty->GetClass()->GetFName().ToString()))
	{
		return;
	}

	uint8* TargetAddress = Handles[2].GetOutputData();
	checkSlow(TargetAddress);

	// TODO: add a specialization for bool to GetDispatchFunctionImpl rather than branching here
	const FBoolProperty* SourceBoolProperty = CastField<FBoolProperty>(SourceProperty);
	if(SourceBoolProperty)
	{
		static_cast<const FBoolProperty*>(TargetProperty)->SetPropertyValue(TargetAddress, SourceBoolProperty->GetPropertyValue(SourceAddress));
	}
	else
	{
		SourceProperty->CopyCompleteValue(TargetAddress, SourceAddress);
	}
}

