// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTrait_OverrideVariables.h"
#include "VariableOverrides.h"
#include "AnimNextRigVMAsset.h"

#if WITH_EDITOR
#include "RigVMModel/RigVMPin.h"
#include "RigVMModel/RigVMController.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMTrait_OverrideVariables)

#if WITH_EDITOR
void FRigVMTrait_OverrideVariables::GetProgrammaticPins(URigVMController* InController, int32 InParentPinIndex, const URigVMPin* InTraitPin, const FString& InDefaultValue, FRigVMPinInfoArray& OutPinArray) const
{
	TArray<FPropertyBagPropertyDesc> Descs;
	Descs.Reserve(Overrides.Num());

	for (const FAnimNextVariableOverrideInfo& VariableOverrideInfo : Overrides)
	{
		if (!VariableOverrideInfo.Name.IsNone() && VariableOverrideInfo.Type.IsValid())
		{
			Descs.Emplace(VariableOverrideInfo.Name, VariableOverrideInfo.Type.GetContainerType(), VariableOverrideInfo.Type.GetValueType(), VariableOverrideInfo.Type.GetValueTypeObject());
		}
	}

	FInstancedPropertyBag Defaults;
	Defaults.AddProperties(Descs);

	for (const FAnimNextVariableOverrideInfo& VariableOverrideInfo : Overrides)
	{
		Defaults.SetValueSerializedString(VariableOverrideInfo.Name, VariableOverrideInfo.DefaultValue);
	}

	const TFunction<ERigVMPinDefaultValueType(const FName&)> DefaultValueTypeGetter = [](const FName& InPropertyName)
	{
		return ERigVMPinDefaultValueType::AutoDetect;
	};

	OutPinArray.AddPins(const_cast<UPropertyBag*>(Defaults.GetPropertyBagStruct()), InController, ERigVMPinDirection::Input, InParentPinIndex, DefaultValueTypeGetter, Defaults.GetValue().GetMemory(), true);
}

bool FRigVMTrait_OverrideVariables::ShouldCreatePinForProperty(const FProperty* InProperty) const
{
	return
		Overrides.ContainsByPredicate([VariableName = InProperty->GetFName()](const FAnimNextVariableOverrideInfo& InVariableOverrideInfo)
		{
			return InVariableOverrideInfo.Name == VariableName;
		});
}
#endif

#if WITH_EDITOR
FString FRigVMTrait_OverrideVariablesForAsset::GetDisplayName() const
{
	return Asset != nullptr ? Asset->GetFName().ToString() : TEXT("None");
}
#endif

void FRigVMTrait_OverrideVariablesForAsset::GenerateOverrides(const FRigVMTraitScope& InTraitScope, TArray<UE::UAF::FVariableOverrides>& OutOverrides) const
{
	if (Asset != nullptr)
	{
		TConstArrayView<FRigVMMemoryHandle> MemoryHandles = InTraitScope.GetAdditionalMemoryHandles();
		int32 NumVariables = MemoryHandles.Num();
		check(Overrides.Num() == NumVariables);

		TArray<UE::UAF::FVariableOverrides::FOverride> AssetOverrides;
		AssetOverrides.Reserve(NumVariables);

		const FInstancedPropertyBag& PropertyBag = Asset->GetVariableDefaults();
		for (int32 VariableIndex = 0; VariableIndex < NumVariables; ++VariableIndex)
		{
			const FAnimNextVariableOverrideInfo& OverrideInfo = Overrides[VariableIndex];
			const FRigVMMemoryHandle& MemoryHandle = MemoryHandles[VariableIndex];
			if (const FPropertyBagPropertyDesc* Desc = PropertyBag.FindPropertyDescByName(OverrideInfo.Name))
			{
				check(OverrideInfo.Type == FAnimNextParamType::FromProperty(Desc->CachedProperty));
				check(OverrideInfo.Type == FAnimNextParamType::FromProperty(MemoryHandle.GetProperty()));
				AssetOverrides.Emplace(Desc->Name, OverrideInfo.Type, const_cast<uint8*>(MemoryHandle.GetInputData()));
			}
		}

		if (AssetOverrides.Num() > 0)
		{
			OutOverrides.Emplace(Asset, MoveTemp(AssetOverrides));
		}
	}
}

#if WITH_EDITOR
FString FRigVMTrait_OverrideVariablesForStruct::GetDisplayName() const
{
	return Struct != nullptr ? Struct->GetName() : TEXT("None");
}
#endif

void FRigVMTrait_OverrideVariablesForStruct::GenerateOverrides(const FRigVMTraitScope& InTraitScope, TArray<UE::UAF::FVariableOverrides>& OutOverrides) const
{
	if (Struct != nullptr)
	{
		TConstArrayView<FRigVMMemoryHandle> MemoryHandles = InTraitScope.GetAdditionalMemoryHandles();
		int32 NumVariables = MemoryHandles.Num();
		check(Overrides.Num() == NumVariables);

		TArray<UE::UAF::FVariableOverrides::FOverride> StructOverrides;
		StructOverrides.Reserve(NumVariables);

		for (int32 VariableIndex = 0; VariableIndex < NumVariables; ++VariableIndex)
		{
			const FAnimNextVariableOverrideInfo& OverrideInfo = Overrides[VariableIndex];
			const FRigVMMemoryHandle& MemoryHandle = MemoryHandles[VariableIndex];
			if (const FProperty* Property = Struct->FindPropertyByName(OverrideInfo.Name))
			{
				check(OverrideInfo.Type == FAnimNextParamType::FromProperty(Property));
				check(OverrideInfo.Type == FAnimNextParamType::FromProperty(MemoryHandle.GetProperty()));
				StructOverrides.Emplace(Property->GetFName(), OverrideInfo.Type, const_cast<uint8*>(MemoryHandle.GetInputData()));
			}
		}

		if (StructOverrides.Num() > 0)
		{
			OutOverrides.Emplace(Struct, MoveTemp(StructOverrides));
		}
	}
}
