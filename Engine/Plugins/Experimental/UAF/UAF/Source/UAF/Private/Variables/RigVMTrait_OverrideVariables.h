// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMTrait.h"
#include "Param/ParamType.h"
#include "RigVMTrait_OverrideVariables.generated.h"

#define UE_API UAF_API

class UUAFRigVMAsset;

namespace UE::UAF
{
	struct FVariableOverrides;
}

// Override info for a single variable
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNextVariableOverrideInfo
{
	GENERATED_BODY()

	FAnimNextVariableOverrideInfo() = default;

	FAnimNextVariableOverrideInfo(FName InName, const FAnimNextParamType& InType, const FString& InDefaultValue)
		: Name(InName)
		, Type(InType)
		, DefaultValue(InDefaultValue)
	{}

	UPROPERTY(EditAnywhere, Category = "Variables")
	FName Name = "Variable";

	UPROPERTY(EditAnywhere, Category = "Variables")
	FAnimNextParamType Type = FAnimNextParamType(FAnimNextParamType::EValueType::Bool);

	UPROPERTY()
	FString DefaultValue;
};

// A trait that allows insertion of specific asset/struct variable overrides onto an override variables node
USTRUCT(meta=(Hidden))
struct FRigVMTrait_OverrideVariables : public FRigVMTrait
{
	GENERATED_BODY()

	// FRigVMTrait interface
#if WITH_EDITOR
	UE_API virtual void GetProgrammaticPins(URigVMController* InController, int32 InParentPinIndex, const URigVMPin* InTraitPin, const FString& InDefaultValue, FRigVMPinInfoArray& OutPinArray) const override;
	UE_API virtual bool ShouldCreatePinForProperty(const FProperty* InProperty) const override;
#endif

	// Override point for derived types to generate overrides
	virtual void GenerateOverrides(const FRigVMTraitScope& InTraitScope, TArray<UE::UAF::FVariableOverrides>& OutOverrides) const {}

	// Cached override data
	UPROPERTY(EditAnywhere, Category = "Variables", meta=(Input, DetailsOnly, TitleProperty = "Name"))
	TArray<FAnimNextVariableOverrideInfo> Overrides;
};

/*
 * A trait to generate overrides for an asset
 */
USTRUCT()
struct FRigVMTrait_OverrideVariablesForAsset : public FRigVMTrait_OverrideVariables
{
	GENERATED_BODY()

#if WITH_EDITOR
	UE_API virtual FString GetDisplayName() const override;
#endif
	UE_API virtual void GenerateOverrides(const FRigVMTraitScope& InTraitScope, TArray<UE::UAF::FVariableOverrides>& OutOverrides) const override;

	// The asset to use to generate overrides
	UPROPERTY(EditAnywhere, Category = "Variables", meta=(Input, Constant, DetailsOnly))
	TObjectPtr<const UUAFRigVMAsset> Asset;
};

/*
 * A trait to generate overrides for a a struct
 */
USTRUCT()
struct FRigVMTrait_OverrideVariablesForStruct : public FRigVMTrait_OverrideVariables
{
	GENERATED_BODY()

#if WITH_EDITOR
	UE_API virtual FString GetDisplayName() const override;
#endif
	UE_API virtual void GenerateOverrides(const FRigVMTraitScope& InTraitScope, TArray<UE::UAF::FVariableOverrides>& OutOverrides) const override;

	// The asset to use to generate overrides
	UPROPERTY(EditAnywhere, Category = "Variables", meta=(Input, Constant, DetailsOnly))
	TObjectPtr<const UScriptStruct> Struct;
};

#undef UE_API
