// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/StructDataCache.h"

#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif

#include "Containers/StripedMap.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectKey.h"

namespace UE::UAF
{

namespace Private
{
	static TStripedMap<32, TObjectKey<const UScriptStruct>, TSharedRef<FStructDataCache>> GStructDataMap;

#if WITH_LIVE_CODING
	void HandleLiveCodingPatchComplete()
	{
		GStructDataMap.ForEach([](const TPair<TObjectKey<const UScriptStruct>, TSharedRef<FStructDataCache>>& InStructData)
		{
			InStructData.Value->Rebuild();
		});
	}

	FDelegateHandle LiveCodingPatchHandle;
#endif
}

FStructDataCache::FStructDataCache(const UScriptStruct* InStruct)
	: WeakStruct(InStruct)
{
	Rebuild();
}

TSharedRef<FStructDataCache> FStructDataCache::GetStructInfo(const UScriptStruct* InStruct)
{
	auto ProduceStructData = [InStruct]()
	{
		TSharedRef<FStructDataCache> StructData = MakeShared<FStructDataCache>(InStruct);
		StructData->Rebuild();
		return StructData;
	};

	return Private::GStructDataMap.FindOrProduce(InStruct, ProduceStructData);
}

void FStructDataCache::Rebuild()
{
	Properties.Reset();

	const UScriptStruct* Struct = WeakStruct.Get();
	if (Struct == nullptr)
	{
		return;
	}

	// Grab all the struct's properties
	for (TFieldIterator<FProperty> It(Struct); It; ++It)
	{
		Properties.Add({*It, FAnimNextParamType::FromProperty(*It) });
	}

	// Sort by offset
	Properties.Sort([](const FPropertyInfo& InA, const FPropertyInfo& InB)
	{
		return InA.Property->GetOffset_ForInternal() < InB.Property->GetOffset_ForInternal();
	});
}

void FStructDataCache::Init()
{
#if WITH_LIVE_CODING
	if (ILiveCodingModule* LiveCoding = FModuleManager::LoadModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME))
	{
		Private::LiveCodingPatchHandle = LiveCoding->GetOnPatchCompleteDelegate().AddStatic(&Private::HandleLiveCodingPatchComplete);
	}
#endif
}

void FStructDataCache::Destroy()
{
#if WITH_LIVE_CODING
	if (ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME))
	{
		LiveCoding->GetOnPatchCompleteDelegate().Remove(Private::LiveCodingPatchHandle);
	}
#endif
}


}