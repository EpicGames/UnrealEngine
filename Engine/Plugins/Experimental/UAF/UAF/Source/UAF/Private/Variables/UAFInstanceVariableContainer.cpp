// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/UAFInstanceVariableContainer.h"

#include "AnimNextRigVMAsset.h"

#include "RigVMRuntimeAsset.h"
#include "Variables/StructDataCache.h"
#include "VariableOverrides.h"
#include "Param/ParamType.h"
#include "Variables/VariableOverridesCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFInstanceVariableContainer)

namespace UE::UAF::Private
{
	// Utility functions for type conversion, adapted from UE::StructUtils::Private in PropertyBag.cpp

	bool CanCastTo(const UStruct* From, const UStruct* To)
	{
		return From != nullptr && To != nullptr && From->IsChildOf(To);
	}

	EPropertyBagResult GetPropertyAsInt64(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, int64& OutValue)
	{
		TValueOrError<int64, void> Result = UE::StructUtils::GetPropertyValueAsInt64(InProperty, Address);
		if (Result.HasValue())
		{
			OutValue = Result.GetValue();
			return EPropertyBagResult::Success;
		}
		return EPropertyBagResult::TypeMismatch;
	}

	EPropertyBagResult GetPropertyAsUInt64(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, uint64& OutValue)
	{
		TValueOrError<uint64, void> Result = UE::StructUtils::GetPropertyValueAsUInt64(InProperty, Address);
		if (Result.HasValue())
		{
			OutValue = Result.GetValue();
			return EPropertyBagResult::Success;
		}
		return EPropertyBagResult::TypeMismatch;
	}

	EPropertyBagResult GetPropertyAsDouble(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, double& OutValue)
	{
		TValueOrError<double, void> Result = UE::StructUtils::GetPropertyValueAsDouble(InProperty, Address);
		if (Result.HasValue())
		{
			OutValue = Result.GetValue();
			return EPropertyBagResult::Success;
		}
		return EPropertyBagResult::TypeMismatch;
	}

	// Generic property getter. Used for FName, FString, FText. 
	template<typename T, typename PropT>
	EPropertyBagResult GetPropertyValue(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, T& OutValue)
	{
		check(Address != nullptr);
		check(InProperty != nullptr);

		if (!InProperty->IsA<PropT>())
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const PropT* Property = CastFieldChecked<PropT>(InProperty);
		OutValue = Property->GetPropertyValue(Address);

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult GetPropertyValueAsEnum(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, const UEnum* RequestedEnum, uint8& OutValue)
	{
		check(Address != nullptr);
		check(InProperty != nullptr);

		if (InValueType != EPropertyBagPropertyType::Enum)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
		{
			const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
			check(UnderlyingProperty);

			if (RequestedEnum != EnumProperty->GetEnum())
			{
				return EPropertyBagResult::TypeMismatch;
			}

			OutValue = static_cast<uint8>(UnderlyingProperty->GetUnsignedIntPropertyValue(Address));
			return EPropertyBagResult::Success;
		}
		else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
		{
			if (RequestedEnum != ByteProperty->Enum)
			{
				return EPropertyBagResult::TypeMismatch;
			}
			OutValue = ByteProperty->GetPropertyValue(Address);
			return EPropertyBagResult::Success;
		}
		else
		{
			return EPropertyBagResult::TypeMismatch;
		}
	}

	EPropertyBagResult GetPropertyValueAsStruct(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, const UScriptStruct* RequestedStruct, uint8* OutValue)
	{
		check(Address != nullptr);
		check(InProperty != nullptr);

		if (InValueType != EPropertyBagPropertyType::Struct)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(InProperty);
		check(StructProperty->Struct);
		check(RequestedStruct);

		if (CanCastTo(StructProperty->Struct, RequestedStruct) == false)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		// We dont use the property here to avoid copying more than we need if we are 'casting' from derived to base
		RequestedStruct->CopyScriptStruct(OutValue, Address, 1);

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult GetPropertyValueAsObject(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, const UClass* RequestedClass, UObject*& OutValue)
	{
		check(Address != nullptr);
		check(InProperty != nullptr);

		if (InValueType != EPropertyBagPropertyType::Object
			&& InValueType != EPropertyBagPropertyType::SoftObject
			&& InValueType != EPropertyBagPropertyType::Class
			&& InValueType != EPropertyBagPropertyType::SoftClass)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const FObjectPropertyBase* ObjectProperty = CastFieldChecked<FObjectPropertyBase>(InProperty);
		check(ObjectProperty->PropertyClass);

		if (RequestedClass != nullptr && CanCastTo(ObjectProperty->PropertyClass, RequestedClass) == false)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		OutValue = ObjectProperty->GetObjectPropertyValue(Address);

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult GetPropertyValueAsSoftPath(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, FSoftObjectPath& OutValue)
	{
		check(Address != nullptr);
		check(InProperty != nullptr);

		if (InValueType != EPropertyBagPropertyType::SoftObject
			&& InValueType != EPropertyBagPropertyType::SoftClass)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const FSoftObjectProperty* SoftObjectProperty = CastFieldChecked<FSoftObjectProperty>(InProperty);
		check(SoftObjectProperty->PropertyClass);

		OutValue = SoftObjectProperty->GetPropertyValue(Address).ToSoftObjectPath();

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult GetValueBool(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, bool& OutValue)
	{
		int64 ReturnValue = 0;
		const EPropertyBagResult Result = GetPropertyAsInt64(InProperty, InValueType, Address, ReturnValue);
		if (Result == EPropertyBagResult::Success)
		{
			OutValue = ReturnValue != 0;
		}
		return Result;
	}

	EPropertyBagResult GetValueByte(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, uint8& OutValue)
	{
		uint64 ReturnValue = 0;
		const EPropertyBagResult Result = GetPropertyAsUInt64(InProperty, InValueType, Address, ReturnValue);
		if (Result == EPropertyBagResult::Success)
		{
			OutValue = static_cast<uint8>(ReturnValue);
		}
		return Result;
	}

	EPropertyBagResult GetValueInt32(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, int32& OutValue)
	{
		int64 ReturnValue = 0;
		const EPropertyBagResult Result = GetPropertyAsInt64(InProperty, InValueType, Address, ReturnValue);
		if (Result == EPropertyBagResult::Success)
		{
			OutValue = static_cast<int32>(ReturnValue);
		}
		return Result;
	}

	EPropertyBagResult GetValueUInt32(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, uint32& OutValue)
	{
		uint64 ReturnValue = 0;
		const EPropertyBagResult Result = GetPropertyAsUInt64(InProperty, InValueType, Address, ReturnValue);
		if (Result == EPropertyBagResult::Success)
		{
			OutValue = static_cast<uint32>(ReturnValue);
		}
		return Result;
	}

	EPropertyBagResult GetValueInt64(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, int64& OutValue)
	{
		return GetPropertyAsInt64(InProperty, InValueType, Address, OutValue);
	}

	EPropertyBagResult GetValueUInt64(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, uint64& OutValue)
	{
		return GetPropertyAsUInt64(InProperty, InValueType, Address, OutValue);
	}

	EPropertyBagResult GetValueFloat(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, float& OutValue)
	{
		double ReturnValue = 0;
		const EPropertyBagResult Result = GetPropertyAsDouble(InProperty, InValueType, Address, ReturnValue);
		if (Result == EPropertyBagResult::Success)
		{
			OutValue = static_cast<float>(ReturnValue);
		}
		return Result;
	}

	EPropertyBagResult GetValueDouble(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, double& OutValue)
	{
		return GetPropertyAsDouble(InProperty, InValueType, Address, OutValue);
	}

	EPropertyBagResult GetValueName(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, FName& OutValue)
	{
		return GetPropertyValue<FName, FNameProperty>(InProperty, InValueType, Address, OutValue);
	}

	EPropertyBagResult GetValueString(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, FString& OutValue)
	{
		return GetPropertyValue<FString, FStrProperty>(InProperty, InValueType, Address, OutValue);
	}

	EPropertyBagResult GetValueText(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, FText& OutValue)
	{
		return GetPropertyValue<FText, FTextProperty>(InProperty, InValueType, Address, OutValue);
	}

	EPropertyBagResult GetValueEnum(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, const UEnum* RequestedEnum, uint8& OutValue)
	{
		return GetPropertyValueAsEnum(InProperty, InValueType, Address, RequestedEnum, OutValue);
	}

	EPropertyBagResult GetValueStruct(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, const UScriptStruct* RequestedStruct, uint8* OutValue)
	{
		return GetPropertyValueAsStruct(InProperty, InValueType, Address, RequestedStruct, OutValue);
	}

	EPropertyBagResult GetValueObject(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, const UClass* RequestedClass, UObject*& OutValue)
	{
		return GetPropertyValueAsObject(InProperty, InValueType, Address, RequestedClass, OutValue);
	}

	EPropertyBagResult GetValueClass(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, const UClass* RequestedClass, UClass*& OutValue)
	{
		UObject* ReturnValue = nullptr;
		const EPropertyBagResult Result = GetPropertyValueAsObject(InProperty, InValueType, Address, nullptr, ReturnValue);
		if (Result != EPropertyBagResult::Success)
		{
			return Result;
		}
		UClass* Class = Cast<UClass>(ReturnValue);
		if (Class == nullptr && ReturnValue != nullptr)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		if (CanCastTo(Class, RequestedClass) == false)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		OutValue = Class;
		return Result;
	}

	EPropertyBagResult GetValueSoftPath(const FProperty* InProperty, EPropertyBagPropertyType InValueType, const void* Address, FSoftObjectPath& OutValue)
	{
		return GetPropertyValueAsSoftPath(InProperty, InValueType, Address, OutValue);
	}

	EPropertyBagResult GetVariableFromMismatchedValueType(const FProperty* InProperty, const FAnimNextParamType& InSrcType, const FAnimNextParamType& InDestType, const uint8* InAddress, uint8* OutResult)
	{
		check(InSrcType != InDestType);	// Function assumes that types are mismatched

		switch (InDestType.GetValueType())
		{
		case EPropertyBagPropertyType::Bool:
			return GetValueBool(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<bool*>(OutResult));
		case EPropertyBagPropertyType::Byte:
			return GetValueByte(InProperty, InSrcType.GetValueType(), InAddress, *OutResult);
		case EPropertyBagPropertyType::Int32:
			return GetValueInt32(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<int32*>(OutResult));
		case EPropertyBagPropertyType::Int64:
			return GetValueInt64(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<int64*>(OutResult));
		case EPropertyBagPropertyType::Float:
			return GetValueFloat(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<float*>(OutResult));
		case EPropertyBagPropertyType::Double:
			return GetValueDouble(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<double*>(OutResult));
		case EPropertyBagPropertyType::Name:
			return GetValueName(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<FName*>(OutResult));
		case EPropertyBagPropertyType::String:
			return GetValueString(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<FString*>(OutResult));
		case EPropertyBagPropertyType::Text:
			return GetValueText(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<FText*>(OutResult));
		case EPropertyBagPropertyType::Enum:
			return GetValueEnum(InProperty, InSrcType.GetValueType(), InAddress, CastChecked<UEnum>(InDestType.GetValueTypeObject()), *OutResult);
		case EPropertyBagPropertyType::Struct:
			return GetValueStruct(InProperty, InSrcType.GetValueType(), InAddress, CastChecked<UScriptStruct>(InDestType.GetValueTypeObject()), OutResult);
		case EPropertyBagPropertyType::Object:
			return GetValueObject(InProperty, InSrcType.GetValueType(), InAddress, CastChecked<UClass>(InDestType.GetValueTypeObject()), *reinterpret_cast<UObject**>(OutResult));
		case EPropertyBagPropertyType::SoftObject:
			return GetValueSoftPath(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<FSoftObjectPath*>(OutResult));
		case EPropertyBagPropertyType::Class:
			return GetValueClass(InProperty, InSrcType.GetValueType(), InAddress, CastChecked<UClass>(InDestType.GetValueTypeObject()), *reinterpret_cast<UClass**>(OutResult));
		case EPropertyBagPropertyType::SoftClass:
			return GetValueSoftPath(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<FSoftObjectPath*>(OutResult));
		case EPropertyBagPropertyType::UInt32:
			return GetValueUInt32(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<uint32*>(OutResult));
		case EPropertyBagPropertyType::UInt64:
			return GetValueUInt64(InProperty, InSrcType.GetValueType(), InAddress, *reinterpret_cast<uint64*>(OutResult));
		default:
			return EPropertyBagResult::TypeMismatch;
		}
	}

	EPropertyBagResult GetVariableFromMismatchedArrayType(const FProperty* InProperty, const FAnimNextParamType& InSrcType, const FAnimNextParamType& InDestType, const uint8* InAddress, uint8* OutResult)
	{
		const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(InProperty);
		const FProperty* ElementProperty = ArrayProperty->Inner;
		FAnimNextParamType SrcValueType(InSrcType.GetValueType(), FAnimNextParamType::EContainerType::None, InSrcType.GetValueTypeObject());
		FAnimNextParamType DestValueType(InDestType.GetValueType(), FAnimNextParamType::EContainerType::None, InDestType.GetValueTypeObject());
		check(SrcValueType != DestValueType);	// Function assumes that types are mismatched

		const TScriptArray<FHeapAllocator>* SrcArray = reinterpret_cast<const TScriptArray<FHeapAllocator>*>(InAddress);
		TScriptArray<FHeapAllocator>* DestArray = reinterpret_cast<TScriptArray<FHeapAllocator>*>(OutResult);

		const int32 NumElements = SrcArray->Num();
		const size_t SrcValueTypeSize = InSrcType.GetValueTypeSize();
		const size_t DestValueTypeSize = InDestType.GetValueTypeSize();
		const size_t DestValueTypeAlignment = InDestType.GetAlignment(); 

		// Reallocate dest array
		DestArray->SetNumUninitialized(NumElements, DestValueTypeSize, DestValueTypeAlignment);

		// Perform per-element conversion
		bool bSucceeded = true;
		const uint8* SrcElement = static_cast<const uint8*>(SrcArray->GetData());
		uint8* DestElement = static_cast<uint8*>(DestArray->GetData());
		for (int32 Index = 0; Index < NumElements; ++Index)
		{
			bSucceeded &= (GetVariableFromMismatchedValueType(ElementProperty, SrcValueType, DestValueType, SrcElement, DestElement) == EPropertyBagResult::Success);
			SrcElement += SrcValueTypeSize;
			DestElement += DestValueTypeSize;
		}

		return bSucceeded ? EPropertyBagResult::Success : EPropertyBagResult::TypeMismatch;
	}

	EPropertyBagResult GetVariableFromMismatchedType(const FProperty* InProperty, const FAnimNextParamType& InSrcType, const FAnimNextParamType& InDestType, const uint8* InAddress, uint8* OutResult)
	{
		switch (InSrcType.GetContainerType())
		{
		case EPropertyBagContainerType::None:
			if (InDestType.GetContainerType() == EPropertyBagContainerType::None)
			{
				return GetVariableFromMismatchedValueType(InProperty, InSrcType, InDestType, InAddress, OutResult);
			}
			return EPropertyBagResult::TypeMismatch;
		case EPropertyBagContainerType::Array:
			if (InDestType.GetContainerType() == EPropertyBagContainerType::Array)
			{
				return GetVariableFromMismatchedArrayType(InProperty, InSrcType, InDestType, InAddress, OutResult);
			}
			return EPropertyBagResult::TypeMismatch;
		default:
			return EPropertyBagResult::TypeMismatch;
		}
	}
}

FUAFInstanceVariableContainer::FUAFInstanceVariableContainer(const FUAFAssetInstance& InHostInstance, const UScriptStruct* InStruct, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides, int32 InOverridesIndex)
	: HostInstance(&InHostInstance)
{
	AssetOrStructData.Set<FStructType>(InStruct);
	StructDataCache = UE::UAF::FStructDataCache::GetStructInfo(InStruct);
	NumVariables = StructDataCache->GetProperties().Num();
	VariablesContainer.Set<FInstancedStruct>(FInstancedStruct());
	VariablesContainer.Get<FInstancedStruct>().InitializeAs(InStruct);

	Overrides = InOverrides;
	OverridesIndex = InOverridesIndex;
	ResolveOverrides();
}

FUAFInstanceVariableContainer::FUAFInstanceVariableContainer(const FUAFAssetInstance& InHostInstance, const UUAFRigVMAsset* InAsset, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides, int32 InOverridesIndex)
	: HostInstance(&InHostInstance)
{
	AssetOrStructData.Set<FAssetType>(InAsset);
	NumVariables = InAsset->GetVariableDefaults().GetNumPropertiesInBag();
	VariablesContainer.Set<FInstancedPropertyBag>(InAsset->GetVariableDefaults());

	Overrides = InOverrides;
	OverridesIndex = InOverridesIndex;
	ResolveOverrides();
}

FUAFInstanceVariableContainer::FUAFInstanceVariableContainer(const FUAFAssetInstance& InHostInstance, const IRigVMRuntimeAssetInterface* InAsset, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides, int32 InOverridesIndex)
	: HostInstance(&InHostInstance)
{
	TArray<FRigVMExternalVariable> ExternalVariables = const_cast<IRigVMRuntimeAssetInterface*>(InAsset)->GetExternalVariables();
	FInstancedPropertyBag DefaultProperties;
	
	for (const FRigVMExternalVariable& Variable : ExternalVariables)
	{
		// Add to schema
		DefaultProperties.AddProperty(Variable.GetName(), Variable.GetProperty());

		// Copy in the default value
		DefaultProperties.SetValue(Variable.GetName(), Variable.GetProperty(), InAsset->GetVariablesMemory());
	}
	
	NumVariables = DefaultProperties.GetNumPropertiesInBag();

	// Copy the variables into the container
	VariablesContainer.Set<FInstancedPropertyBag>(MoveTemp(DefaultProperties));

	const UObject* InterfaceObject = Cast< const UObject>(InAsset);
	TScriptInterface<const IRigVMRuntimeAssetInterface> Interface = InterfaceObject;
	AssetOrStructData.Set<FRigVMAssetType>(Interface);

	Overrides = InOverrides;
	OverridesIndex = InOverridesIndex;
	ResolveOverrides();
	
}


EPropertyBagResult FUAFInstanceVariableContainer::GetVariable(int32 InVariableIndex, const FAnimNextParamType& InType, TArrayView<uint8> OutResult) const
{
	using namespace UE::UAF;

	const bool bHasOverrides = ResolvedOverrides.Num() > 0;
	FAnimNextParamType FoundType;
	const uint8* Memory = nullptr;
	const FProperty* Property = nullptr;
	switch (VariablesContainer.GetIndex())
	{
	case FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			const FInstancedPropertyBag& Variables = VariablesContainer.Get<FInstancedPropertyBag>();
			
			const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
			
			check(PropertyBag != nullptr);
			TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBag->GetPropertyDescs();
			const FPropertyBagPropertyDesc& Desc = Descs[InVariableIndex];
			FoundType = FAnimNextParamType::FromPropertyBagPropertyDesc(Desc);
			Property = Desc.CachedProperty;
			if (bHasOverrides && ResolvedOverrides[InVariableIndex] != nullptr)
			{
				Memory = ResolvedOverrides[InVariableIndex];
			}
			else
			{
				Memory = Desc.CachedProperty->ContainerPtrToValuePtr<uint8>(Variables.GetValue().GetMemory());
			}
			break;
		}
	case FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			const FStructDataCache::FPropertyInfo& PropertyInfo = StructDataCache->GetProperties()[InVariableIndex];
			FoundType = PropertyInfo.Type;
			Property = PropertyInfo.Property;
			if (bHasOverrides && ResolvedOverrides[InVariableIndex] != nullptr)
			{
				Memory = ResolvedOverrides[InVariableIndex];
			}
			else
			{
				const FInstancedStruct& Variables = VariablesContainer.Get<FInstancedStruct>();
				Memory = Property->ContainerPtrToValuePtr<uint8>(Variables.GetMemory());
			}
			break;
		}
	default:
		checkNoEntry();
		break;
	}

	if (FoundType != InType)
	{
		return Private::GetVariableFromMismatchedType(Property, FoundType, InType, Memory, OutResult.GetData());
	}

	if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		*reinterpret_cast<bool*>(OutResult.GetData()) = BoolProperty->GetPropertyValue(Memory);
	}
	else
	{
		Property->CopyCompleteValue(OutResult.GetData(), Memory);
	}

	return EPropertyBagResult::Success;
}

EPropertyBagResult FUAFInstanceVariableContainer::AccessVariable(int32 InVariableIndex, const FAnimNextParamType& InType, TArrayView<uint8>& OutResult)
{
	using namespace UE::UAF;

	const bool bHasOverrides = ResolvedOverrides.Num() > 0;
	FAnimNextParamType FoundType;
	uint8* Memory = nullptr;
	const FProperty* Property = nullptr;
	switch (VariablesContainer.GetIndex())
	{
	case FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			FInstancedPropertyBag& Variables = VariablesContainer.Get<FInstancedPropertyBag>();
			
			const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
			
			check(PropertyBag != nullptr);
			TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBag->GetPropertyDescs();
			const FPropertyBagPropertyDesc& Desc = Descs[InVariableIndex];
			FoundType = FAnimNextParamType::FromPropertyBagPropertyDesc(Desc);
			Property = Desc.CachedProperty;
			if (bHasOverrides && ResolvedOverrides[InVariableIndex] != nullptr)
			{
				Memory = ResolvedOverrides[InVariableIndex];
			}
			else
			{
				Memory = Desc.CachedProperty->ContainerPtrToValuePtr<uint8>(Variables.GetMutableValue().GetMemory());
			}
			break;
		}
	case FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			const FStructDataCache::FPropertyInfo& PropertyInfo = StructDataCache->GetProperties()[InVariableIndex];
			Property = PropertyInfo.Property;
			FoundType = PropertyInfo.Type;
			if (bHasOverrides && ResolvedOverrides[InVariableIndex] != nullptr)
			{
				Memory = ResolvedOverrides[InVariableIndex];
			}
			else
			{
				FInstancedStruct& Variables = VariablesContainer.Get<FInstancedStruct>();
				Memory = PropertyInfo.Property->ContainerPtrToValuePtr<uint8>(Variables.GetMutableMemory());
			}
			break;
		}
	default:
		checkNoEntry();
		break;
	}

	if (FoundType != InType)
	{
		return EPropertyBagResult::TypeMismatch;
	}

	if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		ensureMsgf(BoolProperty->IsNativeBool(), TEXT("Warning: Accessing non-native bool via AccessVariable will result in incorrect results"));
	}

	OutResult = TArrayView<uint8>(Memory, FoundType.GetSize());
	return EPropertyBagResult::Success;
}

void FUAFInstanceVariableContainer::AccessVariableUnchecked(int32 InVariableIndex, const FProperty*& OutProperty, TArrayView<uint8>& OutResult)
{
	using namespace UE::UAF;

	const bool bHasOverrides = ResolvedOverrides.Num() > 0;
	int32 Size = 0;
	uint8* Memory = nullptr;
	switch (VariablesContainer.GetIndex())
	{
	case FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			FInstancedPropertyBag& Variables = VariablesContainer.Get<FInstancedPropertyBag>();
			const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
			
			check(PropertyBag != nullptr);
			TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBag->GetPropertyDescs();
			const FPropertyBagPropertyDesc& Desc = Descs[InVariableIndex];
			OutProperty = Desc.CachedProperty;
			Size = Desc.CachedProperty->GetElementSize();
			if (bHasOverrides && ResolvedOverrides[InVariableIndex] != nullptr)
			{
				Memory = ResolvedOverrides[InVariableIndex];
			}
			else
			{
				Memory = Desc.CachedProperty->ContainerPtrToValuePtr<uint8>(Variables.GetMutableValue().GetMemory());
			}
			break;
		}
	case FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			const FStructDataCache::FPropertyInfo& PropertyInfo = StructDataCache->GetProperties()[InVariableIndex];
			OutProperty = PropertyInfo.Property;
			Size = PropertyInfo.Property->GetElementSize();
			if (bHasOverrides && ResolvedOverrides[InVariableIndex] != nullptr)
			{
				Memory = ResolvedOverrides[InVariableIndex];
			}
			else
			{
				FInstancedStruct& Variables = VariablesContainer.Get<FInstancedStruct>();
				Memory = PropertyInfo.Property->ContainerPtrToValuePtr<uint8>(Variables.GetMutableMemory());
			}
			break;
		}
	default:
		checkNoEntry();
		break;
	}

	if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(OutProperty))
	{
		ensureMsgf(BoolProperty->IsNativeBool(), TEXT("Warning: Accessing non-native bool via AccessVariable will result in incorrect results"));
	}

	OutResult = TArrayView<uint8>(Memory, Size);
}

FName FUAFInstanceVariableContainer::GetFName() const
{
	switch (AssetOrStructData.GetIndex())
	{
		case FAssetOrStructType::IndexOfType<FAssetType>():
			{
				return AssetOrStructData.Get<FAssetType>().GetFName();
			}
		case FAssetOrStructType::IndexOfType<FRigVMAssetType>():
			{
				return AssetOrStructData.Get<FRigVMAssetType>().GetObject()->GetFName();
			}
	case FAssetOrStructType::IndexOfType<FStructType>():
		{
			return AssetOrStructData.Get<FStructType>().GetFName();
		}
	default:
		checkNoEntry();
		break;
	}

	return NAME_None;
}

EPropertyBagResult FUAFInstanceVariableContainer::SetVariable(int32 InVariableIndex, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue)
{
	using namespace UE::UAF;

	const bool bHasOverrides = ResolvedOverrides.Num() > 0;
	FAnimNextParamType FoundType;
	uint8* Memory = nullptr;
	const FProperty* Property = nullptr;
	switch (VariablesContainer.GetIndex())
	{
	case FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			FInstancedPropertyBag& Variables = VariablesContainer.Get<FInstancedPropertyBag>();
			const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
			
			check(PropertyBag != nullptr);
			TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBag->GetPropertyDescs();
			const FPropertyBagPropertyDesc& Desc = Descs[InVariableIndex];
			FoundType = FAnimNextParamType::FromPropertyBagPropertyDesc(Desc);
			Property = Desc.CachedProperty;
			if (bHasOverrides && ResolvedOverrides[InVariableIndex] != nullptr)
			{
				Memory = ResolvedOverrides[InVariableIndex];
			}
			else
			{
				Memory = Desc.CachedProperty->ContainerPtrToValuePtr<uint8>(Variables.GetMutableValue().GetMemory());
			}
			break;
		}
	case FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			const FStructDataCache::FPropertyInfo& PropertyInfo = StructDataCache->GetProperties()[InVariableIndex];
			FoundType = PropertyInfo.Type;
			Property = PropertyInfo.Property;
			if (bHasOverrides && ResolvedOverrides[InVariableIndex] != nullptr)
			{
				Memory = ResolvedOverrides[InVariableIndex];
			}
			else
			{
				FInstancedStruct& Variables = VariablesContainer.Get<FInstancedStruct>();
				Memory = Property->ContainerPtrToValuePtr<uint8>(Variables.GetMutableMemory());
			}
			break;
		}
	default:
		checkNoEntry();
		break;
	}
	
	if (FoundType != InType)
	{
		return Private::GetVariableFromMismatchedType(Property, FoundType, InType, InNewValue.GetData(), Memory);
	}

	if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		BoolProperty->SetPropertyValue(Memory, *reinterpret_cast<const bool*>(InNewValue.GetData()));
	}
	else
	{
		Property->CopyCompleteValue(Memory, InNewValue.GetData());
	}
	return EPropertyBagResult::Success;
}

void FUAFInstanceVariableContainer::GetAllVariablesOfType(const FAnimNextParamType& InType, TArray<FAnimNextVariableReference>& OutVariables) const
{
	using namespace UE::UAF;

	if (NumVariables == 0)
	{
		return;
	}

	switch (VariablesContainer.GetIndex())
	{
	case FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			const FInstancedPropertyBag& Variables = VariablesContainer.Get<FInstancedPropertyBag>();
			const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();;
			const UObject* OwnerObject = nullptr;

			if (AssetOrStructData.IsType<FAssetType>())
			{
				const FAssetType& Asset = AssetOrStructData.Get<FAssetType>();
				OwnerObject = Asset;
			}
			else if(AssetOrStructData.IsType<FRigVMAssetType>())
			{
				FRigVMAssetType AssetInterface = AssetOrStructData.Get<FRigVMAssetType>();
				check(AssetInterface.GetObject());
				OwnerObject = AssetInterface.GetObject();
			}
			
			check(PropertyBag != nullptr);
			for (const FPropertyBagPropertyDesc& Desc : PropertyBag->GetPropertyDescs())
			{
				if (Desc.ValueType == InType.GetValueType() &&
					Desc.ContainerTypes.GetFirstContainerType() == InType.GetContainerType() &&
					Desc.ValueTypeObject == InType.GetValueTypeObject())
				{
					OutVariables.Add(FAnimNextVariableReference::FromName(Desc.Name, OwnerObject));
				}
			}
			break;
		}
	case FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			const UScriptStruct* Struct = AssetOrStructData.Get<FStructType>();
			for (const FStructDataCache::FPropertyInfo& PropertyInfo : StructDataCache->GetProperties())
			{
				if (PropertyInfo.Type == InType)
				{
					OutVariables.Add(FAnimNextVariableReference::FromName(PropertyInfo.Property->GetFName(), Struct));
				}
			}
			break;
		}
	default:
		checkNoEntry();
		break;
	}
}

bool FUAFInstanceVariableContainer::ForEachVariable(TFunctionRef<bool(FName, const FAnimNextParamType&, int32)> InFunction, int32& InOutVariableIndex) const
{
	using namespace UE::UAF;
	
	if (NumVariables == 0)
	{
		return true;
	}

	switch (VariablesContainer.GetIndex())
	{
	case FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			const UPropertyBag* PropertyBag = VariablesContainer.Get<FInstancedPropertyBag>().GetPropertyBagStruct();

			check(PropertyBag != nullptr);
			for (const FPropertyBagPropertyDesc& Desc : PropertyBag->GetPropertyDescs())
			{
				if (!InFunction(Desc.Name, FAnimNextParamType::FromPropertyBagPropertyDesc(Desc), InOutVariableIndex))
				{
					return false;
				}
				InOutVariableIndex++;
			}
			break;
		}
	case FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			for (const FStructDataCache::FPropertyInfo& PropertyInfo : StructDataCache->GetProperties())
			{
				if (!InFunction(PropertyInfo.Property->GetFName(), PropertyInfo.Type, InOutVariableIndex))
				{
					return false;
				}
				InOutVariableIndex++;
			}
			break;
		}
	default:
		checkNoEntry();
		break;
	}

	return true;
}

FAnimNextParamType FUAFInstanceVariableContainer::GetVariableType(int32 InVariableIndex) const
{
	using namespace UE::UAF;

	switch (VariablesContainer.GetIndex())
	{
	case FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			const FInstancedPropertyBag& Variables = VariablesContainer.Get<FInstancedPropertyBag>();
			const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
			
			check(PropertyBag != nullptr);
			TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBag->GetPropertyDescs();
			return FAnimNextParamType::FromPropertyBagPropertyDesc(Descs[InVariableIndex]);
		}
	case FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			const FStructDataCache::FPropertyInfo& PropertyInfo = StructDataCache->GetProperties()[InVariableIndex];
			return PropertyInfo.Type;
		}
	default:
		checkNoEntry();
		break;
	}

	return FAnimNextParamType();
}

void FUAFInstanceVariableContainer::ResolveOverrides()
{
	using namespace UE::UAF;

	TSharedPtr<const FVariableOverridesCollection> PinnedOverrides = Overrides.Pin();
	if (!PinnedOverrides.IsValid())
	{
		ResolvedOverrides.Reset();
		return;
	}

	check(PinnedOverrides->Collection.IsValidIndex(OverridesIndex));
	

	ResolvedOverrides.SetNumZeroed(NumVariables, EAllowShrinking::No);

	switch (AssetOrStructData.GetIndex())
	{
	case FAssetOrStructType::IndexOfType<FAssetType>():
		{
			const UUAFRigVMAsset* Asset = AssetOrStructData.Get<FAssetType>();
			const FInstancedPropertyBag& PropertyBag = Asset->GetVariableDefaults();
			checkf(Asset == PinnedOverrides->Collection[OverridesIndex].AssetOrStructData.Get<FAssetType>(), TEXT("Variable overrides asset does not match"));
			TConstArrayView<FVariableOverrides::FOverride> OverridesView = PinnedOverrides->Collection[OverridesIndex].Overrides;
			const int32 NumOverrides = OverridesView.Num();
			for (int32 OverrideIndex = 0; OverrideIndex < NumOverrides; ++OverrideIndex)
			{
				const FVariableOverrides::FOverride& Override = OverridesView[OverrideIndex];
				const FPropertyBagPropertyDesc* Desc = PropertyBag.FindPropertyDescByName(Override.Name);
				if (Desc == nullptr)
				{
					ensureMsgf(false, TEXT("Variable not found"));
					continue;
				}
				if (Override.Type != FAnimNextParamType::FromPropertyBagPropertyDesc(*Desc))
				{
					ensureMsgf(false, TEXT("Variable type does not match"));
					continue;
				}
				int32 VariableIndex = Desc - PropertyBag.GetPropertyBagStruct()->GetPropertyDescs().GetData();
				ResolvedOverrides[VariableIndex] = Override.Memory;
			}
			break;
		}
	case FAssetOrStructType::IndexOfType<FStructType>():
		{
			const UScriptStruct* Struct = AssetOrStructData.Get<FStructType>();
			check(StructDataCache.IsValid());
			checkf(Struct == PinnedOverrides->Collection[OverridesIndex].AssetOrStructData.Get<FVariableOverrides::FStructType>(), TEXT("Variable overrides struct does not match"));
			TConstArrayView<FVariableOverrides::FOverride> OverridesView = PinnedOverrides->Collection[OverridesIndex].Overrides;
			const int32 NumOverrides = OverridesView.Num();
			for (int32 OverrideIndex = 0; OverrideIndex < NumOverrides; ++OverrideIndex)
			{
				const FVariableOverrides::FOverride& Override = OverridesView[OverrideIndex];
				int32 PropertyIndex = StructDataCache->GetProperties().IndexOfByPredicate([&Override](const FStructDataCache::FPropertyInfo& InPropertyInfo)
				{
					return Override.Name == InPropertyInfo.Property->GetFName();
				});
				if (PropertyIndex == INDEX_NONE)
				{
					ensureMsgf(false, TEXT("Variable not found"));
					continue;	
				}

				const FStructDataCache::FPropertyInfo& PropertyInfo = StructDataCache->GetProperties()[PropertyIndex];
				check(PropertyInfo.Property != nullptr);

				if (Override.Type != PropertyInfo.Type)
				{
					ensureMsgf(false, TEXT("Variable type does not match"));
					continue;
				}
				ResolvedOverrides[PropertyIndex] = Override.Memory;
			}
			break;
		}
		case FAssetOrStructType::IndexOfType<FRigVMAssetType>():
		{
			const IRigVMRuntimeAssetInterface* RigVMAsset = AssetOrStructData.Get<FRigVMAssetType>().GetInterface();
			checkf(RigVMAsset == PinnedOverrides->Collection[OverridesIndex].AssetOrStructData.Get<FVariableOverrides::FRigVMAssetType>().GetInterface(), TEXT("Variable overrides RigVMRuntimeAsset does not match"));
			TConstArrayView<FVariableOverrides::FOverride> OverridesView = PinnedOverrides->Collection[OverridesIndex].Overrides;
			const int32 NumOverrides = OverridesView.Num();
			TArray<FRigVMExternalVariable> ExternalVariables = const_cast<IRigVMRuntimeAssetInterface*>(RigVMAsset)->GetExternalVariables();
			for (int32 OverrideIndex = 0; OverrideIndex < NumOverrides; ++OverrideIndex)
			{
				const FVariableOverrides::FOverride& Override = OverridesView[OverrideIndex];
				int32 VariableIndex = ExternalVariables.IndexOfByPredicate([&Override](const FRigVMExternalVariable& InVariable)
					{
						return InVariable.GetName() == Override.Name;
					});

				if (VariableIndex == INDEX_NONE)
				{
					ensureMsgf(false, TEXT("Variable not found"));
					continue;
				}
				
				const FProperty* Property = ExternalVariables[VariableIndex].GetProperty();
				if (Property == nullptr)
				{
					ensureMsgf(false, TEXT("Property not found"));
					continue;
				}
				
				FPropertyBagPropertyDesc Desc(Override.Name, Property);
				if (Override.Type != FAnimNextParamType::FromPropertyBagPropertyDesc(Desc))
				{
					ensureMsgf(false, TEXT("Variable type does not match"));
					continue;
				}
				ResolvedOverrides[VariableIndex] = Override.Memory;
			}
			break;
		}
	default:
		checkNoEntry();
		break;
	}
}

#if WITH_EDITOR
void FUAFInstanceVariableContainer::Migrate()
{
	using namespace UE::UAF;

	switch (AssetOrStructData.GetIndex())
	{
	case FAssetOrStructType::IndexOfType<FAssetType>():
		{
			FInstancedPropertyBag& PropertyBag = VariablesContainer.Get<FInstancedPropertyBag>();
			PropertyBag.MigrateToNewBagInstance(AssetOrStructData.Get<FAssetType>()->GetVariableDefaults());
			NumVariables = PropertyBag.GetNumPropertiesInBag();
			break;
		}
	case FAssetOrStructType::IndexOfType<FStructType>():
		{
			break;
		}
	case FAssetOrStructType::IndexOfType<FRigVMAssetType>():
		{
			FInstancedPropertyBag& PropertyBag = VariablesContainer.Get<FInstancedPropertyBag>();

			const FRigVMAssetType& RigVMAsset = AssetOrStructData.Get<FRigVMAssetType>();
			TArray<FRigVMExternalVariable> ExternalVariables = const_cast<IRigVMRuntimeAssetInterface*>(RigVMAsset.GetInterface())->GetExternalVariables();
			for (const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
			{
				PropertyBag.AddProperty(ExternalVariable.GetName(), ExternalVariable.GetProperty());
			}

			
			NumVariables = PropertyBag.GetNumPropertiesInBag();
			break;
		}
	default:
		checkNoEntry();
		break;
	}

	ResolveOverrides();
}
#endif

void FUAFInstanceVariableContainer::GetRigVMMemoryForVariables(TConstArrayView<FRigVMExternalVariableDef> InVariableDefs, TArray<FRigVMExternalVariableRuntimeData>& OutRuntimeData)
{
	using namespace UE::UAF;
	
	if (NumVariables == 0)
	{
		return;
	}

	const bool bHasOverrides = ResolvedOverrides.Num() > 0;

	switch (VariablesContainer.GetIndex())
	{
	case FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			FInstancedPropertyBag& Variables = VariablesContainer.Get<FInstancedPropertyBag>();
			uint8* BasePtr = Variables.GetMutableValue().GetMemory();
			TConstArrayView<FPropertyBagPropertyDesc> Descs = Variables.GetPropertyBagStruct()->GetPropertyDescs();
			check(InVariableDefs.Num() == Descs.Num());
			for (int32 VariableIndex = 0; VariableIndex < Descs.Num(); ++VariableIndex)
			{
				if (bHasOverrides && ResolvedOverrides[VariableIndex] != nullptr)
				{
					OutRuntimeData.Emplace(ResolvedOverrides[VariableIndex]);
				}
				else
				{
					const FPropertyBagPropertyDesc& Desc = Descs[VariableIndex];
					const FRigVMExternalVariableDef& Def = InVariableDefs[VariableIndex];

					if (!ensure(Desc.CachedProperty->GetClass() == Def.GetProperty()->GetClass()))
					{
						OutRuntimeData.Empty();
						return;
					}
					OutRuntimeData.Emplace(Desc.CachedProperty->ContainerPtrToValuePtr<uint8>(BasePtr));
				}
			}
			break;
		}
	case FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			FInstancedStruct& Variables = VariablesContainer.Get<FInstancedStruct>();
			uint8* BasePtr = Variables.GetMutableMemory();
			TConstArrayView<FStructDataCache::FPropertyInfo> Properties = StructDataCache->GetProperties();
			check(InVariableDefs.Num() == Properties.Num());
			for (int32 VariableIndex = 0; VariableIndex < Properties.Num(); ++VariableIndex)
			{
				if (bHasOverrides && ResolvedOverrides[VariableIndex] != nullptr)
				{
					OutRuntimeData.Emplace(ResolvedOverrides[VariableIndex]);
				}
				else
				{
					const FStructDataCache::FPropertyInfo& Property = Properties[VariableIndex];
					const FRigVMExternalVariableDef& Def = InVariableDefs[VariableIndex];
					if (!ensure(Property.Property->GetClass() == Def.GetProperty()->GetClass()))
					{
						OutRuntimeData.Empty();
						return;
					}
					OutRuntimeData.Emplace(Property.Property->ContainerPtrToValuePtr<uint8>(BasePtr));
				}
			}
			break;
		}
	default:
		checkNoEntry();
		break;
	}
}

void FUAFInstanceVariableContainer::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	switch (AssetOrStructData.GetIndex())
	{
	case FAssetOrStructType::IndexOfType<FAssetType>():
		{
			Collector.AddReferencedObject(AssetOrStructData.Get<FAssetType>());
			break;
		}
	case FAssetOrStructType::IndexOfType<FStructType>():
		{
			Collector.AddReferencedObject(AssetOrStructData.Get<FStructType>());
			break;
		}
	case FAssetOrStructType::IndexOfType<FRigVMAssetType>():
		{
			Collector.AddReferencedObject(AssetOrStructData.Get<FRigVMAssetType>().GetObjectRef());
			break;
		}
	default:
		checkNoEntry();	
	}

	switch (VariablesContainer.GetIndex())
	{
	case FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
		{
			FInstancedPropertyBag& InstancedPropertyBag = VariablesContainer.Get<FInstancedPropertyBag>();
			Collector.AddPropertyReferencesWithStructARO(FInstancedPropertyBag::StaticStruct(), &InstancedPropertyBag);
			break;
		}
	case FVariablesContainerType::IndexOfType<FInstancedStruct>():
		{
			FInstancedStruct& InstancedStruct = VariablesContainer.Get<FInstancedStruct>();
			Collector.AddPropertyReferencesWithStructARO(FInstancedStruct::StaticStruct(), &InstancedStruct);
			break;
		}
	default:
		checkNoEntry();
		break;
	}
}
