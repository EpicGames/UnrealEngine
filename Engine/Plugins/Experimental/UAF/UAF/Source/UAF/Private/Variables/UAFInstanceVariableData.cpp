// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/UAFInstanceVariableData.h"
#include "AnimNextRigVMAsset.h"
#include "RigVMRuntimeAsset.h"
#include "UAFAssetInstance.h"
#include "Variables/StructDataCache.h"
#include "VariableOverrides.h"
#include "Logging/StructuredLog.h"
#include "Variables/AnimNextVariableReference.h"
#include "VariableOverridesCollection.h"
#include "Variables/UAFInstanceVariableContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFInstanceVariableData)

void FUAFInstanceVariableData::Initialize(const FUAFAssetInstance& InInstance, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides)
{
	using namespace UE::UAF;

	Reset();

	// Add the primary asset's variables
	AddVariablesContainerForAsset(InInstance.GetAsset<UUAFRigVMAsset>(), InInstance, InOverrides);

	// Add referenced asset variables
	for (const UUAFRigVMAsset* ReferencedAsset : InInstance.GetAsset<UUAFRigVMAsset>()->GetReferencedVariableAssets())
	{
		AddVariablesContainerForAsset(ReferencedAsset, InInstance, InOverrides);
	}

	// Add referenced struct variables
	for (const UScriptStruct* Struct : InInstance.GetAsset<UUAFRigVMAsset>()->GetReferencedVariableStructs())
	{
		AddVariablesContainerForStruct(Struct, InInstance, InOverrides);
	}

	// Add referenced rig VM asset variables
	for (TScriptInterface<const IRigVMRuntimeAssetInterface> RigVMAsset : InInstance.GetAsset<UUAFRigVMAsset>()->GetReferencedVariableRigVMAssets())
	{
		AddVariablesContainerForRigVMAsset(RigVMAsset, InInstance, InOverrides);
	}

	// Now any overrides we havent already added
	if (InOverrides.IsValid())
	{
		for (int32 OverrideIndex = 0; OverrideIndex < InOverrides->Collection.Num(); ++OverrideIndex)
		{
			const FVariableOverrides& Override = InOverrides->Collection[OverrideIndex];
			if (Override.AssetOrStructData.IsType<FVariableOverrides::FStructType>())
			{
				const UScriptStruct* Struct = Override.AssetOrStructData.Get<FVariableOverrides::FStructType>();
				if (AllSharedVariableContainersMap.Contains(Struct))
				{
					continue;
				}

				TSharedPtr<FUAFInstanceVariableContainer> NewVariables = MakeShared<FUAFInstanceVariableContainer>(InInstance, Struct, InOverrides, OverrideIndex);
				OwnedVariableContainers.Add(NewVariables.ToSharedRef());
				const int32 ReferencedAssetIndex = AllSharedVariableContainers.Emplace(NewVariables);
				// NOTE: not adding to InternalSharedVariableContainers, as these arent needed for running the graph
				AllSharedVariableContainersMap.Add(Struct, ReferencedAssetIndex);
				NumAllVariables += NewVariables->NumVariables;
			}
			else if(Override.AssetOrStructData.IsType<FVariableOverrides::FAssetType>())
			{
				const UUAFRigVMAsset* Asset = Override.AssetOrStructData.Get<FVariableOverrides::FAssetType>();
				if (AllSharedVariableContainersMap.Contains(Asset))
				{
					continue;
				}

				TSharedPtr<FUAFInstanceVariableContainer> NewVariables = MakeShared<FUAFInstanceVariableContainer>(InInstance, Asset, InOverrides, OverrideIndex);
				OwnedVariableContainers.Add(NewVariables.ToSharedRef());
				const int32 ReferencedAssetIndex = AllSharedVariableContainers.Emplace(NewVariables);
				// NOTE: not adding to InternalSharedVariableContainers, as these arent needed for running the graph
				AllSharedVariableContainersMap.Add(Asset, ReferencedAssetIndex);
				NumAllVariables += NewVariables->NumVariables;
			}
			else if (Override.AssetOrStructData.IsType<FVariableOverrides::FRigVMAssetType>())
			{
				FVariableOverrides::FRigVMAssetType RigVMAsset = Override.AssetOrStructData.Get<FVariableOverrides::FRigVMAssetType>();
				if (AllSharedVariableContainersMap.Contains(RigVMAsset.GetObject()))
				{
					continue;
				}

				if (RigVMAsset.GetInterface() == nullptr)
				{
					continue;
				}

				TSharedPtr<FUAFInstanceVariableContainer> NewVariables = MakeShared<FUAFInstanceVariableContainer>(InInstance, RigVMAsset.GetInterface(), InOverrides, OverrideIndex);
				OwnedVariableContainers.Add(NewVariables.ToSharedRef());
				const int32 ReferencedAssetIndex = AllSharedVariableContainers.Emplace(NewVariables);
				// NOTE: not adding to InternalSharedVariableContainers, as these arent needed for running the graph
				AllSharedVariableContainersMap.Add(RigVMAsset.GetObject(), ReferencedAssetIndex);
				NumAllVariables += NewVariables->NumVariables;
			}
		}
	}

	RebuildNameMaps();
	
	ResolveOverrides();

#if WITH_EDITOR
	bHasBeenInitialized = true;
#endif
}

void FUAFInstanceVariableData::AddVariablesContainerForAsset(const UUAFRigVMAsset* InAsset, const FUAFAssetInstance& InInstance, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides)
{
	using namespace UE::UAF;

	TSharedPtr<FUAFInstanceVariableContainer> NewVariables;

	// First check to see we have an override
	if(InOverrides.IsValid())
	{
		for (int32 OverrideIndex = 0; OverrideIndex < InOverrides->Collection.Num(); ++OverrideIndex)
		{
			const FVariableOverrides& Override = InOverrides->Collection[OverrideIndex];
			if (Override.IsOverrideFor(InAsset))
			{
				NewVariables = MakeShared<FUAFInstanceVariableContainer>(InInstance, InAsset, InOverrides, OverrideIndex);
				OwnedVariableContainers.Add(NewVariables.ToSharedRef());
				break;
			}
		}
	}

	// Check to see if this asset exists in our host chain. If so, we take a reference.
	if (!NewVariables.IsValid())
	{
		const FUAFAssetInstance* CurrentHostInstance = InInstance.GetHost();
		while (CurrentHostInstance)
		{
			if (const int32* SetIndexPtr = CurrentHostInstance->Variables.AllSharedVariableContainersMap.Find(InAsset))
			{
				NewVariables = CurrentHostInstance->Variables.AllSharedVariableContainers[*SetIndexPtr].Pin();
				break;
			}
			CurrentHostInstance = CurrentHostInstance->GetHost();
		}
	}

	// Otherwise, create a new instance that we own
	if (!NewVariables.IsValid())
	{
		NewVariables = MakeShared<FUAFInstanceVariableContainer>(InInstance, InAsset);
		OwnedVariableContainers.Add(NewVariables.ToSharedRef());
	}

	const int32 ReferencedAssetIndex = AllSharedVariableContainers.Emplace(NewVariables);
	InternalSharedVariableContainers.Add(NewVariables);
	AllSharedVariableContainersMap.Add(InAsset, ReferencedAssetIndex);
	NumAllVariables += NewVariables->NumVariables;
	NumInternalVariables += NewVariables->NumVariables;
}

void FUAFInstanceVariableData::AddVariablesContainerForStruct(const UScriptStruct* InStruct, const FUAFAssetInstance& InInstance, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides)
{
	using namespace UE::UAF;

	TSharedPtr<FUAFInstanceVariableContainer> NewVariables;

	// First check to see we have an override
	if(InOverrides.IsValid())
	{
		for (int32 OverrideIndex = 0; OverrideIndex < InOverrides->Collection.Num(); ++OverrideIndex)
		{
			const FVariableOverrides& Override = InOverrides->Collection[OverrideIndex];
			if (Override.IsOverrideFor(InStruct))
			{
				NewVariables = MakeShared<FUAFInstanceVariableContainer>(InInstance, InStruct, InOverrides, OverrideIndex);
				OwnedVariableContainers.Add(NewVariables.ToSharedRef());
				break;
			}
		}
	}

	// Check to see if this asset exists in our host chain. If so, we take a reference.
	if (!NewVariables.IsValid())
	{
		const FUAFAssetInstance* CurrentHostInstance = InInstance.GetHost();
		while (CurrentHostInstance)
		{
			if (const int32* SetIndexPtr = CurrentHostInstance->Variables.AllSharedVariableContainersMap.Find(InStruct))
			{
				NewVariables = CurrentHostInstance->Variables.AllSharedVariableContainers[*SetIndexPtr].Pin();
				break;
			}
			CurrentHostInstance = CurrentHostInstance->GetHost();
		}
	}

	// Otherwise, create a new instance
	if (!NewVariables.IsValid())
	{
		NewVariables = MakeShared<FUAFInstanceVariableContainer>(InInstance, InStruct);
		OwnedVariableContainers.Add(NewVariables.ToSharedRef());
	}

	const int32 ReferencedAssetIndex = AllSharedVariableContainers.Emplace(NewVariables);
	InternalSharedVariableContainers.Add(NewVariables);
	AllSharedVariableContainersMap.Add(InStruct, ReferencedAssetIndex);
	NumAllVariables += NewVariables->NumVariables;
	NumInternalVariables += NewVariables->NumVariables;
}

void FUAFInstanceVariableData::AddVariablesContainerForRigVMAsset(const TScriptInterface<const IRigVMRuntimeAssetInterface>& InRigVMAsset,
	const FUAFAssetInstance& InInstance, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides)
{
	using namespace UE::UAF;

	check(InRigVMAsset.GetInterface() != nullptr);

	TSharedPtr<FUAFInstanceVariableContainer> NewVariables;

	// First check to see we have an override
	if(InOverrides.IsValid())
	{
		for (int32 OverrideIndex = 0; OverrideIndex < InOverrides->Collection.Num(); ++OverrideIndex)
		{
			const FVariableOverrides& Override = InOverrides->Collection[OverrideIndex];
			if (Override.IsOverrideFor(InRigVMAsset.GetInterface()))
			{
				NewVariables = MakeShared<FUAFInstanceVariableContainer>(InInstance, InRigVMAsset.GetInterface(), InOverrides, OverrideIndex);
				OwnedVariableContainers.Add(NewVariables.ToSharedRef());
				break;
			}
		}
	}

	// Check to see if this asset exists in our host chain. If so, we take a reference.
	if (!NewVariables.IsValid())
	{
		const FUAFAssetInstance* CurrentHostInstance = InInstance.GetHost();
		while (CurrentHostInstance)
		{
			if (const int32* SetIndexPtr = CurrentHostInstance->Variables.AllSharedVariableContainersMap.Find(InRigVMAsset.GetObject()))
			{
				NewVariables = CurrentHostInstance->Variables.AllSharedVariableContainers[*SetIndexPtr].Pin();
				break;
			}
			CurrentHostInstance = CurrentHostInstance->GetHost();
		}
	}

	// Otherwise, create a new instance that we own
	if (!NewVariables.IsValid())
	{
		NewVariables = MakeShared<FUAFInstanceVariableContainer>(InInstance, InRigVMAsset.GetInterface());
		OwnedVariableContainers.Add(NewVariables.ToSharedRef());
	}

	const int32 ReferencedAssetIndex = AllSharedVariableContainers.Emplace(NewVariables);
	InternalSharedVariableContainers.Add(NewVariables);
	AllSharedVariableContainersMap.Add(InRigVMAsset.GetObject(), ReferencedAssetIndex);
	NumAllVariables += NewVariables->NumVariables;
	NumInternalVariables += NewVariables->NumVariables;
}

void FUAFInstanceVariableData::RebuildNameMaps()
{
	using namespace UE::UAF;

	LegacyVariableNameMap.Reset();
	VariablesMap.Reset();

	// Setup name maps
	const int32 NumSharedVariableSets = AllSharedVariableContainers.Num();
	for (int32 SharedVariableSetIndex = 0; SharedVariableSetIndex < NumSharedVariableSets; ++SharedVariableSetIndex)
	{
		const TSharedRef<FUAFInstanceVariableContainer>& VariableSet = AllSharedVariableContainers[SharedVariableSetIndex].Pin().ToSharedRef();
		switch (VariableSet->VariablesContainer.GetIndex())
		{
		case FUAFInstanceVariableContainer::FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
			{
				const FInstancedPropertyBag& PropertyBag = VariableSet->VariablesContainer.Get<FInstancedPropertyBag>();
				const int32 NumVariablesInSet = PropertyBag.GetNumPropertiesInBag();
				const UObject* AssetObject = nullptr;
				if (VariableSet->AssetOrStructData.IsType<FUAFInstanceVariableContainer::FAssetType>())
				{
					AssetObject = VariableSet->AssetOrStructData.Get<FUAFInstanceVariableContainer::FAssetType>();
				}
				else if (VariableSet->AssetOrStructData.IsType<FUAFInstanceVariableContainer::FRigVMAssetType>())
				{
					FUAFInstanceVariableContainer::FRigVMAssetType RigVMInterface = VariableSet->AssetOrStructData.Get<FUAFInstanceVariableContainer::FRigVMAssetType>();
					AssetObject = RigVMInterface.GetObject();
				}

				check(AssetObject != nullptr);
				
				if (NumVariablesInSet > 0)
				{
					TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBag.GetPropertyBagStruct()->GetPropertyDescs();
					check(Descs.Num() == NumVariablesInSet);
					for (int32 VariableIndex = 0; VariableIndex < NumVariablesInSet; ++VariableIndex)
					{
						if (!IsInternalVariableName(Descs[VariableIndex].Name))
						{
							FUAFInstanceVariableData::FVariableMapping Mapping(SharedVariableSetIndex, VariableIndex);
							VariablesMap.Add(FAnimNextVariableReference::FromName(Descs[VariableIndex].Name, AssetObject), Mapping);
							LegacyVariableNameMap.Add(Descs[VariableIndex].Name, Mapping);
						}
					}
				}
				break;
			}
		case FUAFInstanceVariableContainer::FVariablesContainerType::IndexOfType<FInstancedStruct>():
			{
				const int32 NumVariablesInSet = VariableSet->NumVariables;
				if (NumVariablesInSet > 0)
				{
					const UScriptStruct* Struct = VariableSet->AssetOrStructData.Get<FUAFInstanceVariableContainer::FStructType>().Get();
					TConstArrayView<FStructDataCache::FPropertyInfo> Properties = VariableSet->StructDataCache->GetProperties();
					check(Properties.Num() == NumVariablesInSet);
					for (int32 VariableIndex = 0; VariableIndex < NumVariablesInSet; ++VariableIndex)
					{
						const FName Name = Properties[VariableIndex].Property->GetFName();
						if (!IsInternalVariableName(Name))
						{
							FUAFInstanceVariableData::FVariableMapping Mapping(SharedVariableSetIndex, VariableIndex);
							VariablesMap.Add(FAnimNextVariableReference::FromProperty(Properties[VariableIndex].Property, Struct), Mapping);
							LegacyVariableNameMap.Add(Name, Mapping);
						}
					}
				}
				break;
			}
		default:
			checkNoEntry();
			break;
		}
	}
}

void FUAFInstanceVariableData::Reset()
{
	OwnedVariableContainers.Reset();
	InternalSharedVariableContainers.Reset();
	AllSharedVariableContainers.Reset();
	AllSharedVariableContainersMap.Reset();
	LegacyVariableNameMap.Reset();
	VariablesMap.Reset();
	NumAllVariables = 0;
	NumInternalVariables = 0;
}

#if WITH_EDITOR
void FUAFInstanceVariableData::Migrate(const FUAFAssetInstance& InInstance)
{
	// Higher level logic needs to check to see if we have already been initialized, as overrides need to be cached first
	check(bHasBeenInitialized);

	const UUAFRigVMAsset* InstanceAsset = InInstance.GetAsset<UUAFRigVMAsset>();
	TConstArrayView<TObjectPtr<const UUAFRigVMAsset>> ReferencedVariableAssets = InstanceAsset->GetReferencedVariableAssets();
	TConstArrayView<TObjectPtr<const UScriptStruct>> ReferencedVariableStructs = InstanceAsset->GetReferencedVariableStructs();
	TConstArrayView<TScriptInterface<const IRigVMRuntimeAssetInterface>> ReferencedVariableRigVMAssets = InstanceAsset->GetReferencedVariableRigVMAssets();

	// Referenced assets + struct + self-container
	const int32 NumExpectedContainers = ReferencedVariableAssets.Num() + ReferencedVariableStructs.Num() + ReferencedVariableRigVMAssets.Num() + 1;
	const int32 NumExistingContainers = AllSharedVariableContainersMap.Num();

		// Verify the number of containers, new ones might have been added or existing ones removed
	const bool bMissingContainers = NumExpectedContainers > NumExistingContainers;
	const bool bOutOfDataContainers = NumExistingContainers > NumExpectedContainers;

	if (bMissingContainers)
	{
		// Referenced asset variables
		for (const UUAFRigVMAsset* ReferencedAsset : ReferencedVariableAssets)
		{
			const bool bHasVariableContainerForAsset = OwnedVariableContainers.ContainsByPredicate([ReferencedAsset](const TSharedRef<FUAFInstanceVariableContainer>& Container)
			{
				if (const FUAFInstanceVariableContainer::FAssetType* VariantPtr = Container->AssetOrStructData.TryGet<TObjectPtr<const UUAFRigVMAsset>>())
				{
					return (*VariantPtr) == ReferencedAsset;
				}
				return false;
			});
		
			if (!bHasVariableContainerForAsset)
			{
				AddVariablesContainerForAsset(ReferencedAsset, InInstance, nullptr);
			}
		}

		// Referenced struct variables
		for (const UScriptStruct* Struct : ReferencedVariableStructs)
		{
			const bool bHasVariableContainerForStruct = OwnedVariableContainers.ContainsByPredicate([Struct](const TSharedRef<FUAFInstanceVariableContainer>& Container)
			{
				if (const FUAFInstanceVariableContainer::FStructType* VariantPtr = Container->AssetOrStructData.TryGet<FUAFInstanceVariableContainer::FStructType>())
				{
					return (*VariantPtr) == Struct;
				}
				return false;
			});

			if (!bHasVariableContainerForStruct)
			{
				AddVariablesContainerForStruct(Struct, InInstance, nullptr);
			}
		}

		// Referenced IRigVMRuntimeAsset variables
		for (TScriptInterface<const IRigVMRuntimeAssetInterface> ReferencedRigVMAsset : ReferencedVariableRigVMAssets)
		{
			const bool bHasVariableContainerForAsset = OwnedVariableContainers.ContainsByPredicate([ReferencedRigVMAsset](const TSharedRef<FUAFInstanceVariableContainer>& Container)
			{
				if (const FUAFInstanceVariableContainer::FRigVMAssetType* VariantPtr = Container->AssetOrStructData.TryGet<FUAFInstanceVariableContainer::FRigVMAssetType>())
				{
					return (*VariantPtr) == ReferencedRigVMAsset;
				}
				return false;
			});
		
			if (!bHasVariableContainerForAsset)
			{
				AddVariablesContainerForRigVMAsset(ReferencedRigVMAsset, InInstance, nullptr);
			}
		}
	}
	else if (bOutOfDataContainers)
	{		
		for (int32 Index = 0; Index < AllSharedVariableContainers.Num(); ++Index)
		{
			const TWeakPtr<FUAFInstanceVariableContainer>& WeakContainer = AllSharedVariableContainers[Index];
			if (TSharedPtr<FUAFInstanceVariableContainer> SharedVariableContainer = WeakContainer.Pin())
			{
				const UObject* Object = nullptr;
				switch (SharedVariableContainer->AssetOrStructData.GetIndex())
				{
				case FUAFInstanceVariableContainer::FAssetOrStructType::IndexOfType<FUAFInstanceVariableContainer::FAssetType>():
					Object = SharedVariableContainer->AssetOrStructData.Get<FUAFInstanceVariableContainer::FAssetType>();
					break;
				case FUAFInstanceVariableContainer::FAssetOrStructType::IndexOfType<FUAFInstanceVariableContainer::FStructType>():
					Object = SharedVariableContainer->AssetOrStructData.Get<FUAFInstanceVariableContainer::FStructType>();
					break;
				case FUAFInstanceVariableContainer::FAssetOrStructType::IndexOfType<FUAFInstanceVariableContainer::FRigVMAssetType>():
					Object = SharedVariableContainer->AssetOrStructData.Get<FUAFInstanceVariableContainer::FRigVMAssetType>().GetObject();
					break;
				default:
					checkNoEntry()
					break;
				}

				// Referenced shared asset, or object corresponds to the instanced asset
				const bool bIsAssetEntry = ReferencedVariableAssets.Contains(Object) || InstanceAsset == Object;
				const bool bIsStructEntry = ReferencedVariableStructs.Contains(Object);
				const bool bIsRigVMEntry = ReferencedVariableRigVMAssets.Contains(Object);

				// Remove variable containers for asset/struct which are no longer referenced/part of this instance
				if (!bIsAssetEntry && !bIsStructEntry && !bIsRigVMEntry)
				{
					AllSharedVariableContainers.RemoveAtSwap(Index);

					OwnedVariableContainers.Remove(SharedVariableContainer.ToSharedRef());
					InternalSharedVariableContainers.Remove(WeakContainer);
					
					--Index;
				}
			}
		}
	}

	// Only migrate required/valid containers
	for (TSharedRef<FUAFInstanceVariableContainer>& OwnedVariableContainer : OwnedVariableContainers)
	{
		OwnedVariableContainer->Migrate();
	}

	NumAllVariables = 0;
	NumInternalVariables = 0;
	AllSharedVariableContainersMap.Reset();

	int32 ContainerIndex = 0;
	for (TWeakPtr<FUAFInstanceVariableContainer>& WeakSharedVariableContainer : AllSharedVariableContainers)
	{
		TSharedPtr<FUAFInstanceVariableContainer> SharedVariableContainer = WeakSharedVariableContainer.Pin();
		check(SharedVariableContainer.IsValid());

		NumAllVariables += SharedVariableContainer->NumVariables;

		const UObject* Object = nullptr;
		switch (SharedVariableContainer->AssetOrStructData.GetIndex())
		{
		case FUAFInstanceVariableContainer::FAssetOrStructType::IndexOfType<FUAFInstanceVariableContainer::FAssetType>():
			Object = SharedVariableContainer->AssetOrStructData.Get<FUAFInstanceVariableContainer::FAssetType>();
			break;
		case FUAFInstanceVariableContainer::FAssetOrStructType::IndexOfType<FUAFInstanceVariableContainer::FStructType>():
			Object = SharedVariableContainer->AssetOrStructData.Get<FUAFInstanceVariableContainer::FStructType>();
			break;
		case FUAFInstanceVariableContainer::FAssetOrStructType::IndexOfType<FUAFInstanceVariableContainer::FRigVMAssetType>():
			Object = SharedVariableContainer->AssetOrStructData.Get<FUAFInstanceVariableContainer::FRigVMAssetType>().GetObject();
			break;
		default:
			checkNoEntry()
			break;
		}

		AllSharedVariableContainersMap.Add(Object, ContainerIndex++);
	}

	RebuildNameMaps();

	for (TWeakPtr<FUAFInstanceVariableContainer>& WeakInternalSharedVariableContainer : InternalSharedVariableContainers)
	{
		TSharedPtr<FUAFInstanceVariableContainer> InternalSharedVariableContainer = WeakInternalSharedVariableContainer.Pin();
		check(InternalSharedVariableContainer.IsValid());
		
		NumInternalVariables += InternalSharedVariableContainer->NumVariables;
	}
}
#endif

void FUAFInstanceVariableData::ResolveOverrides()
{
	for (TSharedRef<FUAFInstanceVariableContainer>& OwnedVariableContainer : OwnedVariableContainers)
	{
		OwnedVariableContainer->ResolveOverrides();
	}
}

const FUAFInstanceVariableData::FVariableMapping* FUAFInstanceVariableData::FindMapping(const FAnimNextVariableReference& InVariable) const
{
	const FVariableMapping* FoundMapping = nullptr;
	if (InVariable.GetObject() == nullptr)
	{
		// Legacy path, lookup by name only
		FoundMapping = LegacyVariableNameMap.Find(InVariable.GetName());
	}
	else
	{
		FoundMapping = VariablesMap.Find(InVariable);
	}
	return FoundMapping;
}

EPropertyBagResult FUAFInstanceVariableData::GetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8> OutResult) const
{
	using namespace UE::UAF;

	const FVariableMapping* FoundMapping = FindMapping(InVariable);
	if (FoundMapping == nullptr)
	{
		return EPropertyBagResult::PropertyNotFound;
	}

	const TWeakPtr<FUAFInstanceVariableContainer>& SharedVariableSet = AllSharedVariableContainers[FoundMapping->AllSharedVariableContainersIndex];
	TSharedPtr<FUAFInstanceVariableContainer> PinnedVariableSet = SharedVariableSet.Pin();
	check(PinnedVariableSet.IsValid());
	return PinnedVariableSet->GetVariable(FoundMapping->VariableIndex, InType, OutResult);
}

EPropertyBagResult FUAFInstanceVariableData::AccessVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8>& OutResult) const
{
	using namespace UE::UAF;

	const FVariableMapping* FoundMapping = FindMapping(InVariable);
	if (FoundMapping == nullptr)
	{
		return EPropertyBagResult::PropertyNotFound;
	}

	const TWeakPtr<FUAFInstanceVariableContainer>& SharedVariableSet = AllSharedVariableContainers[FoundMapping->AllSharedVariableContainersIndex];
	TSharedPtr<FUAFInstanceVariableContainer> PinnedVariableSet = SharedVariableSet.Pin();
	check(PinnedVariableSet.IsValid());
	return PinnedVariableSet->AccessVariable(FoundMapping->VariableIndex, InType, OutResult);
}

EPropertyBagResult FUAFInstanceVariableData::SetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue) const
{
	using namespace UE::UAF;

	const FVariableMapping* FoundMapping = FindMapping(InVariable);
	if (FoundMapping == nullptr)
	{
		return EPropertyBagResult::PropertyNotFound;
	}

	const TWeakPtr<FUAFInstanceVariableContainer>& SharedVariableSet = AllSharedVariableContainers[FoundMapping->AllSharedVariableContainersIndex];
	TSharedPtr<FUAFInstanceVariableContainer> PinnedVariableSet = SharedVariableSet.Pin();
	check(PinnedVariableSet.IsValid());
	return PinnedVariableSet->SetVariable(FoundMapping->VariableIndex, InType, InNewValue);
}

bool FUAFInstanceVariableData::AccessVariablesStructInternal(const UScriptStruct* InStruct, TFunctionRef<void(FStructView)> InFunction) const
{
	const int32* FoundIndexPtr = AllSharedVariableContainersMap.Find(InStruct);
	if (FoundIndexPtr == nullptr)
	{
		return false;
	}

	TSharedPtr<FUAFInstanceVariableContainer> PinnedVariableSet = AllSharedVariableContainers[*FoundIndexPtr].Pin();
	check(PinnedVariableSet.IsValid());
	if (!PinnedVariableSet->VariablesContainer.IsType<FInstancedStruct>())
	{
		// Not an instanced struct (can be a set of overrides), so cant return a view
		return false;
	}

	InFunction(PinnedVariableSet->VariablesContainer.Get<FInstancedStruct>());
	return true;
}

void FUAFInstanceVariableData::ForEachVariablesStructInternal(TFunctionRef<void(FStructView)> InFunction) const
{
	for (const TWeakPtr<FUAFInstanceVariableContainer>& SharedVariableSet : AllSharedVariableContainers)
	{
		TSharedPtr<FUAFInstanceVariableContainer> PinnedVariableSet = SharedVariableSet.Pin();
		check(PinnedVariableSet.IsValid());
		if (!PinnedVariableSet->VariablesContainer.IsType<FInstancedStruct>())
		{
			// Not an instanced struct (can be a set of overrides), so cant supply a view
			continue;
		}

		InFunction(PinnedVariableSet->VariablesContainer.Get<FInstancedStruct>());
	}
}

void FUAFInstanceVariableData::GetAllVariablesOfType(const FAnimNextParamType& InType, TArray<FAnimNextVariableReference>& OutVariables) const
{
	for (const TWeakPtr<FUAFInstanceVariableContainer>& SharedVariableSet : AllSharedVariableContainers)
	{
		TSharedPtr<FUAFInstanceVariableContainer> PinnedVariableSet = SharedVariableSet.Pin();
		PinnedVariableSet->GetAllVariablesOfType(InType, OutVariables);
	}
}

void FUAFInstanceVariableData::ForEachVariable(TFunctionRef<bool(FName, const FAnimNextParamType&, int32)> InFunction) const
{
	int32 VariableIndex = 0;
	for (const TWeakPtr<FUAFInstanceVariableContainer>& SharedVariableSet : AllSharedVariableContainers)
	{
		TSharedPtr<FUAFInstanceVariableContainer> PinnedVariableSet = SharedVariableSet.Pin();
		if (!PinnedVariableSet->ForEachVariable(InFunction, VariableIndex))
		{
			break;
		}
	}
}

void FUAFInstanceVariableData::AccessVariablePropertyByIndex(int32 InIndex, TFunctionRef<void(const FProperty*, TArrayView<uint8>)> InFunction)
{
	int32 VariableIndexOffset = 0;
	for (const TWeakPtr<FUAFInstanceVariableContainer>& SharedVariableSet : AllSharedVariableContainers)
	{
		TSharedPtr<FUAFInstanceVariableContainer> PinnedVariableSet = SharedVariableSet.Pin();
		const int32 BaseIndex = VariableIndexOffset;
		VariableIndexOffset += PinnedVariableSet->NumVariables;
		if (InIndex < VariableIndexOffset)
		{
			const FProperty* Property = nullptr;
			TArrayView<uint8> Data;
			PinnedVariableSet->AccessVariableUnchecked(InIndex - BaseIndex, Property, Data);
			InFunction(Property, Data);
			return;
		}
	}
}

EPropertyBagResult FUAFInstanceVariableData::AccessVariableProperty(const FAnimNextVariableReference& InVariable,
	TFunctionRef<void(const FProperty*, TArrayView<uint8>)> InFunction)
{
	using namespace UE::UAF;

	const FVariableMapping* FoundMapping = FindMapping(InVariable);
	if (FoundMapping == nullptr)
	{
		return EPropertyBagResult::PropertyNotFound;
	}

	const TWeakPtr<FUAFInstanceVariableContainer>& SharedVariableSet = AllSharedVariableContainers[FoundMapping->AllSharedVariableContainersIndex];
	TSharedPtr<FUAFInstanceVariableContainer> PinnedVariableSet = SharedVariableSet.Pin();
	check(PinnedVariableSet.IsValid());

	const FProperty* Property = nullptr;
	TArrayView<uint8> Data;
	PinnedVariableSet->AccessVariableUnchecked(FoundMapping->VariableIndex, Property, Data);
	InFunction(Property, Data);
	return EPropertyBagResult::Success;
}

void FUAFInstanceVariableData::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	for (const TSharedRef<FUAFInstanceVariableContainer>& SharedVariableSet : OwnedVariableContainers)
	{
		Collector.AddPropertyReferencesWithStructARO(FUAFInstanceVariableContainer::StaticStruct(), &SharedVariableSet.Get());
	}
}
