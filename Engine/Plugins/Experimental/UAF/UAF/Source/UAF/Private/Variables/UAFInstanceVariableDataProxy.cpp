// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/UAFInstanceVariableDataProxy.h"

#include "AnimNextRigVMAsset.h"
#include "Variables/StructDataCache.h"
#include "Variables/AnimNextVariableReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFInstanceVariableDataProxy)

FUAFInstanceVariableContainerProxy::FUAFInstanceVariableContainerProxy(const FUAFInstanceVariableContainer& InVariables)
	: FUAFInstanceVariableContainer(InVariables)
{
	DirtyFlags.SetNum(NumVariables, false);
}

EPropertyBagResult FUAFInstanceVariableContainerProxy::AccessVariableWithDirtyFlag(int32 InVariableIndex, const FAnimNextParamType& InType, TArrayView<uint8>& OutResult)
{
	EPropertyBagResult Result = AccessVariable(InVariableIndex, InType, OutResult);
	if (Result == EPropertyBagResult::Success)
	{
		DirtyFlags[InVariableIndex] = true;
	}
	return Result;
}

EPropertyBagResult FUAFInstanceVariableContainerProxy::SetVariableWithDirtyFlag(int32 InVariableIndex, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue)
{
	EPropertyBagResult Result = SetVariable(InVariableIndex, InType, InNewValue);
	if (Result == EPropertyBagResult::Success)
	{
		DirtyFlags[InVariableIndex] = true;
	}
	return Result;
}

void FUAFInstanceVariableDataProxy::Initialize(const FUAFInstanceVariableData& InVariables)
{
	Variables = &InVariables;

	// TODO: To preserve memory for seldom-set variable sets, we could lazily instantiate the proxy set on set/get rather than up-front here
	ProxyVariableSets.Reserve(Variables->AllSharedVariableContainers.Num());
	for (const TWeakPtr<FUAFInstanceVariableContainer>& VariableSet : Variables->AllSharedVariableContainers)
	{
		ProxyVariableSets.Emplace(VariableSet.Pin().ToSharedRef().Get());
	}
	DirtyFlags.SetNum(Variables->AllSharedVariableContainers.Num(), false);
	bIsDirty = false;
}

void FUAFInstanceVariableDataProxy::Reset()
{
	ProxyVariableSets.Reset();
	DirtyFlags.Reset();
	bIsDirty = false;
}

EPropertyBagResult FUAFInstanceVariableDataProxy::WriteVariable(const FAnimNextVariableReference& InVariable,  const FAnimNextParamType& InType, TArrayView<uint8>& OutResult)
{
	if(Variables == nullptr)
	{
		// Not yet initialized
		return EPropertyBagResult::PropertyNotFound;
	}
	
	const FUAFInstanceVariableData::FVariableMapping* FoundMapping = Variables->FindMapping(InVariable);
	if (FoundMapping == nullptr)
	{
		return EPropertyBagResult::PropertyNotFound;
	}

	EPropertyBagResult Result = ProxyVariableSets[FoundMapping->AllSharedVariableContainersIndex].AccessVariableWithDirtyFlag(FoundMapping->VariableIndex, InType, OutResult);
	if (Result == EPropertyBagResult::Success)
	{
		bIsDirty = true;
		DirtyFlags[FoundMapping->AllSharedVariableContainersIndex] = true;
	}
	return Result;
}

EPropertyBagResult FUAFInstanceVariableDataProxy::SetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue)
{
	if(Variables == nullptr)
	{
		// Not yet initialized
		return EPropertyBagResult::PropertyNotFound;
	}

	const FUAFInstanceVariableData::FVariableMapping* FoundMapping =  Variables->FindMapping(InVariable);
	if (FoundMapping == nullptr)
	{
		return EPropertyBagResult::PropertyNotFound;
	}

	EPropertyBagResult Result = ProxyVariableSets[FoundMapping->AllSharedVariableContainersIndex].SetVariableWithDirtyFlag(FoundMapping->VariableIndex, InType, InNewValue);
	if (Result == EPropertyBagResult::Success)
	{
		bIsDirty = true;
		DirtyFlags[FoundMapping->AllSharedVariableContainersIndex] = true;
	}
	return Result;
}

EPropertyBagResult FUAFInstanceVariableDataProxy::AccessVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8>& OutValue)
{
	if (Variables == nullptr)
	{
		// Not yet initialized
		return EPropertyBagResult::PropertyNotFound;
	}

	const FUAFInstanceVariableData::FVariableMapping* FoundMapping =  Variables->FindMapping(InVariable);
	if (FoundMapping == nullptr)
	{
		return EPropertyBagResult::PropertyNotFound;
	}

	return ProxyVariableSets[FoundMapping->AllSharedVariableContainersIndex].AccessVariable(FoundMapping->VariableIndex, InType, OutValue);
}

bool FUAFInstanceVariableDataProxy::WriteVariables(const UScriptStruct* InStruct, TFunctionRef<void(FStructView)> InFunction)
{
	if(Variables == nullptr)
	{
		// Not yet initialized
		return false;
	}

	const int32* FoundIndexPtr = Variables->AllSharedVariableContainersMap.Find(InStruct);
	if (FoundIndexPtr == nullptr)
	{
		return false;
	}

	FUAFInstanceVariableContainerProxy& ProxyVariableSet = ProxyVariableSets[*FoundIndexPtr];
	if (!ProxyVariableSet.VariablesContainer.IsType<FInstancedStruct>())
	{
		// Not an instanced struct (can be a set of overrides), so cant return a view
		return false;
	}

	InFunction(ProxyVariableSet.VariablesContainer.Get<FInstancedStruct>());

	bIsDirty = true;
	DirtyFlags[*FoundIndexPtr] = true;
	ProxyVariableSet.DirtyFlags.SetRange(0, ProxyVariableSet.DirtyFlags.Num(), true);

	return true;
}

void FUAFInstanceVariableDataProxy::CopyDirty()
{
	using namespace UE::UAF;
	
	if(bIsDirty)
	{
		// Copy dirty properties
		for (TConstSetBitIterator<> SetIt(DirtyFlags); SetIt; ++SetIt)
		{
			const int32 SetIndex = SetIt.GetIndex();
			FUAFInstanceVariableContainerProxy& ProxySet = ProxyVariableSets[SetIndex];
			switch (ProxySet.VariablesContainer.GetIndex())
			{
			case FUAFInstanceVariableContainer::FVariablesContainerType::IndexOfType<FInstancedPropertyBag>():
				{
					const FInstancedPropertyBag& ProxyPropertyBag = ProxySet.VariablesContainer.Get<FInstancedPropertyBag>();
					TConstArrayView<FPropertyBagPropertyDesc> ProxyDescs = ProxyPropertyBag.GetPropertyBagStruct()->GetPropertyDescs();
					const uint8* SourceContainerPtr = ProxyPropertyBag.GetValue().GetMemory();

					FUAFInstanceVariableContainer& Set = *Variables->AllSharedVariableContainers[SetIndex].Pin().Get();
					FInstancedPropertyBag& PropertyBag = Set.VariablesContainer.Get<FInstancedPropertyBag>();
					TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBag.GetPropertyBagStruct()->GetPropertyDescs();
					uint8* TargetContainerPtr = PropertyBag.GetMutableValue().GetMemory();

					for (TConstSetBitIterator<> It(ProxySet.DirtyFlags); It; ++It)
					{
						const int32 Index = It.GetIndex();
						const FProperty* SourceProperty = ProxyDescs[Index].CachedProperty;
						const FProperty* TargetProperty = Descs[Index].CachedProperty;
						checkSlow(SourceProperty->GetClass() == TargetProperty->GetClass());
						const uint8* SourcePtr = SourceProperty->ContainerPtrToValuePtr<uint8>(SourceContainerPtr);
						uint8* TargetPtr = TargetProperty->ContainerPtrToValuePtr<uint8>(TargetContainerPtr);
						SourceProperty->CopyCompleteValue(TargetPtr, SourcePtr);
					}
					break;
				}
			case FUAFInstanceVariableContainer::FVariablesContainerType::IndexOfType<FInstancedStruct>():
				{
					const FInstancedStruct& ProxyStruct = ProxySet.VariablesContainer.Get<FInstancedStruct>();
					const uint8* SourceContainerPtr = ProxyStruct.GetMemory();
					TConstArrayView<FStructDataCache::FPropertyInfo> ProxyProperties = ProxySet.StructDataCache->GetProperties();

					FUAFInstanceVariableContainer& Set = *Variables->AllSharedVariableContainers[SetIndex].Pin().Get();
					FInstancedStruct& Struct = Set.VariablesContainer.Get<FInstancedStruct>();
					uint8* TargetContainerPtr = Struct.GetMutableMemory();
					TConstArrayView<FStructDataCache::FPropertyInfo> Properties = Set.StructDataCache->GetProperties();

					for (TConstSetBitIterator<> It(ProxySet.DirtyFlags); It; ++It)
					{
						const int32 Index = It.GetIndex();
						const FProperty* SourceProperty = ProxyProperties[Index].Property;
						const FProperty* TargetProperty = Properties[Index].Property;
						checkSlow(SourceProperty->GetClass() == TargetProperty->GetClass());
						const uint8* SourcePtr = SourceProperty->ContainerPtrToValuePtr<uint8>(SourceContainerPtr);
						uint8* TargetPtr = TargetProperty->ContainerPtrToValuePtr<uint8>(TargetContainerPtr);
						SourceProperty->CopyCompleteValue(TargetPtr, SourcePtr);
					}
					break;
				}
			default:
				checkNoEntry();
				break;
			}

			// Reset sets dirty flags
			ProxySet.DirtyFlags.SetRange(0, ProxySet.DirtyFlags.Num(), false);
		}

		// Reset dirty flags
		DirtyFlags.SetRange(0, DirtyFlags.Num(), false);
		bIsDirty = false;
	}
}

bool FUAFInstanceVariableDataProxy::IsVariableDirty(const FAnimNextVariableReference& InVariable) const
{
	if (Variables == nullptr)
	{
		return false;
	}

	const FUAFInstanceVariableData::FVariableMapping* FoundMapping = Variables->FindMapping(InVariable);
	if (FoundMapping == nullptr)
	{
		return false;
	}

	const FUAFInstanceVariableContainerProxy& VariableSet = ProxyVariableSets[FoundMapping->AllSharedVariableContainersIndex];
	const bool bIsVariableDirty = VariableSet.DirtyFlags[FoundMapping->VariableIndex];

	return bIsVariableDirty;
}