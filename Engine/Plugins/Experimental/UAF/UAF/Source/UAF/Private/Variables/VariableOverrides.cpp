// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariableOverrides.h"

#include "AnimNextRigVMAsset.h"
#include "RigVMRuntimeAsset.h"

namespace UE::UAF
{

FVariableOverrides::FVariableOverrides(const UUAFRigVMAsset* InAsset, TArray<FOverride>&& InOverrides)
	: Overrides(MoveTemp(InOverrides))
{
	check(InAsset);

	AssetOrStructData.Set<FAssetType>(InAsset);
}

FVariableOverrides::FVariableOverrides(const UScriptStruct* InStruct, TArray<FOverride>&& InOverrides)
	: Overrides(MoveTemp(InOverrides))
{
	AssetOrStructData.Set<FStructType>(InStruct);

}

FVariableOverrides::FVariableOverrides(const IRigVMRuntimeAssetInterface* InRigVMAsset, TArray<FOverride>&& InOverrides)
	:Overrides(MoveTemp(InOverrides))
{
	check(InRigVMAsset);
	TScriptInterface<const IRigVMRuntimeAssetInterface> Interface = Cast<const UObject>(InRigVMAsset);
	check(Interface != nullptr);
	AssetOrStructData.Set<FRigVMAssetType>(Interface);
}
}
