// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Param/ParamType.h"

class IRigVMRuntimeAssetInterface;
class UUAFRigVMAsset;
struct FUAFInstanceVariableContainer;
struct FUAFInstanceVariableData;
struct FAnimNextRigVMAssetStructData;
struct FAnimNextVariableOverridesCollection;

namespace UE::UAF
{

// Information about a set of variable overrides.
// These supply an indirection for variables to host RigVM memory handle memory.
// We always ensure we override the 'whole asset's variables' so we adhere to copy-in rules when running inner instances
struct FVariableOverrides
{
	// Override for a specific variable
	struct FOverride
	{
		FOverride(FName InName, const FAnimNextParamType& InType, uint8* InMemory)
			: Name(InName)
			, Type(InType)
			, Memory(InMemory)
		{}

		// The name of the variable
		FName Name;

		// The type of the variable
		FAnimNextParamType Type;

		// Ptr to the variable's memory override.
		// This should be owned by the outer scope (i.e. its lifetime should match) that is supplying the shared variable
		uint8* Memory = nullptr;
	};

	// Construct a set of overrides for an asset
	FVariableOverrides(const UUAFRigVMAsset* InAsset, TArray<FOverride>&& InOverrides);

	// Construct a set of overrides for a struct
	FVariableOverrides(const UScriptStruct* InStruct, TArray<FOverride>&& InOverrides);

	// Construct a set of overrides for a IRigVMRuntimeAssetInterface 
	FVariableOverrides(const IRigVMRuntimeAssetInterface* InRigVMAsset, TArray<FOverride>&& InOverrides);

	bool IsOverrideFor(const UUAFRigVMAsset* InAsset) const
	{
		const TObjectPtr<const UUAFRigVMAsset>* AssetPtr = AssetOrStructData.TryGet<FAssetType>();
		return AssetPtr && *AssetPtr == InAsset;
	}

	bool IsOverrideFor(const UScriptStruct* InStruct) const
	{
		const TObjectPtr<const UScriptStruct>* StructDataPtr = AssetOrStructData.TryGet<FStructType>();
		return StructDataPtr && *StructDataPtr == InStruct;
	}

	bool IsOverrideFor(const IRigVMRuntimeAssetInterface* InRigVMAsset) const
	{
		const FRigVMAssetType* AssetPtr = AssetOrStructData.TryGet<FRigVMAssetType>();
		return AssetPtr && *AssetPtr == InRigVMAsset;
	}

private:
	friend FUAFInstanceVariableContainer;
	friend FUAFInstanceVariableData;
	friend FAnimNextVariableOverridesCollection;

	// Asset or struct to override variables for
	using FAssetType = TObjectPtr<const UUAFRigVMAsset>;
	using FStructType = TObjectPtr<const UScriptStruct>;
	using FRigVMAssetType = TScriptInterface<const IRigVMRuntimeAssetInterface>;
	using FAssetOrStructType = TVariant<FAssetType, FStructType, FRigVMAssetType>;
	FAssetOrStructType AssetOrStructData;

	// Individual variable overrides
	TArray<FOverride> Overrides;
};

}
