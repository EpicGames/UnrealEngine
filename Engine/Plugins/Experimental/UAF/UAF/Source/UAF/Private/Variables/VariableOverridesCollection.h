// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VariableOverrides.h"

namespace UE::UAF
{

// A collection of overrides for asset/struct variables
struct FVariableOverridesCollection
{
	TArray<FVariableOverrides> Collection;
};

}