// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class UUAFRigVMAsset;

namespace UE::UAF
{

// Opaque handle that can be used to call RigVM functions on UUAFRigVMAsset
struct FFunctionHandle
{
	// Check to see if this is a valid handle
	bool IsValid() const
	{
		return FunctionIndex != INDEX_NONE;
	}

#if WITH_EDITOR
	// Check to see if this is a valid handle and that it can be used for the specified VM (hash comparison)
	bool IsValidForVM(uint32 InVMHash) const
	{
		return IsValid() && VMHash == InVMHash;
	}
#endif

private:
	friend ::UUAFRigVMAsset;

	int32 FunctionIndex = INDEX_NONE;
#if WITH_EDITOR
	uint32 VMHash = 0;
#endif
};

}