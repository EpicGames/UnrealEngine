// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AnimNextFunctionReference.generated.h"

/**
 * A reference to a specific parameterless RigVM function on a UUAFRigVMAsset.
 *
 * Analogous to FAnimNextVariableReference but for functions instead of variables.
 * Stores the mangled event name used for VM execution and the owning asset for
 * editor validation.
 */
struct FRigVMGraphFunctionHeader;

USTRUCT(BlueprintType)
struct FAnimNextFunctionReference
{
	GENERATED_BODY()

	FAnimNextFunctionReference() = default;

	/** Construct a reference from a function header and owning asset. Stores the GUID for rename robustness. */
	UAF_API static FAnimNextFunctionReference FromHeader(const FRigVMGraphFunctionHeader& InHeader, const UObject* InAsset);

	/** Construct a reference from an event name and owning asset (backward compat, no GUID). Editor only. */
#if WITH_EDITORONLY_DATA
	UAF_API static FAnimNextFunctionReference FromEventName(FName InEventName, const UObject* InAsset);
#endif

	/** Get the stable function GUID (survives renames). Invalid for old serialized data. */
	const FGuid& GetFunctionGuid() const { return FunctionGuid; }

#if WITH_EDITORONLY_DATA
	/** Get the mangled event name (editor only, used for display fallback). */
	FName GetEventName() const { return EventName; }
#endif

	/** Get the asset that owns the function. */
	TObjectPtr<const UObject> GetObject() const { return Object; }

	/** Check whether this can ever refer to a valid function. */
	bool IsNone() const
	{
#if WITH_EDITORONLY_DATA
		return !FunctionGuid.IsValid() && EventName.IsNone();
#else
		return !FunctionGuid.IsValid();
#endif
	}

	/** Reset this reference to None. */
	void Reset()
	{
		FunctionGuid.Invalidate();
#if WITH_EDITORONLY_DATA
		EventName = NAME_None;
#endif
		Object = nullptr;
	}

	bool operator==(const FAnimNextFunctionReference& Other) const
	{
		if (FunctionGuid.IsValid() && Other.FunctionGuid.IsValid())
		{
			return FunctionGuid == Other.FunctionGuid;
		}
#if WITH_EDITORONLY_DATA
		return EventName == Other.EventName && Object == Other.Object;
#else
		return false;
#endif
	}

	bool operator!=(const FAnimNextFunctionReference& Other) const
	{
		return !(*this == Other);
	}

	friend uint32 GetTypeHash(const FAnimNextFunctionReference& Key)
	{
		if (Key.FunctionGuid.IsValid())
		{
			return GetTypeHash(Key.FunctionGuid);
		}
#if WITH_EDITORONLY_DATA
		return HashCombineFast(GetTypeHash(Key.EventName), GetTypeHash(Key.Object));
#else
		return 0;
#endif
	}

private:
	/** Stable GUID identifying the function (survives renames). Invalid for legacy data. */
	UPROPERTY(EditAnywhere, Category = "Function")
	FGuid FunctionGuid;

#if WITH_EDITORONLY_DATA
	/** The mangled event name (editor only, used for display fallback with legacy data). */
	UPROPERTY(EditAnywhere, Category = "Function")
	FName EventName;
#endif

	/** The asset that owns the function. */
	UPROPERTY(EditAnywhere, Category = "Function")
	TObjectPtr<const UObject> Object;
};
