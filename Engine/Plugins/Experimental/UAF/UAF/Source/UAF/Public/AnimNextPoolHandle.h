// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/NumericLimits.h"

namespace UE::UAF
{

template<typename ElementType>
class TPool;

// Generational handle to a TPool entry
template<typename ElementType>
struct TPoolHandle
{
private:
	uint32 Index = MAX_uint32;
	uint32 SerialNumber = 0;

	friend TPool<ElementType>;
public:
	void Reset()
	{
		Index = MAX_uint32;
		SerialNumber = 0;
	}
	uint64 GetUniqueId() const { return (static_cast<uint64>(Index) << 32) | static_cast<uint64>(SerialNumber);}

	bool IsValid() const
	{
		return Index != MAX_uint32 && SerialNumber != 0;
	}

	bool operator==(const TPoolHandle& InOther) const
	{
		return Index == InOther.Index && SerialNumber == InOther.SerialNumber;
	}
};

}
