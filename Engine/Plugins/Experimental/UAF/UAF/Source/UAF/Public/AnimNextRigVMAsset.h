// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMHost.h"
#include "Param/ParamType.h"
#include "Variables/AnimNextVariableReference.h"
#include "AnimNextRigVMAsset.generated.h"

#define UE_API UAF_API

class UUAFRigVMAsset;
class UAnimNextVariableEntry;
class UUAFRigVMAssetEditorData;
class UAnimNextAnimGraphSettings;
struct FUAFAssetInstance;
struct FUAFInstanceVariableContainer;
struct FUAFInstanceVariableData;
struct FUAFInstanceVariableDataProxy;
struct FAnimNextRigVMFunctionData;
struct FUAFRigVMComponent;
struct FUAFAssetInstanceComponent;
struct FUAFRigVMSystemScriptEventsComponent;

namespace UE::UAF
{
	struct FAnimGraphBuilderContext;
	struct FSystemBuilderContext;
	struct FFunctionHandle;
	struct FModuleEventTickFunction;
	struct FVariableOverrides;
	struct FInjectionInfo;
}

namespace UE::UAF::Tests
{
	class FSystemReference_Creation;
}

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
	class FCompilationScope;
}

#if WITH_EDITORONLY_DATA
UENUM(BlueprintType)
enum class EAnimNextRigVMAssetState : uint8
{
	CompiledWithErrors,
	CompiledWithWarnings,
	CompiledWithSuccess,
	Invalid
};
#endif // WITH_EDITORONLY_DATA

/** Base class for all AnimNext assets that can host RigVM logic */
UCLASS(MinimalAPI, Abstract)
class UUAFRigVMAsset : public URigVMHost
{
	GENERATED_BODY()

protected:
	friend class UAnimNextVariableEntry;
	friend class UUAFRigVMAssetEditorData;
	friend class UAnimNextWorkspaceEditorMode;
	friend UAnimNextAnimGraphSettings;
	friend FUAFRigVMComponent;
	friend FUAFRigVMSystemScriptEventsComponent;
	friend struct UE::UAF::FModuleEventTickFunction;
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend FUAFInstanceVariableData;
	friend UE::UAF::FInjectionInfo;
	friend UE::UAF::FAnimGraphBuilderContext;
	friend UE::UAF::FSystemBuilderContext;
	friend UE::UAF::UncookedOnly::FCompilationScope;
	friend UE::UAF::Tests::FSystemReference_Creation;
	friend FUAFAssetInstance;
	friend class UUAFTestBlueprintLibrary;
	friend struct FAnimNextModuleInstance;

	UE_API UUAFRigVMAsset(const FObjectInitializer& ObjectInitializer);

	// UObject interface
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_API virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;

public:
	// Get variable defaults property bag
	const FInstancedPropertyBag& GetVariableDefaults() const { return VariableDefaults; }

	// Get the RigVM object we host
	const URigVM* GetRigVM() const { return RigVM; }

	// Call a function handle using a property bag as arguments.
	UE_API void CallFunctionHandle(UE::UAF::FFunctionHandle InHandle, FRigVMExtendedExecuteContext& InContext, FInstancedPropertyBag& InArgs);

	// Execute a parameterless function and copy its return value into OutReturnValue.
	// The function must have exactly 1 arg (the return value output). Returns true on success.
	UE_API bool ExecuteParameterlessFunction(UE::UAF::FFunctionHandle InHandle, FRigVMExtendedExecuteContext& InContext, void* OutReturnValue, int32 ReturnValueSize);

	// Get a function handle given its stable GUID
	UE_API UE::UAF::FFunctionHandle GetFunctionHandle(const FGuid& InFunctionGuid) const;

	// Get a function handle given its mangled event name (backward compat)
	UE_API UE::UAF::FFunctionHandle GetFunctionHandle(FName InEventName) const;

#if WITH_EDITOR
	// Delegate called in editor when an asset's compilation has started (i.e. asset data that instances rely on is going to change)
	using FOnCompileJobEvent = TTSMulticastDelegate<void(UUAFRigVMAsset*)>;
	static UE_API FOnCompileJobEvent& OnCompileJobStarted();

	// Delegate called in editor when an asset's compilation has finished (and instances can be re-allocated)
	static UE_API FOnCompileJobEvent& OnCompileJobFinished();
#endif
	
	// Get all the assets whose variables we will reference/instance at runtime
	TConstArrayView<TObjectPtr<const UUAFRigVMAsset>> GetReferencedVariableAssets() const { return ReferencedVariableAssets; }
	
	// Get all the native structs whose variables we will reference/instance at runtime
	TConstArrayView<TObjectPtr<const UScriptStruct>> GetReferencedVariableStructs() const { return ReferencedVariableStructs; }

	// Get all the IRigVMRuntimeAssetInterfaces whose variables we will reference/instance at runtime
	TConstArrayView<TScriptInterface<const IRigVMRuntimeAssetInterface>> GetReferencedVariableRigVMAssets() const { return ReferencedVariableRigVMAssets; }
	
private:
	// URigVMHost interface
	UE_API virtual TArray<FRigVMExternalVariable> GetExternalVariablesImpl(bool bFallbackToBlueprint) const;
	
protected:
	// The ExtendedExecuteContext object holds the common work data used by the RigVM internals. It is populated during the initial VM initialization.
	// Each instance of an AnimGraph requires a copy of this context and a call to initialize the VM instance with the context copy, 
	// so the cached memory handles are updated to the correct memory addresses.
	// This context is used as a reference to copy the common data for all instances created.
	UPROPERTY(Transient)
	FRigVMExtendedExecuteContext ExtendedExecuteContext;

	UPROPERTY()
	TObjectPtr<URigVM> RigVM;

	// Variables and their defaults (including public variables, sorted first)
	UPROPERTY()
	FInstancedPropertyBag VariableDefaults;

	// Variables property bag (including all variables from referenced assets, as external variables need stable property ptrs). 
	// Default values are not valid, this is just stored here for FProperty stability
	UPROPERTY()
	FInstancedPropertyBag CombinedPropertyBag;

	// All the assets whose variables we will reference/instance at runtime
	UPROPERTY()
	TArray<TObjectPtr<const UUAFRigVMAsset>> ReferencedVariableAssets;

	// All the native structs whose variables we will reference/instance at runtime
	UPROPERTY()
	TArray<TObjectPtr<const UScriptStruct>> ReferencedVariableStructs;

	// All the IRigVMRuntimeAssetInterface whose variables we will reference/instance at runtime
	UPROPERTY()
	TArray<TScriptInterface<const IRigVMRuntimeAssetInterface>> ReferencedVariableRigVMAssets;

	// All the natively-callable RigVM functions
	UPROPERTY()
	TArray<FAnimNextRigVMFunctionData> FunctionData;

	// Reference to the default injection site
	UPROPERTY()
	FAnimNextVariableReference DefaultInjectionSite;

	// Default components that are required for instances of this asset
	UPROPERTY()
	TArray<TInstancedStruct<FUAFAssetInstanceComponent>> Components;

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Instanced, Category = "Editor Data", meta = (ShowInnerProperties))
	TObjectPtr<UObject> EditorData;

	UPROPERTY(transient)
	EAnimNextRigVMAssetState CompilationState = EAnimNextRigVMAssetState::CompiledWithSuccess;
#endif
};

#undef UE_API
