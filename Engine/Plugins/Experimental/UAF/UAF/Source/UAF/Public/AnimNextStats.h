// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("UAF"), STATGROUP_AnimNext, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Module Event Tick"), STAT_AnimNext_Module_EventTick, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Module End Tick"), STAT_AnimNext_Module_EndTick, STATGROUP_AnimNext, UAF_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Initialize Instance"), STAT_AnimNext_InitializeInstance, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Create Instance Data"), STAT_AnimNext_CreateInstanceData, STATGROUP_AnimNext, UAF_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Make RefPose"), STAT_AnimNext_Make_RefPose, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Write Pose"), STAT_AnimNext_Write_Pose, STATGROUP_AnimNext, UAF_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Copy Transforms (SoA)"), STAT_AnimNext_CopyTransforms_SoA, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Normalize Rotations (SoA)"), STAT_AnimNext_NormalizeRotations_SoA, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Blend Overwrite (SoA)"), STAT_AnimNext_BlendOverwrite_SoA, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Blend Accumulate (SoA)"), STAT_AnimNext_BlendAccumulate_SoA, STATGROUP_AnimNext, UAF_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Generate Reference Pose"), STAT_AnimNext_GenerateReferencePose, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Remap Pose (From AnimBP)"), STAT_AnimNext_RemapPose_FromAnimBP, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Remap Pose (To AnimBP)"), STAT_AnimNext_RemapPose_ToAnimBP, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Remap Pose (To LocalTransforms)"), STAT_AnimNext_RemapPose_ToLocalTransforms, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Remap Pose (Mesh->Mesh)"), STAT_AnimNext_RemapPose_Mesh2Mesh, STATGROUP_AnimNext, UAF_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF: Convert Local Space To Component Space"), STAT_AnimNext_ConvertLocalSpaceToComponentSpace, STATGROUP_AnimNext, UAF_API);