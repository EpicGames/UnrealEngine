// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "BindableValue/UAFPropertyBinding.h"

#include "UAFBindableTypes.generated.h"

struct FUAFAssetInstance;

/**
 * Base struct for all strongly-typed bindable value wrappers.
 *
 * Each concrete subclass (FBindableBool, FBindableFloat, etc.) co-locates a constant
 * fallback value with an optional runtime Binding. Call HasBinding() to check whether
 * a binding is set.
 *
 * To read the resolved value at runtime, call GetValue(Instance):
 *   - If Instance is null or no binding is set, returns ConstantValue (the fallback).
 *   - Otherwise resolves the binding against the given FUAFAssetInstance, falling back
 *     to ConstantValue on failure.
 *
 */
USTRUCT(MinimalAPI)
struct FBindableValueBase
{
	GENERATED_BODY()

	FBindableValueBase() = default;
	~FBindableValueBase() = default;

	FBindableValueBase(const FBindableValueBase& Other)
		: Binding(Other.Binding ? MakeUnique<FUAFPropertyBinding>(*Other.Binding) : nullptr)
	{}

	FBindableValueBase& operator=(const FBindableValueBase& Other)
	{
		if (this != &Other)
		{
			Binding = Other.Binding ? MakeUnique<FUAFPropertyBinding>(*Other.Binding) : nullptr;
		}
		return *this;
	}

	FBindableValueBase(FBindableValueBase&&) = default;
	FBindableValueBase& operator=(FBindableValueBase&&) = default;

	bool HasBinding() const { return Binding.IsValid(); }
	const FUAFPropertyBinding* GetBinding() const
	{
		return Binding.Get();
	}
	
	void SetBinding(const FUAFPropertyBinding& InBinding)
	{
		if (Binding.IsValid())
		{
			// Overwrite existing binding
			*Binding.Get() = InBinding;
		}
		else
		{
			Binding = MakeUnique<FUAFPropertyBinding>(InBinding);
		}
	}

	void SetBinding(FUAFPropertyBinding&& InBinding)
	{
		if (Binding.IsValid())
		{
			// Overwrite existing binding
			*Binding.Get() = MoveTemp(InBinding);
		}
		else
		{
			Binding = MakeUnique<FUAFPropertyBinding>(MoveTemp(InBinding));
		}
	}

	void ClearBinding()
	{
		Binding.Reset();
	}

	/**
	 * Custom serialization is required because the Binding field is stored as a TUniquePtr<FUAFPropertyBinding>,
	 * which is not supported by UPROPERTY serialization. Three serialization paths are needed:
	 *   - Serialize(FArchive&): binary path (WithSerializer trait)
	 *   - ExportTextItem/ImportTextItem: text path (WithExportTextItem/WithImportTextItem traits),
	 *     used by RigVM text round-trip and property copy/paste
	 * The binding is exported as a __Binding=(...) pseudo-property appended to the struct's text.
	 * Each concrete subclass must declare its own TStructOpsTypeTraits (UE does not inherit traits).
	 */
	UAF_API bool Serialize(FArchive& Ar);
	UAF_API bool ExportTextItem(FString& ValueStr, const FBindableValueBase& DefaultValue,
		UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	UAF_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags,
		UObject* Parent, FOutputDevice* ErrorText);

	/**
	 * Compares this bindable value against Other, including the non-UPROPERTY Binding.
	 * Without this, delta serialization skips the struct when all UPROPERTYs match defaults,
	 * silently dropping the binding. Called by per-type Identical() via DECLARE_BINDABLE_IDENTICAL.
	 */
	UAF_API bool BindingsIdentical(const FBindableValueBase* Other, const UScriptStruct* ConcreteStruct, uint32 PortFlags) const;

protected:
	/** Shared helpers for text serialization — called by concrete subclasses via IMPLEMENT_BINDABLE_TEXT_SERIALIZATION. */
	UAF_API static void ExportBindingText(FString& ValueStr, const FUAFPropertyBinding* InBinding, int32 PortFlags);
	UAF_API static bool ImportBindingText(const TCHAR*& Buffer, TUniquePtr<FUAFPropertyBinding>& OutBinding, int32 PortFlags);

	/** Optional binding. Null = no binding; non-null = GetValue() resolves from a UAF variable at runtime. */
	TUniquePtr<FUAFPropertyBinding> Binding;
};

// Forward-declare the shared resolution helper so the friend declaration in the macro below is valid.
namespace UE::UAF::Private
{
	template<typename T, typename BindableType>
	static T ResolveBindableValue(const BindableType& Bindable, FUAFAssetInstance* Instance);
}

/**
 * Declares ExportTextItem/ImportTextItem on a concrete FBindableXxx type.
 * Also grants friend access to the shared ResolveBindableValue helper (uses GET_MEMBER_NAME_CHECKED on ConstantValue).
 * Place inside the struct body. Implementation is provided by IMPLEMENT_BINDABLE_TEXT_SERIALIZATION in the .cpp.
 */
#define DECLARE_BINDABLE_TEXT_SERIALIZATION(StructType) \
	template<typename T, typename BT> \
	friend T UE::UAF::Private::ResolveBindableValue(const BT&, FUAFAssetInstance*); \
	bool ExportTextItem(FString& ValueStr, const StructType& DefaultValue, \
		UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const; \
	bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, \
		UObject* Parent, FOutputDevice* ErrorText);

/**
 * Declares Identical() on a concrete FBindableXxx type, delegating to the base BindingsIdentical helper.
 * Required so that delta serialization includes the non-UPROPERTY binding in its comparison.
 * Place inside the struct body alongside DECLARE_BINDABLE_TEXT_SERIALIZATION.
 */
#define DECLARE_BINDABLE_IDENTICAL(StructType) \
	bool Identical(const StructType* Other, uint32 PortFlags) const;

/**
 * Declares TStructOpsTypeTraits for a concrete FBindableXxx type.
 * UE does not inherit traits from base structs, so each subclass must declare its own.
 */
#define DECLARE_BINDABLE_STRUCT_OPS(StructType) \
	template<> \
	struct TStructOpsTypeTraits<StructType> : public TStructOpsTypeTraitsBase2<StructType> \
	{ \
		enum \
		{ \
			WithSerializer = true, \
			WithCopy = true, \
			WithIdentical = true, \
			WithExportTextItem = true, \
			WithImportTextItem = true, \
		}; \
	};

template<>
struct TStructOpsTypeTraits<FBindableValueBase> : public TStructOpsTypeTraitsBase2<FBindableValueBase>
{
	enum
	{
		WithSerializer = true,
		WithCopy = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
	};
};

//------------------------------------------------------------------------------
// FBindableBool
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableBool : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableBool)
	DECLARE_BINDABLE_IDENTICAL(FBindableBool)

	FBindableBool() = default;
	FBindableBool(bool InValue) : ConstantValue(InValue) {}

	/** Resolve the binding against Instance (may be null), falling back to ConstantValue. */
	UAF_API bool GetValue(FUAFAssetInstance* Instance) const;

	bool GetConstantValue() const { return ConstantValue; }
	void SetConstantValue(bool InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	bool ConstantValue = false;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableBool)

//------------------------------------------------------------------------------
// FBindableFloat
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableFloat : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableFloat)
	DECLARE_BINDABLE_IDENTICAL(FBindableFloat)

	FBindableFloat() = default;
	FBindableFloat(float InValue) : ConstantValue(InValue) {}

	UAF_API float GetValue(FUAFAssetInstance* Instance) const;

	float GetConstantValue() const { return ConstantValue; }
	void  SetConstantValue(float InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	float ConstantValue = 0.f;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableFloat)

//------------------------------------------------------------------------------
// FBindableDouble
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableDouble : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableDouble)
	DECLARE_BINDABLE_IDENTICAL(FBindableDouble)

	FBindableDouble() = default;
	FBindableDouble(double InValue) : ConstantValue(InValue) {}

	UAF_API double GetValue(FUAFAssetInstance* Instance) const;

	double GetConstantValue() const { return ConstantValue; }
	void   SetConstantValue(double InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	double ConstantValue = 0.0;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableDouble)

//------------------------------------------------------------------------------
// FBindableInt32
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableInt32 : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableInt32)
	DECLARE_BINDABLE_IDENTICAL(FBindableInt32)

	FBindableInt32() = default;
	FBindableInt32(int32 InValue) : ConstantValue(InValue) {}

	UAF_API int32 GetValue(FUAFAssetInstance* Instance) const;

	int32 GetConstantValue() const { return ConstantValue; }
	void  SetConstantValue(int32 InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	int32 ConstantValue = 0;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableInt32)

//------------------------------------------------------------------------------
// FBindableInt64
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableInt64 : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableInt64)
	DECLARE_BINDABLE_IDENTICAL(FBindableInt64)

	FBindableInt64() = default;
	FBindableInt64(int64 InValue) : ConstantValue(InValue) {}

	UAF_API int64 GetValue(FUAFAssetInstance* Instance) const;

	int64 GetConstantValue() const { return ConstantValue; }
	void  SetConstantValue(int64 InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	int64 ConstantValue = 0;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableInt64)

//------------------------------------------------------------------------------
// FBindableByte
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableByte : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableByte)
	DECLARE_BINDABLE_IDENTICAL(FBindableByte)

	FBindableByte() = default;
	FBindableByte(uint8 InValue) : ConstantValue(InValue) {}

	UAF_API uint8 GetValue(FUAFAssetInstance* Instance) const;

	uint8 GetConstantValue() const { return ConstantValue; }
	void  SetConstantValue(uint8 InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	uint8 ConstantValue = 0;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableByte)

//------------------------------------------------------------------------------
// FBindableName
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableName : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableName)
	DECLARE_BINDABLE_IDENTICAL(FBindableName)

	FBindableName() = default;
	FBindableName(FName InValue) : ConstantValue(InValue) {}

	UAF_API FName GetValue(FUAFAssetInstance* Instance) const;

	const FName& GetConstantValue() const { return ConstantValue; }
	void         SetConstantValue(const FName& InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	FName ConstantValue;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableName)

//------------------------------------------------------------------------------
// FBindableVector
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableVector : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableVector)
	DECLARE_BINDABLE_IDENTICAL(FBindableVector)

	FBindableVector() = default;
	FBindableVector(const FVector& InValue) : ConstantValue(InValue) {}

	UAF_API FVector GetValue(FUAFAssetInstance* Instance) const;

	const FVector& GetConstantValue() const { return ConstantValue; }
	void           SetConstantValue(const FVector& InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	FVector ConstantValue = FVector::ZeroVector;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableVector)

//------------------------------------------------------------------------------
// FBindableQuat
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableQuat : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableQuat)
	DECLARE_BINDABLE_IDENTICAL(FBindableQuat)

	FBindableQuat() = default;
	FBindableQuat(const FQuat& InValue) : ConstantValue(InValue) {}

	UAF_API FQuat GetValue(FUAFAssetInstance* Instance) const;

	const FQuat& GetConstantValue() const { return ConstantValue; }
	void         SetConstantValue(const FQuat& InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	FQuat ConstantValue = FQuat::Identity;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableQuat)

//------------------------------------------------------------------------------
// FBindableTransform
//------------------------------------------------------------------------------

USTRUCT(BlueprintType, MinimalAPI)
struct FBindableTransform : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableTransform)
	DECLARE_BINDABLE_IDENTICAL(FBindableTransform)

	FBindableTransform() = default;
	FBindableTransform(const FTransform& InValue) : ConstantValue(InValue) {}

	UAF_API FTransform GetValue(FUAFAssetInstance* Instance) const;

	const FTransform& GetConstantValue() const { return ConstantValue; }
	void              SetConstantValue(const FTransform& InValue) { ConstantValue = InValue; }

private:
	UPROPERTY(EditAnywhere, Category = "Value")
	FTransform ConstantValue = FTransform::Identity;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableTransform)

//------------------------------------------------------------------------------
// FBindableEnum
//------------------------------------------------------------------------------

/**
 * Bindable enum value. Stores the raw int32 fallback alongside the UEnum class so that
 * the editor binding picker can filter compatible enum variables.
 *
 * The declaring node data type should initialize EnumClass and ConstantValue in its
 * default constructor, e.g.:
 *   LoopMode.SetConstantValue((int32)EAnimAssetLoopMode::Auto);
 *   LoopMode.EnumClass = StaticEnum<EAnimAssetLoopMode>();
 *
 * Enums whose underlying type exceeds 32 bits are not supported; a runtime ensure
 * will fire if such an enum is bound.
 */
USTRUCT(BlueprintType, MinimalAPI)
struct FBindableEnum : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableEnum)
	DECLARE_BINDABLE_IDENTICAL(FBindableEnum)

	FBindableEnum() = default;
	explicit FBindableEnum(UEnum* InEnumClass) : EnumClass(InEnumClass) {}

	/** Constructs with EnumClass and ConstantValue both derived from the enum value. */
	template<typename T> requires (bool(TIsUEnumClass<T>::Value))
	explicit FBindableEnum(T InValue)
		: EnumClass(StaticEnum<T>())
		, ConstantValue(static_cast<int32>(InValue))
	{
		static_assert(sizeof(T) <= sizeof(int32),
			"FBindableEnum only supports enums with underlying types up to 32 bits.");
	}

	UAF_API int32 GetValue(FUAFAssetInstance* Instance) const;

	/** Type-safe getter — T must be a UENUM. */
	template<typename T> requires (bool(TIsUEnumClass<T>::Value))
	T GetValue(FUAFAssetInstance* Instance) const
	{
		static_assert(sizeof(T) <= sizeof(int32),
			"FBindableEnum only supports enums with underlying types up to 32 bits.");
		return static_cast<T>(GetValue(Instance));
	}

	int32 GetConstantValue() const { return ConstantValue; }
	void  SetConstantValue(int32 InValue) { ConstantValue = InValue; }

	/** Type-safe constant value getter — T must be a UENUM. */
	template<typename T> requires (bool(TIsUEnumClass<T>::Value))
	T GetConstantValue() const
	{
		static_assert(sizeof(T) <= sizeof(int32),
			"FBindableEnum only supports enums with underlying types up to 32 bits.");
		return static_cast<T>(ConstantValue);
	}

	/** Type-safe constant value setter — T must be a UENUM. */
	template<typename T> requires (bool(TIsUEnumClass<T>::Value))
	void SetConstantValue(T InValue)
	{
		static_assert(sizeof(T) <= sizeof(int32),
			"FBindableEnum only supports enums with underlying types up to 32 bits.");
		ConstantValue = static_cast<int32>(InValue);
	}

	UEnum* GetEnumClass() const { return EnumClass; }
	void   SetEnumClass(UEnum* InEnumClass) { EnumClass = InEnumClass; }

private:

	/** Enum class for variable-picker filtering. Set by the declaring node's constructor. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TObjectPtr<UEnum> EnumClass = nullptr;
	
	UPROPERTY(EditAnywhere, Category = "Value")
	int32 ConstantValue = 0;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableEnum)

//------------------------------------------------------------------------------
// FBindableStruct
//------------------------------------------------------------------------------

/**
 * Bindable struct value.
 *
 * Call GetValue<T>(Instance, OutValue) to resolve the binding into caller-provided storage:
 *   - No binding (or null Instance): copies ConstantValue into OutValue,
 *     or zero-initializes OutValue if ConstantValue is empty.
 *   - Variable binding: reads the named variable from Instance into OutValue.
 *   - SubProperty binding: reads the leaf sub-property from a struct variable into OutValue.
 *
 * T must be a USTRUCT. A runtime ensure verifies that T matches the bound or constant struct type.
 */
USTRUCT(BlueprintType, MinimalAPI)
struct FBindableStruct : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableStruct)
	DECLARE_BINDABLE_IDENTICAL(FBindableStruct)

	FBindableStruct() = default;
	explicit FBindableStruct(UScriptStruct* InStructClass) : StructClass(InStructClass) {}

	/** Constructs with StructClass and ConstantValue both derived from T. */
	template<typename T>
	explicit FBindableStruct(const T& InValue)
		: StructClass(TBaseStructure<T>::Get())
		, ConstantValue(FInstancedStruct::Make(InValue))
	{
	}

	/**
	 * Resolves the binding into OutValue, falling back to ConstantValue on failure.
	 * T must be a USTRUCT. Runtime-asserts that T matches the bound or constant struct type.
	 */
	template<typename T>
	void GetValue(FUAFAssetInstance* Instance, T& OutValue) const
	{
		GetValueToMem(Instance, TBaseStructure<T>::Get(), &OutValue);
	}

	const FInstancedStruct& GetConstantValue() const { return ConstantValue; }
	void SetConstantValue(const FInstancedStruct& InValue) { ConstantValue = InValue; }
	void SetConstantValue(FInstancedStruct&& InValue) { ConstantValue = MoveTemp(InValue); }

	UScriptStruct* GetStructClass() const { return StructClass; }
	void           SetStructClass(UScriptStruct* InStructClass) { StructClass = InStructClass; }

private:
	/** Non-templated implementation; called by GetValue<T>. */
	UAF_API void GetValueToMem(FUAFAssetInstance* Instance, const UScriptStruct* ExpectedStruct, void* OutMem) const;

	/** Struct class for variable-picker filtering. Set by the declaring node's constructor. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TObjectPtr<UScriptStruct> StructClass = nullptr;

	UPROPERTY(EditAnywhere, Category = "Value")
	FInstancedStruct ConstantValue;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableStruct)

//------------------------------------------------------------------------------
// FBindableObject
//------------------------------------------------------------------------------

/**
 * Bindable UObject reference. Stores a hard TObjectPtr<UObject> constant alongside
 * an optional ObjectClass filter for the editor variable picker.
 *
 * The declaring node data type should set ObjectClass in its constructor to restrict
 * which object variables are shown in the picker.
 */
USTRUCT(BlueprintType, MinimalAPI)
struct FBindableObject : public FBindableValueBase
{
	GENERATED_BODY()
	DECLARE_BINDABLE_TEXT_SERIALIZATION(FBindableObject)
	DECLARE_BINDABLE_IDENTICAL(FBindableObject)

	FBindableObject() = default;
	explicit FBindableObject(UObject* InValue) : ConstantValue(InValue) {}
	explicit FBindableObject(UClass* InObjectClass) : ObjectClass(InObjectClass) {}
	FBindableObject(UObject* InValue, UClass* InObjectClass)
		: ObjectClass(InObjectClass)
		, ConstantValue(InValue)
	{
	}

	UAF_API UObject* GetValue(FUAFAssetInstance* Instance) const;

	template<class T> requires(std::is_base_of_v<UObject, std::decay_t<T>>)
	T* GetValue(FUAFAssetInstance* Instance) const
	{
		checkf(ObjectClass == nullptr || ObjectClass == T::StaticClass(), TEXT("Attempting to cast to incompatible object type."));
		return static_cast<T*>(GetValue(Instance));
	}

	UObject* GetConstantValue() const { return ConstantValue; }
	void     SetConstantValue(UObject* InValue) { ConstantValue = InValue; }

	UClass* GetObjectClass() const { return ObjectClass; }
	void    SetObjectClass(UClass* InClass) { ObjectClass = InClass; }

private:
	/** Class filter for variable-picker. Set by the declaring node's constructor. */
	UPROPERTY(EditAnywhere, Category = "Value")
	TObjectPtr<UClass> ObjectClass = nullptr;

	UPROPERTY(EditAnywhere, Category = "Value")
	TObjectPtr<UObject> ConstantValue = nullptr;
};

DECLARE_BINDABLE_STRUCT_OPS(FBindableObject)
