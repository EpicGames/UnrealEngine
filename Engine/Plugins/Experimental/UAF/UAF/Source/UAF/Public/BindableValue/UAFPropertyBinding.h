// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextFunctionReference.h"
#include "PropertyBindingPath.h"
#include "Variables/AnimNextVariableReference.h"

#include "UAFPropertyBinding.generated.h"

/**
 * Describes how to source a bound property value.
 */
UENUM()
enum class EUAFBindingSourceType : uint8
{
	/** Direct UAF variable (top-level, with type-match or type-promotion). */
	Variable,

	/** A dot-separated path into a struct-type UAF variable. */
	SubProperty,

	/** A 0-parameter RigVM function whose return value is used. */
	Function,
};

/**
 * Describes the source side of a property binding: where to read a value from at runtime.
 * The target is implicit — determined by which FBindableXxx member holds this struct.
 */
USTRUCT()
struct FUAFPropertyBinding
{
	GENERATED_BODY()

	/**
	 * The UAF variable to read from.
	 * Used when SourceType is Variable or SubProperty.
	 */
	UPROPERTY(EditAnywhere, Category = "Binding")
	FAnimNextVariableReference SourceVariable;

	/**
	 * Dot-separated path into the struct variable to the leaf property.
	 * Used only when SourceType is SubProperty.
	 */
	UPROPERTY(EditAnywhere, Category = "Binding")
	FPropertyBindingPath SubPropertyPath;

	/**
	 * Reference to the parameterless RigVM function whose return value is used.
	 * Used only when SourceType is Function.
	 */
	UPROPERTY(EditAnywhere, Category = "Binding")
	FAnimNextFunctionReference SourceFunction;

	/** How to source the bound value. */
	UPROPERTY(EditAnywhere, Category = "Binding")
	EUAFBindingSourceType SourceType = EUAFBindingSourceType::Variable;
};
