// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISystemOutputAdapter.h"
#include "Components/ActorComponent.h"
#include "Module/SystemReference.h"
#include "Param/ParamType.h"
#include "TraitCore/TraitEvent.h"
#include "Module/AnimNextModuleInitMethod.h"
#include "Module/ModuleTaskContext.h"
#include "Module/TaskRunLocation.h"

#include "AnimNextComponent.generated.h"

struct FUAFSystemFactoryAsset;

namespace UE::Workspace
{
	class IWorkspaceViewportController;
}

struct FAnimNextComponentInstanceData;
class UUAFSystem;
struct AnimNextVariableReference;
// TODO: if a public API is implemented, the forward declaration of this struct would no longer be needed.
struct FAnimMixerUtils;
struct FUAFWeakSystemReference;

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

namespace UE::UAF::Editor
{
	class FAssetWizard;
	class FSystemViewportController;
	class FAnimGraphViewportController;
	class SUAFGraphAssetPreview;
}

namespace UE::UAF::Tests
{
	struct FUAFSystemTests;
}

// Description of a UAF component input
USTRUCT(meta=(Hidden))
struct FUAFComponentInputDesc
{
	GENERATED_BODY()

	// The component that this input uses for its input pose
	UPROPERTY(EditAnywhere, Category="Input", meta = (UseComponentPicker, AllowedClasses = "/Script/UAF.UAFComponent,/Script/Engine.SkeletalMeshComponent"))
	FComponentReference Component;

	// The input pose variable that this input maps to
	UPROPERTY(EditAnywhere, Category="Input", meta = (AllowedType = "FUAFValueBundle"))
	FAnimNextVariableReference Input;
};

// The operation to perform when outputting values with respect to a skeletal mesh component
UENUM()
enum class EUAFSkeletalMeshComponentOutputMode : uint8
{
	// Write the pose to the skeletal mesh component (i.e. animate it directly)
	WriteToSkeletalMeshComponentPose,

	// Do not write the pose to the skeletal mesh component. 
	// This can be useful to avoid unnecessary work if the skeletal mesh component's output is not rendered, or if this component is generating an 
	// intermediate output
	SkipWriteToSkeletalMeshComponentPose,
};

UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent))
class UUAFComponent : public UActorComponent, public UE::UAF::ISystemOutputAdapter
{
	GENERATED_BODY()

	// UActorComponent interface
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual void Activate(bool bReset = false) override;
	virtual void Deactivate() override;

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;

	// ISystemOutputAdapter interface
	virtual FInputData GetInputData() const override;
	virtual void SignalOutputWritten(const UE::UAF::FModuleTaskContext& InContext) override;
	
public:
	// Sets a system variable's value.
	// @param    Name     The name of the variable to set
	// @param    Value    The value to set the variable to
	UE_DEPRECATED(5.6, "This function is no longer used, please use SetVariableReference (for Blueprint) or SetVariable that takes a FAnimNextVariableReference")
	UFUNCTION(BlueprintCallable, Category = "UAF", DisplayName = "Set Variable", CustomThunk, meta = (CustomStructureParam = Value, UnsafeDuringActorConstruction, DeprecatedFunction, DeprecationMessage = "This function has been deprecated, please use Set Variable that takes a variable reference instead"))
	UAF_API void BlueprintSetVariable(FName Name, int32 Value);

	// Sets a system variable's value.
	// @param    Variable The variable to set
	// @param    Value    The value to set the variable to
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category = "UAF", DisplayName = "Set Variable", CustomThunk, meta = (CustomStructureParam = Value, AutoCreateRefTerm = "Variable, Value", UnsafeDuringActorConstruction))
	UAF_API void BlueprintSetVariableReference(const FAnimNextVariableReference& Variable, const int32& Value);

	// Gets a system variable's value.
	// @param    Variable The variable to get
	// @param    Value    The return value of the variable
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category = "UAF", DisplayName = "Get Variable", CustomThunk, meta = (CustomStructureParam = Value, AutoCreateRefTerm = "Variable, Value", UnsafeDuringActorConstruction))
	UAF_API void BlueprintGetVariableReference(const FAnimNextVariableReference& Variable, int32& Value);
	
	// Sets a system's (of FValueBundle type) variable.
	// @param    Variable The variable to set
	// @param    Component    The component to use as input source
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category = "UAF", DisplayName = "Set Input Binding", meta = (AutoCreateRefTerm = "Variable, Component", UnsafeDuringActorConstruction))
	UAF_API void BlueprintSetInputBinding(const FAnimNextVariableReference& Variable, UActorComponent* Component);
	
	// Set a variable.
	// @param    InVariable         The variable to set
	// @param    InValue            The value to set the variable to
	// @return true if the variable was set correctly
	template<typename ValueType>
	bool SetVariable(const FAnimNextVariableReference& InVariable, const ValueType& InValue)
	{
		return SetVariableInternal(InVariable, FAnimNextParamType::GetType<ValueType>(), TConstArrayView<uint8>(reinterpret_cast<const uint8*>(&InValue), sizeof(ValueType))) == EPropertyBagResult::Success;
	}

	// Get a variable.
	// @param    InVariable         The variable to get
	// @param    OutValue           The return value of the variable
	// @return true if the variable was gotten correctly
	template <typename ValueType>
	bool GetVariable(const FAnimNextVariableReference& InVariable, ValueType& OutValue)
	{
		return GetVariableInternal(InVariable, FAnimNextParamType::GetType<ValueType>(), TArrayView<uint8>(reinterpret_cast<uint8*>(&OutValue), sizeof(ValueType))) == EPropertyBagResult::Success;
	}

	// Accesses a variable for writing.
	// This is intended to allow for copy-free writing of larger data structures & arrays, rather than read access. 
	// @param    InVariable         The variable to set
	// @param    InFunction         The function used to modify the variable, called back immediately on the game thread if the variable exists.
	// @return true if the variable was accessed successfully
	template<typename ValueType>
	bool WriteVariable(const FAnimNextVariableReference& InVariable, TFunctionRef<void(ValueType&)> InFunction)
	{
		return WriteVariableInternal(InVariable, FAnimNextParamType::GetType<ValueType>(), [&InFunction](TArrayView<uint8> InData)
		{
			InFunction(*reinterpret_cast<ValueType*>(InData.GetData()));
		}) == EPropertyBagResult::Success;
	}

	// Enable or disable debug drawing. Note only works in builds with UE_ENABLE_DEBUG_DRAWING enabled
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API void ShowDebugDrawing(bool bShowDebugDrawing);

	// Queue a task to run during execution 
	UAF_API void QueueTask(FName InModuleEventName, TUniqueFunction<void(const UE::UAF::FModuleTaskContext&)>&& InTaskFunction, UE::UAF::ETaskRunLocation InLocation = UE::UAF::ETaskRunLocation::Before);

	// Queues an input trait event
	// Input events will be processed in the next graph update after they are queued
	UAF_API void QueueInputTraitEvent(FAnimNextTraitEventPtr Event);

	// Get a reference to the registered system
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API FUAFWeakSystemReference GetSystemReference() const;

	// Find the tick function for the specified event
	// @param	InEventName			The event associated to the wanted tick function
	UAF_API const FTickFunction* FindTickFunction(FName InEventName) const;

	// Add a prerequisite tick function dependency to the specified event
	// @param	InObject			The object that owns the tick function
	// @param	InTickFunction		The tick function to depend on
	// @param	InEventName			The event to add the dependency to
	UAF_API void AddPrerequisite(UObject* InObject, FTickFunction& InTickFunction, FName InEventName);

	// Add a prerequisite dependency on the component's primary tick function to the specified event
	// The component will tick before the event
	// @param	Component			The component to add as a prerequisite 
	// @param	EventName			The event to add the dependency to
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API void AddComponentPrerequisite(UActorComponent* Component, FName EventName);

	// Add a subsequent tick function dependency to the specified event
	// @param	InObject			The object that owns the tick function
	// @param	InTickFunction		The tick function to depend on
	// @param	InEventName			The event to add the dependency to
	UAF_API void AddSubsequent(UObject* InObject, FTickFunction& InTickFunction, FName InEventName);

	// Add a subsequent dependency on the component's primary tick function to the specified event
	// The component will tick after the event
	// @param	Component			The component to add as a subsequent of the event
	// @param	EventName			The event to add the dependency to
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API void AddComponentSubsequent(UActorComponent* Component, FName EventName);

	// Remove a prerequisite tick function dependency from the specified event
	// @param	InObject			The object that owns the tick function
	// @param	InTickFunction		The tick function that was depended on
	// @param	InEventName			The event to remove the dependency from
	UAF_API void RemovePrerequisite(UObject* InObject, FTickFunction& InTickFunction, FName InEventName);

	// Remove a prerequisite on the component's primary tick function from the specified event
	// @param	Component			The component to remove as a prerequisite 
	// @param	EventName			The event to add the dependency to
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API void RemoveComponentPrerequisite(UActorComponent* Component, FName EventName);
	
	// Remove a prerequisite tick function dependency from the specified event
	// @param	InObject			The object that owns the tick function
	// @param	InTickFunction		The tick function that was depended on
	// @param	InEventName			The event to remove the dependency from
	UAF_API void RemoveSubsequent(UObject* InObject, FTickFunction& InTickFunction, FName InEventName);

	// Remove a subsequent dependency on the component's primary tick function from the specified event
	// @param	Component			The component to remove as a subsequent of the event
	// @param	EventName			The event to add the dependency to
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API void RemoveComponentSubsequent(UActorComponent* Component, FName EventName);
	
	// Add a prerequisite anim next event dependency to the specified event
	// @param	InEventName			The event name in this component
	// @param	InTickFunction		The other component we want a prerequisite on
	// @param	InEventName			The other component's event name
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API void AddModuleEventPrerequisite(FName InEventName, UUAFComponent* OtherAnimNextComponent, FName OtherEventName);

	// Add a subsequent anim next event dependency to the specified event
	// @param	InEventName			The event name in this component
	// @param	InTickFunction		The other component we want to add a prerequisite to
	// @param	InEventName			The other component's event name
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API void AddModuleEventSubsequent(FName InEventName, UUAFComponent* OtherAnimNextComponent, FName OtherEventName);

	// Remove a prerequisite anim next event dependency from the specified event
	// @param	InEventName			The event name in this component
	// @param	InTickFunction		The other component we want to remove a prerequisite from
	// @param	InEventName			The other component's event name
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API void RemoveModuleEventPrerequisite(FName InEventName, UUAFComponent* OtherAnimNextComponent, FName OtherEventName);

	// Remove a subsequent anim next event dependency from the specified event
	// @param	InEventName			The event name in this component
	// @param	InTickFunction		The other component we want to remove a prerequisite to
	// @param	InEventName			The other component's event name
	UFUNCTION(BlueprintCallable, Category = "UAF")
	UAF_API void RemoveModuleEventSubsequent(FName InEventName, UUAFComponent* OtherAnimNextComponent, FName OtherEventName);

	TConstStructView<FUAFSystemFactoryAsset> GetAssetData() const { return AssetData; }

	// can only be called when component is not registered
	UAF_API void SetAsset(TInstancedStruct<FUAFSystemFactoryAsset>&& AssetData);

private:
	UAF_API void SetAssetFromObject(const UObject* InAsset);
	UAF_API void SetAssetInternal(TConstStructView<FUAFSystemFactoryAsset> AssetData);
	UAF_API void SetAssetInternal(TInstancedStruct<FUAFSystemFactoryAsset>&& AssetData);

	UAF_API void RegisterSystem();
	UAF_API void UnregisterSystem();

	UAF_API bool IsModuleValid();

	DECLARE_FUNCTION(execBlueprintSetVariable);
	DECLARE_FUNCTION(execBlueprintSetVariableReference);
	DECLARE_FUNCTION(execBlueprintGetVariableReference);

	// Sets the value of the specified variable
	UAF_API EPropertyBagResult SetVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue);

	// Gets the value of the specified variable, expects OutNewValue to be correctly sized
	UAF_API EPropertyBagResult GetVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8> OutNewValue);
	
	// Accesses value of the specified variable for writing
	UAF_API EPropertyBagResult WriteVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>)> InFunction);

	// Accesses value of the specified variable for reading
	UAF_API EPropertyBagResult ReadVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TConstArrayView<uint8>)> InFunction);

	// Determine what skeletal mesh component to write to
	USkeletalMeshComponent* DetermineSkeletalMeshComponent() const;

	// Called on registration, sets up IO
	void SetupInputOutput();

	// Called on registration, or lazily on dependent registration, as component registration order is not guaranteed 
	void AllocateSystem();
	
	// If applicable, sets up a tick order dependency between this UAF System and the Character Movement Component
	// Only set up for characters and if this component targets the character mesh and bSetupCharacterMovementDependency is set
	void SetupCharacterMovementDependency();
	// Removes the tick dependency between the system this component runs and the character movement component, if it was previously set
	void ReleaseCharacterMovementDependency();

private:
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend struct FAssetBlueprintTests;
	friend class UE::UAF::Editor::FAssetWizard;
	// TODO ideally there would be a public API for the things FAnimMixerUtils does, so it wouldn't be necessary.
	friend struct FAnimMixerUtils;
	friend class UE::UAF::Editor::FSystemViewportController;
	friend class UE::UAF::Editor::FAnimGraphViewportController;
	friend class UE::UAF::Editor::SUAFGraphAssetPreview;
	friend UE::UAF::Tests::FUAFSystemTests;

	// The asset that this component will run
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "UAnimNextComponent::Asset is replaced by AssetData")
	UPROPERTY()
	TObjectPtr<UObject> Asset_DEPRECATED = nullptr;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category="Animation", meta=(ExcludeBaseStruct))
	TInstancedStruct<FUAFSystemFactoryAsset> AssetData;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UUAFSystem> Module_DEPRECATED = nullptr;
#endif

	// The inputs to this UAF system. These map the outputs of other UAF components to the specified input values of this component's current system.
	// ~ Note that these are EditDefaultsOnly to work around limitations with array editing for component instances. If we want to make these
	// ~ per-instance-editable, then we may need a custom handler for component instance data to support that.
	UPROPERTY(EditDefaultsOnly, Category="Input")
	TArray<FUAFComponentInputDesc> Inputs;

	// The skeletal mesh component that this UAF system will animate against.
	// If this is not set, then the first component that is found on the actor will be used.
	// Note that setting this does not necessarily mean that the mesh's pose will be written by the UAF system, only that the output value will match
	// the skeletal mesh component's 'shape'.
	// ~ Note that this is EditDefaultsOnly to complement Inputs.
	UPROPERTY(EditDefaultsOnly, Category="Output", meta = (UseComponentPicker, AllowedClasses = "/Script/Engine.SkeletalMeshComponent"))
	FComponentReference OutputComponent;

	// The operation to perform when outputting values with respect to a skeletal mesh component
	UPROPERTY(EditDefaultsOnly, Category="Output")
	EUAFSkeletalMeshComponentOutputMode SkeletalMeshComponentOutputMode = EUAFSkeletalMeshComponentOutputMode::WriteToSkeletalMeshComponentPose;

	UPROPERTY()
	TObjectPtr<USkeletalMeshComponent> CachedSkeletalMeshComponent;

	// Owning reference to the registered system
	UE::UAF::FSystemReference SystemReference;
	
	// Cached mapping between publicly exposed FValueBundle variables and their currently bound actor-components
	UPROPERTY(transient)
	TMap<FAnimNextVariableReference, TWeakObjectPtr<UActorComponent>> InputToComponentMappings;

	// Serial number, updated on worker threads when output is written
	uint32 SerialNumber = 0;

	// How to initialize the system
	UPROPERTY(EditAnywhere, Category="Animation")
	EAnimNextModuleInitMethod InitMethod = EAnimNextModuleInitMethod::InitializeAndPauseInEditor;

	/** When checked, the system's debug drawing instructions are drawn in the viewport */
	UPROPERTY(EditAnywhere, Category = "Animation")
	uint8 bShowDebugDrawing : 1 = false;
	
	// When set, this component automatically adds a tick prerequisite to ensure the owning character's movement component ticks before this system.
	// Only takes effect if the owner is or inherits from ACharacter and this component outputs to its Mesh component.
	// This mimics the default behavior on a character when animation blueprints are used. */
	UPROPERTY(EditAnywhere, Category = "Animation")
	uint8 bSetupCharacterMovementDependency : 1 = true;
	uint8 bCharacterMovementDependencyHasBeenSet : 1 = false;
};
