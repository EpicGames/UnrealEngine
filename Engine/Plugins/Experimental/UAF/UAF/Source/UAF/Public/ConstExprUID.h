// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Hash/Fnv.h"

namespace UE::UAF
{
	// Type alias for a raw UID, not typesafe
	using FFConstExprUIDRaw = uint32;

	/**
	 * A constexpr safe UID
	 *
	 * Encapsulates an constexpr UID.
	 * The string is exposed in non-shipping builds for logging and debugging purposes.
	 * The UID should be generated from the provided string using FNV1a with 32 bits.
	 *
	 * The whole struct is meant to be 'constexpr' to allow inlining.
	 */
	struct FConstExprUID final
	{
		// Constructs an invalid UID
		constexpr FConstExprUID() noexcept
			: UID(INVALID_UID)
#if !UE_BUILD_SHIPPING
			, DebugName(TEXT("<Invalid ConstExprUID UID>"))
#endif
		{
		}

		// Constructs a UID
		[[nodiscard]] explicit constexpr FConstExprUID(FFConstExprUIDRaw InUID, const TCHAR* InDebugName = TEXT("<Unknown ConstExprUID Name>")) noexcept
			: UID(InUID)
#if !UE_BUILD_SHIPPING
			, DebugName(InDebugName)
#endif
		{
		}

		// Constructs a UID
		[[nodiscard]] static constexpr FConstExprUID MakeFromString(const TCHAR* InName) noexcept
		{
			return FConstExprUID(UE::HashStringFNV1a32(MakeStringView(InName)), InName);
		}

#if !UE_BUILD_SHIPPING
		// Returns a literal string to the debug name
		[[nodiscard]] constexpr const TCHAR* GetDebugName() const noexcept { return DebugName; }
#endif

		// Returns the raw UID
		[[nodiscard]] constexpr FFConstExprUIDRaw GetUID() const noexcept { return UID; }

		// Returns whether this UID is valid or not
		[[nodiscard]] constexpr bool IsValid() const noexcept { return UID != INVALID_UID; }

		// Returns the hash of this UID
		[[nodiscard]] constexpr friend uint32 GetTypeHash(FConstExprUID InUID)
		{
			// Return the UID as-is since it is already a hashed value
			return InUID.UID;
		}

	private:
		static constexpr FFConstExprUIDRaw INVALID_UID = 0;

		FFConstExprUIDRaw	UID;

#if !UE_BUILD_SHIPPING
		const TCHAR*		DebugName;
#endif
	};

	// Compares for equality and inequality
	constexpr bool operator==(FConstExprUID LHS, FConstExprUID RHS) noexcept { return LHS.GetUID() == RHS.GetUID(); }
	constexpr bool operator!=(FConstExprUID LHS, FConstExprUID RHS) noexcept { return LHS.GetUID() != RHS.GetUID(); }
	constexpr bool operator==(FConstExprUID LHS, FFConstExprUIDRaw RHS) noexcept { return LHS.GetUID() == RHS; }
	constexpr bool operator!=(FConstExprUID LHS, FFConstExprUIDRaw RHS) noexcept { return LHS.GetUID() != RHS; }
	constexpr bool operator==(FFConstExprUIDRaw LHS, FConstExprUID RHS) noexcept { return LHS == RHS.GetUID(); }
	constexpr bool operator!=(FFConstExprUIDRaw LHS, FConstExprUID RHS) noexcept { return LHS != RHS.GetUID(); }

	// For sorting
	constexpr bool operator<(FConstExprUID LHS, FConstExprUID RHS) { return LHS.GetUID() < RHS.GetUID(); }
}
