// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataRegistryTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "Containers/Ticker.h"

#define UE_API UAF_API

class USkeletalMeshComponent;
struct FReferenceSkeleton;

namespace UE::UAF
{

class FAnimNextModuleImpl;

struct FDataHandle;

// Global registry of animation data
// Holds ref counted data that gets released when the last DataHandle of that element goes out of scope
// Calling public functions from multiple threads is expected. Data races are guarded by a FRWLock.
// TODO : Memory management will have to be implemented to avoid fragmentation and performance reasons
class FDataRegistry
{
public:

	// Access the global registry
	static UE_API FDataRegistry* Get();


	// --- Reference Pose Handling ---

	// Generates and registers a reference pose for the SkeletalMesh asset of the SkeletalMeshComponent
	// and modifies it with the additional required bones 
	// or the visibility state of the bones of the SkeletalMeshComponent
	UE_API FDataHandle RegisterReferencePose(USkeletalMeshComponent* SkeletalMeshComponent);

	// Returns a ref counted handle to the refence pose of the given skeletal mesh component
	UE_API FDataHandle GetOrGenerateReferencePose(USkeletalMeshComponent* SkeletalMeshComponent);

	// Removes a previously registered reference pose for the given SkeletalMeshComponent
	UE_API void RemoveReferencePose(USkeletalMeshComponent* SkeletalMeshComponent);
	
	// Generates and registers a reference pose for a SkeletalMesh asset
	UE_API FDataHandle RegisterReferencePose(USkeletalMesh* SkeletalMesh);

	// Returns a ref counted handle to the reference pose of the given skeletal mesh
	UE_API FDataHandle GetOrGenerateReferencePose(USkeletalMesh* SkeletalMesh);

	// Removes a previously registered reference pose for the given SkeletalMesh
	UE_API void RemoveReferencePose(USkeletalMesh* SkeletalMesh);

	// --- AnimationData Storage / Retrieval  --- 

	// Registers an anim data handle with arbitrary data using an FName
	// Note that AnimDataHandles are refcounted, so this makes them permanent until unregistered
	UE_API void RegisterData(const FName& Id, const FDataHandle& AnimationDataHandle);

	// Unregisters a previously registered anim data handle 
	UE_API void UnregisterData(const FName& Id);

	// Obtains the data hanle for the passed Id, if it exists.
	// If there is no anim data handle registered, the handle IsValid will be false
	UE_API FDataHandle GetRegisteredData(const FName& Id) const;

	
	// --- Supported types registration ---

	// Registers a type and sets the desired preallocation block size
	// If a type is allocated without registering, a default block size of 32 will be used
	template<typename DataType>
	inline void RegisterDataType(int32 AllocationBlockSize)
	{
		RegisterDataType_Impl<DataType>(AllocationBlockSize);
	}

	// --- Persistent Data ---

	// Allocates uninitialized memory for a type (leaving the initialization to the caller)
	// Returns a refcounted animation data handle
	// Allocated memory will be released once the refcount reaches 0
	template<typename DataType>
	FDataHandle PreAllocateMemory(const int32 NumElements)
	{
		FAnimNextParamType ParamType = FAnimNextParamType::GetType<DataType>();

		bool bIsTypeDefValid = false;
		FDataTypeDef TypeDef;
		{
			FRWScopeLock Lock(DataTypeDefsLock, SLT_ReadOnly);
			if (const FDataTypeDef* TypeDefPtr = DataTypeDefs.Find(ParamType))
			{
				bIsTypeDefValid = true;
				TypeDef = *TypeDefPtr;
			}
		}

		if (!bIsTypeDefValid)
		{
			TypeDef = RegisterDataType_Impl<DataType>(DEFAULT_BLOCK_SIZE);
			// TODO : Log if we allocate more than DEFAULT_BLOCK_SIZE elements of that type
		}

		if (ensure(TypeDef.ParamType.IsValid()))
		{
			const int32 ElementSize = TypeDef.ElementSize;
			const int32 ElementAlign = TypeDef.ElementAlign;
			const int32 AlignedSize = Align(ElementSize, ElementAlign);

			const int32 BufferSize = NumElements * AlignedSize;

			uint8* Memory = (uint8*)FMemory::Malloc(BufferSize, TypeDef.ElementAlign);    // TODO : This should come from preallocated chunks, use malloc / free for now

			Private::FAllocatedBlock* AllocatedBlock = new Private::FAllocatedBlock(Memory, NumElements, ParamType); // TODO : avoid memory fragmentation
			AllocatedBlock->AddRef();

			FRWScopeLock Lock(AllocatedBlocksLock, SLT_Write);
			AllocatedBlocks.Add(AllocatedBlock);
			return FDataHandle(AllocatedBlock);
		}

		return FDataHandle();
	}

	// Allocates memory for a type, initialized with the default constructor (with optional passed arguments)
	// Returns a refcounted animation data handle
	// Allocated memory will be released once the refcount reaches 0
	template<typename DataType, typename... ArgTypes>
	FDataHandle AllocateData(const int32 NumElements, ArgTypes&&... Args)
	{
		FDataHandle Handle = PreAllocateMemory<DataType>(NumElements);

		DataType* RetVal = Handle.GetPtr<DataType>();
		for (int i = 0; i < NumElements; i++)
		{
			// perform a placement new per element
			new ((uint8*)&RetVal[i]) DataType(Forward<ArgTypes>(Args)...);
		}

		return Handle;
	}

private:
	typedef void (*DestroyFnSignature)(uint8* TargetBuffer, int32 NumElem);

	static constexpr int32 DEFAULT_BLOCK_SIZE = 32;

	// structure holding each registered type information
	struct FDataTypeDef
	{
		FAnimNextParamType ParamType;
		DestroyFnSignature DestroyTypeFn = nullptr;
		int32 ElementSize = 0;
		int32 ElementAlign = 0;
		int32 AllocationBlockSize = 0;
	};

	struct FReferencePoseData
	{
		FReferencePoseData() = default;
		
		FReferencePoseData(const FDataHandle& InAnimationDataHandle, const FDelegateHandle& InDelegateHandle)
			: AnimationDataHandle(InAnimationDataHandle)
			, DelegateHandle(InDelegateHandle)
		{
		}

		FReferencePoseData(FDataHandle&& InAnimationDataHandle, FDelegateHandle&& InDelegateHandle)
			: AnimationDataHandle(MoveTemp(InAnimationDataHandle))
			, DelegateHandle(MoveTemp(InDelegateHandle))
		{
		}

		FDataHandle AnimationDataHandle;
		FDelegateHandle DelegateHandle;
	};

	// Map holding registered types
	TMap<FAnimNextParamType, FDataTypeDef> DataTypeDefs;
	// Lock for registered types map
	FRWLock DataTypeDefsLock;

	TSet<Private::FAllocatedBlock*> AllocatedBlocks;
	// Lock for allocated blocks
	FRWLock AllocatedBlocksLock;

	// Map holding named data
	TMap <FName, FDataHandle> StoredData;
	mutable FRWLock StoredDataLock;

	// Map holding reference poses for Skeletal Mesh Components
	TMap <TObjectKey<USkeletalMeshComponent>, FReferencePoseData> SkeletalMeshComponentReferencePoses;
	mutable FRWLock SkeletalMeshComponentReferencePosesLock;

	TMap <TObjectKey<USkeletalMesh>, FReferencePoseData> SkeletalMeshReferencePoses;
	mutable FRWLock SkeletalMeshReferencePosesLock;

	// A handle to our ticker when GC runs and we need to clean things up
	FTSTicker::FDelegateHandle GCCleanupTickerHandle;

	// Array of keys to clean up post-GC
	TArray<TObjectKey<USkeletalMeshComponent>> GCComponentKeysToCleanup;
	TArray<TObjectKey<USkeletalMesh>> GCMeshKeysToCleanup;

	// The current index of GC keys to process
	int32 GCComponentKeysProcessedIndex = 0;
	int32 GCMeshKeysProcessedIndex = 0;

	// Registers a type and sets the allocation block size
	template<typename DataType>
	FDataTypeDef RegisterDataType_Impl(int32 AllocationBlockSize)
	{
		FAnimNextParamType ParamType = FAnimNextParamType::GetType<DataType>();
		check(ParamType.IsValid());

		const int32 ElementSize = ParamType.GetSize();
		const int32 ElementAlign = ParamType.GetAlignment();

		// If we use raw types, I need a per element destructor
		DestroyFnSignature DestroyFn = [](uint8* TargetBuffer, int32 NumElem)->void
		{
			const DataType* Ptr = (DataType*)TargetBuffer;
			for (int i = 0; i < NumElem; i++)
			{
				Ptr[i].~DataType();
			}
		};

		{
			FRWScopeLock WriteLock(DataTypeDefsLock, SLT_Write);

			FDataTypeDef* AddedDef = &DataTypeDefs.FindOrAdd(ParamType, { ParamType, DestroyFn, ElementSize, ElementAlign, AllocationBlockSize });
			check(AddedDef->ParamType == ParamType); // check we have not added two different types with the same ID

			return *AddedDef;
		}
	}

	UE_API void OnLODRequiredBonesUpdate(USkeletalMeshComponent* SkeletalMeshComponent, int32 LODLevel, const TArray<FBoneIndexType>& LODRequiredBones);
	
	UE_API void FreeAllocatedBlock(Private::FAllocatedBlock * AllocatedBlock);

	UE_API void ReleaseReferencePoseData();

	static bool GCCleanupTicker(float DeltaTime);
	bool RunGCCleanup(double TimeBudget);

// --- ---
private:
	friend class FAnimNextModuleImpl;
	friend struct FDataHandle;

	// Initialize the global registry
	static UE_API void Init();

	// Shutdown the global registry
	static UE_API void Destroy();

	// Called before engine shutdown to clear internal state while UObjects are still valid
	static void OnPreExit();

	volatile int32 HandleCounter = 0;


private:
	static UE_API void HandlePostGarbageCollect();
};

} // namespace UE::UAF

#undef UE_API
