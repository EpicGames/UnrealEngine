// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "Param/ParamType.h"
#include "Animation/AnimTypes.h"
#include "ReferenceSkeleton.h"
#include "BoneIndices.h"

#define UE_API UAF_API

namespace UE::UAF
{

namespace Private
{

struct FAllocatedBlock
{
	void* Memory = nullptr;
	int32 NumElem = 0;
	mutable int32 NumRefs = 0;
	FAnimNextParamType Type;

	FAllocatedBlock(void* InMemory, int32 InNumElem, FAnimNextParamType InType)
		: Memory(InMemory)
		, NumElem(InNumElem)
		, Type(InType)
	{
	}

	inline uint32 AddRef() const
	{
		return uint32(FPlatformAtomics::InterlockedIncrement(&NumRefs));
	}

	inline uint32 Release() const
	{
		check(NumRefs > 0);

		const int32 Refs = FPlatformAtomics::InterlockedDecrement(&NumRefs);
		check(Refs >= 0);

		return uint32(Refs);
	}

	inline uint32 GetRefCount() const
	{
		return uint32(NumRefs);
	}

private:
	FAllocatedBlock() = delete;
	FAllocatedBlock(const FAllocatedBlock& Other) = delete;
	FAllocatedBlock(FAllocatedBlock&& Other) = delete;
};

} // end namespace Private

struct FDataHandle
{
	FDataHandle() = default;

	FDataHandle(Private::FAllocatedBlock* InAllocatedBlock)
		: AllocatedBlock(InAllocatedBlock)
	{
	}

	UE_API ~FDataHandle();

	FDataHandle(const FDataHandle& Other)
		: AllocatedBlock(Other.AllocatedBlock)
	{
		if (AllocatedBlock != nullptr)
		{
			const int32 CurrentCount = AllocatedBlock->AddRef();
			check(CurrentCount > 1);
		}
	}

	FDataHandle& operator= (const FDataHandle& Other)
	{
		FDataHandle Tmp(Other);

		Swap(*this, Tmp);
		return *this;
	}

	FDataHandle(FDataHandle&& Other)
		: FDataHandle()
	{
		Swap(*this, Other);
	}

	FDataHandle& operator= (FDataHandle&& Other)
	{
		Swap(*this, Other);
		return *this;
	}

	inline bool IsValid() const
	{
		return AllocatedBlock != nullptr;
	}

	template<typename DataType>
	inline TArrayView<DataType> AsArrayView()
	{
		check(IsValid());
		return TArrayView<DataType>((DataType*)AllocatedBlock->Memory, AllocatedBlock->NumElem);
	}

	template<typename DataType>
	inline TArrayView<DataType> AsArrayView() const
	{
		check(IsValid());
		return TArrayView<DataType>(AllocatedBlock->Memory, AllocatedBlock->NumElem);
	}

	template<typename DataType>
	inline DataType* GetPtr()
	{
		check(IsValid());
		DataType* Data = static_cast<DataType*>(AllocatedBlock->Memory);
		check(Data != nullptr);
		return Data;
	}

	template<typename DataType>
	inline const DataType* GetPtr() const
	{
		check(IsValid());
		const DataType* Data = static_cast<const DataType*>(AllocatedBlock->Memory);
		return Data;
	}

	template<typename DataType>
	inline DataType& GetRef()
	{
		check(IsValid());
		DataType* Data = static_cast<DataType*>(AllocatedBlock->Memory);
		check(Data != nullptr);
		return *Data;
	}

	template<typename DataType>
	inline const DataType& GetRef() const
	{
		check(IsValid());
		const DataType* Data = static_cast<const DataType*>(AllocatedBlock->Memory);
		check(Data != nullptr);
		return *Data;
	}

	inline FAnimNextParamType GetType() const
	{
		return AllocatedBlock != nullptr ? AllocatedBlock->Type : FAnimNextParamType();
	}

private:
	Private::FAllocatedBlock* AllocatedBlock = nullptr;
};

enum class ETransformFlags : uint8
{
	None = 0,
	ComponentSpaceSet = 1 << 0
};


} // namespace UE::UAF

#undef UE_API
