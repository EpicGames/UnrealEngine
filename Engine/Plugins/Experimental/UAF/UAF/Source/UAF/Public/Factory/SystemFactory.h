// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Factory/UAFSystemFactoryParams.h"
#include "Templates/SubScriptStructOf.h"

#define UE_API UAF_API

struct FUAFSystemFactoryAsset;
class UUAFSystem;
class UObject;
struct FUAFAssetInstanceComponent;
class IUAFAssetData;

namespace UE::UAF
{
	struct ISystemBuilder;
	struct FSystemFactory;
	class FAnimNextModuleImpl;
}

namespace UE::UAF
{

// Creates or recycles programmatically-generated systems
// Uses the hash of the recipe to determine if the system has been created already.
// If the hash matches one that has already been created but the system has already been GCed it will be re-created as needed.
struct FSystemFactory
{
	// Make a system from the specified recipe
	UE_API static const UUAFSystem* BuildSystem(const ISystemBuilder& InBuilder);

	// Get default params used to build & initialize a system according to registered object class, copying InObject via the registered initializer
	// Effective equivalent of GetDefaultParamsForClass and InitializeDefaultParamsForObject.
	UE_API static FUAFSystemFactoryParams GetDefaultParamsForAsset(TConstStructView<FUAFSystemFactoryAsset> Asset);

	// Function used to initialize the object for the factory-generated system
	using FParamsInitializer = TFunction<void(const TConstStructView<FUAFSystemFactoryAsset>&, FUAFSystemFactoryParams&)>;

	template<typename TAssetData>
	using TParamsInitializer = TFunction<void(const TAssetData&, FUAFSystemFactoryParams&)>;

	UE_API static void RegisterAsset(const TSubScriptStructOf<FUAFSystemFactoryAsset>& AssetStructType, FUAFSystemFactoryParams&& InParams, FParamsInitializer&& InInitializer);

	template<typename TAssetData>
	static void RegisterAsset(FUAFSystemFactoryParams&& InParams, TParamsInitializer<TAssetData>&& InInitializer = nullptr)
	{
		if (InInitializer)
		{
			RegisterAsset(TAssetData::StaticStruct(), MoveTemp(InParams), [Initializer = MoveTemp(InInitializer)](const TConstStructView<FUAFSystemFactoryAsset>& AssetData, FUAFSystemFactoryParams& InOutParams)
			{
				Initializer(*AssetData.GetPtr<TAssetData>(), InOutParams);
			});
		}
		else
		{
			RegisterAsset(TAssetData::StaticStruct(), MoveTemp(InParams), nullptr);
		}
	}

	// Get all registered classes used to generate factory systems
	UE_API static TConstArrayView<TSubScriptStructOf<FUAFSystemFactoryAsset>> GetRegisteredAssetDataTypes();

private:
	friend FAnimNextModuleImpl;

	// Called on module init
	UE_API static void Init();

	// Called on module shutdown to unload everything
	UE_API static void Destroy();

	// Called before engine shutdown to clear internal state while UObjects are still valid
	static void OnPreExit();
};

}

#undef UE_API