// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFSystemBuilder.h"
#include "StructUtils/StructView.h"
#include "UAFSimpleSystemBuilder.generated.h"

#define UE_API UAF_API

class UUAFRigVMAsset;
class IRigVMRuntimeAssetInterface;
struct FUAFSimpleSystemBuilder;
struct FUAFSystemFactoryParams;
struct FUAFAssetInstanceComponent;

namespace UE::UAF
{
	struct FSystemFactory;
};

USTRUCT()
struct FUAFSimpleSystemBuilder : public FUAFSystemBuilder
{
	GENERATED_BODY()

	// Add a variables struct to this builder
	UE_API void AddVariablesInstancedStruct(FInstancedStruct&& InInstancedStruct);
	UE_API void AddVariablesStructView(FConstStructView InStructView);

	// Add a struct by type, initializing to default
	UE_API void AddVariablesStruct(const TObjectPtr<const UScriptStruct>& InScriptStruct);

	UE_API void AddVariablesRigVMAsset(const TScriptInterface<const IRigVMRuntimeAssetInterface>& RigVMAssetInterface);
	UE_API void AddVariablesAsset(const TObjectPtr<const UUAFRigVMAsset>& UAFAsset);

	// Add a component struct to this builder
	UE_API void AddComponentInstancedStruct(TInstancedStruct<FUAFAssetInstanceComponent>&& InInstancedStruct);
	UE_API void AddComponentStructView(TConstStructView<FUAFAssetInstanceComponent> InStructView);

	// Check whether this method is empty or not
	bool IsValid() const
	{
		return ComponentStructs.Num() > 0;
	}

	// Reset back to empty state
	void Reset()
	{
		ComponentStructs.Empty();
		VariablesStructs.Empty();
		ReferencedVariableRigVMAssets.Empty();
		ReferencedVariableAssets.Empty();
	}

private:
	friend UE::UAF::FSystemFactory;
	friend FUAFSystemFactoryParams;

	// FUAFSystemBuilder interface
	UE_API virtual bool Build(UE::UAF::FSystemBuilderContext& InContext) const override;
	UE_API virtual uint64 RecalculateKey() const override;

	// The structs we used to generate the system's components
	UPROPERTY()
	TArray<TInstancedStruct<FUAFAssetInstanceComponent>> ComponentStructs;

	// The structs we used to generate the system's variables
	UPROPERTY()
	TArray<FInstancedStruct> VariablesStructs;

	// All the IRigVMRuntimeAssetInterface whose variables we will reference/instance at runtime
	UPROPERTY()
	TArray<TScriptInterface<const IRigVMRuntimeAssetInterface>> ReferencedVariableRigVMAssets;

	// All the assets whose variables we will reference/instance at runtime
	UPROPERTY()
	TArray<TObjectPtr<const UUAFRigVMAsset>> ReferencedVariableAssets;
};

#undef UE_API