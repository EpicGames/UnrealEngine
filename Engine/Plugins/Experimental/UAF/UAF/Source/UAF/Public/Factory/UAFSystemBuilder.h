// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UAFSystemBuilder.generated.h"

#define UE_API UAF_API

namespace UE::UAF
{
	struct FSystemFactory;
	struct IBuilder;
	struct FSystemBuilderContext;
}

namespace UE::UAF
{

// A method for making a system in FSystemFactory procedurally
struct ISystemBuilder
{
	virtual ~ISystemBuilder() = default;

protected:
	static constexpr uint64 InvalidKey = 0;

private:
	friend FSystemFactory;

	// Primary entry point for factory method for procedurally generating systems
	// Fills the supplied context used to generate the system
	// @return true if the system factory context was written successfully, false otherwise
	virtual bool Build(FSystemBuilderContext& InContext) const { return false; };

	// Get a uniquely-identifying key that we use to avoid rebuilding systems
	UE_API virtual uint64 GetKey() const;
};

}

// Base struct for procedural system methods that need to persist or be referenced in data
USTRUCT()
struct FUAFSystemBuilder
#if CPP
	: public UE::UAF::ISystemBuilder
#endif
{
	GENERATED_BODY()

protected:
	// Invalidate the key if the method's params change, it will be lazily recalculated
	void InvalidateKey()
	{
		CachedKey = UE::UAF::ISystemBuilder::InvalidKey;
	}
	
	// Recalculates the key - called lazily when the key is retrieved
	virtual uint64 RecalculateKey() const { return UE::UAF::ISystemBuilder::InvalidKey; }

	// ISystemBuilder interface
	UE_API virtual uint64 GetKey() const override;

private:
	// Cached key based on serialized properties
	mutable uint64 CachedKey = UE::UAF::ISystemBuilder::InvalidKey;
};

#undef UE_API