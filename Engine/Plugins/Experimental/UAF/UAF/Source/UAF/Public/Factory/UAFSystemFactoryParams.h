// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFSystemBuilder.h"
#include "InstanceTask.h"
#include "InstanceTaskContext.h"
#include "UAFAssetInstanceComponent.h"
#include "UAFSimpleSystemBuilder.h"
#include "UAFSystemFactoryParams.generated.h"

#define UE_API UAF_API

struct FAnimNextModuleInstance;

namespace UE::UAF
{
	struct FSystemFactory;
}

// Set of parameters that describe a way of generating & initializing a UAF system
USTRUCT(BlueprintType)
struct FUAFSystemFactoryParams
{
	GENERATED_BODY()

	FUAFSystemFactoryParams() = default;

	// Whether these params are valid for application
	bool IsValid() const
	{
		return Builder.IsValid();
	}

	// Access a component struct during initialization
	template<typename StructType>
	bool AccessComponentStruct(TFunctionRef<void(StructType&)> InFunction)
	{
		for (TInstancedStruct<FUAFAssetInstanceComponent>& StructData : Builder.ComponentStructs)
		{
			if (StructData.GetScriptStruct() == TBaseStructure<StructType>::Get())
			{
				InFunction(StructData.GetMutable<StructType>());
				return true;
			}
		}
		return false;
	}

	// Access a variables struct during initialization
	template<typename StructType>
	bool AccessVariablesStruct(TFunctionRef<void(StructType&)> InFunction)
	{
		for (FInstancedStruct& StructData : Builder.VariablesStructs)
		{
			if (StructData.GetScriptStruct() == TBaseStructure<StructType>::Get())
			{
				InFunction(StructData.GetMutable<StructType>());
				return true;
			}
		}
		return false;
	}

	// Add a public variables struct to these params. This will act as part of the public API of the system.
	// If a struct already exists of this type then its values be overriden by virtue of the initializer running later
	template<typename StructType>
	void AddPublicVariablesStruct()
	{
		FInstancedStruct InstancedStruct(TBaseStructure<StructType>::Get());
		Builder.AddVariablesInstancedStruct(MoveTemp(InstancedStruct));
	}

	// Add a public variables struct to these params. This will act as part of the public API of the system.
	// If a struct already exists of this type then its values be overriden by virtue of the initializer running later
	template<typename StructType>
	TStructView<StructType> AddPublicVariablesStruct(const StructType& InStruct)
	{
		return Builder.AddVariablesStructView(InStruct);
	}

	void AddPublicVariablesStructByType(const TObjectPtr<const UScriptStruct>& InStruct)
	{
		Builder.AddVariablesStruct(InStruct);
	} 

	// Add a public variables rig VM asset to these params. This will act as part of the public API of the system.
	// If a asset reference already exists of this type then its values be overriden by virtue of the initializer running later
	void AddPublicVariablesRigVMAsset(const TScriptInterface<const IRigVMRuntimeAssetInterface>& RigVMAsset)
	{
		Builder.AddVariablesRigVMAsset(RigVMAsset);
	}

	// Add a public shared variables asset to these params. This will act as part of the public API of the system.
	// If a asset reference already exists of this type then its values be overriden by virtue of the initializer running later
	void AddPublicVariablesAsset(const TObjectPtr<const UUAFRigVMAsset>& UAFAsset)
	{
		Builder.AddVariablesAsset(UAFAsset);
	}

	template<typename ComponentType>
	void AddComponent()
	{
		static_assert(std::is_base_of_v<FUAFAssetInstanceComponent,  ComponentType>, "Components must derive from FUAFAssetInstanceComponent");
		ComponentType Component;
		Builder.AddComponentStructView(TConstStructView<FUAFAssetInstanceComponent>(Component));
	}

	template<typename ComponentType>
	void AddComponent(const ComponentType& InStruct)
	{
		static_assert(std::is_base_of_v<FUAFAssetInstanceComponent,  ComponentType>, "Components must derive from FUAFAssetInstanceComponent");
		Builder.AddComponentStructView(InStruct);
	}

	// Get the builder that these params hold
	const UE::UAF::ISystemBuilder& GetBuilder() const
	{
		return Builder;
	}

	// Add an initialize task used to set up an instance when these params are applied
	FUAFSystemFactoryParams& AddInitializeTask(UE::UAF::FInstanceTask&& InTask)
	{
		InitializeTasks.Add(MoveTemp(InTask));
		return *this;
	}

	// Clear these parameters back to empty
	UE_API void Reset();

private:
	friend UE::UAF::FSystemFactory;
	friend FAnimNextModuleInstance;

	// Apply these params to the supplied instance.
	UE_API void InitializeInstance(FUAFAssetInstance& InInstance) const;

	// Method used to create a system
	UPROPERTY(EditAnywhere, Category = Parameters)
	FUAFSimpleSystemBuilder Builder;

	// Tasks to run on the factory-generated system instance via InitializeInstance, used for initial setup of variables etc.
	TArray<UE::UAF::FInstanceTask> InitializeTasks;
};

#undef UE_API
