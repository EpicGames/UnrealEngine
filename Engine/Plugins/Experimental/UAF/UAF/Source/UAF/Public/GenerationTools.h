// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneIndices.h"
#include "Animation/AttributesRuntime.h"
#include "Containers/IndirectArray.h"
#include "ReferencePose.h"
#include "LODPose.h"

#define UE_API UAF_API

class USkeletalMeshComponent;
class USkeletalMesh;
class FSkeletalMeshLODRenderData;
struct FPoseContext;
class UPhysicsAsset;

namespace UE::UAF
{

struct FGenerationLODData
{
	TArray<FBoneIndexType> RequiredBones;									// All the bones required for the LOD
	TArray<FBoneIndexType> ExcludedBones;									// List of bones excluded from LOD 0
	TArray<FBoneIndexType> ExcludedBonesFromPrevLOD;						// list of bones excluded from previous LOD
};

class FGenerationTools
{
public:

	// Generates the reference pose data from a SkeletalMeshComponent and / or Skeletal Mesh asset
	// If no SkeletalMeshComponent is passed, the reference pose will not exclude invisible bones and will not include shadow shapes required bones
	// If no SkeletalMesh asset is passed, there will be no generation
	static UE_API bool GenerateReferencePose(const USkeletalMeshComponent* SkeletalMeshComponent
		, USkeletalMesh* SkeletalMesh
		, UE::UAF::FReferencePose& OutAnimationReferencePose);

	// Generates the full list of bones required by a LOD, based on the SkeletalRetrieve RequiredBones
	// Compute ExcludedBones versus LOD0 and PreviousLOD
	// Remove ExcludedBonesFromPrevLOD from BonesInAllLODS
	static UE_API void GenerateRawLODData(const USkeletalMeshComponent* SkeletalMeshComponent
		, const USkeletalMesh* SkeletalMesh
		, const int32 LODIndex
		, const TIndirectArray<FSkeletalMeshLODRenderData>& LODRenderData
		, TArray<FBoneIndexType>& OutRequiredBones
		, TArray<FBoneIndexType>& OutFillComponentSpaceTransformsRequiredBones);

	// For each LOD > 0:
	// Retrieve RequiredBones
	// Compute ExcludedBones versus LOD0 and PreviousLOD
	static UE_API void GenerateLODData(const USkeletalMeshComponent* SkeletalMeshComponent
		, const USkeletalMesh* SkeletalMesh
		, const int32 StartLOD
		, const int32 NumLODs
		, const TIndirectArray<FSkeletalMeshLODRenderData>& LODRenderData
		, const TArray<FBoneIndexType>& RequiredBones_LOD0
		, TArray<FGenerationLODData>& GenerationLODData
		, TArray<FGenerationLODData>& GenerationComponentSpaceLODData);

	// Calculate the bone indexes difference from LOD0 for LODIndex
	static UE_API void CalculateDifferenceFromParentLOD(int32 LODIndex, TArray<FGenerationLODData>& GenerationLODData);

	// Check the required bones in LOD(N) are required in LOD(N-1) 
	// and add missing bones at LOD(N-1), to enable fast path on malformed LODs
	static UE_API void FixLODRequiredBones(const int32 NumLODs
		, const USkeletalMesh* SkeletalMesh
		, TArray<FGenerationLODData>& GenerationLODData
		, TArray<FGenerationLODData>& GenerationComponentSpaceLODData);

	// For each LOD :
	// Check the excluded bones in LOD(N) contain all the bones excluded in LOD(N-1)
	static UE_API bool CheckExcludedBones(const int32 NumLODs
		, const TArray<FGenerationLODData>& GenerationLODData
		, const USkeletalMesh* SkeletalMesh);

	// For each LOD :
	// Generates an unified list of bones, in LOD order (if possible)
	// Returns true if the unified list could be created, false otherwise
	static UE_API bool GenerateOrderedBoneList(const USkeletalMesh* SkeletalMesh
		, TArray<FGenerationLODData>& GenerationLODData
		, TArray<FBoneIndexType>& OrderedBoneList);

	/**
	 *	Utility for taking two arrays of bone indices, which must be strictly increasing, and finding the A - B.
	 *	That is - any items left in A, after removing B.
	 */
	static UE_API void DifferenceBoneIndexArrays(const TArray<FBoneIndexType>& A, const TArray<FBoneIndexType>& B, TArray<FBoneIndexType>& Output);

	// Checks if all sockets of a skeletal mesh are set to always animate, as it is a requirement for generating a single reference pose,
	// where the local space pose and the component space pose use the same bone indexes
	static UE_API bool CheckSkeletalAllMeshSocketsAlwaysAnimate(const USkeletalMesh* SkeletalMesh);

	// Converts AnimBP pose to AnimNext Pose
	// This function expects both poses to have the same LOD (number of bones and indexes)
	// The target pose should be assigned to the correct reference pose prior to this call
	static UE_API void RemapPose(const FPoseContext& SourcePose, FLODPose& TargetPose);

	// Converts AnimNext pose to AnimBP Pose
	// This function expects both poses to have the same LOD (number of bones and indexes)
	// The target pose should be assigned to the correct reference pose prior to this call
	static UE_API void RemapPose(const FLODPose& SourcePose, FPoseContext& TargetPose);

	// Converts AnimNext pose to local space transform array
	// This function expects the output pose to have the same or a greater number of bones (as it may be being calculated
	// for a lower LOD)
	// The target pose should be assigned to the correct reference pose prior to this call, as transforms will not be filled
	// in by this call if they are not affected by the current LOD.
	static UE_API void RemapPose(const FLODPose& SourcePose, TArrayView<FTransform> TargetTransforms);

	// Converts AnimNext attributes to AnimBP attributes
	static UE_API void RemapAttributes(
		const FLODPose& LODPose,
		const UE::Anim::FHeapAttributeContainer& InAttributes,
		UE::Anim::FMeshAttributeContainer& OutAttributes);

	static UE_API void RemapAttributes(
		const FLODPose& LODPose,
		const UE::Anim::FStackAttributeContainer& InAttributes,
		UE::Anim::FMeshAttributeContainer& OutAttributes);

	static UE_API void RemapAttributes(
		const FLODPose& LODPose,
		const UE::Anim::FMeshAttributeContainer& InAttributes,
		UE::Anim::FStackAttributeContainer& OutAttributes);

	// Converts AnimNext attributes to AnimBP attributes
	static UE_API void RemapAttributes(
		const FLODPose& LODPose,
		const UE::Anim::FHeapAttributeContainer& InAttributes,
		FPoseContext& OutPose);

	// Converts AnimNext attributes to AnimBP attributes
	static UE_API void RemapAttributes(
		const FLODPose& LODPose,
		const UE::Anim::FStackAttributeContainer& InAttributes,
		FPoseContext& OutPose);

	// Converts AnimBP attributes to AnimNext attributes
	static UE_API void RemapAttributes(
		const FPoseContext& OutPose,
		const FLODPose& LODPose,
		UE::Anim::FHeapAttributeContainer& OutAttributes
	);

	// Converts AnimBP attributes to AnimNext attributes
	static UE_API void RemapAttributes(
		const FPoseContext& OutPose,
		const FLODPose& LODPose,
		UE::Anim::FStackAttributeContainer& OutAttributes
	);

	// Converts a local space to component space buffer given a number of required bones
	static UE_API void ConvertLocalSpaceToComponentSpace(TConstArrayView<FBoneIndexType> InParentIndices, TConstArrayView<FTransform> InBoneSpaceTransforms, TConstArrayView<FBoneIndexType> InRequiredBoneIndices, TArrayView<FTransform> OutComponentSpaceTransforms);
};

} // namespace UE::UAF

#undef UE_API
