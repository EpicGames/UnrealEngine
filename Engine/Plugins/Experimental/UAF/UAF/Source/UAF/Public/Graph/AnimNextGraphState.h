// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/PropertyBag.h"
#include "AnimNextGraphState.generated.h"

// Default state for a graph entry point
USTRUCT()
struct FAnimNextGraphState
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	void Reset()
	{
		State.Reset();
		PublicParameterStartIndex = INDEX_NONE;
	}
#endif

	// All state, both public and private
	UPROPERTY()
	FInstancedPropertyBag State;

	// Index of the first public parameter, of INDEX_NONE if no public parameter is present
	UPROPERTY()
	int32 PublicParameterStartIndex = INDEX_NONE;
};