// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectPtr.h"
#include "Components/SkeletalMeshComponent.h"

class USkeletalMesh;
class USkeletalMeshComponent;
struct FAnimNextGraphLODPose;
struct FAnimNextGraphReferencePose;
struct FUAFSystemOutputComponent;

namespace UE::UAF
{
struct FModuleTaskContext;
}

namespace UE::UAF
{

// Interface used by any host that wants to read an output from a UAF system
// Register this with FUAFSystemOutputComponent
class ISystemOutputAdapter
{
public:
	virtual ~ISystemOutputAdapter() = default;
	
	// All input data required to generate an output value
	struct FInputData
	{
		FInputData(TNonNullPtr<USkeletalMesh> InSkeletalMesh, int32 InLOD)
			: SkeletalMesh(InSkeletalMesh)
			, LOD(InLOD)
		{}

		FInputData(TNonNullPtr<USkeletalMeshComponent> InSkeletalMeshComponent)
			: SkeletalMesh(InSkeletalMeshComponent->GetSkeletalMeshAsset())
			, SkeletalMeshComponent(InSkeletalMeshComponent)
			, LOD(InSkeletalMeshComponent->GetPredictedLODLevel())
		{}

		// Mesh to use, can be nullptr when using SkeletalMeshComponent (as the component can have a null mesh), but must be valid when not
		USkeletalMesh* SkeletalMesh = nullptr;

		// Legacy mesh component to use, can be nullptr
		USkeletalMeshComponent* SkeletalMeshComponent = nullptr;

		// LOD to run at
		int32 LOD = 0;
	};

private:
	friend struct ::FUAFSystemOutputComponent;

	// Called back on a worker thread by FUAFSystemOutputComponent to retrieve the LOD and mesh to use when system execution begins
	// The mesh & LOD is then consistent across a system's execution
	// Consistency is validated with an ensure() in FUAFSystemOutputComponent prior to calling back via GenerateRenderData
	virtual FInputData GetInputData() const = 0;

	// Called back on a worker thread by FUAFSystemOutputComponent when an output is written
	virtual void SignalOutputWritten(const UE::UAF::FModuleTaskContext& InContext) = 0;
};

}
