// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

namespace UE::UAF
{
	struct FInstanceTaskContext;

	// A task that can be queued on an instance
	using FInstanceTask = TFunction<void(const FInstanceTaskContext&)>;

	// A unique task that can be queued on an instance
	using FUniqueInstanceTask = TUniqueFunction<void(const FInstanceTaskContext&)>;
}
