// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Param/ParamType.h"
#include "Templates/Function.h"

class UUAFSharedVariables;
struct FUAFAssetInstance;
struct FAnimNextVariableReference;
struct FAnimNextFactoryParams;
struct FUAFSystemFactoryParams;
class UInjectionCallbackProxy;
struct FAnimNextGraphInstance;

namespace UE::UAF
{
	struct FInjectionSiteTrait;
}

namespace UE::UAF
{

// Context struct passed to instance tasks, allowing setting of variables etc.
// @see FInjectionRequest::QueueTask
// @see FModuleTaskContext
struct FInstanceTaskContext
{
public:
	// Access a variable's value.
	// Type must match strictly, no conversions are performed.
	// @param	InVariable			The variable to get the value of
	// @param	InFunction			Function that will be called if no errors occur
	// @return true if successful
	template<typename ValueType>
	bool AccessVariable(const FAnimNextVariableReference& InVariable, TFunctionRef<void(ValueType&)> InFunction) const
	{
		return AccessVariableInternal(InVariable, FAnimNextParamType::GetType<ValueType>(), [&InFunction](TArrayView<uint8> InArrayView)
		{
			InFunction(*reinterpret_cast<ValueType*>(InArrayView.GetData()));
		});
	}

	// Set a variable's value.
	// If the type does not match exactly then a conversion will be attempted.
	// @param	InVariable			The variable to set the value of
	// @param	OutResult			Result that will be filled if no errors occur
	// @return see EPropertyBagResult
	template<typename ValueType>
	bool SetVariable(const FAnimNextVariableReference& InVariable, const ValueType& InNewValue) const
	{
		return SetVariableInternal(InVariable, FAnimNextParamType::GetType<ValueType>(), TConstArrayView<uint8>(reinterpret_cast<const uint8*>(&InNewValue), sizeof(ValueType)));
	}

	// Access the memory of the shared variable struct directly.
	// @param	InFunction			Function called with a reference to the variable's struct 
	// @return true if the variables could be accessed. Variables can exist but be unable to be accessed if user overrides are set
	template<typename StructType>
	bool AccessVariablesStruct(TFunctionRef<void(StructType&)> InFunction) const
	{
		return AccessVariablesStructInternal(TBaseStructure<StructType>::Get(), [&InFunction](FStructView InStructView)
		{
			InFunction(InStructView.Get<StructType>());
		});
	}

	// Get the asset instance
	FUAFAssetInstance& GetAssetInstance() const
	{
		return Instance;
	}

protected:
	friend FInjectionSiteTrait;
	friend ::FAnimNextFactoryParams;
	friend ::FUAFSystemFactoryParams;
	friend ::UInjectionCallbackProxy;
	friend ::FAnimNextGraphInstance;

	UAF_API FInstanceTaskContext(FUAFAssetInstance& InInstance);

	// Access a variable's value.
	UAF_API bool AccessVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>&)> InFunction) const;

	// Set a variable's value.
	UAF_API bool SetVariableInternal(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue) const;

	// Access the memory of the shared variable struct directly.
	UAF_API bool AccessVariablesStructInternal(const UScriptStruct* InStruct, TFunctionRef<void(FStructView)> InFunction) const;

	FUAFAssetInstance& Instance;
};

}

 