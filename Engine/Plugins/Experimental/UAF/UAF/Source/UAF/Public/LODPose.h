// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ReferencePose.h"
#include "BoneIndices.h"
#include "TransformArray.h"
#include "TransformArrayOperations.h"
#include "LODPose.generated.h"

// Handy define to set uninitialized pose buffers to 0xCDCDCDCD to help find issues
#define UE_ENABLE_POSE_DEBUG_FILL 0

namespace UE::UAF
{

enum class ELODPoseFlags : uint8
{
	None				= 0,
	Additive			= 1 << 0,
	DisableRetargeting	= 1 << 1,
	UseRawData			= 1 << 2,
	UseSourceData		= 1 << 3,
	MeshSpaceAdditive	= 1 << 4,
	LocalSpaceAdditive	= 1 << 5,
};

ENUM_CLASS_FLAGS(ELODPoseFlags);

struct FLODPose
{
	static constexpr int32 INVALID_LOD_LEVEL = -1;

	FTransformArrayView LocalTransformsView;
	const FReferencePose* RefPose = nullptr;
	int32 LODLevel = INVALID_LOD_LEVEL;
	ELODPoseFlags Flags = ELODPoseFlags::None;

	FLODPose() = default;

	void CopyFrom(const FLODPose& SourcePose)
	{
		if (!SourcePose.IsValid())
		{
			return;
		}
		
		check(RefPose && SourcePose.RefPose);
		check(RefPose->ReferenceLocalTransforms.Num() == SourcePose.RefPose->ReferenceLocalTransforms.Num());

		// Copy over the flags from our source.
		Flags = SourcePose.Flags;

		if (SourcePose.LODLevel == LODLevel)
		{
			// LOD levels match, just copy the full set of bone transforms.
			CopyTransforms(LocalTransformsView, SourcePose.LocalTransformsView);
		}
		else if (SourcePose.LODLevel < LODLevel)
		{
			// The source pose is set to a higher-quality LOD level and contains more bone transforms than we need. Just copy the ones we actually need.
			CopyTransforms(LocalTransformsView, SourcePose.LocalTransformsView, /*StartIndex=*/0, LocalTransformsView.Num());
		}
		else
		{
			// The source pose is missing transforms as it is set to a lower-quality LOD level, initialize the missing bone transforms with the reference pose transforms.
			const int32 NumSourceBones = SourcePose.LocalTransformsView.Num();
			const int32 NumTargetBones = LocalTransformsView.Num();
			const int32 NumAdditionalBones = NumTargetBones - NumSourceBones;
			SetRefPose(SourcePose.IsAdditive(), NumSourceBones, NumAdditionalBones);
			CopyTransforms(LocalTransformsView, SourcePose.LocalTransformsView, /*StartIndex=*/0, NumSourceBones);
		}
	}

	// This function copies transforms from a AoS source
	// Warning : The source number of transforms has to match the local pose size
	template<typename OtherAllocator>
	void CopyTransformsFrom(const TArray<FTransform, OtherAllocator>& SourceTransforms)
	{
		const int32 NumTransforms = SourceTransforms.Num();
		check(LocalTransformsView.Num() == NumTransforms);

		const FTransform* RESTRICT SourceTransform = SourceTransforms.GetData();
		const FTransform* RESTRICT SourceTransformEnd = SourceTransforms.GetData() + NumTransforms;
		FVector* RESTRICT TargetTranslations = &LocalTransformsView[0].Translation;
		FQuat* RESTRICT TargetRotations = &LocalTransformsView[0].Rotation;
		FVector* RESTRICT TargetScales = &LocalTransformsView[0].Scale3D;

		for (; SourceTransform < SourceTransformEnd; SourceTransform++, TargetTranslations++, TargetRotations++, TargetScales++)
		{
			//LocalTransformsView[Index] = SourceTransforms[Index]; // If LocalTransforms is in SoA format, there is a conversion from AoS to SoA
			(*TargetTranslations) = SourceTransform->GetTranslation();
			(*TargetRotations) = SourceTransform->GetRotation();
			(*TargetScales) = SourceTransform->GetScale3D();
		}
	}

	// This function copies transforms to an AoS target array
	// Warning : The target number of transforms has to match the local pose size
	template<typename OtherAllocator>
	void CopyTransformsTo(TArray<FTransform, OtherAllocator>& OutTransforms) const
	{
		const int32 NumTransforms = LocalTransformsView.Num();
		
		check(NumTransforms == OutTransforms.Num());
		for (int32 Index = 0; Index < NumTransforms; Index++)
		{
			OutTransforms[Index] = LocalTransformsView[Index]; // If LocalTransforms is in SoA format, there is a conversion from SoA to AoS
		}
	}

	void SetRefPose(bool bAdditive = false, int32 StartIndex = 0, int32 NumTransformsToCopy = -1)
	{
		const int32 NumTransforms = LocalTransformsView.Num();
		if (NumTransforms > 0)
		{
			if (NumTransformsToCopy < 0)
			{
				NumTransformsToCopy = NumTransforms - StartIndex;
			}

			if (bAdditive)
			{
				SetIdentity(bAdditive, StartIndex, NumTransformsToCopy);
			}
			else
			{
				check(RefPose != nullptr);
				CopyTransforms(LocalTransformsView, RefPose->ReferenceLocalTransforms.GetConstView(), StartIndex, NumTransformsToCopy);
			}
		}

		Flags = (ELODPoseFlags)(bAdditive ? (Flags | ELODPoseFlags::Additive) : Flags & ELODPoseFlags::Additive);
	}

	const FReferencePose& GetRefPose() const
	{
		check(RefPose != nullptr);
		return *RefPose;
	}

	void SetIdentity(bool bAdditive = false, int32 StartIndex = 0, int32 NumTransformsToSet = -1)
	{
		UE::UAF::SetIdentity(LocalTransformsView, bAdditive, StartIndex, NumTransformsToSet);
	}

	int32 GetNumBones() const
	{
		return RefPose != nullptr ? RefPose->GetNumBonesForLOD(LODLevel) : 0;
	}

	const TArrayView<const FBoneIndexType> GetLODBoneIndexToParentLODBoneIndexMap() const
	{
		if (LODLevel != INVALID_LOD_LEVEL && RefPose != nullptr)
		{
			return RefPose->GetLODBoneIndexToParentLODBoneIndexMap(LODLevel);
		}
		else
		{
			return TArrayView<const FBoneIndexType>();
		}
	}

	const TArrayView<const FBoneIndexType> GetLODBoneIndexToMeshBoneIndexMap() const
	{
		if (LODLevel != INVALID_LOD_LEVEL && RefPose != nullptr)
		{
			return RefPose->GetLODBoneIndexToMeshBoneIndexMap(LODLevel);
		}
		else
		{
			return TArrayView<const FBoneIndexType>();
		}
	}

	const TArrayView<const FBoneIndexType> GetLODBoneIndexToSkeletonBoneIndexMap() const
	{
		if (LODLevel != INVALID_LOD_LEVEL && RefPose != nullptr)
		{
			return RefPose->GetLODBoneIndexToSkeletonBoneIndexMap(LODLevel);
		}
		else
		{
			return TArrayView<const FBoneIndexType>();
		}
	}

	const TArrayView<const FBoneIndexType> GetMeshBoneIndexToLODBoneIndexMap() const
	{
		if (RefPose != nullptr)
		{
			return RefPose->GetMeshBoneIndexToLODBoneIndexMap();
		}
		else
		{
			return TArrayView<const FBoneIndexType>();
		}

	}

	const TArrayView<const FBoneIndexType> GetMeshBoneIndexToParentMeshBoneIndexMap() const
	{
		if (RefPose != nullptr)
		{
			return RefPose->GetMeshBoneIndexToParentMeshBoneIndexMap();
		}
		else
		{
			return TArrayView<const FBoneIndexType>();
		}
	}

	const TArrayView<const FBoneIndexType> GetSkeletonBoneIndexToLODBoneIndexMap() const
	{
		if (RefPose != nullptr)
		{
			return RefPose->GetSkeletonBoneIndexToLODBoneIndexMap();
		}
		else
		{
			return TArrayView<const FBoneIndexType>();
		}
	}

	// Query to find a LODBoneIndex for an associated BoneName
	// Returns INVALID_BONE_INDEX if missing ReferencePose or if no bone found for a given name
	FBoneIndexType FindLODBoneIndexFromBoneName(FName BoneName) const
	{
		if (RefPose != nullptr)
		{
			return RefPose->FindLODBoneIndexFromBoneName(BoneName);
		}
		return INVALID_BONE_INDEX;
	}

	// Query whether bone with LODIndex ChildLODBoneIndex is a child of bone with LODIndex ParentLODBoneIndex
	// Returns:
	//	True - ChildLODBoneIndex is a child of ParentLODBoneIndex
	//  False - ChildLODBoneIndex is not a child of ParentLODBoneIndex
	bool IsBoneChildOf(FBoneIndexType ChildLODBoneIndex, FBoneIndexType ParentLODBoneIndex) const
	{
		check(ChildLODBoneIndex != INVALID_BONE_INDEX);
		check(ParentLODBoneIndex != INVALID_BONE_INDEX);
		check(ChildLODBoneIndex != ParentLODBoneIndex);
		
		if (RefPose == nullptr)
		{
			return false;
		}

		const TArrayView<const FBoneIndexType> LODBoneIndexToParentMap = RefPose->GetLODBoneIndexToParentLODBoneIndexMap(LODLevel); 
		
		FBoneIndexType ParentOfChild = LODBoneIndexToParentMap[ChildLODBoneIndex];
		while (ParentOfChild != INVALID_BONE_INDEX)
		{
			if (ParentOfChild == ParentLODBoneIndex)
			{
				return true;
			}

			// Keep looking
			ChildLODBoneIndex = ParentOfChild;
			ParentOfChild = LODBoneIndexToParentMap[ChildLODBoneIndex];
		}

		return false;
	}

	// Get the LODBoneIndex of the parent of the bone at ChildLODBoneIndex
	// Returns INVALID_BONE_INDEX if no reference pose or if ChilddLODBoneIndex is the root
	FBoneIndexType GetLODBoneParentIndex(FBoneIndexType ChildLODBoneIndex) const
	{
		if (RefPose == nullptr)
		{
			return INVALID_BONE_INDEX;
		}
		
		return RefPose->GetLODParentBoneIndex(LODLevel, ChildLODBoneIndex);
	}

	const USkeleton* GetSkeletonAsset() const
	{
		return RefPose != nullptr ? RefPose->Skeleton.Get() : nullptr;
	}

	bool IsValid() const
	{
		return (RefPose != nullptr && RefPose->IsValid());
	}

	bool IsAdditive() const
	{
		return (Flags & ELODPoseFlags::Additive) != ELODPoseFlags::None;
	}

	bool IsMeshSpaceAdditive() const
	{
		return (Flags & ELODPoseFlags::MeshSpaceAdditive) != ELODPoseFlags::None;
	}

	bool IsLocalSpaceAdditive() const
	{
		return (Flags & ELODPoseFlags::LocalSpaceAdditive) != ELODPoseFlags::None;
	}

	/** Disable Retargeting */
	void SetDisableRetargeting(bool bDisableRetargeting)
	{
		Flags = (ELODPoseFlags)(bDisableRetargeting ? (Flags | ELODPoseFlags::DisableRetargeting) : Flags & ELODPoseFlags::DisableRetargeting);
	}

	/** True if retargeting is disabled */
	bool GetDisableRetargeting() const
	{
		return (Flags & ELODPoseFlags::DisableRetargeting) != ELODPoseFlags::None;
	}

	/** Ignore compressed data and use RAW data instead, for debugging. */
	void SetUseRAWData(bool bUseRAWData)
	{
		Flags = (ELODPoseFlags)(bUseRAWData ? (Flags | ELODPoseFlags::UseRawData) : Flags & ELODPoseFlags::UseRawData);
	}

	/** True if we're requesting RAW data instead of compressed data. For debugging. */
	bool ShouldUseRawData() const
	{
		return (Flags & ELODPoseFlags::UseRawData) != ELODPoseFlags::None;
	}

	/** Use Source data instead.*/
	void SetUseSourceData(bool bUseSourceData)
	{
		Flags = (ELODPoseFlags)(bUseSourceData ? (Flags | ELODPoseFlags::UseSourceData) : Flags & ELODPoseFlags::UseSourceData);
	}

	/** True if we're requesting Source data instead of RawAnimationData. For debugging. */
	bool ShouldUseSourceData() const
	{
		return (Flags & ELODPoseFlags::UseSourceData) != ELODPoseFlags::None;
	}
};

template <typename AllocatorType>
struct TLODPose : public FLODPose
{
	TTransformArray<AllocatorType> LocalTransforms;

	TLODPose() = default;
	
	TLODPose(const TLODPose& Other) : FLODPose(Other), LocalTransforms(Other.LocalTransforms)
	{
		LocalTransformsView = LocalTransforms.GetView();
	}

	TLODPose(const FReferencePose& InRefPose, int32 InLODLevel, bool bSetRefPose = true, bool bAdditive = false)
	{
		PrepareForLOD(InRefPose, InLODLevel, bSetRefPose, bAdditive);
	}

	TLODPose& operator=(const TLODPose& Other)
	{
		FLODPose::operator=(Other);
		LocalTransforms = Other.LocalTransforms;
		LocalTransformsView = LocalTransforms.GetView();
		return *this;
	}

	bool ShouldPrepareForLOD(const FReferencePose& InRefPose, int32 InLODLevel, bool bAdditive = false) const
	{
		return
			LODLevel != InLODLevel ||
			RefPose != &InRefPose ||
			bAdditive != EnumHasAnyFlags(Flags, ELODPoseFlags::Additive);
	}
	
	void PrepareForLOD(const FReferencePose& InRefPose, int32 InLODLevel, bool bSetRefPose = true, bool bAdditive = false)
	{
		LODLevel = InLODLevel;
		RefPose = &InRefPose;

		const int32 NumTransforms = InRefPose.GetNumBonesForLOD(InLODLevel);
		LocalTransforms.SetNumUninitialized(NumTransforms, EAllowShrinking::No);	// Don't shrink, our LOD might increase back causing us to churn
		LocalTransformsView = LocalTransforms.GetView();

		Flags = (ELODPoseFlags)(bAdditive ? (Flags | ELODPoseFlags::Additive) : Flags & ELODPoseFlags::Additive);

		if (NumTransforms > 0)
		{
			if (bSetRefPose)
			{
				SetRefPose(bAdditive);
			}
#if UE_ENABLE_POSE_DEBUG_FILL
			else
			{
				LocalTransforms.DebugFill();
			}
#endif
		}
	}
};

using FLODPoseHeap = TLODPose<FDefaultAllocator>;
using FLODPoseStack = TLODPose<FAnimStackAllocator>;

}

// USTRUCT wrapper for LOD pose
USTRUCT()
struct FAnimNextLODPose
#if CPP
	: UE::UAF::FLODPoseHeap
#endif
{
	GENERATED_BODY()
};