// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Variables/AnimNextSharedVariables.h"
#include "RigVMCore/RigVM.h"
#include "RigVMHost.h"
#include "Graph/AnimNextGraphState.h"
#include "Module/RigVMTrait_ModuleEventDependency.h"

#include "AnimNextModule.generated.h"

#define UE_API UAF_API

class UEdGraph;
class UUAFSystem;
class UAnimGraphNode_AnimNextGraph;
struct FAnimNode_AnimNextGraph;
struct FRigUnit_AnimNextGraphEvaluator;
struct FAnimNextGraphInstance;
struct FAnimNextScheduleGraphTask;
struct FAnimNextEditorParam;
struct FAnimNextParam;
struct FAnimNextModuleInstance;

namespace UE::UAF
{
	struct FContext;
	struct FExecutionContext;
	class FAnimNextModuleImpl;
	struct FTestUtils;
	struct FParametersProxy;
}

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

namespace UE::UAF::Editor
{
	class FModuleEditor;
	class FVariableCustomization;
}

// Root asset represented by a component when instantiated
UCLASS(MinimalAPI, BlueprintType, DisplayName="UAF System")
class UUAFSystem : public UUAFSharedVariables
{
	GENERATED_BODY()

public:
	UE_API UUAFSystem(const FObjectInitializer& ObjectInitializer);

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;

protected:
	friend class UUAFSystemFactory;
	friend class UUAFSystem_EditorData;
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend class UE::UAF::Editor::FModuleEditor;
	friend struct UE::UAF::FTestUtils;
	friend FAnimNextGraphInstance;
	friend class UAnimGraphNode_AnimNextGraph;
	friend UE::UAF::FExecutionContext;
	friend struct FAnimNextScheduleGraphTask;
	friend UE::UAF::FAnimNextModuleImpl;
	friend class UE::UAF::Editor::FVariableCustomization;
	friend struct UE::UAF::FParametersProxy;
	friend FAnimNextModuleInstance;

	// All dependencies that should be set up when the module initializes
	UPROPERTY()
	TArray<TInstancedStruct<FRigVMTrait_ModuleEventDependency>> Dependencies;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<TObjectPtr<UScriptStruct>> RequiredComponents_DEPRECATED;

	UPROPERTY()
	FAnimNextGraphState DefaultState_DEPRECATED;
	
	UPROPERTY()
	FInstancedPropertyBag PropertyBag_DEPRECATED;
#endif
};

#undef UE_API
