// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextModuleInitMethod.generated.h"

UENUM()
enum class EAnimNextModuleInitMethod : uint8
{
	// Do not perform any initial update, set up data structures only
	None,

	// Set up data structures, perform an initial update and then pause
	InitializeAndPause,

	// Set up data structures, perform an initial update and then pause in editor only, otherwise act like InitializeAndRun
	InitializeAndPauseInEditor,

	// Set up data structures then continue updating
	InitializeAndRun
};