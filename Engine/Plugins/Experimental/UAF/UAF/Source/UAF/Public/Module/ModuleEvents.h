// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraitCore/TraitEvent.h"

/**
 * Module Action Event
 * 
 * Module action events are trait events that are processed at the end of a module's execution.
 * If the event is not thread-safe, it is dispatched to execute on the main thread.
 */
struct FAnimNextModule_ActionEvent : public FAnimNextTraitEvent
{
	DECLARE_ANIM_TRAIT_EVENT(FAnimNextModule_ActionEvent, FAnimNextTraitEvent)

	FAnimNextModule_ActionEvent() = default;

	FAnimNextModule_ActionEvent(TUniqueFunction<void(void)>&& InActionFunction)
		: ActionFunction(MoveTemp(InActionFunction))
	{}

	// Whether or not this event is thread-safe and can execute on any thread
	// Events that are not thread-safe will execute on the main thread
	virtual bool IsThreadSafe() const { return false; }

	// Executes the schedule action
	// Derived types can override this and implement whatever they wish instead of using a lambda
	virtual void Execute() const
	{
		if (ActionFunction.IsSet())
		{
			ActionFunction();
		}
	}

	// The optional action to execute
	TUniqueFunction<void(void)> ActionFunction;
};
