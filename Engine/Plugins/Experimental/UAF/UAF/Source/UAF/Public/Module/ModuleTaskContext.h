// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InstanceTaskContext.h"

#include "TraitCore/TraitEvent.h"

struct FUAFAssetInstanceComponent;
struct FAnimNextModuleInstance;
struct FInstancedPropertyBag;
struct FAnimNextModuleInjectionComponent;
struct FUAFSystemOutputComponent;

namespace UE::UAF
{
	struct FScheduleContext;
	enum class EParameterScopeOrdering : int32;
	struct FModuleEventTickFunction;
}

namespace UE::UAF
{

// Context passed to schedule task callbacks
struct FModuleTaskContext : public FInstanceTaskContext
{
public:
	// Queues an input trait event
	// Input events will be processed in the next graph update after they are queued
	UAF_API void QueueInputTraitEvent(FAnimNextTraitEventPtr Event) const;

	// Queues an output trait event
	// Output events will be processed at the end of the schedule tick
	UAF_API void QueueOutputTraitEvent(FAnimNextTraitEventPtr Event) const;

	// Access a module instance component of the specified type. If the component exists, then InFunction will be called
	UAF_API void TryAccessComponent(UScriptStruct* InComponentType, TFunctionRef<void(FUAFAssetInstanceComponent&)> InFunction) const;

	// Access a module instance component of the specified type. 
	// If the component does not exist, it will be created.
	UAF_API void AccessComponent(UScriptStruct* InComponentType, TFunctionRef<void(FUAFAssetInstanceComponent&)> InFunction) const;

	// Access a module instance component of the specified type. If the component exists, then InFunction will be called
	template<typename ComponentType>
	void TryAccessComponent(TFunctionRef<void(ComponentType&)> InFunction) const
	{
		TryAccessComponent(TBaseStructure<ComponentType>::Get(), [&InFunction](FUAFAssetInstanceComponent& InComponent)
		{
			InFunction(static_cast<ComponentType&>(InComponent));
		});
	}
	
	// Access a module instance component of the specified type. 
	// If the component does not exist, it will be created.
	template<typename ComponentType>
	void AccessComponent(TFunctionRef<void(ComponentType&)> InFunction) const
	{
		AccessComponent(TBaseStructure<ComponentType>::Get(), [&InFunction](FUAFAssetInstanceComponent& InComponent)
		{
			InFunction(static_cast<ComponentType&>(InComponent));
		});
	}

	UAF_API FAnimNextModuleInstance* const GetModuleInstance() const;

private:
	FModuleTaskContext(FAnimNextModuleInstance& InModuleInstance);

	// The module instance currently running
	FAnimNextModuleInstance& ModuleInstance;

	friend UE::UAF::FModuleEventTickFunction;
	friend ::FAnimNextModuleInjectionComponent;
	friend ::FUAFSystemOutputComponent;
};

using FUniqueSystemTask = TUniqueFunction<void(const UE::UAF::FModuleTaskContext&)>;

}