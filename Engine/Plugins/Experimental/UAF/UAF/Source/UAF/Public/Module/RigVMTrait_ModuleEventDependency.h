// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMTrait.h"
#include "RigVMTrait_ModuleEventDependency.generated.h"

#define UE_API UAF_API

struct FTickFunction;

// The relative ordering of a module dependency
UENUM()
enum class EAnimNextModuleEventDependencyOrdering : uint8
{
	// This dependency executes before the specified module event
	Before,
	
	// This dependency executes after the specified module event
	After
};

namespace UE::UAF
{
	// Context passed to module dependency functions
	struct FModuleDependencyContext
	{
		FModuleDependencyContext(UObject* InObject, FTickFunction& InTickFunction)
			: Object(InObject)
			, TickFunction(InTickFunction)
		{}
		
		// The object that the module is bound to
		UObject* Object = nullptr;

		// The module's tick function that we want to depend on
		FTickFunction& TickFunction;
	};
}

// A trait that acts as dependency that can be established between an external system and an AnimNext event
USTRUCT(meta=(Hidden))
struct FRigVMTrait_ModuleEventDependency : public FRigVMTrait
{
	GENERATED_BODY()

	UE_API FRigVMTrait_ModuleEventDependency();

	// Override point that adds the dependency
	virtual void OnAddDependency(const UE::UAF::FModuleDependencyContext& InContext) const {}

	// Override point that removes the dependency
	virtual void OnRemoveDependency(const UE::UAF::FModuleDependencyContext& InContext) const {}

	// How to execute relative to the event
	UPROPERTY(EditAnywhere, Category="Dependency")
	EAnimNextModuleEventDependencyOrdering Ordering;

	// The event to execute relative to
	UPROPERTY(EditAnywhere, Category="Dependency", meta=(CustomWidget=AnimNextModuleEvent))
	FName EventName;
};

#undef UE_API
