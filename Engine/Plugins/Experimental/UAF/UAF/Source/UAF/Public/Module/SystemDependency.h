// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::UAF
{

// Dependency type passed to AddDependency in FSystemReference/FUAFWeakSystemReference etc.
enum class ESystemDependency : uint8
{
	// Dependency runs before the specified event
	Prerequisite,
	// Dependency runs after the specified event
	Subsequent
};

}