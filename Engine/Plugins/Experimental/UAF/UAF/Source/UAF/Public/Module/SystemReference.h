// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"
#include "UAFModuleInstanceComponent.h"
#include "Module/TaskRunLocation.h"
#include "StructUtils/StructView.h"
#include "Subsystems/EngineSubsystem.h"
#include "SystemReference.generated.h"

#define UE_API UAF_API

struct FUAFSystemFactoryAsset;
class UUAFSystem;
struct FAnimNextParamType;
struct FAnimNextVariableReference;
struct FAnimNextModuleInstance;
enum class EAnimNextModuleInitMethod : uint8;
enum class EPropertyBagResult : uint8;
struct FAnimNextTraitEvent;
struct FTickFunction;
class UUAFComponent;
struct FUAFWeakSystemReference;
class UUAFRigVMAsset;
struct FMassUAFFragment;
class UMassUAFProcessor;
class UMassUAFInitializer;
class UMeshEntityComponentImpl_Skinned;
class UEntitySkeletonModifierStackImpl;
class UMassUAFDestructor;

namespace UE::UAF
{
enum class ESystemDependency : uint8;
struct FModuleTaskContext;
struct FInjectionRequest;
struct FModuleEventTickFunction;
class FAnimNextModuleImpl;
}

namespace UE::UAF::Tests
{
class FSystemReference_Creation;
}

namespace UE::UAF
{

// An owning reference to a UAF system.
// Provides a partially thread-safe API for interacting with UAF systems. Check individual methods for thread-safety. Methods that can only be called
// on the game thread will assert when called from non-game threads.
struct FSystemReference
{
	// Default-constructable only because it needs to be embedded in UObjects/UStructs
	FSystemReference() = default;

	// Move only, non-copyable
	FSystemReference(const FSystemReference&) = delete;
	FSystemReference& operator=(const FSystemReference&) = delete;
	UE_API FSystemReference(FSystemReference&&);
	UE_API FSystemReference& operator=(FSystemReference&&);

private:
	friend UUAFComponent;
	friend FUAFWeakSystemReference;
	friend UE::UAF::FModuleEventTickFunction;
	friend UE::UAF::FAnimNextModuleImpl;
	friend UE::UAF::Tests::FSystemReference_Creation;
	friend UEntitySkeletonModifierStackImpl;
	friend UMeshEntityComponentImpl_Skinned;

	// @TODO: Temporarily friending Mass types, so that Mass can act as a system host similar to UAnimNextComponent.
	friend FMassUAFFragment;
	friend UMassUAFProcessor;
	friend UMassUAFInitializer;
	friend UMassUAFDestructor;

	// Construct an owning handle to a system instance.
	// Constructs a FAnimNextModuleInstance for use by worker threads.
	// Should be called only from the game thread
	UE_API FSystemReference(TConstStructView<FUAFSystemFactoryAsset> InAsset, UObject* InObject, EAnimNextModuleInitMethod InitMethod);

	// Starts up system pool
	static void Init();

	// Shuts down the system pool
	static void Destroy();

	// Reset this handle to invalid - gives up ownership
	// Should be called only from the game thread
	UE_API void Reset();

	// Check if this handle is valid
	// Can be called on any thread.
	UE_API bool IsValid() const;

	// Get the system that this handle references
	// Can be called on any thread.
	UE_API const UUAFSystem* GetSystem() const;

	// Enables or disables the system
	// This operation is deferred until just before the next world tick (FWorldDelegates::OnWorldPreActorTick)
	// Can be called on any thread.
	UE_API void SetEnabled(bool bInEnabled) const;

#if UE_ENABLE_DEBUG_DRAWING
	// Enables or disables the system's debug drawing
	// This operation is deferred until just before the next world tick (FWorldDelegates::OnWorldPreActorTick)
	// Can be called on any thread.
	UE_API void ShowDebugDrawing(bool bInShowDebugDrawing) const;
#endif

	// Queue a task to run before the first user event
	// Can be called on any thread.
	// @param	InTaskFunction		The function to run
	void QueueTask(TUniqueFunction<void(const UE::UAF::FModuleTaskContext&)>&& InTaskFunction) const
	{
		QueueTask(FName(), MoveTemp(InTaskFunction), UE::UAF::ETaskRunLocation::Before);
	}
	
	// Queue a task to run at a particular point in a system's execution
	// Can be called on any thread.
	// @param	InSystemEventName	The name of the event in the system to run the supplied task relative to. If this is NAME_None, then the first user event will be used.
	// @param	InTaskFunction		The function to run
	// @param	InLocation			Where to run the task, before or after
	UE_API void QueueTask(FName InSystemEventName, TUniqueFunction<void(const UE::UAF::FModuleTaskContext&)>&& InTaskFunction, UE::UAF::ETaskRunLocation InLocation = UE::UAF::ETaskRunLocation::Before) const;

	// Queue an input trait event to be fired before the first user event
	// Can be called on any thread.
	// @param	InEvent				The event to queue
	UE_API void QueueInputTraitEvent(TSharedPtr<FAnimNextTraitEvent> Event) const;

	// Find the const tick function for the specified event. Useful to analyze/log prerequisites. Use AddDependency, RemoveDependency to actually modify dependencies
	// Should be called only from the game thread
	// @param	InEventName			The event associated to the wanted tick function
	UE_API const FTickFunction* FindTickFunction(FName InEventName) const;

	// Get the event name of the first user event
	// Should be called only from the game thread
	UE_API FName GetFirstUserEventName() const;

	// Get the event name of the last user event
	// Should be called only from the game thread
	UE_API FName GetLastUserEventName() const;

	// Run a specific Event in-place.
	// @TODO: Should only be called on unregistered tick functions.
	// @param	InEventName		The event we want to run.
	UE_API void RunEvent(FName InEventName, float DeltaTime) const;

	// Add a dependency on a tick function to the specified event
	// Should be called only from the game thread
	// @param	InObject			The object that owns the tick function
	// @param	InTickFunction		The tick function to depend on
	// @param	InSystemEventName	The event to add the dependency to
	// @param	InDependency		The kind of dependency to add
	UE_API void AddDependency(UObject* InObject, FTickFunction& InTickFunction, FName InSystemEventName, UE::UAF::ESystemDependency InDependency) const;

	// Remove a dependency on a tick function from the specified event
	// Should be called only from the game thread
	// @param	InObject			The object that owns the tick function
	// @param	InTickFunction		The tick function that was depended on
	// @param	InSystemEventName	The event to remove the dependency from
	// @param	InDependency		The kind of dependency to remove
	UE_API void RemoveDependency(UObject* InObject, FTickFunction& InTickFunction, FName InSystemEventName, UE::UAF::ESystemDependency InDependency) const;

	// Add a dependency on a system event to the specified system event
	// Should be called only from the game thread
	// @param	InSystemEventName		The event to add the dependency to/from
	// @param	InOtherSystem			The reference to the other system whose dependencies are being modified
	// @param	InOtherSystemEventName	The other system's event to add the dependency to/from
	// @param	InDependency			The kind of dependency to add
	UE_API void AddSystemEventDependency(FName InSystemEventName, FUAFWeakSystemReference InOtherSystem, FName InOtherSystemEventName, UE::UAF::ESystemDependency InDependency) const;

	// Remove a dependency on a system event from the specified system event
	// Should be called only from the game thread
	// @param	InSystemEventName		The event to remove the dependency to/from
	// @param	InOtherSystem			The reference to the other system whose dependencies are being modified
	// @param	InOtherSystemEventName	The other system's event to remove the dependency to/from
	// @param	InDependency			The kind of dependency to remove
	UE_API void RemoveSystemEventDependency(FName InSystemEventName, FUAFWeakSystemReference InOtherSystem, FName InOtherSystemEventName, UE::UAF::ESystemDependency InDependency) const;

	// Set the value of the specified variable
	// Can be called on any thread.
	// @param	InVariable			The variable we want to set
	// @param	InType				The type of the variable we want to set
	// @param	InData				The data to set the variable with
	// @return see EPropertyBagResult
	UE_API EPropertyBagResult SetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InData) const;

	// Accesses the variable of the specified name for writing.
	// General read access is not provided via this API due to the current double-buffering strategy used to communicate variable writes to worker threads.
	// This is intended to allow for copy-free writing of larger data structures & arrays, rather than read access.
	// Can be called on any thread.
	// @param	InVariable			The variable we want to access
	// @param	InType				The type of the variable we want to access
	// @param	InFunction			Function that will be called to allow modification of the variables
	// @return see EPropertyBagResult
	UE_API EPropertyBagResult WriteVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>)> InFunction) const;

	// Access a variable's value for read only.
	// This performs non-recursive locks on the system instance to ensure that concurrent access is respected, so care must be taken calling this from
	// animation worker tasks as deadlocks can occur.
	// Type must match strictly, no conversions are performed.
	// Can be called on any thread.
	// @param	InVariable			The variable to get the value of
	// @param	InType				The type of the variable
	// @param	InFunction			Function that will be called if no errors occur
	// @return see EPropertyBagResult
	UE_API EPropertyBagResult ReadVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TConstArrayView<uint8>)> InFunction) const;
	
	// Access the supplied system component for reading.
	// Can be called on any thread.
	// @param	InFunction			Function that will be called to access the struct
	// @return true if the struct was available, otherwise false
	template<typename ComponentType>
	bool ReadComponent(TFunctionRef<void(TConstStructView<ComponentType> InComponentStruct)> InFunction) const
	{
		static_assert(std::is_base_of_v<FUAFModuleInstanceComponent, ComponentType>, "Component struct must be a child of FUAFModuleInstanceComponent");
		return ReadComponent(ComponentType::StaticStruct(), [&InFunction](FConstStructView InStructView)
			{
				InFunction(TConstStructView<ComponentType>(InStructView.Get<ComponentType>()));
			});
	}

#if WITH_EDITOR
	// Runs a linearized version of the initial tick of the system to support generating an output pose for editor worlds that dont tick
	// Should be called only from the game thread
	UE_API void RunInitialTickInEditor() const;
#endif

private:
	// Access the supplied system component for reading.
	UE_API bool ReadComponent(UScriptStruct* InComponentType, TFunctionRef<void(FConstStructView InComponentStruct)> InFunction) const;

	// Internal only constructor for selective access to owned-ref APIs
	explicit FSystemReference(FAnimNextModuleInstance& InInstance);

	// Pointer to the instance
	TSharedPtr<FAnimNextModuleInstance> Ptr;
};

}

// Engine subsystem used to manage system compilation routing, ticking and GC
UCLASS(MinimalAPI)
class UUAFEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

private:
	UE_API UUAFEngineSubsystem();

	UE_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	// Refresh any entries that use the provided asset as it has been recompiled.
	UE_API void OnCompileJobFinished(UUAFRigVMAsset* InAsset);
#endif

#if WITH_EDITOR
	// Handle used to hook asset compilation
	FDelegateHandle OnCompileJobFinishedHandle;
#endif

	// Handle used to hook world tick
	FDelegateHandle OnWorldPreActorTickHandle;

	// Handle used to hook into GC
	FDelegateHandle OnPreGarbageCollectHandle;
};

#undef UE_API