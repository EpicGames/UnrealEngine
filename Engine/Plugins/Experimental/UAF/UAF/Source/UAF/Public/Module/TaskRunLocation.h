// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::UAF
{

enum class ETaskRunLocation : int32
{
	// Run the task before the specified task
	Before,

	// Run the task after the specified task
	After,
};

}