// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFAssetInstance.h"
#include "UAFAssetInstanceComponent.h"
#include "UAFModuleInstanceComponent.generated.h"

struct FAnimNextModuleInstance;
struct FAnimNextTraitEvent;

#define UE_API UAF_API

/** A system instance component is attached to and owned by a system instance. */
USTRUCT(meta=(Hidden, Abstract))
struct FUAFModuleInstanceComponent : public FUAFAssetInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	using ContainerType = FAnimNextModuleInstance;

	virtual ~FUAFModuleInstanceComponent() override = default;

	// Returns the owning system instance this component lives on.
	// Can return nullptr if an owning system instance has not been created (such as a default component on an asset)
	UE_API FAnimNextModuleInstance* GetModuleInstancePtr();

	// Returns the owning system instance this component lives on
	// Will assert if an owning system instance has not been created (such as a default component on an asset)
	UE_API FAnimNextModuleInstance& GetModuleInstance();

	// Returns the owning system instance this component lives on
	// Will assert if an owning system instance has not been created (such as a default component on an asset)
	UE_API const FAnimNextModuleInstance& GetModuleInstance() const;

	// Called during system execution for any events to be handled
	virtual void OnTraitEvent(FAnimNextTraitEvent& Event) {}

	// Called at start of system execution each frame (this runs during the first user event - prior events, e.g. bindings will run before this)
	virtual void OnBeginExecution(float InDeltaTime) {}

	// Called at end of system execution each frame
	virtual void OnEndExecution(float InDeltaTime) {}

private:
	friend struct FAnimNextModuleInstance;
};

#undef UE_API