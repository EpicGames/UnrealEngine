// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Module/ModuleEvents.h"
#include "Module/UAFModuleInstanceComponent.h"

#include "UAFNotifyDispatcherComponent.generated.h"

struct FAnimNextTraitEvent;
struct FAnimNotifyEventReference;

namespace UE::UAF
{
	struct FNotifyQueueDispatchEvent;
}

#define UE_API UAF_API

USTRUCT()
struct FUAFNotifyDispatcherComponent : public FUAFModuleInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	// FUAFModuleInstanceComponent interface
	virtual void OnTraitEvent(FAnimNextTraitEvent& Event) override;
	virtual void OnEndExecution(float InDeltaTime) override;

	// Triggers a single anim notify in the dispatcher
	void TriggerSingleAnimNotify(float InDeltaTime, UE::UAF::FNotifyQueueDispatchEvent* InDispatcher, const FAnimNotifyEventReference& EventReference);

	// FUAFAssetInstanceComponent interface
	virtual void OnBindToInstance() override;

	// Notify queue to dispatch
	UPROPERTY()
	FAnimNotifyQueue NotifyQueue;

	TArray<const FAnimNotifyEvent*> ActiveAnimNotifyState;

	UPROPERTY()
	TArray<FAnimNotifyEventReference> ActiveAnimNotifyEventReference;

	// Skeletal mesh component to 'fake' dispatch from
	UPROPERTY()
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;
};

namespace UE::UAF
{
	// Event that dispatches to the module
	struct FNotifyDispatchEvent : public FAnimNextTraitEvent
	{
		DECLARE_ANIM_TRAIT_EVENT(FNotifyDispatchEvent, FAnimNextTraitEvent)

		// Notifies to be dispatched
		TArray<FAnimNotifyEventReference> Notifies;

		// Weight at dispatch time
		float Weight = 1.0f;
	};

	// Event that dispatches from the module to gameplay
	struct FNotifyQueueDispatchEvent : public FAnimNextModule_ActionEvent
	{
		DECLARE_ANIM_TRAIT_EVENT(FNotifyQueueDispatchEvent, FAnimNextModule_ActionEvent)

		// FAnimNextModule_ActionEvent interface
		virtual bool IsThreadSafe() const override { return bIsThreadSafe; }
		virtual void Execute() const override;

		// The various events to dispatch
		TArray<FAnimNotifyEventReference> EventsToNotify;
		TArray<FAnimNotifyEventReference> EventsToEnd;
		TArray<FAnimNotifyEventReference> EventsToBegin;
		TArray<FAnimNotifyEventReference> EventsToTick;

		// Skeletal mesh component to dispatch as
		TWeakObjectPtr<USkeletalMeshComponent> WeakSkeletalMeshComponent;

		// Delta time to apply to notifies
		float DeltaSeconds = 0.0f;

		// Whether this queue thread safe to dispatch
		bool bIsThreadSafe = false;
	};
}

#undef UE_API