// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UAF/UAFAssetData.h"
#include "Module/AnimNextModule.h"

#include "UAFSystemAssetData.generated.h"

class UUAFAnimChooserTable;
class UUAFSystem;

USTRUCT(DisplayName="System Asset")
struct FUAFSystemFactoryAsset_System : public FUAFSystemFactoryAsset
{
	GENERATED_BODY()

	FUAFSystemFactoryAsset_System(const UUAFSystem* InSystem) : System(InSystem){ }
	FUAFSystemFactoryAsset_System() = default;

	virtual bool Validate() const override { return System != nullptr; }
	
	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const override { OutReferencedObjects.Add(System); }

	UPROPERTY(EditAnywhere, Category = System)
	TObjectPtr<const UUAFSystem> System;
};

