// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextPoolHandle.h"
#include "ModuleTaskContext.h"
#include "TaskRunLocation.h"
#include "UAFModuleInstanceComponent.h"
#include "UAFWeakSystemReference.generated.h"

#define UE_API UAF_API

class UUAFSystem;
class UUAFAnimGraph;
struct FAnimNextModuleInstance;

namespace UE::UAF
{
	enum class ESystemDependency : uint8;
	struct FInjectionRequest;
	struct FModuleEventTickFunction;
	struct FSystemReference;
}

// A non-owning reference to a UAF system.
// ~Provides a thread-safe API for interacting with UAF systems.
USTRUCT(BlueprintType, DisplayName="UAF System Reference")
struct FUAFWeakSystemReference
{
	GENERATED_BODY()

	FUAFWeakSystemReference() = default;

	UAF_API explicit FUAFWeakSystemReference(const UE::UAF::FSystemReference& InHandle);

	bool operator==(const FUAFWeakSystemReference& InOther) const
	{
		return WeakPtr == InOther.WeakPtr;
	}

	// Check to see if this handle is for a valid instance
	// Can be called on any thread.
	bool IsValid() const
	{
		return WeakPtr.IsValid();
	}

	// Queue a task to run before the first user event
	// Can be called on any thread.
	// @param	InTaskFunction		The function to run
	void QueueTask(TUniqueFunction<void(const UE::UAF::FModuleTaskContext&)>&& InTaskFunction) const
	{
		QueueTask(FName(), MoveTemp(InTaskFunction), UE::UAF::ETaskRunLocation::Before);
	}

	// Queue a task to run at a particular point in a system's execution
	// Can be called on any thread.
	// @param	InSystemEventName	The name of the event in the system to run the supplied task relative to. If this is NAME_None, then the first user event will be used.
	// @param	InTaskFunction		The function to run
	// @param	InLocation			Where to run the task, before or after
	UE_API void QueueTask(FName InSystemEventName, TUniqueFunction<void(const UE::UAF::FModuleTaskContext&)>&& InTaskFunction, UE::UAF::ETaskRunLocation InLocation = UE::UAF::ETaskRunLocation::Before) const;

	// Queue an input trait event to be fired before the first user event
	// Can be called on any thread.
	// @param	InEvent				The event to queue
	UE_API void QueueInputTraitEvent(TSharedPtr<FAnimNextTraitEvent> InEvent) const;

	// Set the value of the specified variable.
	// Note: Sets the proxy variables that will be deferred-transferred to the system on its next ExecuteBindings_WT call.
	// Can be called on any thread.
	// @param	InVariable			The variable we want to set
	// @param	InType				The type of the variable we want to set
	// @param	InData				The data to set the variable with
	// @return see EPropertyBagResult
	UE_API EPropertyBagResult SetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InData) const;

	// Accesses the variable of the specified name for writing.
	// Note: Sets the proxy variables that will be deferred-transferred to the system on its next ExecuteBindings_WT call.
	// General read access is not provided via this API due to the current double-buffering strategy used to communicate variable writes to worker threads.
	// This is intended to allow for copy-free writing of larger data structures & arrays, rather than read access.
	// Can be called on any thread.
	// @param	InVariable			The variable we want to access
	// @param	InType				The type of the variable we want to access
	// @param	InFunction			Function that will be called to allow modification of the variables
	// @return see EPropertyBagResult
	UE_API EPropertyBagResult WriteVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TArrayView<uint8>)> InFunction) const;

	// Access a variable's value for read only.
	// This performs non-recursive locks on the system instance to ensure that concurrent access is respected, so care must be taken calling this from
	// animation worker tasks as deadlocks can occur.
	// Type must match strictly, no conversions are performed.
	// Can be called on any thread.
	// @param	InVariable			The variable to get the value of
	// @param	InType				The type of the variable
	// @param	InFunction			Function that will be called if no errors occur
	// @return see EPropertyBagResult
	UE_API EPropertyBagResult ReadVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TFunctionRef<void(TConstArrayView<uint8>)> InFunction) const;

	// Access the supplied system component for reading.
	// Can be called on any thread.
	// @param	InFunction			Function that will be called to access the struct
	// @return true if the struct was available, otherwise false
	template<typename ComponentType>
	bool ReadComponent(TFunctionRef<void(TConstStructView<ComponentType> InComponentStruct)> InFunction) const
	{
		static_assert(std::is_base_of_v<FUAFModuleInstanceComponent, ComponentType>, "Component struct must be a child of FUAFModuleInstanceComponent");
		return ReadComponent(ComponentType::StaticStruct(), [&InFunction](FConstStructView InStructView)
		{
			InFunction(TConstStructView<ComponentType>(InStructView.Get<ComponentType>()));
		});
	}

	UE_API const UUAFSystem* GetSystem() const;

private:
	friend UE::UAF::FSystemReference;
	friend FAnimNextModuleInstance;
	friend UE::UAF::FModuleEventTickFunction;
	friend UE::UAF::FInjectionRequest;
	friend UUAFAnimGraph;

	UE_API bool ReadComponent(UScriptStruct* InComponentType, TFunctionRef<void(FConstStructView InComponentStruct)> InFunction) const;

	// Ptr to the instance
	TWeakPtr<FAnimNextModuleInstance> WeakPtr;
};

#undef UE_API