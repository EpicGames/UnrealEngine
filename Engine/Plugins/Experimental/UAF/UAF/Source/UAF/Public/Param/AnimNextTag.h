// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextTag.generated.h"

// Empty struct to act as a named tag for parameters
USTRUCT(BlueprintType)
struct FAnimNextTag
{
	GENERATED_BODY()
};

// Empty struct to tag named scopes
USTRUCT(BlueprintType)
struct FAnimNextScope : public FAnimNextTag
{
	GENERATED_BODY()
};

// Empty struct to tag named entry points
USTRUCT(BlueprintType)
struct FAnimNextEntryPoint : public FAnimNextTag
{
	GENERATED_BODY()
};
