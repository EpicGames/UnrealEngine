// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{

enum class EParamCompatibility : int32
{
	// Parameters types are incompatble, completely unrelated (e.g. Vector -> bool)
	Incompatible,

	// Parameters types are incompatible because data loss can occur (e.g. int32 -> int16)
	Incompatible_DataLoss,

	// Parameters types are compatible with a type promotion (e.g. int32 -> int64)
	Compatible_Promotion,

	// Parameters types are compatible with an object cast (e.g. UAnimSequence -> UAnimationAsset)
	Compatible_Cast,

	// Parameter types are the same
	Compatible_Equal
};

struct FParamCompatibility
{
	FParamCompatibility(EParamCompatibility InCompatibility)
		: Compatibility(InCompatibility)
	{}

	EParamCompatibility Compatibility;

	static FParamCompatibility Equal()
	{
		return EParamCompatibility::Compatible_Cast;
	}

	static FParamCompatibility Compatible()
	{
		return EParamCompatibility::Compatible_Promotion;
	}

	static FParamCompatibility IncompatibleWithDataLoss()
	{
		return EParamCompatibility::Incompatible_DataLoss;
	}

	bool IsEqual() const
	{
		return Compatibility >= EParamCompatibility::Compatible_Cast;
	}

	bool IsCompatible() const
	{
		return Compatibility >= EParamCompatibility::Compatible_Promotion;
	}

	bool IsCompatibleWithDataLoss() const
	{
		return Compatibility >= EParamCompatibility::Incompatible_DataLoss;
	}

	bool operator<(const FParamCompatibility& InOther) const
	{
		return Compatibility < InOther.Compatibility;
	}

	bool operator<=(const FParamCompatibility& InOther) const
	{
		return Compatibility <= InOther.Compatibility;
	}

	bool operator==(const FParamCompatibility& InOther) const
	{
		return Compatibility == InOther.Compatibility;
	}

	bool operator!=(const FParamCompatibility& InOther) const
	{
		return Compatibility != InOther.Compatibility;
	}

	bool operator>=(const FParamCompatibility& InOther) const
	{
		return Compatibility >= InOther.Compatibility;
	}

	bool operator>(const FParamCompatibility& InOther) const
	{
		return Compatibility > InOther.Compatibility;
	}
};

}