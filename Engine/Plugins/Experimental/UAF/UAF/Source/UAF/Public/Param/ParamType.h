// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/PropertyBag.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "Concepts/BaseStructureProvider.h"
#include "RigVMCore/RigVMTemplate.h"
#include "ParamType.generated.h"

#define UE_API UAF_API

namespace UE::UAF
{
	struct FParamHelpers;
}

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

namespace UE::UAF::Private
{
	template <typename T>
	struct TIsSoftObjectPtr
	{
		enum { Value = false };
	};

	template <typename T> struct TIsSoftObjectPtr<               TSoftObjectPtr<T>> { enum { Value = true }; };
	template <typename T> struct TIsSoftObjectPtr<const          TSoftObjectPtr<T>> { enum { Value = true }; };
	template <typename T> struct TIsSoftObjectPtr<      volatile TSoftObjectPtr<T>> { enum { Value = true }; };
	template <typename T> struct TIsSoftObjectPtr<const volatile TSoftObjectPtr<T>> { enum { Value = true }; };

	template <typename T>
	struct TIsSoftClassPtr
	{
		enum { Value = false };
	};

	template <typename T> struct TIsSoftClassPtr<               TSoftClassPtr<T>> { enum { Value = true }; };
	template <typename T> struct TIsSoftClassPtr<const          TSoftClassPtr<T>> { enum { Value = true }; };
	template <typename T> struct TIsSoftClassPtr<      volatile TSoftClassPtr<T>> { enum { Value = true }; };
	template <typename T> struct TIsSoftClassPtr<const volatile TSoftClassPtr<T>> { enum { Value = true }; };
}

/**
 * Representation of a UAF variable's type.
 */
USTRUCT(BlueprintType, DisplayName = "UAF Variable Type")
struct FAnimNextParamType
{
public:
	GENERATED_BODY()

	using EValueType = ::EPropertyBagPropertyType;
	using EContainerType = ::EPropertyBagContainerType;

	friend struct UE::UAF::FParamHelpers;
	friend struct UE::UAF::UncookedOnly::FUtils;

	FAnimNextParamType() = default;

	/** Construct a parameter type from the passed in value, container and object type. */
	FAnimNextParamType(EValueType InValueType, EContainerType InContainerType = EContainerType::None, const UObject* InValueTypeObject = nullptr)
		: ValueTypeObject(InValueTypeObject)
		, ValueType(InValueType)
		, ContainerType(InContainerType)
	{
	}

private:
	/** Pointer to object that defines the Enum, Struct, or Class. */
	UPROPERTY()
	TObjectPtr<const UObject> ValueTypeObject = nullptr;

	/** Type of the value described by this parameter. */
	UPROPERTY()
	EPropertyBagPropertyType ValueType = EPropertyBagPropertyType::None;

	/** Type of the container described by this parameter. */
	UPROPERTY()
	EPropertyBagContainerType ContainerType = EPropertyBagContainerType::None;
	
private:
	/** Helper function for GetType */
	template<typename ParamType>
	static FAnimNextParamType GetTypeInner(EContainerType InContainerType)
	{
		using NonPtrParamType = std::remove_pointer_t<ParamType>;

		if constexpr (std::is_same_v<ParamType, bool>)
		{
			return FAnimNextParamType(EValueType::Bool, InContainerType);
		}
		else if constexpr (std::is_same_v<ParamType, uint8>)
		{
			return FAnimNextParamType(EValueType::Byte, InContainerType);
		}
		else if constexpr (std::is_same_v<ParamType, int32> || std::is_same_v<ParamType, int>)
		{
			return FAnimNextParamType(EValueType::Int32, InContainerType);
		}
		else if constexpr (std::is_same_v<ParamType, int64>)
		{
			return FAnimNextParamType(EValueType::Int64, InContainerType);
		}
		else if constexpr (std::is_same_v<ParamType, float>)
		{
			return FAnimNextParamType(EValueType::Float, InContainerType);
		}
		else if constexpr (std::is_same_v<ParamType, double>)
		{
			return FAnimNextParamType(EValueType::Double, InContainerType);
		}
		else if constexpr (std::is_same_v<ParamType, FName>)
		{
			return FAnimNextParamType(EValueType::Name, InContainerType);
		}
		else if constexpr (std::is_same_v<ParamType, FString>)
		{
			return FAnimNextParamType(EValueType::String, InContainerType);
		}
		else if constexpr (std::is_same_v<ParamType, FText>)
		{
			return FAnimNextParamType(EValueType::Text, InContainerType);
		}
		else if constexpr (TIsUEnumClass<ParamType>::Value)
		{
			return FAnimNextParamType(EValueType::Enum, InContainerType, StaticEnum<ParamType>());
		}
		else if constexpr (TModels<CStaticStructProvider, ParamType>::Value)
		{
			return FAnimNextParamType(EValueType::Struct, InContainerType, ParamType::StaticStruct());
		}
		else if constexpr (TModels<CBaseStructureProvider, ParamType>::Value)
		{
			return FAnimNextParamType(EValueType::Struct, InContainerType, TBaseStructure<ParamType>::Get());
		}
		else if constexpr (TModels<CStaticClassProvider, NonPtrParamType>::Value)
		{
			if constexpr (std::is_same_v<NonPtrParamType, UClass>)
			{
				return FAnimNextParamType(EValueType::Class, InContainerType, UObject::StaticClass());
			}
			else
			{
				return FAnimNextParamType(EValueType::Object, InContainerType, NonPtrParamType::StaticClass());
			}
		}
		else if constexpr (TIsTObjectPtr<ParamType>::Value)
		{
			if constexpr (std::is_same_v<ParamType, TObjectPtr<UClass>>)
			{
				return FAnimNextParamType(EValueType::Class, InContainerType, UObject::StaticClass());
			}
			else
			{
				return FAnimNextParamType(EValueType::Object, InContainerType, ParamType::ElementType::StaticClass());
			}
		}
		else if constexpr (TIsTSubclassOf<ParamType>::Value)
		{
			return FAnimNextParamType(EValueType::Class, InContainerType, ParamType::ElementType::StaticClass());
		}
		else if constexpr (UE::UAF::Private::TIsSoftObjectPtr<ParamType>::Value)
		{
			return FAnimNextParamType(EValueType::SoftObject, InContainerType, ParamType::ElementType::StaticClass());
		}
		else if constexpr (UE::UAF::Private::TIsSoftClassPtr<ParamType>::Value)
		{
			return FAnimNextParamType(EValueType::SoftClass, InContainerType, ParamType::ElementType::StaticClass());
		}
		else if constexpr (std::is_same_v<ParamType, uint32>)
		{
			return FAnimNextParamType(EValueType::UInt32, InContainerType);
		}
		else if constexpr (std::is_same_v<ParamType, uint64>)
		{
			return FAnimNextParamType(EValueType::UInt64, InContainerType);
		}
		else
		{
			static_assert(sizeof(ParamType) == 0, "Type is not expressible as a FAnimNextParamType for available types.");
			return FAnimNextParamType();
		}
	}

	/** Helper function for IsValid */
	UE_API bool IsValidObject() const;

public:
	/** Get a parameter type based on the passed-in built-in type */
	template<typename ParamType>
	static FAnimNextParamType GetType()
	{
		using NonConstType = std::remove_const_t<ParamType>;

		if constexpr (TIsTArray<NonConstType>::Value)
		{
			return GetTypeInner<typename NonConstType::ElementType>(EContainerType::Array);
		}
		else
		{
			return GetTypeInner<NonConstType>(EContainerType::None);
		}
	}

	/** Get the pointer to the object that defines the Enum, Struct, or Class. */
	const UObject* GetValueTypeObject() const
	{
		return ValueTypeObject;
	}

	/** Get the type of the value described by this parameter. */
	EValueType GetValueType() const
	{
		return ValueType;
	}

	/** Get the type of the container described by this parameter. */
	EContainerType GetContainerType() const
	{
		return ContainerType;
	}

	/** 
	 * Helper function returning the size of the type.
	 *
	 * @returns Size in bytes of the type.
	 */
	UE_API size_t GetSize() const;

	/** 
	 * Helper function returning the size of the value type.
	 * This is identical to GetSize for non containers
	 *
	 * @returns Size in bytes of the value type.
	 */	
	UE_API size_t GetValueTypeSize() const;
	
	/** 
	 * Helper function returning the alignment of the type.
	 *
	 * @returns Alignment of the type.
	 */	
	UE_API size_t GetAlignment() const;

	/** 
	 * Helper function returning the alignment of the value type.
	 * This is identical to GetAlignment for non containers
	 *
	 * @returns Alignment of the value type.
	 */	
	UE_API size_t GetValueTypeAlignment() const;

	/** Append a string representing this type to the supplied string builder */
	UE_API void ToString(FStringBuilderBase& InStringBuilder) const;

	/** Get a string representing this type */
	UE_API FString ToString() const;

	/** Get a type from a string */
	static UE_API FAnimNextParamType FromString(const FString& InString);

	/** Get a FRigVMTemplateArgumentType from this type */
	UE_API FRigVMTemplateArgumentType ToRigVMTemplateArgument() const;
	
	/** Construct a parameter type from the passed in FRigVMTemplateArgumentType. */
	static UE_API FAnimNextParamType FromRigVMTemplateArgument(const FRigVMTemplateArgumentType& RigVMType);

	/** Construct a parameter type from the passed in FProperty. */
	static UE_API FAnimNextParamType FromProperty(const FProperty* InProperty);

	/** Construct a parameter type from the passed in FProperty. */
	static FAnimNextParamType FromPropertyBagPropertyDesc(const FPropertyBagPropertyDesc& InPropertyDesc)
	{
		check(InPropertyDesc.ContainerTypes.Num() <= 1);	// Cannot create a param type from a nested container descriptor
		return FAnimNextParamType(InPropertyDesc.ValueType, InPropertyDesc.ContainerTypes.GetFirstContainerType(), InPropertyDesc.ValueTypeObject);
	}

	/** Equality operator */
	friend bool operator==(const FAnimNextParamType& InLHS, const FAnimNextParamType& InRHS)
	{
		return InLHS.ValueType == InRHS.ValueType && InLHS.ContainerType == InRHS.ContainerType && InLHS.ValueTypeObject == InRHS.ValueTypeObject;
	}

	/** Inequality operator */
	friend bool operator!=(const FAnimNextParamType& InLHS, const FAnimNextParamType& InRHS)
	{
		return InLHS.ValueType != InRHS.ValueType || InLHS.ContainerType != InRHS.ContainerType || InLHS.ValueTypeObject != InRHS.ValueTypeObject;
	}

	/** Type hash for TMap storage */
	friend uint32 GetTypeHash(const FAnimNextParamType& InType)
	{
		return HashCombineFast(GetTypeHash((uint32)InType.ValueType | ((uint32)InType.ContainerType << 8)), GetTypeHash(InType.ValueTypeObject));
	}

	/** @return whether this type is explicitly none */
	bool IsNone() const
	{
		return ValueType == EValueType::None && ContainerType == EContainerType::None && !ValueTypeObject;
	}

	/** @return whether this type actually describes a type */
	bool IsValid() const
	{
		const bool bHasValidValueType = ValueType != EValueType::None;
		const bool bHasValidContainerType = (ContainerType == EContainerType::None) || (bHasValidValueType && ContainerType != EContainerType::None);
		const bool bHasValidObjectType = ((ValueType < EValueType::Enum) || ValueType == EValueType::UInt32 || ValueType == EValueType::UInt64) || (ValueType >= EValueType::Enum && ValueType <= EValueType::SoftClass && IsValidObject());
		return bHasValidValueType && bHasValidContainerType && bHasValidObjectType;
	}

	/** @return whether this type represents an object (object/class/softobject/softclass) */
	UE_API bool IsObjectType() const;
};

#undef UE_API
