// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ContainersFwd.h"
#include "UObject/WeakObjectPtr.h"
#include "DataRegistry.h"
#include <algorithm>
#include "Animation/AnimTypes.h"
#include "ReferenceSkeleton.h"
#include "BoneIndices.h"
#include "TransformArray.h"
#include "Components/SkeletalMeshComponent.h"
#include "ReferencePose.generated.h"

class USkeletalMesh;
class USkeleton;

namespace UE::UAF
{

enum class EReferencePoseGenerationFlags : uint8
{
	None = 0,
	FastPath = 1 << 0
};

ENUM_CLASS_FLAGS(EReferencePoseGenerationFlags);

template <typename AllocatorType, typename SetAllocator>
struct TReferencePose
{
	// Transform array of our bind pose sorted by LOD, allows us to truncate the array for a specific LOD
	// Higher LOD come first
	TTransformArray<AllocatorType> ReferenceLocalTransforms;

	// A mapping of LOD sorted bone indices to their parent LOD sorted bone indices per LOD
	// Each list of bone indices is a mapping of: LODSortedBoneIndex -> LODSortedBoneIndex
	// When fast path is enabled, we have a single LOD entry that we truncate to the number of bones for each LOD
	TArray<TArray<FBoneIndexType, AllocatorType>, AllocatorType> LODBoneIndexToParentLODBoneIndexMapPerLOD;

	// A mapping of LOD sorted bone indices to skeletal mesh indices per LOD
	// Each list of bone indices is a mapping of: LODSortedBoneIndex -> SkeletalMeshBoneIndex
	// When fast path is enabled, we have a single LOD entry that we truncate to the number of bones for each LOD
	TArray<TArray<FBoneIndexType, AllocatorType>, AllocatorType> LODBoneIndexToMeshBoneIndexMapPerLOD;

	// A mapping of LOD sorted bone indices to skeleton indices per LOD
	// Each list of bone indices is a mapping of: LODSortedBoneIndex -> SkeletonBoneIndex
	// When fast path is enabled, we have a single LOD entry that we truncate to the number of bones for each LOD
	TArray<TArray<FBoneIndexType, AllocatorType>, AllocatorType> LODBoneIndexToSkeletonBoneIndexMapPerLOD;

	// List of skeleton bone indices
	// Each list of skeleton bone indices is a mapping of: SkeletonBoneIndex -> LODSortedBoneIndex
	// Size of the map equals the number of mesh bones. Mesh bones that are not part of the pose hold a INDEX_NONE.
	// Note: This map is LOD independent. Please use IsBoneEnabled() to check if a given bone is part of a LOD level.
	TArray<FBoneIndexType, AllocatorType> SkeletonBoneIndexToLODBoneIndexMap;

	// List of pose bone indices by mesh bone index
	// List of bone indices is a mapping of: SkeletalMeshBoneIndex -> LODSortedBoneIndex
	// Size of the map equals the number of mesh bones. Mesh bones that are not part of the pose hold a INDEX_NONE.
	// Note: This map is LOD independent. Please use IsBoneEnabled() to check if a given bone is part of a LOD level.
	TArray<FBoneIndexType, AllocatorType> MeshBoneIndexToLODBoneIndexMap;

	// Number of bones for each LOD
	TArray<int32, AllocatorType> LODNumBones;

	// Mapping of mesh bone indices to mesh parent indices for each bone
	TArray<FBoneIndexType, AllocatorType> MeshBoneIndexToParentMeshBoneIndexMap;

	// Mapping of bone names to LOD Bone Indices
	TMap<FName, FBoneIndexType, SetAllocator> BoneNameToLODBoneIndexMap;

	// Curves of interest (material/morph target)
	UE::Anim::FBulkCurveFlags CurveFlags;

	TWeakObjectPtr<const USkeletalMeshComponent> SkeletalMeshComponent = nullptr;
	TWeakObjectPtr<const USkeletalMesh> SkeletalMesh = nullptr;
	TWeakObjectPtr<const USkeleton> Skeleton = nullptr;
	EReferencePoseGenerationFlags GenerationFlags = EReferencePoseGenerationFlags::None;

	static constexpr FBoneIndexType RootBoneIndex = 0;

	TReferencePose() = default;

	bool IsValid() const
	{
		return ReferenceLocalTransforms.Num() > 0;
	}

	int32 GetNumBonesForLOD(int32 LODLevel) const
	{
		const int32 NumLODS = LODNumBones.Num();

		return (LODLevel < NumLODS) ? LODNumBones[LODLevel] : (NumLODS > 0) ? LODNumBones[0] : 0;
	}

	bool IsBoneEnabled(int32 BoneIndex, int32 LODLevel) const
	{
		return (BoneIndex < GetNumBonesForLOD(LODLevel));
	}

	bool IsFastPath() const
	{
		return (GenerationFlags & EReferencePoseGenerationFlags::FastPath) != EReferencePoseGenerationFlags::None;
	}

	void Initialize(const FReferenceSkeleton& RefSkeleton
		, const TArray<TArray<FBoneIndexType>>& InLODBoneIndexToParentLODBoneIndexMapPerLOD
		, const TArray<TArray<FBoneIndexType>>& InLODBoneIndexToMeshBoneIndexMapPerLOD
		, const TArray<TArray<FBoneIndexType>>& InLODBoneIndexToSkeletonBoneIndexMapPerLOD
		, const TArray<FBoneIndexType>& InSkeletonBoneIndexToLODBoneIndexMap
		, const TArray<FBoneIndexType>& InMeshBoneIndexToLODBoneIndexMap
		, const TArray<int32, AllocatorType>& InLODNumBones
		, const TMap<FName, FBoneIndexType>& InNameToLODBoneIndexMap
		, const UE::Anim::FBulkCurveFlags& InCurveFlags
		, bool bFastPath)
	{
		const int32 NumBonesLOD0 = !InLODNumBones.IsEmpty() ? InLODNumBones[0] : 0;
		const int32 NumBonesMesh = RefSkeleton.GetRefBoneInfo().Num();

		MeshBoneIndexToParentMeshBoneIndexMap.SetNum(NumBonesMesh);
		ReferenceLocalTransforms.SetNum(NumBonesLOD0);
		LODBoneIndexToParentLODBoneIndexMapPerLOD = InLODBoneIndexToParentLODBoneIndexMapPerLOD;
		LODBoneIndexToMeshBoneIndexMapPerLOD = InLODBoneIndexToMeshBoneIndexMapPerLOD;
		LODBoneIndexToSkeletonBoneIndexMapPerLOD = InLODBoneIndexToSkeletonBoneIndexMapPerLOD;
		SkeletonBoneIndexToLODBoneIndexMap = InSkeletonBoneIndexToLODBoneIndexMap;
		MeshBoneIndexToLODBoneIndexMap = InMeshBoneIndexToLODBoneIndexMap;
		BoneNameToLODBoneIndexMap = InNameToLODBoneIndexMap;
		CurveFlags = InCurveFlags;
		LODNumBones = InLODNumBones;

		const TArray<FTransform>& RefBonePose = RefSkeleton.GetRefBonePose();
		const TArray<FBoneIndexType>& BoneLODIndexToMeshIndexMap0 = InLODBoneIndexToMeshBoneIndexMapPerLOD[0]; // Fill the transforms with the LOD0 indexes
		const TArray<FMeshBoneInfo>& RefBoneInfo = RefSkeleton.GetRefBoneInfo();

		for (int32 LODBoneIndex = 0; LODBoneIndex < NumBonesLOD0; ++LODBoneIndex)
		{
			// TODO : For SoA this is un-optimal, as we are using a TransformAdapter. Evaluate using a specific SoA iterator
			ReferenceLocalTransforms[LODBoneIndex] = RefBonePose[BoneLODIndexToMeshIndexMap0[LODBoneIndex]];
		}

		for (int32 BoneIndex = 0; BoneIndex < NumBonesMesh; ++BoneIndex)
		{
			MeshBoneIndexToParentMeshBoneIndexMap[BoneIndex] = RefBoneInfo[BoneIndex].ParentIndex;
		}

		GenerationFlags = bFastPath ? EReferencePoseGenerationFlags::FastPath : EReferencePoseGenerationFlags::None;
	}

	// Returns a list of LOD sorted parent bone indices, a mapping of: LODSortedBoneIndex -> LODSortedBoneIndex
	const TArrayView<const FBoneIndexType> GetLODBoneIndexToParentLODBoneIndexMap(int32 LODLevel) const
	{
		TArrayView<const FBoneIndexType> ArrayView;

		if (LODLevel >= 0 && (IsFastPath() || LODLevel < LODBoneIndexToParentLODBoneIndexMapPerLOD.Num()))
		{
			const int32 NumBonesForLOD = GetNumBonesForLOD(LODLevel);
			const int32 LODIndex = IsFastPath() ? 0 : LODLevel;

			ArrayView = MakeArrayView(LODBoneIndexToParentLODBoneIndexMapPerLOD[LODIndex].GetData(), NumBonesForLOD);
		}

		return ArrayView;
	}

	// Returns a list of LOD sorted skeletal mesh bone indices, a mapping of: LODSortedBoneIndex -> SkeletalMeshBoneIndex
	const TArrayView<const FBoneIndexType> GetLODBoneIndexToMeshBoneIndexMap(int32 LODLevel) const
	{
		TArrayView<const FBoneIndexType> ArrayView;

		if (LODLevel >= 0 && (IsFastPath() || LODLevel < LODBoneIndexToMeshBoneIndexMapPerLOD.Num()))
		{
			const int32 NumBonesForLOD = GetNumBonesForLOD(LODLevel);
			const int32 LODIndex = IsFastPath() ? 0 : LODLevel;

			ArrayView = MakeArrayView(LODBoneIndexToMeshBoneIndexMapPerLOD[LODIndex].GetData(), NumBonesForLOD);
		}

		return ArrayView;
	}

	// Returns a list of LOD sorted skeleton bone indices, a mapping of: LODSortedBoneIndex -> SkeletonBoneIndex
	const TArrayView<const FBoneIndexType> GetLODBoneIndexToSkeletonBoneIndexMap(int32 LODLevel) const
	{
		TArrayView<const FBoneIndexType> ArrayView;

		if (LODLevel >= 0 && (IsFastPath() || LODLevel < LODBoneIndexToSkeletonBoneIndexMapPerLOD.Num()))
		{
			const int32 NumBonesForLOD = GetNumBonesForLOD(LODLevel);
			const int32 LODIndex = IsFastPath() ? 0 : LODLevel;

			ArrayView = MakeArrayView(LODBoneIndexToSkeletonBoneIndexMapPerLOD[LODIndex].GetData(), NumBonesForLOD);
		}

		return ArrayView;
	}

	// Returns a list of LOD bone indices, a mapping of: SkeletonBoneIndex -> LODSortedBoneIndex
	const TArrayView<const FBoneIndexType> GetSkeletonBoneIndexToLODBoneIndexMap() const
	{
		return SkeletonBoneIndexToLODBoneIndexMap;
	}

	// Return a list of LOD bone indices, a mapping of: SkeletalMeshBoneIndex -> LODSortedBoneIndex
	const TArrayView<const FBoneIndexType> GetMeshBoneIndexToLODBoneIndexMap() const
	{
		return MeshBoneIndexToLODBoneIndexMap;
	}

	const TMap<FName, FBoneIndexType>& GetBoneNameToLODBoneIndexMap() const
	{
		return BoneNameToLODBoneIndexMap;
	}

	// Query to find a LODBoneIndex for an associated BoneName
	// Returns INVALID_BONE_INDEX if no bone found for a given name
	FBoneIndexType FindLODBoneIndexFromBoneName(FName BoneName) const
	{
		if (const FBoneIndexType* BoneIndex = BoneNameToLODBoneIndexMap.Find(BoneName))
		{
			return *BoneIndex;
		}
		return INVALID_BONE_INDEX;
	}

	int32 GetMeshBoneIndexFromLODBoneIndex(FBoneIndexType LODBoneIndex) const
	{
		const int32 NumBonesLOD0 = LODBoneIndexToMeshBoneIndexMapPerLOD[0].Num();
		check(LODBoneIndex < NumBonesLOD0);
		return LODBoneIndexToMeshBoneIndexMapPerLOD[0][LODBoneIndex];
	}

	// Returns a mapping of mesh bone indices to mesh parent indices for each bone
	TConstArrayView<FBoneIndexType> GetMeshBoneIndexToParentMeshBoneIndexMap() const
	{
		return MeshBoneIndexToParentMeshBoneIndexMap;
	}

	// Returns the corresponding skeleton Bone Index to a LOD Bone Index
	FBoneIndexType GetSkeletonBoneIndexFromLODBoneIndex(FBoneIndexType LODBoneIndex) const
	{
		const int32 NumBonesLOD0 = LODBoneIndexToSkeletonBoneIndexMapPerLOD[0].Num();
		check(LODBoneIndex < NumBonesLOD0);
		return LODBoneIndexToSkeletonBoneIndexMapPerLOD[0][LODBoneIndex];
	}

	// Translate a skeletal mesh bone index to a pose bone index.
	FBoneIndexType GetLODBoneIndexFromMeshBoneIndex(FBoneIndexType MeshBoneIndex) const
	{
		check(MeshBoneIndex < MeshBoneIndexToLODBoneIndexMap.Num());
		return MeshBoneIndexToLODBoneIndexMap[MeshBoneIndex];
	}

	FBoneIndexType GetLODBoneIndexFromSkeletonBoneIndex(FBoneIndexType SkeletionBoneIndex) const
	{
		check(SkeletionBoneIndex < SkeletonBoneIndexToLODBoneIndexMap.Num());
		return SkeletonBoneIndexToLODBoneIndexMap[SkeletionBoneIndex];
	}

	FBoneIndexType GetLODParentBoneIndex(int32 LODLevel, FBoneIndexType LODBoneIndex) const
	{
		const TArrayView<const FBoneIndexType> ParentLODMap = GetLODBoneIndexToParentLODBoneIndexMap(LODLevel);
		if (ParentLODMap.IsValidIndex(LODBoneIndex))
		{
			const FBoneIndexType LODParentBoneIndex = ParentLODMap[LODBoneIndex];
			if (LODParentBoneIndex != UINT16_MAX)
			{
				return LODParentBoneIndex;
			}
		}
		return INVALID_BONE_INDEX;
	}

	const USkeletalMesh* GetSkeletalMeshAsset() const
	{
		return SkeletalMesh.Get();
	}
	const USkeleton* GetSkeletonAsset() const
	{
		return Skeleton.Get();
	}

	FTransform GetRefPoseTransform(FBoneIndexType LODBoneIndex) const
	{
		const int32 NumBonesLOD0 = LODBoneIndexToMeshBoneIndexMapPerLOD[0].Num();
		check(LODBoneIndex < NumBonesLOD0);

		return ReferenceLocalTransforms[LODBoneIndex];
	}

	const FQuat GetRefPoseRotation(FBoneIndexType LODBoneIndex) const
	{
		const int32 NumBonesLOD0 = LODBoneIndexToMeshBoneIndexMapPerLOD[0].Num();
		check(LODBoneIndex < NumBonesLOD0);

		return ReferenceLocalTransforms[LODBoneIndex].GetRotation();
	}

	const FVector GetRefPoseTranslation(FBoneIndexType LODBoneIndex) const
	{
		const int32 NumBonesLOD0 = LODBoneIndexToMeshBoneIndexMapPerLOD[0].Num();
		check(LODBoneIndex < NumBonesLOD0);

		return ReferenceLocalTransforms[LODBoneIndex].GetTranslation();
	}

	const FVector GetRefPoseScale3D(FBoneIndexType LODBoneIndex) const
	{
		const int32 NumBonesLOD0 = LODBoneIndexToMeshBoneIndexMapPerLOD[0].Num();
		check(LODBoneIndex < NumBonesLOD0);

		return ReferenceLocalTransforms[LODBoneIndex].GetScale3D();
	}

	// Get the LOD level of the 'source' of this pose.
	// If this pose is generated from a dynamic source, such as a skeletal mesh component, returns the LOD of the component
	// TODO: As the predicted LOD level can vary across the frame, the LOD should be cached at a module-component level so modules have a consistent view
	// of the current LOD across the frame. For now we are OK as the LOD level is precalculated in the mesh component and we have manual tick dependencies
	// set up to prevent races on the value
	int32 GetSourceLODLevel() const
	{
		const USkeletalMeshComponent* MeshComponent = SkeletalMeshComponent.Get();
		if (MeshComponent == nullptr)
		{
			return 0;
		}
		return MeshComponent->GetPredictedLODLevel();
	}
};

/**
* Hide the allocator usage
*/
using FReferencePose = TReferencePose<FDefaultAllocator, FDefaultSetAllocator>;

} // namespace UE::UAF

// USTRUCT wrapper for reference pose
USTRUCT()
struct FAnimNextReferencePose
#if CPP
	: UE::UAF::FReferencePose
#endif
{
	GENERATED_BODY()
};
