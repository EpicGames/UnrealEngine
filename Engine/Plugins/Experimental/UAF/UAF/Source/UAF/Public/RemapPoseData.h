// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AttributesRuntime.h"
#include "ReferencePose.h"
#include "LODPose.h"
#include "BoneIndices.h"

#include "RemapPoseData.generated.h"

USTRUCT()
struct FRemappedBone
{
	GENERATED_BODY()

	FBoneIndexType SourceBoneIndex = INVALID_BONE_INDEX;
	FBoneIndexType TargetBoneIndex = INVALID_BONE_INDEX;
};

USTRUCT()
struct FBoneRemapping
{
	GENERATED_BODY()

	/**
	 * Source to target bone index array for bones from the target pose that are also present in the source pose are stored.
	 * This is a subset of the bones from the target pose. The smaller the subset, the less bones to remap.
	 */
	TArray<FRemappedBone> BoneIndexMap;

	/**
	 * Target root bone with index 0 mapped to the source skeletal mesh. The bone index stored here represents the bone index on the source
	 * skeletal mesh of the bone with the same name as the root bone on the target. This is used as bone attachment point.
	 */
	int32 TargetRootToSourceBoneIndex = INDEX_NONE;
};

USTRUCT()
struct FRemapPoseData
{
	GENERATED_BODY()

	/**
	 * Check if we need to re-initialize the mapping. This will return true in case of first time usage of the data,
	 * or in the case that either the source or target pose changed.
	 */
	bool ShouldReinit(const UE::UAF::FReferencePose& InSourceRefPose,
		const UE::UAF::FReferencePose& InTargetRefPose) const;

	/**
	 * Re-initializes the mapping for each LOD. Call ShouldReinit() before to avoid unnecessary compute.
	 * Compare the bones from the source and target pose and find the subset of the target that is also present in the source.
	 */
	void Reinit(const UE::UAF::FReferencePose& InSourceRefPose,
		const UE::UAF::FReferencePose& InTargetRefPose);

	/**
	 * Remap a pose using the previously cached mapping. Call this at runtime.
	 * Prepares the target pose internally (allocates, fills ref pose for unmapped bones).
	 */
	UAF_API void RemapPose(const UE::UAF::FLODPoseHeap& SourcePose,
		UE::UAF::FLODPoseHeap& OutTargetPose) const;

	/**
	 * Remap bone transforms from source into an already-prepared target pose.
	 * The target must already have memory allocated and ref pose set for unmapped bones.
	 * Operates on FLODPose base, works with any allocator (heap or stack).
	 */
	UAF_API void RemapBones(const UE::UAF::FLODPose& SourcePose, UE::UAF::FLODPose& OutTargetPose) const;

	/**
	 * Remap attributes from source into an already-prepared target attribute container.
	 * Bone indices are translated from source LOD pose space to target LOD pose space via the skeleton bone name.
	 * Attributes whose source bone has no counterpart in the target skeleton are silently dropped.
	 * Templated to support both heap (FHeapAttributeContainer) and stack (FStackAttributeContainer) output containers.
	 */
	template<typename OutContainerType>
	void RemapAttributes(const UE::UAF::FLODPose& SourceLODPose,
		const UE::Anim::FHeapAttributeContainer& InAttributes,
		const UE::UAF::FLODPose& TargetLODPose,
		OutContainerType& OutAttributes) const;

	/**
	 * Temporary calculate modelspace transform, until the LODPose is able to do that.
	 */
	FTransform RecursiveCalcModelspaceTransform(const UE::UAF::FLODPose& Pose, FBoneIndexType BoneIndex) const;

	const UE::UAF::FReferencePose* SourceRefPose = nullptr;
	const UE::UAF::FReferencePose* TargetRefPose = nullptr;

	/**
	 * Bone mapping from the source to the target skeletal mesh per LOD level combination.
	 * The array's length equals the number of source skeletal mesh LODs available. Each of the elements
	 * will contain a bone mapping based on the given source LOD for the target skeletal mesh LOD. The number of elements
	 * of the sub-array equals to the number of LOD levels of the target skeletal mesh.
	 * Example: SourceToTargetBoneIndexMapPerLOD[4][2] gives a bone mapping from source LOD 4 to target LOD 2 and will usually contain
	 * less elements than the target skeletal mesh has bones.
	 */
	TArray<TArray<FBoneRemapping>> SourceToTargetBoneIndexMapPerLOD;
};