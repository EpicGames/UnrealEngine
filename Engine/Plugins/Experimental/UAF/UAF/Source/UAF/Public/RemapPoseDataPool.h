// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemapPoseData.h"
#include "ReferencePose.h"
#include "Misc/ScopeRWLock.h"

#define UE_API UAF_API

namespace UE::UAF
{

/**
 * Shared pool for FRemapPoseData instances, keyed by (source, target) reference pose pairs.
 * Avoids per-node caching and provides fast lookup with minimal locking.
 * Uses a read-write lock: concurrent reads are lock-free, writes only lock when a new pair is first encountered or needs to be reinited.
 */
class FRemapPoseDataPool
{
public:
	/** Access the singleton pool instance. */
	static UE_API FRemapPoseDataPool& Get();

	/**
	 * Returns true if the source and target reference poses refer to different skeletons, meaning a copy between them requires remapping.
	 * This is a cheap reference pose pointer comparison, no locking or pool access.
	 */
	[[nodiscard]] static bool NeedsRemapping(const FReferencePose& InSourceRefPose, const FReferencePose& InTargetRefPose)
	{
		return &InSourceRefPose != &InTargetRefPose;
	}

	/**
	 * Get or create remap data for a source->target reference pose pair.
	 * On first access for a given pair, FRemapPoseData::Reinit() is called to build the bone mapping. Subsequent calls with the same pair return the cached data.
	 * If either reference pose's skeletal mesh has changed since the last Reinit, the data is refreshed.
	 * Thread safety notes: GetRemapData() returns a reference to the cached FRemapPoseData. The read lock is released before the reference is used by the caller. This is safe because:
	 *   1. GarbageCollect() only runs from FDataRegistry::HandlePostGarbageCollect, which executes on the game thread between frames and never during animation evaluation.
	 *   2. ShouldReinit() only returns true if the skeletal mesh pointer changed, which doesn't happen mid-frame.
	 *   3. All animation evaluation completes within a single frame before GC can run. Therefore, returned references remain valid for the duration of their use during evaluation.
	 */
	UE_API const FRemapPoseData& GetRemapData(const FReferencePose& InSourceRefPose, const FReferencePose& InTargetRefPose);

	/** Remove pool entries that reference stale skeletal meshes. Call periodically (e.g. on GC). */
	UE_API void GarbageCollect();

private:
	struct FRefPosePair
	{
		const FReferencePose* Source;
		const FReferencePose* Target;

		bool operator==(const FRefPosePair& Other) const
		{
			return Source == Other.Source && Target == Other.Target;
		}

		friend uint32 GetTypeHash(const FRefPosePair& Pair)
		{
			return HashCombine(PointerHash(Pair.Source), PointerHash(Pair.Target));
		}
	};

	TMap<FRefPosePair, TUniquePtr<FRemapPoseData>> Pool;
	mutable FRWLock PoolLock;
};

} // namespace UE::UAF

#undef UE_API
