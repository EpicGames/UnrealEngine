// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ReferencePose.h"
#include "LODPose.h"
#include "AnimEncoding.h"
#include "AnimationRuntime.h"
#include "Animation/Skeleton.h"
#include "Animation/SkeletonRemapping.h"
#include "RemapPoseData.h"

namespace UE::UAF
{
	class FRetargetingTools
	{
	public:
		/**
		 * Build sparse bone-track pairs for each translation retargeting mode.
		 * Pre-filters Source to target bone relations into three lists:
		 * AnimationScaled, AnimationRelative, and OrientAndScale.
		 * @param SourcePose                Source pose to retarget from.
		 * @param TargetPose                Retargeting target pose.
		 * @param RemapPoseData             Cached source to target bone mapping per LOD.
		 * @param SourceSkeleton            Source skeleton to retarget from.
		 * @param TargetSkeleton            Retargeting target skeleton.
		 * @param bUseSourceRetargetModes   Set to true when you wish to extract the retarget mode from the source skeleton instead of the target skeleton.
		 * @param bDisableRetargeting       Is translational retargeting disabled?
		 * @param OutSkeletonPairs          Output pairs for EBoneTranslationRetargetingMode::Skeleton retargeting mode where bones use ref pose translation but rotate along with the source.
		 * @param OutAnimScalePairs         Output pairs for AnimationScaled.
		 * @param OutAnimRelativePairs      Output pairs for AnimationRelative.
		 * @param OutOrientAndScalePairs    Output pairs for OrientAndScale.
		 */
		static UAF_API void BuildRetargetingPairs(
			const FLODPose& SourcePose,
			const FLODPose& TargetPose,
			const FRemapPoseData& RemapPoseData,
			const USkeleton* SourceSkeleton,
			const USkeleton* TargetSkeleton,
			bool bUseSourceRetargetModes,
			bool bDisableRetargeting,
			BoneTrackArray& OutSkeletonPairs,
			BoneTrackArray& OutAnimScalePairs,
			BoneTrackArray& OutAnimRelativePairs,
			BoneTrackArray& OutOrientAndScalePairs);

		/**
		 * Per-bone reference-pose translation correction.
		 * This is the step that moves authored offsets across differing proportions without touching rotation.
		 * @tparam bIsAdditive              True if the source pose is additive (deltas relative to a reference pose),
		 *                                  in which case additive-space retargeting is applied. False for absolute poses.
		 * @param SourceSkeleton            Source skeleton to retarget from.
		 * @param TargetSkeleton            Retargeting target skeleton.
		 * @param SkeletonRemapping         Maps source and target skeleton bone indices.
		 * @param bUseSourceRetargetModes   Set to true when you wish to extract the retarget mode from the source skeleton instead of the target skeleton.
		 * @param bDisableRetargeting       Is translational retargeting disabled?
		 * @param PoseBoneIndex             Target pose bone index.
		 * @param TargetSkeletonBoneIndex   Target skeleton bone index.
		 * @param OutPose                   Target pose receiving the translation fix.
		 */
		template<bool bIsAdditive>
		static FORCEINLINE void TranslationRetargetReferencePoseBone(
			const USkeleton* SourceSkeleton,
			const USkeleton* TargetSkeleton,
			const FSkeletonRemapping& SkeletonRemapping,
			bool bUseSourceRetargetModes,
			bool bDisableRetargeting,
			const int32 PoseBoneIndex,
			const int32 TargetSkeletonBoneIndex,
			FLODPose& OutPose)
		{
			const int32 SourceSkeletonBoneIndex = SkeletonRemapping.GetSourceSkeletonBoneIndex(TargetSkeletonBoneIndex);
			const EBoneTranslationRetargetingMode::Type RetargetMode = FAnimationRuntime::GetBoneTranslationRetargetingMode(
				bUseSourceRetargetModes,
				SourceSkeletonBoneIndex,
				TargetSkeletonBoneIndex,
				SourceSkeleton,
				TargetSkeleton,
				bDisableRetargeting);

			if (RetargetMode != EBoneTranslationRetargetingMode::Skeleton)
			{
				FVector BoneTranslation = OutPose.LocalTransformsView[PoseBoneIndex].GetTranslation();

				if constexpr (bIsAdditive)
				{
					BoneTranslation = SkeletonRemapping.RetargetAdditiveTranslationToTargetSkeleton(TargetSkeletonBoneIndex, BoneTranslation);
				}
				else
				{
					BoneTranslation = SkeletonRemapping.RetargetBoneTranslationToTargetSkeleton(TargetSkeletonBoneIndex, BoneTranslation);
				}

				OutPose.LocalTransformsView[PoseBoneIndex].SetTranslation(BoneTranslation);
			}
		}

		/**
		 * Reference-pose correction per bone.
		 * @param SourceSkeleton            Source skeleton to retarget from.
		 * @param TargetSkeleton            Retargeting target skeleton.
		 * @param SkeletonRemapping         Maps source and target skeleton bone indices.
		 * @param bUseSourceRetargetModes   Set to true when you wish to extract the retarget mode from the source skeleton instead of the target skeleton.
		 * @param bDisableRetargeting       Is translational retargeting disabled?
		 * @param OutPose                   Target pose to work on.
		 */
		static UAF_API void ReferencePoseRetarget(
			const USkeleton* SourceSkeleton,
			const USkeleton* TargetSkeleton,
			const FSkeletonRemapping& SkeletonRemapping,
			bool bUseSourceRetargetModes,
			bool bDisableRetargeting,
			FLODPose& OutPose);

		/**
		 * Translation retarget a pose.
		 * Applies the bone’s EBoneTranslationRetargetingMode:
		 *  - AnimationScaled: scale local translation by source/target ref-length ratio.
		 *  - AnimationRelative: transfer delta from authored base into target basis.
		 *    and add to current transform (skip if baked additive upstream).
		 *  - OrientAndScale: orientation-aware scaling when enabled.
		 * @param TargetReferencePose       Target reference pose.
		 * @param SourceSkeleton            Source skeleton to retarget from.
		 * @param TargetSkeleton            Retargeting target skeleton.
		 * @param RetargetTransforms        The name of the retarget source to find.
		 * @param BoneTransform             .
		 * @param SkeletonBoneIndex         Target skeleton bone index.
		 * @param LODBoneIndex              Bone index in current LOD view.
		 * @param bIsBakedAdditive          Treat input as baked additive.
		 * @param bDisableRetargeting       Early-out passthrough.
		 */
		static UAF_API void RetargetBoneTransform(
			const FReferencePose& TargetReferencePose,
			const USkeleton* SourceSkeleton,
			const USkeleton* TargetSkeleton,
			const FName& SourceName,
			const TArray<FTransform>& RetargetTransforms,
			FTransform& BoneTransform,
			const int32 SkeletonBoneIndex,
			const int32 LODBoneIndex,
			const bool bIsBakedAdditive,
			const bool bDisableRetargeting);

		/**
		 * Translation retarget a pose.
		 * Convenience path that pulls the source’s ref local poses (RetargetSource)
		 * and forwards to the explicit-transform overload.
		 * @param TargetReferencePose       Target reference pose.
		 * @param SourceSkeleton            Source skeleton to retarget from.
		 * @param TargetSkeleton            Retargeting target skeleton.
		 * @param RetargetTransforms        The name of the retarget source to find.
		 * @param BoneTransform             .
		 * @param SkeletonBoneIndex         Target skeleton bone index.
		 * @param LODBoneIndex              Bone index in current LOD view.
		 * @param bIsBakedAdditive          Treat input as baked additive.
		 * @param bDisableRetargeting       Early-out passthrough.
		 */
		static FORCEINLINE void RetargetBoneTransform(
			const FReferencePose& TargetReferencePose,
    		const USkeleton* SourceSkeleton,
    		const USkeleton* TargetSkeleton,
    		const FName& RetargetSource,
    		FTransform& BoneTransform,
    		const int32 SkeletonBoneIndex,
    		const int32 LODBoneIndex,
    		const bool bIsBakedAdditive,
    		const bool bDisableRetargeting)
    	{
    		if (SourceSkeleton)
    		{
    			const TArray<FTransform>& RetargetTransforms = SourceSkeleton->GetRefLocalPoses(RetargetSource);
    			RetargetBoneTransform(TargetReferencePose, SourceSkeleton, TargetSkeleton, RetargetSource, RetargetTransforms, BoneTransform, SkeletonBoneIndex, LODBoneIndex, bIsBakedAdditive, bDisableRetargeting);
    		}
    	}

		/**
		 * AnimationRelative path: apply authored-base delta in target basis.
		 * @param TargetPoseBoneIndex       Bone index of the bone on the target pose.
		 * @param TargetSkeletonBoneIndex   Corresponding bone index in the target's skeleton.
		 * @param TargetReferencePose       Target reference pose.
		 * @param SourceToTargetRemapping   Maps source and target skeleton bone indices.
		 * @param BaseTransform             Authored base (source) local-space transform.
		 * @param InOutBoneTransform        In/out current local transform to receive delta.
		 */
		static UAF_API void RetargetBone_AnimRelative(
			const int32 TargetPoseBoneIndex,
			const int32 TargetSkeletonBoneIndex,
			const FReferencePose& TargetReferencePose,
			const FSkeletonRemapping& SourceToTargetRemapping,
			FTransform BaseTransform,
			FTransform& InOutBoneTransform);

		/**
		 * Translation retarget a pose.
		 * Runs the three per-mode passes (Scaled, Relative, Orient+Scale) over OutPose.
		 * Work scales with number of pairs, not bones.
		 * @param TargetReferencePose			Target reference pose.
		 * @param SourceToTargetRemapping		Maps source and target skeleton bone indices.
		 * @param SkeletonRetargetingPairs		Pairs used for retargeting where bones use ref pose translation but rotate along with the source.
		 * @param AnimScaleRetargetingPairs		Pairs for AnimationScaled pass.
		 * @param AnimRelativeRetargetingPairs	Pairs for AnimationRelative pass.
		 * @param OrientAndScaleRetargetingPairs Pairs for OrientAndScale pass.
		 * @param SourceReferencePoseTransforms  Source reference pose transforms.
		 * @param OutPose						Target pose to work on in-place.
		 */
		static UAF_API void RetargetPose(
			const FReferencePose& TargetReferencePose,
			const FSkeletonRemapping& SourceToTargetRemapping,
			BoneTrackArray& SkeletonRetargetingPairs,
			BoneTrackArray& AnimScaleRetargetingPairs,
			BoneTrackArray& AnimRelativeRetargetingPairs,
			BoneTrackArray& OrientAndScaleRetargetingPairs,
			const TArray<FTransform>& SourceReferencePoseTransforms,
			FLODPose& OutPose);
	};
} // end namespace UE::UAF
