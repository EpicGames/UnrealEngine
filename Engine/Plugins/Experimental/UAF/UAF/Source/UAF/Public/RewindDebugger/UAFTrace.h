// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#include "ObjectTrace.h"
#include "StructUtils/PropertyBag.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "ObjectTraceDefines.h"
#include "Trace/Trace.h"
#include "Templates/SharedPointer.h"

#define UAF_TRACE_ENABLED (OBJECT_TRACE_ENABLED && !(UE_BUILD_SHIPPING || UE_BUILD_TEST))

#if UAF_TRACE_ENABLED

UE_TRACE_CHANNEL_EXTERN(UAFChannel, UAF_API);

class UObject;
struct FAnimNextModuleInstance;
struct FUAFAssetInstance;
struct FAnimNextGraphInstance;
struct FUAFInstanceVariableContainer;

namespace UE::UAF
{
struct FUAFTrace
{
	UAF_API static const FGuid CustomVersionGUID;
	
	UAF_API static void Reset();
	
	UAF_API static void OutputUAFInstance(const FUAFAssetInstance* DataInterface, const UObject* OuterObject);
	UAF_API static void OutputUAFVariables(const FUAFAssetInstance* DataInterface, const UObject* OuterObject);

private:
	UAF_API static bool OutputUAFVariableSet(const TSharedRef<FUAFInstanceVariableContainer>& VariableSet, uint64 InstanceId, const UObject* OuterObject);
};

}


#define TRACE_UAF_INSTANCE(DataInterface, OuterObject) UE::UAF::FUAFTrace::OutputUAFInstance(DataInterface, OuterObject);
#define TRACE_UAF_VARIABLES(DataInterface, OuterObject) UE::UAF::FUAFTrace::OutputUAFVariables(DataInterface, OuterObject);
#else
#define TRACE_UAF_INSTANCE(DataInterface, OuterObject)
#define TRACE_UAF_VARIABLES(DataInterface, OuterObject)
#endif
