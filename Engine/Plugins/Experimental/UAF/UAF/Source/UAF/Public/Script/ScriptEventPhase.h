// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace UE::UAF
{

// Phase is used as a general ordering constraint on event execution
enum class EScriptEventPhase : uint8
{
	// Before any execution, e.g. for copying data from the game thread
	PreExecute,

	// General execution, e.g. a prephysics event
	Execute,
};

}
