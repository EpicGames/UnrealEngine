// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Function.h"

struct FTickFunction;
struct FUAFAssetInstance;

namespace UE::UAF
{

// Binding context for tick functions
struct FTickFunctionBindingContext
{
	FTickFunctionBindingContext(FUAFAssetInstance& InAssetInstance, UObject* InObject, UWorld* InWorld)
		: AssetInstance(InAssetInstance)
		, Object(InObject)
		, World(InWorld)
	{}

	// The asset instance
	FUAFAssetInstance& AssetInstance;

	// Object that a asset instance is bound on (e.g. a component)
	UObject* Object = nullptr;

	// World that the object is contained within
	UWorld* World = nullptr;
};

// Function called to bind a tick function to a script event, set up prerequisites etc.
using FTickFunctionBindingFunction = TFunction<void(const FTickFunctionBindingContext& InContext, FTickFunction& InTickFunction)>;

}
