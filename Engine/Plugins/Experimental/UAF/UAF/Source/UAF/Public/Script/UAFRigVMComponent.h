// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFScriptComponent.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "UAFRigVMComponent.generated.h"

struct FAnimNextGraphInstance;
struct FAnimNextModuleInstance;

#define UE_API UAF_API

// Asset instance component supplying work memory for RigVM execution
USTRUCT()
struct FUAFRigVMComponent : public FUAFScriptComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	// Get the RigVM extended execute context
	FRigVMExtendedExecuteContext& GetExtendedExecuteContext()
	{
#if DO_CHECK
		check(bIsInitialized);
#endif
		check(ExtendedExecuteContext.VMHash != 0);
		return ExtendedExecuteContext;
	}

	// FUAFScriptComponent interface
	virtual void CallEventByName(TConstStructView<FUAFScriptContextData> InContextData) override
	{
		StaticCallEvent(TStructView<FUAFScriptComponent>(GetScriptStruct(), reinterpret_cast<uint8*>(this)), InContextData);
	}
	virtual TConstArrayView<UE::UAF::FScriptEventInfo> GetScriptEvents() override;

private:
	// Calls an event in RigVM with appropriate context setup
	UE_API static void StaticCallEvent(TStructView<FUAFScriptComponent> InScriptComponent, TConstStructView<FUAFScriptContextData> InContextData);
	
	// FUAFAssetInstanceComponent interface
	virtual void OnBindToInstance() override;

	friend FAnimNextGraphInstance;
	friend FAnimNextModuleInstance;

	// Cached events 
	TArray<UE::UAF::FScriptEventInfo> ImplementedEvents; 

	// Extended execute context instance for our asset instance, we own it
	UPROPERTY(Transient)
	FRigVMExtendedExecuteContext ExtendedExecuteContext;

	// VM used for execution
	UPROPERTY(Transient)
	TObjectPtr<URigVM> VM;

#if DO_CHECK
	// Flag for initialization checks
	bool bIsInitialized = false;
#endif
};

#undef UE_API