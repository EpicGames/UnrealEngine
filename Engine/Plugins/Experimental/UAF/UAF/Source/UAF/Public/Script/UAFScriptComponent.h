// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFAssetInstanceComponent.h"
#include "Engine/EngineBaseTypes.h"
#include "Script/ScriptEventPhase.h"
#include "Script/ScriptEventTickFunctionBindings.h"
#include "StructUtils/StructView.h"
#include "UAFScriptComponent.generated.h"

#define UE_API UAF_API

struct FAnimNextGraphInstance;
struct FAnimNextModuleInstance;
struct FUAFScriptContextData;
struct FUAFScriptComponent;

namespace UE::UAF
{

// Implemented script event
struct FScriptEvent
{
	// Function used to call an event
	using FEventFunc = void (*)(TStructView<FUAFScriptComponent> /* InScriptComponent */, TConstStructView<FUAFScriptContextData> /* InContextData */);

	FScriptEvent() = default;

	FScriptEvent(FEventFunc InEventFunc, FName InEventName)
		: EventFunc(InEventFunc)
		, EventName(InEventName)
	{}

	// Check whether this even can be called
	bool IsCallable() const
	{
		return EventFunc != nullptr && !EventName.IsNone();
	}

	// Get the name of this event
	FName GetEventName() const
	{
		return EventName;
	}

private:
	friend ::FUAFScriptComponent;

	// Function ptr used to call the event
	FEventFunc EventFunc = nullptr;

	// The name of the event
	FName EventName;
};

// Implemented script event wrapper
struct FScriptEventInfo
{
	// The event function itself (can be nullptr for non-native events)
	FScriptEvent Event;

	// Associated event's struct (e.g. for RigVM events)
	UScriptStruct* Struct = nullptr;

	// Binding callback, called when event is first set up for a system
	FTickFunctionBindingFunction Binding;

	// Sort order used to determine the order withing a tick group the event runs in
	int32 SortOrder = 0;

	// The phase of execution that the event runs in
	EScriptEventPhase Phase = EScriptEventPhase::Execute;

	// The tick group the event runs in
	ETickingGroup TickGroup = ETickingGroup::TG_PrePhysics;

	// The tick group the event completes in
	ETickingGroup EndTickGroup = ETickingGroup::TG_PrePhysics;

	// Whether this is a user-authored event, or a compiler-generated event
	uint8 bUserEvent : 1 = true;

	// Whether this event is a 'task' that runs in a tick function, or whether it will run attached to another tick function
	uint8 bIsTask : 1 = true;

	// Whether this is a task that runs on the game thread, as opposed to a worker thread
	uint8 bIsGameThreadTask : 1 = false;
};

}

// Asset instance component supplying user-authored scripting
USTRUCT(meta=(Hidden, Abstract))
struct FUAFScriptComponent : public FUAFAssetInstanceComponent
{
	GENERATED_BODY()
	DECLARE_UAF_ASSET_INSTANCE_COMPONENT()

	// Call an event on this script component given the cached FScriptEvent
	void CallEvent(const UE::UAF::FScriptEvent& InEvent, TConstStructView<FUAFScriptContextData> InContextData)
	{
		check(InEvent.IsCallable());
		(InEvent.EventFunc)(TStructView<FUAFScriptComponent>(GetScriptStruct(), reinterpret_cast<uint8*>(this)), InContextData);
	}

	// Call an event with just its name, left up to derived components to implement (usually slower due to lookup)
	virtual void CallEventByName(TConstStructView<FUAFScriptContextData> InContextData) PURE_VIRTUAL(FUAFScriptComponent::CallEventByName, );

	// Get info about all script events that are implemented in this component
	virtual TConstArrayView<UE::UAF::FScriptEventInfo> GetScriptEvents() PURE_VIRTUAL(FUAFScriptComponent::GetScriptEvents, return TConstArrayView<UE::UAF::FScriptEventInfo>(); )
};

template<>
struct TStructOpsTypeTraits<FUAFScriptComponent> : public TStructOpsTypeTraitsBase2<FUAFScriptComponent>
{
	enum
	{
		WithPureVirtual = true,
	};
};

#undef UE_API