// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ConstExprUID.h"
#include "TraitCore/TraitEventLifetime.h"

#include "TraitEvent.generated.h"

namespace UE::UAF
{
	struct FTraitEventList;

	// Create an alias to improve readability
	using FTraitEventUID = UE::UAF::FConstExprUID;
}

// Helper macros
// In a trait event struct declaration, this macro declares the necessary boilerplate we require
#define DECLARE_ANIM_TRAIT_EVENT(EventName, SuperEventName) \
	/* FAnimNextTraitEvent impl */ \
	static constexpr UE::UAF::FTraitEventUID TypeUID = UE::UAF::FTraitEventUID::MakeFromString(TEXT(#EventName)); \
	virtual UE::UAF::FTraitEventUID GetTypeUID() const override { return TypeUID; } \
	virtual bool IsA(UE::UAF::FTraitEventUID InTypeUID) const override { return InTypeUID == TypeUID ? true : SuperEventName::IsA(InTypeUID); } \


/**
 * Trait Event
 * 
 * Encapsulates an event in the trait system.
 * Events can be marked as handled to signal to future handlers that no further action needs to be taken.
 * Consuming an event prevents it from propagating to other handlers.
 */
USTRUCT(BlueprintType)
struct FAnimNextTraitEvent
{
	GENERATED_BODY()

	// Creates a transient event
	FAnimNextTraitEvent() = default;

	// Allow typesafe destruction of derived types
	virtual ~FAnimNextTraitEvent() = default;

	// Returns the event type UID
	virtual UE::UAF::FTraitEventUID GetTypeUID() const { return UE::UAF::FTraitEventUID(); }

	// Returns whether or not this event derives from the specified type
	template<class EventType>
	bool IsA() const { return IsA(EventType::TypeUID); }

	// Returns whether or not this event derives from the specified type
	virtual bool IsA(UE::UAF::FTraitEventUID InTypeUID) const { return false; }

	// Returns whether or not this event is valid
	// An event is valid if it hasn't been consumed and if it isn't expired
	bool IsValid() const { return !IsConsumed() && !IsExpired(); }

	// Returns the lifetime of this event
	UE::UAF::FTraitEventLifetime GetLifetime() const { return Lifetime; }

	// Sets the desired lifetime for this event
	void SetLifetime(UE::UAF::FTraitEventLifetime InLifetime) { Lifetime = InLifetime; }

	// Marks this event as being handled.
	// Subsequent handlers can use this to determine if they should take action or not.
	// Returns whether or not this thread succeeded in marking this event as handled
	bool MarkHandled() { return FPlatformAtomics::InterlockedExchange(&bIsHandled, 1) == 0; }

	// Returns whether or not this event has been handled by a previous handler
	bool IsHandled() const { return FPlatformAtomics::AtomicRead_Relaxed(&bIsHandled) != 0; }

	// Marks this event as being consumed.
	// This event won't be forward to any other handlers.
	// Returns whether or not this thread succeeded in marking this event as consumed
	bool MarkConsumed() { return FPlatformAtomics::InterlockedExchange(&bIsConsumed, 1) == 0; }

	// Returns whether or not this event has been consumed by a previous handler
	bool IsConsumed() const { return FPlatformAtomics::AtomicRead_Relaxed(&bIsConsumed) != 0; }

	// Returns whether or not this event has expired
	bool IsExpired() const { return Lifetime.IsExpired(); }

	// Returns whether or not this event has infinite duration
	bool IsInfinite() const { return Lifetime.IsInfinite(); }

	// Returns whether or not this event has transient duration
	bool IsTransient() const { return Lifetime.IsTransient(); }

	// Decrements the lifetime count and returns whether or not this event has expired in the process
	// OnExpired will be called on the derived type if we expire and new output events can be appended
	UAF_API bool DecrementLifetime(UE::UAF::FTraitEventList& OutputEventList);

	// Returns a pointer to this event cast to the specified type if the TypeUIDs match, nullptr otherwise
	template<class EventType>
	EventType* AsType() { return IsA<EventType>() ? static_cast<EventType*>(this) : nullptr; }

	// Returns a pointer to this event cast to the specified type if the TypeUIDs match, nullptr otherwise
	template<class EventType>
	const EventType* AsType() const { return IsA<EventType>() ? static_cast<const EventType*>(this) : nullptr; }

protected:
	// Creates an event of the specified type and associated lifetime
	explicit FAnimNextTraitEvent(UE::UAF::FTraitEventLifetime InLifetime)
		: Lifetime(InLifetime)
	{
	}

	// Called when an events' lifetime expires
	virtual void OnExpired(UE::UAF::FTraitEventList& OutputEventList) {}

private:
	// The specified lifetime of this event
	UE::UAF::FTraitEventLifetime Lifetime = UE::UAF::FTraitEventLifetime::MakeTransient();

	// Signals whether someone has opted to handle this event
	int8 bIsHandled = 0;

	// Signals whether someone consumed this event and it thus shouldn't be forwarded to other handlers
	int8 bIsConsumed = 0;
};

// Create a shared pointer alias for trait events
using FAnimNextTraitEventPtr = TSharedPtr<FAnimNextTraitEvent, ESPMode::ThreadSafe>;

template<class EventType, typename... TArgs>
inline TSharedPtr<EventType, ESPMode::ThreadSafe> MakeTraitEvent(TArgs&&... Args)
{
	return MakeShared<EventType>(Forward<TArgs>(Args)...);
}
