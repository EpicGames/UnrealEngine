// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{
	/**
	 * Trait Event Lifetime
	 * 
	 * This encapsulates a specific lifetime duration.
	 */
	struct FTraitEventLifetime final
	{
		// Creates a lifetime that extends until the next graph update terminates
		static constexpr FTraitEventLifetime MakeTransient() noexcept
		{
			return FTraitEventLifetime(TRANSIENT_LIFETIME);
		}

		// Creates a lifetime that extends for the specified number of graph updates
		static constexpr FTraitEventLifetime MakeUntil(uint32 LifetimeCount) noexcept
		{
			return FTraitEventLifetime(LifetimeCount);
		}

		// Creates a lifetime that extends forever
		static constexpr FTraitEventLifetime MakeInfinite() noexcept
		{
			return FTraitEventLifetime(INFINITE_LIFETIME);
		}

		// Creates an expired lifetime
		constexpr FTraitEventLifetime() noexcept
			: LifetimeCount(EXPIRED_LIFETIME)
		{
		}

		// Returns whether or not this lifetime has expired
		constexpr bool IsExpired() const { return LifetimeCount == EXPIRED_LIFETIME; }

		// Returns whether or not this lifetime is infinite
		constexpr bool IsInfinite() const { return LifetimeCount == INFINITE_LIFETIME; }

		// Returns whether or not this lifetime is transient
		constexpr bool IsTransient() const { return LifetimeCount == TRANSIENT_LIFETIME; }

		// Decrements the lifetime count and returns whether or not it has expired in the process
		UAF_API bool Decrement();

	private:
		// Infinite lifetime duration
		static constexpr uint32 INFINITE_LIFETIME = ~0;

		// Transient lifetime duration
		static constexpr uint32 TRANSIENT_LIFETIME = 1;

		// Expired lifetime duration
		static constexpr uint32 EXPIRED_LIFETIME = 0;

		constexpr explicit FTraitEventLifetime(uint32 InLifetimeCount) noexcept
			: LifetimeCount(InLifetimeCount)
		{
		}

		uint32 LifetimeCount;
	};
}
