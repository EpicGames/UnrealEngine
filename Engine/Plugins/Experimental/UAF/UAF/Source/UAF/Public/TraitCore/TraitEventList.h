// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TraitCore/TraitEvent.h"

namespace UE::UAF
{
	/**
	 * Trait Event List
	 * 
	 * Encapsulates a list of trait events.
	 */
	struct FTraitEventList
	{
		using IteratorType = TArray<FAnimNextTraitEventPtr>::RangedForIteratorType;
		using ConstIteratorType = TArray<FAnimNextTraitEventPtr>::RangedForConstIteratorType;

		FTraitEventList() = default;

		void Push(FAnimNextTraitEventPtr Event) { Events.Add(MoveTemp(Event)); }
		void Reset() { Events.Reset(); }
		int32 Num() const { return Events.Num(); }
		bool IsEmpty() const { return Events.IsEmpty(); }

		void SetNum(int32 NewNum, EAllowShrinking AllowShrinking = EAllowShrinking::Default) { Events.SetNum(NewNum, AllowShrinking); }

		void Append(const FTraitEventList& Source) { Events.Append(Source.Events); }

		FAnimNextTraitEventPtr& operator[](int32 EventIndex) { return Events[EventIndex]; }
		const FAnimNextTraitEventPtr& operator[](int32 EventIndex) const { return Events[EventIndex]; }

		IteratorType begin() { return Events.begin(); }
		ConstIteratorType begin() const { return Events.begin(); }
		IteratorType end() { return Events.end(); }
		ConstIteratorType end() const { return Events.end(); }

	private:
		// A list of events
		TArray<FAnimNextTraitEventPtr, TInlineAllocator<4>> Events;
	};

	// Decrements and purges expired entries from the specified event list
	// Expired events can generate new output events if they wish
	UAF_API void DecrementLifetimeAndPurgeExpired(FTraitEventList& EventList, FTraitEventList& OutputEventList);
}
