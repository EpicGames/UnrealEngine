// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimTypes.h"
#include "TransformArrayView.h"
#include "AnimNextStats.h"
#include <algorithm>

#define ANIM_ENABLE_POINTER_ITERATION 1

namespace UE::UAF
{

extern const FTransform UAF_API TransformAdditiveIdentity;

template <typename AllocatorType>
struct TTransformArrayAoS
{
	TArray<FTransform, AllocatorType> Transforms;

	// Default empty (no memory) Transform
	TTransformArrayAoS() = default;

	// Create a TransformArray of N elements
	explicit TTransformArrayAoS(const int NumTransforms)
	{
		SetNum(NumTransforms);
	}

	~TTransformArrayAoS() = default;
	TTransformArrayAoS(const TTransformArrayAoS& Other) = default;
	TTransformArrayAoS& operator= (const TTransformArrayAoS& Other) = default;
	TTransformArrayAoS(TTransformArrayAoS&& Other) = default;
	TTransformArrayAoS& operator= (TTransformArrayAoS&& Other) = default;

	inline void Reset(int NumTransforms)
	{
		Transforms.Reset(NumTransforms);
	}

	void SetNum(int32 NumTransforms, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		Transforms.SetNum(NumTransforms, AllowShrinking);
	}

	void SetNumUninitialized(int32 NumTransforms, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		Transforms.SetNumUninitialized(NumTransforms, AllowShrinking);
	}

	inline void SetIdentity(bool bAdditiveIdentity = false)
	{
		std::fill(Transforms.begin(), Transforms.end(), bAdditiveIdentity ? TransformAdditiveIdentity : FTransform::Identity);
	}

	inline void CopyTransforms(const TTransformArrayAoS& Other, int32 Index, int32 NumTransforms)
	{
		check(Index < Num() && Index < Other.Num());
		check(Index + (NumTransforms - 1) < Num() && Index + (NumTransforms - 1) < Other.Num());

		std::copy(Other.Transforms.GetData() + Index, Other.Transforms.GetData() + (Index + NumTransforms), Transforms.GetData() + Index);
	}

	inline int32 Num() const
	{
		return Transforms.Num();
	}

	inline TArray<FTransform>& GetTransforms()
	{
		return Transforms;
	}
	inline const TArray<FTransform>& GetTransforms() const
	{
		return Transforms;
	}

	inline FTransform& operator[](int32 Index)
	{
		return Transforms[Index];
	}

	inline const FTransform& operator[](int32 Index) const
	{
		return Transforms[Index];
	}

	FTransformArrayAoSView GetView()
	{
		return FTransformArrayAoSView(Transforms);
	}

	FTransformArrayAoSConstView GetConstView() const
	{
		return FTransformArrayAoSConstView(Transforms);
	}

	/** Set this transform array to the weighted blend of the supplied two transforms. */
	void Blend(const TTransformArrayAoS& AtomArray1, const TTransformArrayAoS& AtomArray2, float BlendWeight)
	{
		if (FMath::Abs(BlendWeight) <= ZERO_ANIMWEIGHT_THRESH)
		{
			// if blend is all the way for child1, then just copy its bone atoms
			//(*this) = Atom1;
			CopyTransforms(AtomArray1, 0, Num());
		}
		else if (FMath::Abs(BlendWeight - 1.0f) <= ZERO_ANIMWEIGHT_THRESH)
		{
			// if blend is all the way for child2, then just copy its bone atoms
			//(*this) = Atom2;
			CopyTransforms(AtomArray2, 0, Num());
		}
		else
		{
			const int32 NumTransforms = Num();

			// Right now we only support same size blend
			check(AtomArray1.Num() == NumTransforms);
			check(AtomArray2.Num() == NumTransforms);

#if ANIM_ENABLE_POINTER_ITERATION
			// This version is faster, but without the range checks the TArray has on operator[]
			const FTransform* RESTRICT Transforms1 = AtomArray1.Transforms.GetData();
			const FTransform* RESTRICT Transforms1End = AtomArray1.Transforms.GetData() + NumTransforms;
			const FTransform* RESTRICT Transforms2 = AtomArray2.Transforms.GetData();
			FTransform* RESTRICT TransformsResult = Transforms.GetData();

			for (; Transforms1 != Transforms1End; ++Transforms1, ++Transforms2, ++TransformsResult)
			{
				//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
				//				// Check that all bone atoms coming from animation are normalized
				//				check(Transforms1[i].IsRotationNormalized());
				//				check(Transforms2[i].IsRotationNormalized());
				//#endif
				TransformsResult->Blend(*Transforms1, *Transforms2, BlendWeight);
			}
#else
			// This version is a bit slower, but has range checks on the array
			const TArray<FTransform, AllocatorType>& Transforms1 = AtomArray1.Transforms;
			const TArray<FTransform, AllocatorType>& Transforms2 = AtomArray2.Transforms;
			for (int32 i = 0; i < NumTransforms; ++i)
			{
				//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
				//				// Check that all bone atoms coming from animation are normalized
				//				check(Transforms1[i].IsRotationNormalized());
				//				check(Transforms2[i].IsRotationNormalized());
				//#endif
				Transforms[i].Blend(Transforms1[i], Transforms2[i], BlendWeight);
			}
#endif // ANIM_ENABLE_POINTER_ITERATION

			DiagnosticCheckNaN_All();
		}
	}

	bool ContainsNaN() const
	{
		for (const auto& Transform : Transforms)
		{
			if (Transform.GetRotation().ContainsNaN())
			{
				return true;
			}
			if (Transform.GetTranslation().ContainsNaN())
			{
				return true;
			}
			if (Transform.GetScale3D().ContainsNaN())
			{
				return true;
			}
		}

		return false;
	}

	bool IsValid() const
	{
		if (ContainsNaN())
		{
			return false;
		}

		for (const auto& Transform : Transforms)
		{
			if (Transform.GetRotation().IsNormalized() == false)
			{
				return false;
			}
		}

		return true;
	}

	// Sets the whole array to 0xCDCDCDCD to aid debugging
	void DebugFill()
	{
		FMemory::Memset(Transforms.GetData(), 0xCD, sizeof(FTransform) * Transforms.Num());
	}

private:
#if ENABLE_NAN_DIAGNOSTIC
	inline static void DiagnosticCheckNaN_Rotate(const FQuat& Rotation)
	{
		if (Rotation.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TTransformArraySoA Rotation contains NaN"));
		}
	}

	inline static void DiagnosticCheckNaN_Scale3D(const FVector& Scale3D)
	{
		if (Scale3D.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TTransformArraySoA Scale3D contains NaN"));
		}
	}

	inline static void DiagnosticCheckNaN_Translate(const FVector& Translation)
	{
		if (Translation.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TTransformArraySoA Translation contains NaN"));
		}
	}

	inline void DiagnosticCheckNaN_All() const
	{
		for (const auto& Transform : Transforms)
		{
			DiagnosticCheckNaN_Rotate(Transform.GetRotation());
			DiagnosticCheckNaN_Translate(Transform.GetTranslation());
			DiagnosticCheckNaN_Scale3D(Transform.GetScale3D());
		}
	}

	inline void DiagnosticCheck_IsValid() const
	{
		DiagnosticCheckNaN_All();
		if (!IsValid())
		{
			logOrEnsureNanError(TEXT("TTransformArraySoA is not valid"));
		}
	}

#else
	inline static void DiagnosticCheckNaN_Translate(const FVector& Translation) {}
	inline static void DiagnosticCheckNaN_Rotate(const FQuat& Rotation) {}
	inline static void DiagnosticCheckNaN_Scale3D(const FVector& Scale3D) {}
	inline void DiagnosticCheckNaN_All() const {}
	inline void DiagnosticCheck_IsValid() const {}
#endif
};

/**
* Transform Array using ArrayOfStructs model
*/
using FTransformArrayAoSHeap = TTransformArrayAoS<FDefaultAllocator>;
using FTransformArrayAoSStack = TTransformArrayAoS<FAnimStackAllocator>;

/**
* Transform Array Test using StructOfArrays model
*/
template <typename AllocatorType>
struct TTransformArraySoA
{
	// We add a bit of padding at the end of our buffer to allow 32-byte wide load/stores
	// without the risk of triggering access exceptions
	static constexpr int32 PaddingSize = sizeof(double);
	static constexpr int32 TransformSize = sizeof(FVector) + sizeof(FQuat) + sizeof(FVector);

	TArrayView<FVector> Translations;
	TArrayView<FQuat> Rotations;
	TArrayView<FVector> Scales3D;

	// Default empty (no memory) Transform
	TTransformArraySoA()
	{
		UpdateViews(nullptr, 0);
	}

	// Create a TransformArray of N elements. Optionally initializes to identity (default)
	explicit TTransformArraySoA(int NumTransforms, bool bSetIdentity = true, bool bAdditiveIdentity = false)
	{
		SetNum(NumTransforms);

		if (bSetIdentity)
		{
			SetIdentity(bAdditiveIdentity);
		}
	}

	~TTransformArraySoA() = default;

	TTransformArraySoA(const TTransformArraySoA& Other)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_CopyTransforms_SoA);

		AllocatedMemory = Other.AllocatedMemory;

		UpdateViews(AllocatedMemory.GetData(), Other.Num());
	}

	TTransformArraySoA& operator= (const TTransformArraySoA& Other)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_CopyTransforms_SoA);

		AllocatedMemory = Other.AllocatedMemory;

		UpdateViews(AllocatedMemory.GetData(), Other.Num());

		return *this;
	}

	TTransformArraySoA(TTransformArraySoA&& Other)
	{
		Swap(*this, Other);
	}

	TTransformArraySoA& operator= (TTransformArraySoA&& Other)
	{
		Swap(*this, Other);
		return *this;
	}

	inline bool IsEmpty() const
	{
		return Num() == 0;
	}

	inline void Empty(int32 Slack = 0)
	{
		const int32 Padding = Slack != 0 ? PaddingSize : 0;
		AllocatedMemory.Empty(Slack + Padding);
		UpdateViews(AllocatedMemory.GetData(), 0);
	}

	inline void Reset(int32 NumTransforms)
	{
		const int32 Padding = NumTransforms != 0 ? PaddingSize : 0;
		AllocatedMemory.Reset(NumTransforms * TransformSize + Padding);
	}

	void SetNum(int32 NumTransforms, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		const int32 Padding = NumTransforms != 0 ? PaddingSize : 0;
		AllocatedMemory.SetNum(NumTransforms * TransformSize + Padding, AllowShrinking);

		UpdateViews(AllocatedMemory.GetData(), NumTransforms);
	}

	void SetNumUninitialized(int32 NumTransforms, EAllowShrinking AllowShrinking = UE::Core::Private::AllowShrinkingByDefault<AllocatorType>())
	{
		const int32 Padding = NumTransforms != 0 ? PaddingSize : 0;
		AllocatedMemory.SetNumUninitialized(NumTransforms * TransformSize + Padding, AllowShrinking);

		UpdateViews(AllocatedMemory.GetData(), NumTransforms);
	}

	inline void SetIdentity(bool bAdditiveIdentity = false)
	{
		std::fill(Translations.begin(), Translations.end(), FVector::ZeroVector);
		std::fill(Rotations.begin(), Rotations.end(), FQuat::Identity);
		std::fill(Scales3D.begin(), Scales3D.end(), bAdditiveIdentity ? FVector::ZeroVector : FVector::OneVector);
	}

	inline void CopyTransforms(const TTransformArraySoA& Other, int32 Index, int32 NumTransforms)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_CopyTransforms_SoA);

		check(Index < Num() && Index < Other.Num());
		check(Index + (NumTransforms - 1) < Num() && Index + (NumTransforms - 1) < Other.Num());

		std::copy(Other.Translations.begin() + Index, Other.Translations.begin() + (Index + NumTransforms), Translations.begin());
		std::copy(Other.Rotations.begin() + Index, Other.Rotations.begin() + (Index + NumTransforms), Rotations.begin());
		std::copy(Other.Scales3D.begin() + Index, Other.Scales3D.begin() + (Index + NumTransforms), Scales3D.begin());
	}

	inline void CopyTransforms(const FTransformArraySoAConstView& Other, int32 Index, int32 NumTransforms)
	{
		SCOPE_CYCLE_COUNTER(STAT_AnimNext_CopyTransforms_SoA);

		check(Index < Num() && Index < Other.Num());
		check(Index + (NumTransforms - 1) < Num() && Index + (NumTransforms - 1) < Other.Num());

		std::copy(Other.Translations.begin() + Index, Other.Translations.begin() + (Index + NumTransforms), Translations.begin());
		std::copy(Other.Rotations.begin() + Index, Other.Rotations.begin() + (Index + NumTransforms), Rotations.begin());
		std::copy(Other.Scales3D.begin() + Index, Other.Scales3D.begin() + (Index + NumTransforms), Scales3D.begin());
	}

	/** Set this transform array to the weighted blend of the supplied two transforms. */
	void Blend(const TTransformArraySoA& AtomArray1, const TTransformArraySoA& AtomArray2, float BlendWeight)
	{
		if (FMath::Abs(BlendWeight) <= ZERO_ANIMWEIGHT_THRESH)
		{
			// if blend is all the way for child1, then just copy its bone atoms
			//(*this) = Atom1;
			CopyTransforms(AtomArray1, 0, Num());
		}
		else if (FMath::Abs(BlendWeight - 1.0f) <= ZERO_ANIMWEIGHT_THRESH)
		{
			// if blend is all the way for child2, then just copy its bone atoms
			//(*this) = Atom2;
			CopyTransforms(AtomArray2, 0, Num());
		}
		else
		{
			const int32 NumTransforms = Num();

#if ANIM_ENABLE_POINTER_ITERATION
			using TransformVectorRegister = TVectorRegisterType<double>;

			// Right now we only support same size blend
			check(AtomArray1.Translations.Num() == Translations.Num() && AtomArray1.Rotations.Num() == Rotations.Num() && AtomArray1.Scales3D.Num() == Scales3D.Num());
			check(AtomArray2.Translations.Num() == Translations.Num() && AtomArray2.Rotations.Num() == Rotations.Num() && AtomArray2.Scales3D.Num() == Scales3D.Num());

			const FVector* RESTRICT Translations1 = AtomArray1.Translations.GetData();
			const FVector* RESTRICT Translations1End = AtomArray1.Translations.GetData() + NumTransforms;
			const FVector* RESTRICT Translations2 = AtomArray2.Translations.GetData();
			FVector* RESTRICT TranslationsResult = Translations.GetData();
			for (; Translations1 != Translations1End; ++Translations1, ++Translations2, ++TranslationsResult)
			{
				*TranslationsResult = FMath::Lerp(*Translations1, *Translations2, BlendWeight);
			}

			const FQuat* RESTRICT Rotations1 = AtomArray1.Rotations.GetData();
			const FQuat* RESTRICT Rotations1End = AtomArray1.Rotations.GetData() + NumTransforms;
			const FQuat* RESTRICT Rotations2 = AtomArray2.Rotations.GetData();
			FQuat* RESTRICT RotationsResult = Rotations.GetData();
			for (; Rotations1 != Rotations1End; ++Rotations1, ++Rotations2, ++RotationsResult)
			{
				//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
				//				// Check that all bone atoms coming from animation are normalized
				//				check(Rotations1->IsNormalized());
				//				check(Rotations2->IsNormalized());
				//#endif

				*RotationsResult = FQuat::FastLerp(*Rotations1, *Rotations2, BlendWeight).GetNormalized();
			}

			const FVector* RESTRICT Scales3D1 = AtomArray1.Scales3D.GetData();
			const FVector* RESTRICT Scales3D1End = AtomArray1.Scales3D.GetData() + NumTransforms;
			const FVector* RESTRICT Scales3D2 = AtomArray2.Scales3D.GetData();
			FVector* RESTRICT Scales3DResult = Scales3D.GetData();
			for (; Scales3D1 != Scales3D1End; ++Scales3D1, ++Scales3D2, ++Scales3DResult)
			{
				*Scales3DResult = FMath::Lerp(*Scales3D1, *Scales3D2, BlendWeight);
			}
#else
			// This version is slower than AoS and a lot slower than the version using pointers, but has range checks on the TArrayView
			const TArrayView<FVector>& Translations1 = AtomArray1.Translations;
			const TArrayView<FVector>& Translations2 = AtomArray2.Translations;
			for (int32 i = 0; i < NumTransforms; ++i)
			{
				Translations[i] = FMath::Lerp(Translations1[i], Translations2[i], BlendWeight);
			}
			const TArrayView<FQuat>& Rotations1 = AtomArray1.Rotations;
			const TArrayView<FQuat>& Rotations2 = AtomArray2.Rotations;
			for (int32 i = 0; i < NumTransforms; ++i)
			{
				//#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_EDITORONLY_DATA
				//				// Check that all bone atoms coming from animation are normalized
				//				check(Transforms1[i].IsRotationNormalized());
				//				check(Transforms2[i].IsRotationNormalized());
				//#endif
				Rotations[i] = FQuat::FastLerp(Rotations1[i], Rotations2[i], BlendWeight).GetNormalized();
			}
			const TArrayView<FVector>& Scales3D1 = AtomArray1.Scales3D;
			const TArrayView<FVector>& Scales3D2 = AtomArray2.Scales3D;
			for (int32 i = 0; i < NumTransforms; ++i)
			{
				Scales3D[i] = FMath::Lerp(Scales3D1[i], Scales3D2[i], BlendWeight);
			}

#endif // ANIM_ENABLE_POINTER_ITERATION

			DiagnosticCheckNaN_All();
		}
	}

	inline int32 Num() const
	{
		return Translations.Num();
	}

	FTransformSoAAdapter operator[] (int Index)
	{
		return FTransformSoAAdapter(Rotations[Index], Translations[Index], Scales3D[Index]);
	}

	const FTransformSoAAdapterConst operator[] (int Index) const
	{
		return FTransformSoAAdapterConst(Rotations[Index], Translations[Index], Scales3D[Index]);
	}

	FTransformArraySoAView GetView()
	{
		FTransformArraySoAView View;
		View.Translations = Translations;
		View.Rotations = Rotations;
		View.Scales3D = Scales3D;
		return View;
	}

	FTransformArraySoAConstView GetConstView() const
	{
		FTransformArraySoAConstView View;
		View.Translations = Translations;
		View.Rotations = Rotations;
		View.Scales3D = Scales3D;
		return View;
	}

	bool ContainsNaN() const
	{
		for (const FQuat& Rotation : Rotations)
		{
			if (Rotation.ContainsNaN())
			{
				return true;
			}
		}
		for (const FVector& Translation : Translations)
		{
			if (Translation.ContainsNaN())
			{
				return true;
			}
		}
		for (const FVector& Scale3D : Scales3D)
		{
			if (Scale3D.ContainsNaN())
			{
				return true;
			}
		}

		return false;
	}

	bool IsValid() const
	{
		if (ContainsNaN())
		{
			return false;
		}

		for (const FQuat& Rotation : Rotations)
		{
			if (Rotation.IsNormalized() == false)
			{
				return false;
			}
		}

		return true;
	}

	// Sets the whole array to 0xCDCDCDCD to aid debugging
	void DebugFill()
	{
		FMemory::Memset(AllocatedMemory.GetData(), 0xCD, AllocatedMemory.Num());
	}

private:
	TArray<uint8, AllocatorType> AllocatedMemory; // TODO : use the allocator directly

	void UpdateViews(uint8* Memory, int32 NumTransforms)
	{
		if (NumTransforms > 0)
		{
			Rotations = TArrayView<FQuat>((FQuat*)(Memory), NumTransforms);

			const int32 TranslationsOffset = sizeof(FQuat) * NumTransforms;
			Translations = TArrayView<FVector>((FVector*)(Memory + TranslationsOffset), NumTransforms);

			const int32 Scales3DOffset = TranslationsOffset + sizeof(FVector) * NumTransforms;
			Scales3D = TArrayView<FVector>((FVector*)(Memory + Scales3DOffset), NumTransforms);
		}
		else
		{
			Rotations = TArrayView<FQuat>();
			Translations = TArrayView<FVector>();
			Scales3D = TArrayView<FVector>();
		}
	}

#if ENABLE_NAN_DIAGNOSTIC
	inline static void DiagnosticCheckNaN_Scale3D(const FVector& Scale3D)
	{
		if (Scale3D.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TTransformArraySoA Scale3D contains NaN"));
		}
	}

	inline static void DiagnosticCheckNaN_Translate(const FVector& Translation)
	{
		if (Translation.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TTransformArraySoA Translation contains NaN"));
		}
	}

	inline static void DiagnosticCheckNaN_Rotate(const FQuat& Rotation)
	{
		if (Rotation.ContainsNaN())
		{
			logOrEnsureNanError(TEXT("TTransformArraySoA Rotation contains NaN"));
		}
	}

	inline void DiagnosticCheckNaN_All() const
	{
		for (const FQuat& Rotation : Rotations)
		{
			DiagnosticCheckNaN_Rotate(Rotation);
		}
		for (const FVector& Translation : Translations)
		{
			DiagnosticCheckNaN_Translate(Translation);
		}
		for (const FVector& Scale3D : Scales3D)
		{
			DiagnosticCheckNaN_Scale3D(Scale3D);
		}
	}

	inline void DiagnosticCheck_IsValid() const
	{
		DiagnosticCheckNaN_All();
		if (!IsValid())
		{
			logOrEnsureNanError(TEXT("TTransformArraySoA is not valid"));
		}
	}

#else
	inline static void DiagnosticCheckNaN_Translate(const FVector& Translation) {}
	inline static void DiagnosticCheckNaN_Rotate(const FQuat& Rotation) {}
	inline static void DiagnosticCheckNaN_Scale3D(const FVector& Scale3D) {}
	inline void DiagnosticCheckNaN_All() const {}
	inline void DiagnosticCheck_IsValid() const {}
#endif

};

/**
* Transform Array using StructsOfArrays model
*/
using FTransformArraySoAHeap = TTransformArraySoA<FDefaultAllocator>;
using FTransformArraySoAStack = TTransformArraySoA<FAnimStackAllocator>;


//****************************************************************************

#define DEFAULT_SOA 1
#if DEFAULT_SOA
template <typename AllocatorType>
using TTransformArray = TTransformArraySoA<AllocatorType>;

using FTransformArrayHeap = FTransformArraySoAHeap;
using FTransformArrayStack = FTransformArraySoAStack;

using FTransformArray = FTransformArraySoAHeap;

#else
template <typename AllocatorType>
using TTransformArray = TTransformArrayAoS<AllocatorType>;

using FTransformArrayHeap = FTransformArrayAoSHeap;
using FTransformArrayStack = FTransformArrayAoSStack;

using FTransformArray = FTransformArrayAoSHeap;

#endif

}