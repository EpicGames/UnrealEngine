// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TransformArrayView.h"
#include "BoneIndices.h"

namespace UE::UAF
{
	// Sets the identity (regular or additive) over the whole destination array
	UAF_API void SetIdentity(const FTransformArrayAoSView& Dest, bool bIsAdditive);
	UAF_API void SetIdentity(const FTransformArrayAoSView& Dest, bool bIsAdditive, int32 StartIndex, int32 NumTransformsToSet);
	UAF_API void SetIdentity(const FTransformArraySoAView& Dest, bool bIsAdditive);
	UAF_API void SetIdentity(const FTransformArraySoAView& Dest, bool bIsAdditive, int32 StartIndex, int32 NumTransformsToSet);

	// Copies the specified number of transforms from a source into a destination starting at the specified start index
	// If NumToCopy is -1, we copy until the end
	UAF_API void CopyTransforms(const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source, int32 StartIndex = 0, int32 NumToCopy = -1);
	UAF_API void CopyTransforms(const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source, int32 StartIndex = 0, int32 NumToCopy = -1);

	// Normalizes rotations in input transform view
	UAF_API void NormalizeRotations(const FTransformArrayAoSView& Input);
	UAF_API void NormalizeRotations(const FTransformArraySoAView& Input);

	// Convert a pose from local space to mesh space (rotations only)
	UAF_API void ConvertPoseLocalToMeshRotation(const FTransformArrayAoSView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap);
	UAF_API void ConvertPoseLocalToMeshRotation(const FTransformArraySoAView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap);

	// Convert a pose from mesh space to local space (rotations only)
	UAF_API void ConvertPoseMeshToLocalRotation(const FTransformArrayAoSView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap);
	UAF_API void ConvertPoseMeshToLocalRotation(const FTransformArraySoAView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap);
	
	// Convert a pose from local space to mesh space (rotation and translation only)
	UAF_API void ConvertPoseLocalToMeshRotationTranslation(const FTransformArrayAoSView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap);
	UAF_API void ConvertPoseLocalToMeshRotationTranslation(const FTransformArraySoAView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap);

	// Convert a pose from mesh space to local space (rotation and translation only)
	UAF_API void ConvertPoseMeshToLocalRotationTranslation(const FTransformArrayAoSView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap);
	UAF_API void ConvertPoseMeshToLocalRotationTranslation(const FTransformArraySoAView& Input, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap);

	// The additive transform view is blended with the additive identity using the provided blend weight
	// We then accumulate the resulting transforms on top of the base transforms
	// Delta = Blend(Identity, Additive, BlendWeight)
	// Base.Accumulate(Delta);
	UAF_API void BlendWithIdentityAndAccumulate(const FTransformArrayAoSView& Base, const FTransformArrayAoSConstView& Additive, const float BlendWeight);
	UAF_API void BlendWithIdentityAndAccumulate(const FTransformArraySoAView& Base, const FTransformArraySoAConstView& Additive, const float BlendWeight);

	// Performs a BlendWithIdentityAndAccumulate in mesh space, result is in local space
	UAF_API void BlendWithIdentityAndAccumulateMesh(
		const FTransformArrayAoSView& Base, const FTransformArrayAoSConstView& Additive, 
		const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap, const float BlendWeight);
	UAF_API void BlendWithIdentityAndAccumulateMesh(
		const FTransformArraySoAView& Base, const FTransformArraySoAConstView& Additive, 
		const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap, const float BlendWeight);

	// @TODO: AoS version is untested because AoS currently does not compile in UAF.
	UAF_API void BlendWithIdentityAndAccumulatePerBoneWeight(const FTransformArrayAoSView& Base, const FTransformArrayAoSConstView& Additive,
		const TArrayView<const int32>& TransformIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultBlendWeight);
	UAF_API void BlendWithIdentityAndAccumulatePerBoneWeight(const FTransformArraySoAView& Base, const FTransformArraySoAConstView& Additive,
		const TArrayView<const int32>& TransformIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultBlendWeight);

	// Performs a BlendWithIdentityAndAccumulatePerBoneWeight in mesh space, result is in local space
	// @TODO: AoS version is untested because AoS currently does not compile in UAF.
	UAF_API void BlendWithIdentityAndAccumulatePerBoneWeightMesh(
		const FTransformArrayAoSView& Base, const FTransformArrayAoSConstView& Additive, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap, 
		const TArrayView<const int32>& TransformIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultBlendWeight);
	// Performs a BlendWithIdentityAndAccumulatePerBoneWeight in mesh space, result is in local space
	UAF_API void BlendWithIdentityAndAccumulatePerBoneWeightMesh(
		const FTransformArraySoAView& Base, const FTransformArraySoAConstView& Additive, const TArrayView<const FBoneIndexType>& LODBoneIndexToParentLODBoneIndexMap, 
		const TArrayView<const int32>& TransformIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultBlendWeight);

	// The source transforms are scaled by the provided weight and the result is written in the destination
	// Dest = Source * ScaleWeight
	UAF_API void BlendOverwriteWithScale(const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source, const float ScaleWeight);
	UAF_API void BlendOverwriteWithScale(const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source, const float ScaleWeight);

	// The source transforms are scaled by the provided weight and the result is added to the destination
	// Dest = Dest + (Source * ScaleWeight)
	UAF_API void BlendAddWithScale(const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source, const float ScaleWeight);
	UAF_API void BlendAddWithScale(const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source, const float ScaleWeight);

	// The source transforms are scaled by the provided per bone weight and the result is written in the destination
	// If bInvert is set, the bone weights are set to (1 - weight), DefaultScaleWeight remains unchanged, i.e. it is not inverted
	// Dest = Source * (WeightIndex != INDEX_NONE ? Weights[WeightIndex] : DefaultScaleWeight)
	UAF_API void BlendOverwritePerBoneWithScale(
		const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source,
		const TArrayView<const int32>& LODBoneIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultScaleWeight, const bool bInvert = false);
	UAF_API void BlendOverwritePerBoneWithScale(
		const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source,
		const TArrayView<const int32>& LODBoneIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultScaleWeight, const bool bInvert = false);

	// The source transforms are scaled by the provided per bone weight and the result is added to the destination
	// Dest = Dest + (Source * (WeightIndex != INDEX_NONE ? Weights[WeightIndex] : DefaultScaleWeight))
	UAF_API void BlendAddPerBoneWithScale(
		const FTransformArrayAoSView& Dest, const FTransformArrayAoSConstView& Source,
		const TArrayView<const int32>& LODBoneIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultScaleWeight);
	UAF_API void BlendAddPerBoneWithScale(
		const FTransformArraySoAView& Dest, const FTransformArraySoAConstView& Source,
		const TArrayView<const int32>& LODBoneIndexToWeightIndexMap, const TArrayView<const float>& BoneWeights, const float DefaultScaleWeight);
	
	// Convert Input into an Additive Pose, by doing Output = Input - Base 
	// @TODO: AoS version is untested because AoS currently does not compile in UAF.
	UAF_API void ConvertToAdditive(const FTransformArrayAoSConstView& Input, const FTransformArrayAoSConstView& Base, const FTransformArrayAoSView& Output);
	UAF_API void ConvertToAdditive(const FTransformArraySoAConstView& Input, const FTransformArraySoAConstView& Base, const FTransformArraySoAView& Output);
}
