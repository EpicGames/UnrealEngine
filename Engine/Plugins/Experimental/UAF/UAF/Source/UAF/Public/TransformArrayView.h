// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{
	// This enables using a SoA array with operator[]
	template <class RotationType, class TranslationType, class ScaleType>
	struct TTransformSoAAdapter
	{
		RotationType& Rotation;
		TranslationType& Translation;
		ScaleType& Scale3D;

		TTransformSoAAdapter(RotationType& InRotation, TranslationType& InTranslation, ScaleType& InScale3D)
			: Rotation(InRotation)
			, Translation(InTranslation)
			, Scale3D(InScale3D)
		{
		}

		inline RotationType& GetRotation() const
		{
			return Rotation;
		}

		inline void SetRotation(const FQuat& InRotation)
		{
			Rotation = InRotation;
		}

		inline TranslationType& GetTranslation() const
		{
			return Translation;
		}

		inline void SetTranslation(const FVector& InTranslation)
		{
			Translation = InTranslation;
		}

		inline ScaleType& GetScale3D() const
		{
			return Scale3D;
		}

		inline void SetScale3D(const FVector& InScale3D)
		{
			Scale3D = InScale3D;
		}

		inline operator FTransform()
		{
			return FTransform(Rotation, Translation, Scale3D);
		}

		inline operator FTransform() const
		{
			return FTransform(Rotation, Translation, Scale3D);
		}

		inline void operator= (const FTransform& Transform)
		{
			Rotation = Transform.GetRotation();
			Translation = Transform.GetTranslation();
			Scale3D = Transform.GetScale3D();
		}

		inline void ScaleTranslation(const FVector::FReal& Scale)
		{
			Translation *= Scale;
			//DiagnosticCheckNaN_Translate();
		}

		inline void NormalizeRotation()
		{
			Rotation.Normalize();
			//DiagnosticCheckNaN_Rotate();
		}
	};

	using FTransformSoAAdapter = TTransformSoAAdapter<FQuat, FVector, FVector>;
	using FTransformSoAAdapterConst = TTransformSoAAdapter<const FQuat, const FVector, const FVector>;

	using FTransformArrayAoSView = TArrayView<FTransform>;
	using FTransformArrayAoSConstView = TArrayView<const FTransform>;

	struct FTransformArraySoAView
	{
		TArrayView<FVector> Translations;
		TArrayView<FQuat> Rotations;
		TArrayView<FVector> Scales3D;

		bool IsEmpty() const { return Rotations.IsEmpty(); }
		int32 Num() const { return Rotations.Num(); }

		inline FTransformSoAAdapter operator[](int32 Index)
		{
			return FTransformSoAAdapter(Rotations[Index], Translations[Index], Scales3D[Index]);
		}

		inline const FTransformSoAAdapterConst operator[](int32 Index) const
		{
			return FTransformSoAAdapterConst(Rotations[Index], Translations[Index], Scales3D[Index]);
		}

		bool IsValid() const
		{
			for (const FQuat& Rotation : Rotations)
			{
				if (Rotation.ContainsNaN() || !Rotation.IsNormalized())
				{
					return false;
				}
			}

			for (const FVector& Translation : Translations)
			{
				if (Translation.ContainsNaN())
				{
					return false;
				}
			}

			for (const FVector& Scale3D : Scales3D)
			{
				if (Scale3D.ContainsNaN())
				{
					return false;
				}
			}

			return true;
		}
	};

	struct FTransformArraySoAConstView
	{
		TArrayView<const FVector> Translations;
		TArrayView<const FQuat> Rotations;
		TArrayView<const FVector> Scales3D;

		FTransformArraySoAConstView() = default;
		FTransformArraySoAConstView(const FTransformArraySoAView& Other)	// Safely coerce from mutable view
			: Translations(Other.Translations)
			, Rotations(Other.Rotations)
			, Scales3D(Other.Scales3D)
		{}

		bool IsEmpty() const { return Rotations.IsEmpty(); }
		int32 Num() const { return Rotations.Num(); }

		inline const FTransformSoAAdapterConst operator[](int32 Index) const
		{
			return FTransformSoAAdapterConst(Rotations[Index], Translations[Index], Scales3D[Index]);
		}

		bool IsValid() const
		{
			for (const FQuat& Rotation : Rotations)
			{
				if (Rotation.ContainsNaN() || !Rotation.IsNormalized())
				{
					return false;
				}
			}

			for (const FVector& Translation : Translations)
			{
				if (Translation.ContainsNaN())
				{
					return false;
				}
			}

			for (const FVector& Scale3D : Scales3D)
			{
				if (Scale3D.ContainsNaN())
				{
					return false;
				}
			}

			return true;
		}
	};

#define DEFAULT_SOA_VIEW 1
#if DEFAULT_SOA_VIEW
	using FTransformArrayView = FTransformArraySoAView;
	using FTransformArrayConstView = FTransformArraySoAConstView;
#else
	using FTransformArrayView = FTransformArrayAoSView;
	using FTransformArrayConstView = FTransformArrayAoSConstView;
#endif
}
