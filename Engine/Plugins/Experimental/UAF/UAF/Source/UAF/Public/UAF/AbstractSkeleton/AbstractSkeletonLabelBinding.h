// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AbstractSkeletonLabelBinding.generated.h"

#define UE_API UAF_API

class UAbstractSkeletonLabelCollection;
class USkeleton;

USTRUCT()
struct FAbstractSkeleton_LabelBinding
{
	GENERATED_BODY()

	UPROPERTY()
	FName Label;

	UPROPERTY()
	FName BoneName;

	UPROPERTY()
	TObjectPtr<const UAbstractSkeletonLabelCollection> SourceCollection;
};

UCLASS(MinimalAPI, BlueprintType)
class UAbstractSkeletonLabelBinding : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	UE_API bool BindBoneToLabel(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection, const FName BoneName, const FName Label);

	UE_API bool UnbindBoneFromLabel(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection, const FName BoneName, const FName Label);

	UE_API bool AddLabelCollection(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection);

	UE_API bool RemoveLabelCollection(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection);
	
	UE_API bool SetSkeleton(const TObjectPtr<USkeleton> InSkeleton);

#endif // WITH_EDITOR

	UE_API bool IsLabelBound(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection, const FName LabelName) const;

	UE_API FName GetLabelBinding(const TObjectPtr<const UAbstractSkeletonLabelCollection> InCollection, const FName LabelName) const;

	UE_API TConstArrayView<FAbstractSkeleton_LabelBinding> GetLabelBindings() const;
	
	UE_API TObjectPtr<USkeleton> GetSkeleton() const;

	UE_API TConstArrayView<TObjectPtr<const UAbstractSkeletonLabelCollection>> GetLabelCollections() const;

private:
	UPROPERTY()
	TArray<TObjectPtr<const UAbstractSkeletonLabelCollection>> LabelCollections;

	UPROPERTY(AssetRegistrySearchable)
	TObjectPtr<USkeleton> Skeleton;

	UPROPERTY()
	TArray<FAbstractSkeleton_LabelBinding> LabelBindings;
};

#undef UE_API
