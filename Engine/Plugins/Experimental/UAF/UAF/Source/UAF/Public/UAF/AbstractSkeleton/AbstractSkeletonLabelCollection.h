// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AbstractSkeletonLabelCollection.generated.h"

#define UE_API UAF_API

UCLASS(MinimalAPI, BlueprintType)
class UAbstractSkeletonLabelCollection : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	UE_API bool AddLabel(const FName InLabel);

	UE_API bool RenameLabel(const FName InOldLabel, const FName InNewLabel);

	UE_API bool RemoveLabel(const FName InLabel);

	UE_API TArray<FName>& GetMutableLabels();
#endif // WITH_EDITOR

	UE_API bool HasLabel(const FName InLabel) const;

	UE_API TConstArrayView<FName> GetLabels() const;

private:
	UPROPERTY()
	TArray<FName> Labels; 
};

#undef UE_API
