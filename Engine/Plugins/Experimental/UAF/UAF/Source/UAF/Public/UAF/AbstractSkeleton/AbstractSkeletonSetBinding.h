// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimData/AttributeIdentifier.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "UAF/Attributes/AttributeBindingIndex.h"
#include "UAF/Attributes/AttributeBindingData.h"
#include "UAF/ValueRuntime/ValueBundle.h"

#include "AbstractSkeletonSetBinding.generated.h"

#define UE_API UAF_API

class UAbstractSkeletonSetCollection;
class USkeleton;

USTRUCT()
struct FAbstractSkeleton_BoneBinding
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Default)
	FName SetName;

	UPROPERTY(EditAnywhere, Category = Default)
	FName BoneName;
};

// Curves are represented as float attributes detached from any bone
USTRUCT()
struct FAbstractSkeleton_AttributeBinding
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Default)
	FName SetName;

	UPROPERTY(EditAnywhere, Category = Default)
	FAnimationAttributeIdentifier Attribute;
};

UCLASS(MinimalAPI, BlueprintType)
class UAbstractSkeletonSetBinding : public UObject
{
	GENERATED_BODY()

public:
	// UObject implementation
	virtual void PostLoad() override;
	virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;

	UE_API bool SetSkeleton(const TObjectPtr<USkeleton> InSkeleton);

	UE_API TObjectPtr<USkeleton> GetSkeleton() const;

	UE_API bool SetSetCollection(const TObjectPtr<UAbstractSkeletonSetCollection> InSetCollection);
	
	UE_API TObjectPtr<const UAbstractSkeletonSetCollection> GetSetCollection() const;
	
	// Bone Bindings

	UE_API bool AddBoneToSet(const FName BoneName, const FName SetName);

	UE_API bool RemoveBoneFromSet(const FName BoneName);

	UE_API bool IsBoneInSet(const FName BoneName) const;

	UE_API bool IsBoneInSet(const FName BoneName, const FName SetName) const;

	// Returns the name of the set that the provided bone is in or NAME_None if bone is unbound.
	UE_API FName GetBoneSet(const FName BoneName);

	UE_API const TConstArrayView<FAbstractSkeleton_BoneBinding> GetBoneBindings() const;
		
	// Attribute Bindings

	// Adds all the default attributes that should always be present
	UE_API void AddDefaultAttributes();

	UE_API bool AddAttributeToSet(const FAnimationAttributeIdentifier Attribute, const FName SetName);

	// Note this function does two different things depending on the type of attribute provided.
	// If the attribute is bound then it is unbound from its set. This does not remove it from the
	// set binding and instead just assigns it to the NANE_None set.
	// If the attribute is unbound then it is completely removed from the set binding.
	// TODO: This function needs to be changed to avoid confusion
	UE_API bool RemoveAttributeFromSet(const FAnimationAttributeIdentifier Attribute);

	// Unbinds all attributes bound to a set. Returns whether or not any attributes were removed.
	UE_API bool RemoveAllFromSet(const FName SetName);
	
	// Returns if an attribute with the given name is currently stored in this set binding.
	// Note that an attribute existing does not mean it is bound to a set.
	[[nodiscard]] UE_API bool ContainsAttribute(const FName AttributeName) const;
	
	// Returns true if the provided attribute is in ANY set
	UE_API bool IsAttributeInSet(const FAnimationAttributeIdentifier Attribute) const;

	// Returns true if the provided attribute is in the provided set
	UE_API bool IsAttributeInSet(const FAnimationAttributeIdentifier Attribute, const FName SetName) const;

	UE_API const TConstArrayView<FAbstractSkeleton_AttributeBinding> GetAttributeBindings() const;

#if WITH_EDITOR
	UE_API FDelegateHandle RegisterOnBindingsChanged(FSimpleMulticastDelegate::FDelegate&& InDelegate);
	UE_API bool UnregisterOnBindingsChanged(const FDelegateHandle InHandle);
#endif

private:
#if WITH_EDITOR
	void OnSkeletonChanged();

	void OnSetCollectionChanged();

	// If a change is made to the set collection while we are loaded/unloaded we need
	// to remove any bindings that are bound to invalid sets
	void FilterOrphanedBindings();
#endif

	// Called internally whenever a change is made to the attribute bindings (bones + attributes)
	void OnBindingsChanged();

	void RebuildBindingData();

	UPROPERTY()
	TObjectPtr<UAbstractSkeletonSetCollection> SetCollection;
	
	UPROPERTY(AssetRegistrySearchable)
	TObjectPtr<USkeleton> Skeleton;

	UPROPERTY()
	TArray<FAbstractSkeleton_BoneBinding> BoneBindings;

	UPROPERTY()
	TArray<FAbstractSkeleton_AttributeBinding> AttributeBindings;

	// TODO: Cache sets in PostLoad for cooked builds

#if WITH_EDITOR
	FDelegateHandle SetCollectionChangedHandle;

	FSimpleMulticastDelegate OnBindingsChangedDelegate;
#endif
};

#undef UE_API
