// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AbstractSkeletonSetCollection.generated.h"

#define UE_API UAF_API

USTRUCT()
struct FAbstractSkeletonSet
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Default)
	FName SetName;

	UPROPERTY(EditAnywhere, Category = Default)
	FName ParentSetName;
};

UCLASS(MinimalAPI, BlueprintType)
class UAbstractSkeletonSetCollection : public UObject
{
	GENERATED_BODY()

public:
	UE_API bool AddSet(const FName SetName, const FName ParentSetName = NAME_None);

	UE_API bool HasSet(const FName SetName) const;

	UE_API bool RenameSet(const FName OldSetName, const FName NewSetName);

	// Removes this set and moves its children to this set's parent
	UE_API bool RemoveSet(const FName SetName);

	UE_API bool ReparentSet(const FName SetName, const FName NewParentName);

	// Returns the sets in this set collection sorted topologically
	UE_API const TArrayView<const FAbstractSkeletonSet> GetSetHierarchy() const;

	FName GetParentSetName(const FName SetName) const;

	// Returns true if SetName is a descendant of AncestorSetName.
	// This is for strict descendants, i.e. a set is not a descendant of itself.
	UE_API bool IsDescendantOf(const FName SetName, const FName AncestorSetName) const;
	
#if WITH_EDITOR
	UE_API FDelegateHandle RegisterOnSetsChanged(FSimpleMulticastDelegate::FDelegate&& InDelegate);
	UE_API bool UnregisterOnSetsChanged(const FDelegateHandle InHandle);
#endif

private:
	bool IsValidSetName(const FName SetName) const;

	int32 GetSetIndex(const FName SetName) const;

	void SortSetHierarchy();

	FAbstractSkeletonSet& GetSetRef(const FName SetName);

	UPROPERTY()
	TArray<FAbstractSkeletonSet> SetHierarchy;

#if WITH_EDITOR
	FSimpleMulticastDelegate OnSetsChanged;
#endif
};

#undef UE_API
