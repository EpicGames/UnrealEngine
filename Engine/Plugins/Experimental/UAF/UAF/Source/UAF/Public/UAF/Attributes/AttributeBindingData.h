// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "UAF/Attributes/AttributeNamedSet.h"
#include "UAF/Attributes/AttributeTypedSet.h"
#include "UAF/ValueRuntime/PoseValueBundle.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API UAF_API

class USkeletalMesh;
class UAbstractSkeletonSetBinding;

namespace UE::UAF
{
	using FAttributeBindingDataPtr = TRefCountPtr<const class FAttributeBindingData>;

	// Our baked abstract skeleton binding data
	// This class is ref counted and all named/typed sets increment/decrement its ref count
	class FAttributeBindingData : public UE::Private::FQueryableRefCountedObject
	{
	public:
		using NamedSetMapType = TSortedMap<FName, FAttributeNamedSet*, FDefaultAllocator, FNameFastLess>;

		class FNamedSetIterator
		{
		public:
			[[nodiscard]] UE_FORCEINLINE_HINT explicit FNamedSetIterator(const NamedSetMapType& NamedSetMap)
				: Iterator(NamedSetMap.CreateConstIterator())
			{
			}

			UE_FORCEINLINE_HINT FNamedSetIterator& operator++()
			{
				++Iterator;
				return *this;
			}

			/** conversion to "bool" returning true if the iterator is valid. */
			[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
			{
				return !!Iterator;
			}

			[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const FNamedSetIterator& Rhs) const
			{
				return Iterator == Rhs.Iterator;
			}
			[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const FNamedSetIterator& Rhs) const
			{
				return Iterator != Rhs.Iterator;
			}

			[[nodiscard]] UE_FORCEINLINE_HINT FAttributeNamedSetPtr operator*() const
			{
				return FAttributeNamedSetPtr(Iterator.Value(), true);
			}

			[[nodiscard]] UE_FORCEINLINE_HINT const FAttributeNamedSet* operator->() const
			{
				return Iterator.Value();
			}

		private:
			NamedSetMapType::TConstIterator Iterator;
		};

		// Returns the number of named sets, O(1)
		[[nodiscard]] UE_API int32 NumNamedSets() const;

		// Returns the number of LODs, O(1)
		[[nodiscard]] UE_API int32 NumLODs() const;

		// Returns whether or not this binding is empty, O(1)
		[[nodiscard]] UE_API bool IsEmpty() const;

		// Returns the number of attributes of the specified type, O(1)
		// Note that some attributes might not be part of any named/typed sets
		[[nodiscard]] UE_API int32 NumAttributes(UScriptStruct* AttributeType) const;

		// Returns the named set with the specified name, O(logN)
		// Returns null if no set with that name is found
		[[nodiscard]] UE_API FAttributeNamedSetPtr FindNamedSet(FName SetName) const;

		// Returns an iterator over all named sets contained within
		[[nodiscard]] UE_FORCEINLINE_HINT FNamedSetIterator CreateNamedSetIterator() const { return FNamedSetIterator(NamedSetMap); }

		// Returns the attribute binding index for the specified attribute, O(N)
		// Returns an invalid index if the attribute isn't found
		[[nodiscard]] UE_API FAttributeBindingIndex FindBindingIndex(FName AttributeName, UScriptStruct* AttributeType) const;

		// Returns a collection of the default attribute values for the specified named set
		[[nodiscard]] UE_API FPoseValueBundlePtr GetDefaultAttributeValues(FName SetName) const;

		// UE::Private::FQueryableRefCountedObject API
		UE_API uint32 Release() const;

	private:
		UE_NONCOPYABLE(FAttributeBindingData);	// We disallow copy/move semantics

		FAttributeBindingData(TNonNullPtr<const UAbstractSkeletonSetBinding> Binding, const USkeletalMesh* SkeletalMesh);
		void ResetInternalReferences();

		// TODO: We should allocate the binding data, named sets, and typed sets data contiguously in a single buffer that is
		//       owned by the binding data to avoid wasted padding, multiple allocations, and reduce TLB pressure

		// A named set entry per LOD (NumSets * NumLODs)
		// Using the LOD index within the named set, we can quickly navigate to other LODs using a pointer offset
		TArray<FAttributeNamedSet> NamedSets;

		// A typed set entry per type and LOD for each named set (NumSets * NumTypes * NumLODs)
		// Using the LOD index within the outer named set, we can quickly navigate to other LODs using a pointer offset
		TArray<FAttributeTypedSet> TypedSets;

		// A map of set names to its named set data (at LOD 0)
		NamedSetMapType NamedSetMap;

		// A map of attribute type to a map of their name to their attribute binding index
		// Bone attribute names aren't sorted, they are in the same order as in the skeleton
		// All other attribute names are sorted by FName
		TMap<UScriptStruct*, TArray<FName>> AttributeNamesByType;

		// Default attribute values for each named set
		TMap<FName, FPoseValueBundlePtr> DefaultAttributeValuesByName;

		// Only filled when executing with a skeletal mesh
		TMap<FSkeletonPoseBoneIndex, FMeshPoseBoneIndex> SkeletonToMeshIndicesMap;

		// The source binding asset used to create this binding data
		TWeakObjectPtr<const UAbstractSkeletonSetBinding> BindingAsset;

		// The number of internal references for the purposes of reference counting
		uint32 NumInternalReferences = 0;

		friend UE_API FAttributeBindingDataPtr MakeAttributeBindingData(TNonNullPtr<const UAbstractSkeletonSetBinding> Binding, const USkeletalMesh* SkeletalMesh);
	};

	// Creates a new attribute binding data instance from the specified binding
	// Returns null if the binding isn't valid
	[[nodiscard]] UE_API FAttributeBindingDataPtr MakeAttributeBindingData(TNonNullPtr<const UAbstractSkeletonSetBinding> Binding, const USkeletalMesh* SkeletalMesh);
}

#undef UE_API
