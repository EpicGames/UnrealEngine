// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AttributeBindingData.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "UObject/ObjectKey.h"

#define UE_API UAF_API

namespace UE::UAF
{
	namespace Private
	{
		class FSetBindingAttributeBindingDataCache
		{
		public:
			FSetBindingAttributeBindingDataCache() = delete;
			explicit FSetBindingAttributeBindingDataCache(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding);

			FAttributeBindingDataPtr GetOrAdd(const USkeletalMesh* SkeletalMesh);

		private:
			const FAttributeBindingDataPtr SkeletonBindingData;
			const TWeakObjectPtr<const UAbstractSkeletonSetBinding> WeakSetBinding;

			TMap<FObjectKey, FAttributeBindingDataPtr> SkeletalMeshBindingData;
			FTransactionallySafeRWLock CacheLock;
		};
	}

	using FSetBindingAttributeBindingDataCachePtr = TSharedPtr<Private::FSetBindingAttributeBindingDataCache>;

	// A thread-safe global cache for mapping set bindings to the cache of attribute data for each skeletal mesh using that binding
	class FAttributeBindingDataCache
	{
	public:
		UE_API FAttributeBindingDataPtr GetOrAdd(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding, const USkeletalMesh* SkeletalMesh);

		UE_API void ResetSetBinding(TNonNullPtr<const UAbstractSkeletonSetBinding> SetBinding);
		UE_API void Reset();
		
	private:
		FTransactionallySafeRWLock CacheLock;
		TMap<FObjectKey, FSetBindingAttributeBindingDataCachePtr> SetBindingToCache;
	};

	UE_API extern FAttributeBindingDataCache GAttributeBindingDataCache;
}

#undef UE_API