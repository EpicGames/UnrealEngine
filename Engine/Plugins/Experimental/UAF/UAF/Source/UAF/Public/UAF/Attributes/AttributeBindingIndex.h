// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneIndices.h"

namespace UE::UAF
{
	// This represents an attribute index within a binding asset
	struct FAttributeBindingIndex final : public FBoneIndexBase
	{
	public:
		// Constructs an invalid index
		FAttributeBindingIndex() = default;

		// Constructs an index with the specified value
		explicit FAttributeBindingIndex(int32 InAttributeBindingIndex) { BoneIndex = InAttributeBindingIndex; }

		// Constructs an index with the specified skeleton bone index
		explicit FAttributeBindingIndex(FSkeletonPoseBoneIndex SkeletonBoneIndex)
		{
			// For now, the skeleton bone indices map directly to the attribute binding indices
			BoneIndex = SkeletonBoneIndex.GetInt();
		}

		// Returns the skeleton bone index that maps to this binding index (only valid for bones)
		FSkeletonPoseBoneIndex GetSkeletonBoneIndex() const
		{
			// For now, the skeleton bone indices map directly to the attribute binding indices
			return FSkeletonPoseBoneIndex(BoneIndex);
		}

		// Returns the skeleton bone index that maps to this binding index (only valid for bones)
		operator FSkeletonPoseBoneIndex() const
		{
			// For now, the skeleton bone indices map directly to the attribute binding indices
			return FSkeletonPoseBoneIndex(BoneIndex);
		}

		UE_BONE_INDEX_OPERATORS(FAttributeBindingIndex)
	};
}
