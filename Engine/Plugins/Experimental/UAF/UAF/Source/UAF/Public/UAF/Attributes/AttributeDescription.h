// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{
	// An attribute description
	struct FAttributeDescription
	{
		// Our attribute name
		FName Name;

		// The attribute name of our parent (or None if we are at the root)
		FName ParentName;

		// The attribute type
		UScriptStruct* Type = nullptr;

		// The LOD at which this attribute is active
		int32 LOD = 0;

		FAttributeDescription() = default;

		FAttributeDescription(FName InName, FName InParentName, UScriptStruct* InType, int32 InLOD)
			: Name(InName)
			, ParentName(InParentName)
			, Type(InType)
			, LOD(InLOD)
		{
		}
	};
}
