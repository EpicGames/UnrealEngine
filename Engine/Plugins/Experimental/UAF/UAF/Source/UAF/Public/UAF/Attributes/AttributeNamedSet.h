// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "UAF/Attributes/AttributeTypedSet.h"

#define UE_API UAF_API

namespace UE::UAF
{
	class FAttributeBindingData;

	using FAttributeBindingDataPtr = TRefCountPtr<const class FAttributeBindingData>;
	using FPoseValueBundlePtr = TSharedPtr<class FPoseValueBundle, ESPMode::ThreadSafe>;
	using FAttributeNamedSetPtr = TRefCountPtr<const class FAttributeNamedSet>;

	class FAttributeNamedSet
	{
	public:
		using TypedSetMapType = TSortedMap<UScriptStruct*, FAttributeTypedSet*>;

		class FTypedSetIterator
		{
		public:
			[[nodiscard]] UE_FORCEINLINE_HINT explicit FTypedSetIterator(const TypedSetMapType& TypedSetMap)
				: Iterator(TypedSetMap.CreateConstIterator())
			{
			}

			UE_FORCEINLINE_HINT FTypedSetIterator& operator++()
			{
				++Iterator;
				return *this;
			}

			/** conversion to "bool" returning true if the iterator is valid. */
			[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
			{
				return !!Iterator;
			}

			[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const FTypedSetIterator& Rhs) const
			{
				return Iterator == Rhs.Iterator;
			}
			[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const FTypedSetIterator& Rhs) const
			{
				return Iterator != Rhs.Iterator;
			}

			[[nodiscard]] UE_FORCEINLINE_HINT FAttributeTypedSetPtr operator*() const
			{
				return FAttributeTypedSetPtr(Iterator.Value(), true);
			}

			[[nodiscard]] UE_FORCEINLINE_HINT const FAttributeTypedSet* operator->() const
			{
				return Iterator.Value();
			}

		private:
			TypedSetMapType::TConstIterator Iterator;
		};

		// Returns the owning binding data this named set is part of
		[[nodiscard]] UE_API FAttributeBindingDataPtr GetBindingData() const;

		// Returns the name of this set, O(1)
		[[nodiscard]] UE_API FName GetName() const;

		// Returns the LOD of this set, O(1)
		[[nodiscard]] UE_API int32 GetLOD() const;

		// Returns the number of typed sets within this set, O(1)
		[[nodiscard]] UE_API int32 NumTypedSets() const;

		// Returns the number of distinct LODs in this set, O(1)
		[[nodiscard]] UE_API int32 NumLODs() const;

		// Returns whether or not this set is empty, O(1)
		[[nodiscard]] UE_API bool IsEmpty() const;

		// Returns the typed set associated with the specified type, O(logN)
		template <class AttributeType>
		[[nodiscard]] FAttributeTypedSetPtr FindTypedSet() const;

		// Returns the typed set associated with the specified type, O(logN)
		[[nodiscard]] UE_API FAttributeTypedSetPtr FindTypedSet(UScriptStruct* Type) const;

		// Returns this named set at the specified LOD, O(1)
		// Returns null if the LOD specified is invalid
		[[nodiscard]] UE_API FAttributeNamedSetPtr AtLOD(int32 LOD) const;

		// Returns an iterator over all typed sets contained within
		[[nodiscard]] UE_FORCEINLINE_HINT FTypedSetIterator CreateTypedSetIterator() const { return FTypedSetIterator(TypedSetMap); }

		// Returns a collection of the default attribute values for this named set
		[[nodiscard]] UE_API FPoseValueBundlePtr GetDefaultAttributeValues() const;

		// Reference counting API
		UE_API void AddRef() const;
		UE_API uint32 Release() const;
		[[nodiscard]] UE_API uint32 GetRefCount() const;

	private:
		UE_NONCOPYABLE(FAttributeNamedSet);	// We disallow copy/move semantics

		FAttributeNamedSet();

		// The name of this set
		FName Name;

		// The LOD of this named set instance
		int32 LOD = 0;

		// The number of LODs within this named set instance
		int32 SetNumLODs = 0;

		// The binding data that owns our memory
		FAttributeBindingData* Owner = nullptr;

		// A map of attribute type to its typed set within our named set
		TypedSetMapType TypedSetMap;

		friend FAttributeBindingData;
		friend FAttributeTypedSet;
	};

	//////////////////////////////////////////////////////////////////////////
	// Implementation

	template <class AttributeType>
	inline FAttributeTypedSetPtr FAttributeNamedSet::FindTypedSet() const
	{
		return FindTypedSet(AttributeType::StaticStruct());
	}
}

#undef UE_API
