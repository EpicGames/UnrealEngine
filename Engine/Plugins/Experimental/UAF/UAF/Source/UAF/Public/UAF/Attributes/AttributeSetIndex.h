// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneIndices.h"

namespace UE::UAF
{
	// This represents an attribute index within an attribute set
	struct FAttributeSetIndex final : public FBoneIndexBase
	{
	public:
		// Constructs an invalid index
		FAttributeSetIndex() = default;

		// Constructs an index with the specified value
		explicit FAttributeSetIndex(int32 InAttributeIndex) { BoneIndex = InAttributeIndex; }

		UE_BONE_INDEX_OPERATORS(FAttributeSetIndex)
	};
}
