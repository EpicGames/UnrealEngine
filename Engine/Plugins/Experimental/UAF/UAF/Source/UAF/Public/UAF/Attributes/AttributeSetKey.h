// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/Attributes/AttributeBindingIndex.h"

#define UE_API UAF_API

namespace UE::UAF
{
	// The key by which to sort out attributes within a set
	// We want attributes with a higher LOD value to come first (e.g. LOD 0 before LOD 1) to
	// allow us to truncate the attribute array for a specific LOD
	// We want parent attributes to come before their children to facilitate space conversion
	// It has the following format:
	//   - Bits [0, 15]:	Attribute binding index (1-based) (16 bits)
	//   - Bits [16, 31]:	Parent attribute binding index (1-based, 0 if we have no parent) (16 bits)
	//   - Bits [32, 39]:	LOD (8 bits)
	// This allows us to sort smaller values first
	// This means that set keys are consistent within a binding and can be used by multiple named/typed sets within
	// and compared accordingly. However, they cannot be used across binding assets.
	class FAttributeSetKey final
	{
	public:
		constexpr FAttributeSetKey() = default;
		constexpr FAttributeSetKey(const FAttributeSetKey& Other) : Value(Other.Value) {}
		constexpr FAttributeSetKey(FAttributeSetKey&& Other) : Value(Other.Value) {}

		[[nodiscard]] UE_API FAttributeSetKey(FAttributeBindingIndex BindingIndex, FAttributeBindingIndex ParentBindingIndex, int32 LOD);

		[[nodiscard]] bool IsValid() const { return Value != 0; }

		[[nodiscard]] int32 GetLOD() const { return static_cast<int32>((Value >> kLODShift) & kLODMask); }
		[[nodiscard]] FAttributeBindingIndex GetBindingIndex() const { return FAttributeBindingIndex(static_cast<uint32>(Value & kIndexMask) - 1); }
		[[nodiscard]] FAttributeBindingIndex GetParentBindingIndex() const { return FAttributeBindingIndex(static_cast<uint32>((Value >> kParentIndexShift) & kIndexMask) - 1); }

		FAttributeSetKey& operator=(const FAttributeSetKey& Rhs) { Value = Rhs.Value; return *this; }
		FAttributeSetKey& operator=(FAttributeSetKey&& Rhs) { Value = Rhs.Value; return *this; }

		[[nodiscard]] explicit operator bool() const { return IsValid(); }

		[[nodiscard]] friend uint32 GetTypeHash(FAttributeSetKey SetKey) { return GetTypeHash(SetKey.Value); }

		// Relational operators
		[[nodiscard]] bool operator==(FAttributeSetKey Rhs) const { return Value == Rhs.Value; }
		[[nodiscard]] bool operator!=(FAttributeSetKey Rhs) const { return Value != Rhs.Value; }
		[[nodiscard]] bool operator>(FAttributeSetKey Rhs) const { return Value > Rhs.Value; }
		[[nodiscard]] bool operator>=(FAttributeSetKey Rhs) const { return Value >= Rhs.Value; }
		[[nodiscard]] bool operator<(FAttributeSetKey Rhs) const { return Value < Rhs.Value; }
		[[nodiscard]] bool operator<=(FAttributeSetKey Rhs) const { return Value <= Rhs.Value; }

	private:
		static constexpr uint64 kLODShift = 32;
		static constexpr uint64 kParentIndexShift = 16;
		static constexpr uint64 kIndexMask = 0x0000FFFFULL;
		static constexpr uint64 kLODMask = 0x000000FFULL;

		uint64 Value = 0;
	};
}

#undef UE_API
