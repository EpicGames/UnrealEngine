// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/RefCounting.h"
#include "UAF/Attributes/AttributeBindingIndex.h"
#include "UAF/Attributes/AttributeDescription.h"
#include "UAF/Attributes/AttributeSetIndex.h"
#include "UAF/Attributes/AttributeSetKey.h"

#define UE_API UAF_API

namespace UE::UAF
{
	class FAttributeBindingData;
	class FAttributeNamedSet;

	using FAttributeBindingDataPtr = TRefCountPtr<const class FAttributeBindingData>;
	using FAttributeNamedSetPtr = TRefCountPtr<const class FAttributeNamedSet>;
	using FAttributeTypedSetPtr = TRefCountPtr<const class FAttributeTypedSet>;

	class FAttributeTypedSet
	{
	public:
		UE_NONCOPYABLE(FAttributeTypedSet);	// We disallow copy/move semantics

		// Returns the named set this typed set is part of
		[[nodiscard]] UE_API FAttributeNamedSetPtr GetNamedSet() const;

		// Returns the owning binding data this typed set is part of
		[[nodiscard]] UE_API FAttributeBindingDataPtr GetBindingData() const;

		// Returns the type of attributes held in this set, O(1)
		[[nodiscard]] UE_API UScriptStruct* GetType() const;

		// Returns the LOD of this set, O(1)
		[[nodiscard]] UE_API int32 GetLOD() const;

		// Returns the number of attributes in this set, O(1)
		[[nodiscard]] UE_API int32 Num() const;

		// Returns the number of distinct LODs in this set, O(1)
		[[nodiscard]] UE_API int32 NumLODs() const;

		// Returns whether or not this set is empty, O(1)
		[[nodiscard]] UE_API bool IsEmpty() const;

		// Returns the typed set of the same type at the specified LOD, O(1)
		// Returns null if the LOD specified is invalid
		[[nodiscard]] UE_API FAttributeTypedSetPtr AtLOD(int32 LOD) const;

		// Searches for an attribute set key using an attribute name, O(logN)
		// Returns an invalid key if the attribute isn't found within the set
		[[nodiscard]] UE_API FAttributeSetKey FindKey(FName AttributeName) const;

		// Searches for an attribute name using an attribute set key, O(logN)
		// Returns None if the attribute isn't found within the set
		[[nodiscard]] UE_API FName FindName(FAttributeSetKey AttributeKey) const;

		// Searches for an attribute set index using an attribute name, O(logN)
		// Returns an invalid index if the attribute isn't found within the set
		[[nodiscard]] UE_API FAttributeSetIndex FindIndex(FName AttributeName) const;

		// Searches for an attribute set index using an attribute set key, O(logN)
		// Returns an invalid index if the attribute isn't found within the set
		[[nodiscard]] UE_API FAttributeSetIndex FindIndex(FAttributeSetKey AttributeKey) const;

		// Searches for an attribute binding index using an attribute name, O(N)
		// Returns an invalid index if the attribute isn't found within the binding
		[[nodiscard]] UE_API FAttributeBindingIndex FindBindingIndex(FName AttributeName) const;

		// Retrieve an attribute set index using an attribute binding index, O(1)
		// Returns an invalid index if the attribute isn't found within the set
		[[nodiscard]] UE_API FAttributeSetIndex GetIndex(FAttributeBindingIndex BindingIndex) const;

		// Retrieve an attribute binding index using an attribute set index, O(1)
		// Returns an invalid index if the attribute set index is invalid
		[[nodiscard]] UE_API FAttributeBindingIndex GetBindingIndex(FAttributeSetIndex AttributeIndex) const;

		// Retrieves the parent attribute index of the specified attribute, O(1)
		// Returns an invalid index if the child attribute index is invalid or the parent isn't within this set
		[[nodiscard]] UE_API FAttributeSetIndex GetParentIndex(FAttributeSetIndex AttributeIndex) const;

		// Retrieve the name of the specified attribute, O(1)
		// Returns None if the attribute set index is invalid
		[[nodiscard]] UE_API FName GetName(FAttributeSetIndex AttributeIndex) const;

		// Retrieves the set key of the specified attribute, O(1)
		// Returns an invalid set key if the attribute set index is invalid
		[[nodiscard]] UE_API FAttributeSetKey GetKey(FAttributeSetIndex AttributeIndex) const;

		// Reference counting API
		UE_API void AddRef() const;
		UE_API uint32 Release() const;
		[[nodiscard]] UE_API uint32 GetRefCount() const;

	private:
		FAttributeTypedSet();
		void Init(TConstArrayView<FAttributeDescription> SetAttributes, TConstArrayView<FName> BindingAttributes);
		void Reset();

		// The type that attributes in this set have
		UScriptStruct* Type = nullptr;

		// The outer container that owns our memory
		FAttributeBindingData* Owner = nullptr;

		// The named set that contains us
		FAttributeNamedSet* NamedSet = nullptr;

		// Sorted list of attribute names, maps a name to its attribute index
		TArray<TTuple<FName, uint16>> AttributeNameToIndexMap;

		// List of attributes (sorted by key)
		TArray<FAttributeSetKey> AttributeKeys;

		// List of attribute names (sorted by key)
		TArray<FName> AttributeNames;

		// List of parent attribute set indices (sorted by key)
		TArray<uint16> ParentAttributeIndices;

		// List of attribute binding indices (sorted by key)
		TArray<uint16> BindingIndices;

		// Maps each attribute binding index to an attribute set index
		TArray<uint16> BindingIndexToSetIndexMap;

		friend FAttributeBindingData;
	};
}

#undef UE_API
