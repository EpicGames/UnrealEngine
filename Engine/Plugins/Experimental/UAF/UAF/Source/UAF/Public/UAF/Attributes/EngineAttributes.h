// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/BuiltInAttributeTypes.h"

#include "EngineAttributes.generated.h"

#define UE_API UAF_API

// A bone transform is a special attribute type that allows for template specialization
USTRUCT(BlueprintType)
struct FBoneTransformAnimationAttribute : public FTransformAnimationAttribute
{
	GENERATED_BODY()

	// Allow promotion from plain transform to bone transform
	using FTransformAnimationAttribute::operator=;
};

namespace UE::Anim
{
	template<>
	struct TAttributeTypeTraits<FBoneTransformAnimationAttribute> : public TAttributeTypeTraits<FTransformAnimationAttribute>
	{
		// Inherit the flags from FTransformAnimationAttribute
	};
}

namespace UE::UAF
{
	// Handles registration for built-in engine attributes and value types
	UE_API void RegisterEngineAttributes();
}

#undef UE_API
