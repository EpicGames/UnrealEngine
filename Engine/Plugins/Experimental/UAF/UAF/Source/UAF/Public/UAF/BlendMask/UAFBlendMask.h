// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HierarchyTable.h"
#include "HierarchyTableBlendProfile.h"

#include "UAFBlendMask.generated.h"

#define UE_API UAF_API

class UUAFBlendMaskFactory;
namespace UE::UAF
{
class FBlendMaskEditorToolkit;
}

// An asset to define what parts to include in a blend when layering.
UCLASS(MinimalAPI, EditInlineNew, BlueprintType)
class UUAFBlendMask : public UObject
{
	GENERATED_BODY()

public:
	using FSkeletonBoneWeightArray = TArray<float>;
	using FSkeletonCurveWeightArray = UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement>;
	using FSkeletonAttributeWeightArray = TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight>;

	// Begin UObject
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#endif
	// End UObject

#if WITH_EDITOR
	UE_API void UpdateHierarchy();

	UE_API void UpdateCachedData();
#endif

	UE_API TObjectPtr<USkeleton> GetSkeleton() const;

	UE_API const FSkeletonBoneWeightArray& GetSkeletonBoneWeights() const;
	UE_API const FSkeletonCurveWeightArray& GetSkeletonCurveWeights() const;
	UE_API const FSkeletonAttributeWeightArray& GetSkeletonAttributeWeights() const;

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UHierarchyTable> Table;
#endif // WITH_EDITORONLY_DATA

	// The flattened version of the table data for use at runtime instead of,
	// slowly traversing the table tree.
	UPROPERTY()
	FBlendProfileStandaloneCachedData CachedBlendProfileData;

private:
#if WITH_EDITOR
	void SetSkeleton(TObjectPtr<USkeleton> InSkeleton);

	void HandleSkeletonHierarchyChanged();
#endif // WITH_EDITOR

	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = Default)
	TObjectPtr<USkeleton> Skeleton;

	friend class UUAFBlendMaskFactory;
	friend class UE::UAF::FBlendMaskEditorToolkit;
};

#undef UE_API