// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HierarchyTable.h"
#include "HierarchyTableBlendProfile.h"

#include "UAFBlendProfile.generated.h"

#define UE_API UAF_API

class UUAFBlendProfileFactory;
namespace UE::UAF
{
	class FBlendProfileEditorToolkit;

	inline const FName AssetRegistryTag_BlendProfileType = TEXT("BlendProfileType");
	inline const FString AssetRegistryTag_BlendProfileType_TimeProfile = TEXT("Time");
	inline const FString AssetRegistryTag_BlendProfileType_WeightProfile = TEXT("Weight");
}

UENUM()
enum class EUAFBlendProfileType
{
	WeightFactor,
	TimeFactor,
};

// An asset to define how a blend should be applied in a transition with per-bone controls.
UCLASS(MinimalAPI, EditInlineNew, BlueprintType)
class UUAFBlendProfile : public UObject
{
	GENERATED_BODY()

public:
	using FSkeletonBoneWeightArray = TArray<float>;
	using FSkeletonCurveWeightArray = UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement>;
	using FSkeletonAttributeWeightArray = TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight>;
	
	// Begin UObject
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#endif
	// End UObject

#if WITH_EDITOR
	UE_API void UpdateHierarchy();

	UE_API void UpdateCachedData();
#endif

	UE_API TObjectPtr<USkeleton> GetSkeleton() const;

	// Returns whether this blend profile is a time-based or weight-based time profile.
	UE_API EUAFBlendProfileType GetType() const;

	// Similar to GetType but returns the value in terms of the engine blend profile mode.
	// Used for reusing UBlendProfile engine code for UAF blend profiles.
	// Will never return EBlendProfileMode::BlendMask.
	UE_API EBlendProfileMode GetEngineType() const;

	UE_API const FSkeletonBoneWeightArray& GetSkeletonBoneWeights() const;
	UE_API const FSkeletonCurveWeightArray& GetSkeletonCurveWeights() const;
	UE_API const FSkeletonAttributeWeightArray& GetSkeletonAttributeWeights() const;

private:
	UPROPERTY(VisibleAnywhere, Category = Default, AssetRegistrySearchable)
	EUAFBlendProfileType Type;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UHierarchyTable> Table;
#endif // WITH_EDITORONLY_DATA

	// The flattened version of the table data for use at runtime instead of,
	// slowly traversing the table tree.
	UPROPERTY()
	FBlendProfileStandaloneCachedData CachedBlendProfileData;

private:
#if WITH_EDITOR
	void SetSkeleton(TObjectPtr<USkeleton> InSkeleton);

	void HandleSkeletonHierarchyChanged();
#endif // WITH_EDITOR

	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = Default)
	TObjectPtr<USkeleton> Skeleton;

	friend class UUAFBlendProfileFactory;
	friend class UE::UAF::FBlendProfileEditorToolkit;
};

#undef UE_API