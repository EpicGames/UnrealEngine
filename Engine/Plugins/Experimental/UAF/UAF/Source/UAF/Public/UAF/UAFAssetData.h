// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"

#include "UAFAssetData.generated.h"

/* FUAFAssetData is the encapsulation of a UAF compatible asset. It is used by the System and Graph factories
 * to produce UAF assets.
 *
 * Generally a UAFAssetData will correspond to e.g. an animation sequence, a blend space, a chooser table etc.
 *
 * Plugins can define their own asset data types and register them with the appropriate factories (e.g. FAnimGraphFactory, FSystemFactory)
 * to define how the asset should be transformed into a UAF. It can also be registered with the UAF::FAssetDataFactory to enable drag and drop
 * functionality from UObject
 */
USTRUCT(meta=(Hidden))
struct FUAFAssetData
{
	GENERATED_BODY()
	virtual ~FUAFAssetData() {}

	// If false, this data is not valid to attempt to construct a UAF asset from
	virtual bool Validate() const { return false; }

	virtual void GetObjectReferences(TArray<const UObject*>& OutReferencedObjects) const { }

	virtual bool CompareAsset(const TConstStructView<FUAFAssetData>& Other) const { unimplemented(); return false; }

	virtual uint32 GetAssetTypeHash() const { unimplemented(); return 0; }

#if WITH_EDITOR
	virtual UAF_API FString GetName() const;
#endif
};

// Assets deriving from this type can be used to create an animation system via the system factory
USTRUCT(meta=(Hidden))
struct FUAFSystemFactoryAsset : public FUAFAssetData
{
	GENERATED_BODY()
};

// Assets deriving from this type can be used to create an animation graph via the animation graph factory
USTRUCT(meta=(Hidden))
struct FUAFGraphFactoryAsset : public FUAFSystemFactoryAsset
{
	GENERATED_BODY()
};

// Useful aliases
namespace UE::UAF
{
using FAssetHandle = TInstancedStruct<FUAFAssetData>;

using FSystemAssetHandle = TInstancedStruct<FUAFSystemFactoryAsset>;
using FSystemAssetHandleConstView = TConstStructView<FUAFSystemFactoryAsset>;

using FGraphAssetHandle = TInstancedStruct<FUAFGraphFactoryAsset>;
using FGraphAssetHandleConstView = TConstStructView<FUAFGraphFactoryAsset>;
}
