// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/SubScriptStructOf.h"

struct FUAFAssetData;

namespace UE::UAF
{
/*
 * The FAssetDataFactory is used to produce FUAFAssetData from UObjects. This is mostly used for UI interactions 
 * in editor when users drop objects on to FUAFAssetData slots, for deprecation and also for some legacy API that takes
 * UObject* directly to create UAF graphs + systems.
 *
 * This factory links UObject class types to FUAFAssetData types.
 *
 * Note: It is not a requirement for a FUAFAssetData be created from a UObject. It could also be created manually
 */
struct FAssetDataFactory
{
public:

	// Create a UAFAssetData object from the supplied Object
	static UAF_API TInstancedStruct<FUAFAssetData> CreateUAFAssetDataFromObject(const UObject* Object);

	// Attempt to create a typed version of a UAF asset data from a UObject. This checks that the desired TAssetDataType
	// Is a base class of the found initializer that corresponds to the UObject class supplied
	template<typename TAssetDataType>
	static TInstancedStruct<TAssetDataType> CreateUAFAssetDataFromObject(const UObject* Object)
	{
		check(Object != nullptr);
	
		const FInitializerInfo* InitializerInfo = FindAssetInitializer(Object->GetClass());
		if (ensureMsgf(InitializerInfo != nullptr, TEXT("Unsupported object type (%s) passed to UAF::FAssetFactory"), *Object->GetClass()->GetFullName()))
		{
			// Check type is compatible
			if (InitializerInfo->AssetDataType->IsChildOf(TAssetDataType::StaticStruct()))
			{
				TInstancedStruct<FUAFAssetData> BaseAssetData = InitializerInfo->Initializer(Object);
				TInstancedStruct<TAssetDataType> DerivedType;
				// Use reinterpret cast so we can move the memory without an additional allocation
				// We know this is OK because we just validated the type using IsChildOf
				// and since TInstancedStruct simply contains a FInstancedStruct it is 'safe' to do this
				DerivedType = MoveTemp(*reinterpret_cast<TInstancedStruct<TAssetDataType>*>(&BaseAssetData));

				return DerivedType;
			}
		}

		return TInstancedStruct<TAssetDataType>();
	}

	template<typename TAssetDataType, typename TObjectType>
	using TAssetInitializer = TFunction<TAssetDataType(const TObjectType*)>;

	// Register an UAF asset with the factory
	// This defines a relationship between the TObjectType and the TAssetDataType.
	// The initializer definnes how to create a TAssetDataType from a TObjectType instance
	// @return a FTopLevelAssetPath that can be used to unregister the asset type via UnregisterAsset
	template<typename TAssetDataType, typename TObjectType>
	[[nodiscard]] static FTopLevelAssetPath RegisterAsset(TAssetInitializer<TAssetDataType, TObjectType>&& Initializer)
	{
		static_assert(std::is_base_of_v<FUAFAssetData, TAssetDataType>); // Check for invalid types. TAssetDataType must derive from FUAFAssetData
		return RegisterAsset(TAssetDataType::StaticStruct(), TObjectType::StaticClass(), [Initializer](const UObject* ObjectIn)->TInstancedStruct<FUAFAssetData>
			{
				return TInstancedStruct<TAssetDataType>::Make(Initializer(Cast<TObjectType>(ObjectIn)));
			});
	}

	// Unregister an UAF asset with the factory
	static UAF_API void UnregisterAsset(const FTopLevelAssetPath& InObjectClassPath);

	// Get all registered UClass types
	static UAF_API TArray<UClass*> GetRegisteredObjectClasses();

	// Get registered UClass types that correspond to a specific baseclass of FUUAFAssetData
	static UAF_API TArray<UClass*> GetRegisteredObjectClasses(TSubScriptStructOf<FUAFAssetData> BaseAssetType);

	template<typename TAssetDataType>
	static TArray<UClass*> GetRegisteredObjectClasses()
	{
		static_assert(std::is_base_of_v<FUAFAssetData, TAssetDataType>);
		return GetRegisteredObjectClasses(TAssetDataType::StaticStruct());
	}
private:

	// Untyped UObject initializer
	using FAssetInitializer = TFunction<TInstancedStruct<FUAFAssetData>(const UObject*)>;

	struct FInitializerInfo
	{
		FInitializerInfo(TSubScriptStructOf<FUAFAssetData> Type, FAssetInitializer Initializer)
			: AssetDataType(Type), Initializer(Initializer){}

		FInitializerInfo() = default;
		
		TSubScriptStructOf<FUAFAssetData> AssetDataType;
		FAssetInitializer Initializer;
	};
	
	
	// Register how to create a FUAFAssetReference from a ObjectClass
	static UAF_API FTopLevelAssetPath RegisterAsset(TSubScriptStructOf<FUAFAssetData> AssetDataType, UClass* ObjectClass, FAssetInitializer&& Initializer);
	static UAF_API const FInitializerInfo* FindAssetInitializer(const UClass* Class);
	static TMap<FTopLevelAssetPath, FInitializerInfo> AssetInitializers;
	static TArray<FTopLevelAssetPath> AllSupportedObjectClassPaths;
	
	// Helper function to recursively gather all valid non-registered subclasses of SuperClass and add them into InOutCompatibleClasses
	// Stops each branch at the first subclass that has its own registered initializer in AssetInitializers to avoid adding subclasses that have their 
	// own or closer related registered initializer already
	static void GatherDerivedCompatibleObjectClasses(const UClass* SuperClass, TArray<UClass*>& InOutCompatibleClasses); 
	
	friend class FUAFAssetDataPropertyCustomization;
};
}
