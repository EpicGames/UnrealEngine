// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/AssetUserData.h"

#include "UAFSkeletonUserData.generated.h"

#define UE_API UAF_API

class UAbstractSkeletonLabelBinding;
class UAbstractSkeletonSetBinding;
class USkeleton;

// Encapsulates the extra skeleton user data necessary for UAF
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "UAF Skeleton Data"))
class UUAFSkeletonUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	// Returns the UAF skeleton user data associated with the specified skeleton
	// If one isn't set by the user, a new one is allocated on demand and cached
	static UE_API UUAFSkeletonUserData* FromSkeleton(TNonNullPtr<USkeleton> Skeleton);

	// The default abstract skeleton label binding asset to use
	UE_API UAbstractSkeletonLabelBinding* GetLabelBinding() const;

	// The default abstract skeleton set binding asset to use
	UE_API UAbstractSkeletonSetBinding* GetSetBinding() const;

private:
	// The default abstract skeleton label binding asset to use
	UPROPERTY(EditAnywhere, Category = AbstractSkeleton)
	TObjectPtr<UAbstractSkeletonLabelBinding> LabelBinding;

	// The default abstract skeleton set binding asset to use
	UPROPERTY(EditAnywhere, Category = AbstractSkeleton)
	TObjectPtr<UAbstractSkeletonSetBinding> SetBinding;
};

#undef UE_API
