// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API UAF_API

namespace UE::UAF
{
	// An attribute mapping key encodes a mapping from an attribute type to a value type
	// akin to the key in something like TMap<TPair<AttributeType, ValueType>, TArray<ValueType>>
	// Mapping keys have a default sort order where we first sort by ValueType then by AttributeType
	class FAttributeMappingKey
	{
	public:
		// Smallest possible value for a mapping key
		UE_API static const FAttributeMappingKey SMALLEST_VALUE;

		// Largest possible value for a mapping key
		UE_API static const FAttributeMappingKey LARGEST_VALUE;

		// Creates a default mapping key which will have the smallest value possible (aka zero)
		constexpr FAttributeMappingKey() = default;

		// Creates a mapping key from the specified type which maps onto itself
		[[nodiscard]] explicit constexpr FAttributeMappingKey(UScriptStruct* AttributeAndValueType);

		// Creates a mapping key from the specified values
		[[nodiscard]] constexpr FAttributeMappingKey(UScriptStruct* AttributeType, UScriptStruct* ValueType);

		// Returns a new mapping key where we map from the specified attribute type to nothing
		template<class AttributeType>
		[[nodiscard]] static FAttributeMappingKey MakeFrom();

		// Returns a new mapping key where we map from the specified attribute type to nothing
		[[nodiscard]] static constexpr FAttributeMappingKey MakeFrom(UScriptStruct* AttributeType);

		// Returns a new mapping key where we map from nothing to the specified value type
		template<class ValueType>
		[[nodiscard]] static FAttributeMappingKey MakeTo();

		// Returns a new mapping key where we map from nothing to the specified value type
		[[nodiscard]] static constexpr FAttributeMappingKey MakeTo(UScriptStruct* ValueType);

		// Returns a new mapping key where we map onto itself
		template<class AttributeAndValueType>
		[[nodiscard]] static FAttributeMappingKey MakeFromTo();

		// Returns a new mapping key where we map onto itself
		[[nodiscard]] static constexpr FAttributeMappingKey MakeFromTo(UScriptStruct* AttributeAndValueType);

		// Returns a new mapping key where we map from the specified attribute type to the specified value type
		template<class FromAttributeType, class ToValueType>
		[[nodiscard]] static FAttributeMappingKey MakeFromTo();

		// Returns a new mapping key where we map from the specified attribute type to the specified value type
		[[nodiscard]] static constexpr FAttributeMappingKey MakeFromTo(UScriptStruct* FromAttributeType, UScriptStruct* ToValueType);

		// Returns a new mapping key where we map from the specified attribute type
		template<class FromAttributeType>
		[[nodiscard]] FAttributeMappingKey From() const;

		// Returns a new mapping key where we map from the specified attribute type
		[[nodiscard]] constexpr FAttributeMappingKey From(UScriptStruct* FromAttributeType) const;

		// Returns a new mapping key where we map to the specified value type
		template<class ToValueType>
		[[nodiscard]] FAttributeMappingKey To() const;

		// Returns a new mapping key where we map to the specified value type
		[[nodiscard]] constexpr FAttributeMappingKey To(UScriptStruct* ToValueType) const;

		// Returns the attribute type that we map from
		[[nodiscard]] constexpr UScriptStruct* GetAttributeType() const { return AttributeType; }

		// Returns the value type that we map to
		[[nodiscard]] constexpr UScriptStruct* GetValueType() const { return ValueType; }

		// LessThan operator for sorting and searching operations
		[[nodiscard]] bool operator<(const FAttributeMappingKey& Other) const;

		// Equality operator
		[[nodiscard]] bool operator==(const FAttributeMappingKey& Other) const;

	private:
		// The attribute type of our typed set whose values we map to
		UScriptStruct* AttributeType = nullptr;

		// The type of values we map onto our set
		UScriptStruct* ValueType = nullptr;
	};

	//////////////////////////////////////////////////////////////////////////
	// Implementation

	constexpr FAttributeMappingKey::FAttributeMappingKey(UScriptStruct* AttributeAndValueType)
		: AttributeType(AttributeAndValueType)
		, ValueType(AttributeAndValueType)
	{
	}

	constexpr FAttributeMappingKey::FAttributeMappingKey(UScriptStruct* InAttributeType, UScriptStruct* InValueType)
		: AttributeType(InAttributeType)
		, ValueType(InValueType)
	{
	}

	template<class AttributeType>
	inline FAttributeMappingKey FAttributeMappingKey::MakeFrom()
	{
		return FAttributeMappingKey(AttributeType::StaticStruct(), nullptr);
	}

	constexpr FAttributeMappingKey FAttributeMappingKey::MakeFrom(UScriptStruct* AttributeType)
	{
		return FAttributeMappingKey(AttributeType, nullptr);
	}

	template<class ValueType>
	inline FAttributeMappingKey FAttributeMappingKey::MakeTo()
	{
		return FAttributeMappingKey(nullptr, ValueType::StaticStruct());
	}

	constexpr FAttributeMappingKey FAttributeMappingKey::MakeTo(UScriptStruct* ValueType)
	{
		return FAttributeMappingKey(nullptr, ValueType);
	}

	template<class AttributeAndValueType>
	inline FAttributeMappingKey FAttributeMappingKey::MakeFromTo()
	{
		return FAttributeMappingKey(AttributeAndValueType::StaticStruct(), AttributeAndValueType::StaticStruct());
	}

	constexpr FAttributeMappingKey FAttributeMappingKey::MakeFromTo(UScriptStruct* AttributeAndValueType)
	{
		return FAttributeMappingKey(AttributeAndValueType, AttributeAndValueType);
	}

	template<class FromAttributeType, class ToValueType>
	inline FAttributeMappingKey FAttributeMappingKey::MakeFromTo()
	{
		return FAttributeMappingKey(FromAttributeType::StaticStruct(), ToValueType::StaticStruct());
	}

	constexpr FAttributeMappingKey FAttributeMappingKey::MakeFromTo(UScriptStruct* FromAttributeType, UScriptStruct* ToValueType)
	{
		return FAttributeMappingKey(FromAttributeType, ToValueType);
	}

	template<class FromAttributeType>
	inline FAttributeMappingKey FAttributeMappingKey::From() const
	{
		return FAttributeMappingKey(FromAttributeType::StaticStruct(), ValueType);
	}

	constexpr FAttributeMappingKey FAttributeMappingKey::From(UScriptStruct* FromAttributeType) const
	{
		return FAttributeMappingKey(FromAttributeType, ValueType);
	}

	template<class ToValueType>
	inline FAttributeMappingKey FAttributeMappingKey::To() const
	{
		return FAttributeMappingKey(AttributeType, ToValueType::StaticStruct());
	}

	constexpr FAttributeMappingKey FAttributeMappingKey::To(UScriptStruct* ToValueType) const
	{
		return FAttributeMappingKey(AttributeType, ToValueType);
	}

	inline bool FAttributeMappingKey::operator<(const FAttributeMappingKey& Other) const
	{
		// Avoiding branches in sorting algorithms
		const ptrdiff_t AttributeTypeDelta = reinterpret_cast<intptr_t>(AttributeType) - reinterpret_cast<intptr_t>(Other.AttributeType);
		const ptrdiff_t ValueTypeDelta = reinterpret_cast<intptr_t>(ValueType) - reinterpret_cast<intptr_t>(Other.ValueType);

		// If the value type differs, sort using it, otherwise sort by the attribute type
		const ptrdiff_t SortDelta = ValueTypeDelta != 0 ? ValueTypeDelta : AttributeTypeDelta;
		return SortDelta < 0;
	}

	inline bool FAttributeMappingKey::operator==(const FAttributeMappingKey& Other) const
	{
		// Logical AND to avoid branching in sorting algorithms
		return (AttributeType == Other.AttributeType) & (ValueType == Other.ValueType);
	}
}

#undef UE_API
