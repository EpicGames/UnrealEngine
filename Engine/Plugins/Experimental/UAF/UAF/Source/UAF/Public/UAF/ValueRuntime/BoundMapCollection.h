// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/Attributes/AttributeNamedSet.h"
#include "UAF/ValueRuntime/AttributeMappingKey.h"
#include "UAF/ValueRuntime/BoundValueMap.h"
#include "UAF/ValueRuntime/IndirectAllocator.h"

#define UE_API UAF_API

namespace UE::UAF
{
	// A iterator that iterates over all bound value maps within a bound map collection
	template<class ContainerType>
	class TBoundMapCollectionIterator
	{
	public:
		static constexpr bool bIsContainerConst = std::is_const_v<ContainerType>;

		using EntryPtrType = typename std::conditional<bIsContainerConst, const typename ContainerType::FMapEntry*, typename ContainerType::FMapEntry*>::type;
		using MapType = typename std::conditional<bIsContainerConst, const FBoundValueMap*, FBoundValueMap*>::type;

		// Creates an empty iterator
		[[nodiscard]] TBoundMapCollectionIterator() = default;

		// Creates an iterator over the map entries within the specified collection
		[[nodiscard]] TBoundMapCollectionIterator(ContainerType& Collection);

		// Increments and moves the iterator to the next map entry
		TBoundMapCollectionIterator& operator++();

		// Returns whether or not the iterator still contains values
		[[nodiscard]] explicit operator bool() const;

		// Returns the mapping key of the current map entry
		[[nodiscard]] const FAttributeMappingKey& GetKey() const;

		// Returns the attribute type of the current map entry
		[[nodiscard]] UScriptStruct* GetAttributeType() const;

		// Returns the value type of the current map entry
		[[nodiscard]] UScriptStruct* GetValueType() const;

		// Returns the map from the current entry
		[[nodiscard]] MapType GetMap() const;

	private:
		// Current map entry
		EntryPtrType EntryPtr = nullptr;

		// One past the last map entry
		EntryPtrType EntryEndPtr = nullptr;
	};

	/*
	 * Bound Value Map Collection
	 *
	 * Holds a sorted collection of bound value maps.
	 * Entries are sorted by value type, then by attribute type.
	 */
	class FBoundMapCollection
	{
	private:
		struct FMapEntry
		{
			// The mapping key that identifies this entry
			FAttributeMappingKey Key;

			// The map containing our values
			FBoundValueMap* Map = nullptr;
		};

	public:
		UE_API FBoundMapCollection(const FAttributeNamedSetPtr& NamedSet, FReallocFun InReallocFun);
		UE_API FBoundMapCollection(const FBoundMapCollection& Other);
		UE_API FBoundMapCollection(FBoundMapCollection&& Other);
		UE_API ~FBoundMapCollection();

		UE_API FBoundMapCollection& operator=(const FBoundMapCollection& Other);
		UE_API FBoundMapCollection& operator=(FBoundMapCollection&& Other);

		// Returns whether or not we are empty
		[[nodiscard]] UE_API bool IsEmpty() const;

		// Returns the named set this collection maps to
		[[nodiscard]] UE_API const FAttributeNamedSetPtr& GetNamedSet() const;

		// Resets the collection with the specified named set
		UE_API void Reset(const FAttributeNamedSetPtr& NamedSet);

		// Clears the collection of any content
		UE_API void Empty();

		// Returns the allocator function this collection was initialized with
		[[nodiscard]] UE_API FReallocFun GetAllocator() const;

		// Returns the number of maps contained within
		[[nodiscard]] UE_API int32 Num() const;

		// Constructs and adds a new bound value map with the specified mapping (attribute -> value type).
		// Returns nullptr if we fail to add an entry (e.g. duplicate)
		template <class ValueType, typename... TArgs> [[nodiscard]] TBoundValueMap<ValueType>* Add(const FAttributeMappingKey& MappingKey, TArgs... Args);

		// Constructs and adds a new bound value map with the specified mapping (attribute -> value type).
		// Returns nullptr if we fail to add an entry (e.g. duplicate)
		[[nodiscard]] UE_API FBoundValueMap* Add(const FAttributeMappingKey& MappingKey);

		// Adds a map to this collection.
		// Returns false if we fail to add an entry (e.g. duplicate, mismatched allocator)
		UE_API bool Add(FBoundValueMap* Map);

		// Appends a new map to this collection (the input must belong at the end of the sorted map list).
		// Returns false if we fail to add an entry (e.g. duplicate, mismatched allocator, not sorted)
		UE_API bool Append(FBoundValueMap* Map);

		// Removes the specified map from this collection
		// Returns whether or not the entry was successfully removed
		UE_API bool Remove(FBoundValueMap* Map);

		// Removes the specified mapping from this collection
		// Returns whether or not the entry was successfully removed
		UE_API bool Remove(const FAttributeMappingKey& MappingKey);

		// Finds a map of ValueType to a set of AttributeType in this collection
		// Returns nullptr if this collection does not contain such a mapping
		template <class ValueType> [[nodiscard]] TBoundValueMap<ValueType>* Find(const FAttributeMappingKey& MappingKey);

		// Finds a map of ValueType to a set of AttributeType in this collection
		// Returns nullptr if this collection does not contain such a mapping
		template <class ValueType> [[nodiscard]] const TBoundValueMap<ValueType>* Find(const FAttributeMappingKey& MappingKey) const;

		// Finds a map of ValueType to a set of AttributeType in this collection
		// Returns nullptr if this collection does not contain such a mapping
		[[nodiscard]] UE_API FBoundValueMap* Find(const FAttributeMappingKey& MappingKey);

		// Finds a map of ValueType to a set of AttributeType in this collection
		// Returns nullptr if this collection does not contain such a mapping
		[[nodiscard]] UE_API const FBoundValueMap* Find(const FAttributeMappingKey& MappingKey) const;

		// Finds or adds a map of ValueType to a set of AttributeType in this collection
		// Returns nullptr if we fail to add an entry (e.g. duplicate)
		template <class ValueType, typename... TArgs> [[nodiscard]] TBoundValueMap<ValueType>* FindOrAdd(const FAttributeMappingKey& MappingKey, TArgs... Args);

		// Finds or adds a map of ValueType to a set of AttributeType in this collection
		// Returns nullptr if we fail to add an entry (e.g. duplicate)
		[[nodiscard]] UE_API FBoundValueMap* FindOrAdd(const FAttributeMappingKey& MappingKey);

		// Iterators
		using FIterator = TBoundMapCollectionIterator<FBoundMapCollection>;
		using FConstIterator = TBoundMapCollectionIterator<const FBoundMapCollection>;

		// Returns an iterator over all maps contained within, sorted by their key (value type first, then attribute type)
		[[nodiscard]] FIterator CreateIterator();
		[[nodiscard]] FConstIterator CreateConstIterator() const;

	private:
		UE_API FBoundValueMap** AddImpl(const FBoundValueMap::FConstructArgs& Args, bool bConstructMap);

		// The named set we are based on, every map uses a typed set within it
		FAttributeNamedSetPtr NamedSet;

		// An array of maps (sorted by mapping key)
		FMapEntry* Maps = nullptr;

		// How many maps are contained within this collection
		int32 MapCount = 0;

		// How many maps we can contain within this collection
		int32 MapCapacity = 0;

		// The indirect allocator function pointer
		FReallocFun ReallocFun = nullptr;

		friend FIterator;
		friend FConstIterator;
	};

	// A heap allocated bound map collection
	class FBoundMapCollectionHeap : public FBoundMapCollection
	{
	public:
		// Constructs an empty collection
		FBoundMapCollectionHeap();

		// Constructs an empty collection based on the specified named set
		explicit FBoundMapCollectionHeap(const FAttributeNamedSetPtr& NamedSet);

		using FBoundMapCollection::operator=;
	};

	// A Mem-Stack allocated bound map collection
	class FBoundMapCollectionStack : public FBoundMapCollection
	{
	public:
		// Constructs an empty collection
		FBoundMapCollectionStack();

		// Constructs an empty collection based on the specified named set
		explicit FBoundMapCollectionStack(const FAttributeNamedSetPtr& NamedSet);

		using FBoundMapCollection::operator=;
	};

	// An operator structure to join bound map collections using InnerJoin/OuterJoin/etc
	struct FBoundMapCollectionJoinOp
	{
		const FAttributeMappingKey& GetLargestKey() const { return FAttributeMappingKey::LARGEST_VALUE; }

		const FAttributeMappingKey& GetKey(const FBoundMapCollection::FIterator& It) const { return It.GetKey(); }
		const FAttributeMappingKey& GetKey(const FBoundMapCollection::FConstIterator& It) const { return It.GetKey(); }

		FBoundValueMap* GetValue(const FBoundMapCollection::FIterator& It) const { return It.GetMap(); }
		const FBoundValueMap* GetValue(const FBoundMapCollection::FConstIterator& It) const { return It.GetMap(); }
	};
}

//////////////////////////////////////////////////////////////////////////
// Implementation

namespace UE::UAF
{
	template <class ValueType, typename... TArgs>
	inline TBoundValueMap<ValueType>* FBoundMapCollection::Add(const FAttributeMappingKey& MappingKey, TArgs... Args)
	{
		if (!NamedSet)
		{
			// We are not initialized
			return nullptr;
		}

		UScriptStruct* ValueTypeStruct = ValueType::StaticStruct();
		if (!ensureMsgf(ValueTypeStruct == MappingKey.GetValueType(), TEXT("Mismatch between the return value type and mapping key value type")))
		{
			return nullptr;
		}

		FAttributeTypedSetPtr TypedSet = NamedSet->FindTypedSet(MappingKey.GetAttributeType());
		if (!TypedSet)
		{
			// Unknown attribute type, it isn't contained within our named set
			return nullptr;
		}

		// Forward the extra arguments
		typename TBoundValueMap<ValueType>::FConstructArgs ConstructArgs(TypedSet, ValueTypeStruct, ReallocFun, Args...);

		// Don't construct the map internally, it'll have to use indirection to do so and here we know the concrete type
		constexpr bool bConstructMap = false;

		FBoundValueMap** MapPtr = AddImpl(ConstructArgs, bConstructMap);
		if (MapPtr == nullptr)
		{
			return nullptr;
		}

		TBoundValueMap<ValueType>* Map = MakeBoundValueMap<ValueType>(ConstructArgs);

		check(*MapPtr == nullptr);
		*MapPtr = Map;

		return Map;
	}

	template <class ValueType>
	inline TBoundValueMap<ValueType>* FBoundMapCollection::Find(const FAttributeMappingKey& MappingKey)
	{
		UScriptStruct* ValueTypeStruct = ValueType::StaticStruct();
		if (!ensureMsgf(ValueTypeStruct == MappingKey.GetValueType(), TEXT("Mismatch between the return value type and mapping key value type")))
		{
			return nullptr;
		}

		return Cast<ValueType>(Find(MappingKey));
	}

	template <class ValueType>
	inline const TBoundValueMap<ValueType>* FBoundMapCollection::Find(const FAttributeMappingKey& MappingKey) const
	{
		UScriptStruct* ValueTypeStruct = ValueType::StaticStruct();
		if (!ensureMsgf(ValueTypeStruct == MappingKey.GetValueType(), TEXT("Mismatch between the return value type and mapping key value type")))
		{
			return nullptr;
		}

		return Cast<ValueType>(Find(MappingKey));
	}

	template <class ValueType, typename... TArgs>
	inline TBoundValueMap<ValueType>* FBoundMapCollection::FindOrAdd(const FAttributeMappingKey& MappingKey, TArgs... Args)
	{
		UScriptStruct* ValueTypeStruct = ValueType::StaticStruct();
		if (!ensureMsgf(ValueTypeStruct == MappingKey.GetValueType(), TEXT("Mismatch between the return value type and mapping key value type")))
		{
			return nullptr;
		}

		if (FBoundValueMap* Map = Find(MappingKey))
		{
			return Cast<ValueType>(Map);
		}

		return Add<ValueType>(MappingKey, Args...);
	}

	inline typename FBoundMapCollection::FIterator FBoundMapCollection::CreateIterator()
	{
		return FIterator(*this);
	}

	inline typename FBoundMapCollection::FConstIterator FBoundMapCollection::CreateConstIterator() const
	{
		return FConstIterator(*this);
	}

	inline FBoundMapCollectionHeap::FBoundMapCollectionHeap()
		: FBoundMapCollection(nullptr, &FAllocatorTypeTrait<FDefaultAllocator>::Realloc)
	{
	}

	inline FBoundMapCollectionHeap::FBoundMapCollectionHeap(const FAttributeNamedSetPtr& InNamedSet)
		: FBoundMapCollection(InNamedSet, &FAllocatorTypeTrait<FDefaultAllocator>::Realloc)
	{
	}

	inline FBoundMapCollectionStack::FBoundMapCollectionStack()
		: FBoundMapCollection(nullptr, &FAllocatorTypeTrait<TMemStackAllocator<>>::Realloc)
	{
	}

	inline FBoundMapCollectionStack::FBoundMapCollectionStack(const FAttributeNamedSetPtr& InNamedSet)
		: FBoundMapCollection(InNamedSet, &FAllocatorTypeTrait<TMemStackAllocator<>>::Realloc)
	{
	}

	template<class ContainerType>
	inline TBoundMapCollectionIterator<ContainerType>::TBoundMapCollectionIterator(ContainerType& Collection)
		: EntryPtr(Collection.Maps)
		, EntryEndPtr(Collection.Maps + Collection.MapCount)
	{
	}

	template<class ContainerType>
	inline TBoundMapCollectionIterator<ContainerType>& TBoundMapCollectionIterator<ContainerType>::operator++()
	{
		++EntryPtr;
		return *this;
	}

	template<class ContainerType>
	inline TBoundMapCollectionIterator<ContainerType>::operator bool() const
	{
		return EntryPtr != EntryEndPtr;
	}

	template<class ContainerType>
	inline const FAttributeMappingKey& TBoundMapCollectionIterator<ContainerType>::GetKey() const
	{
		return EntryPtr->Key;
	}

	template<class ContainerType>
	inline UScriptStruct* TBoundMapCollectionIterator<ContainerType>::GetAttributeType() const
	{
		return EntryPtr->Key.GetAttributeType();
	}

	template<class ContainerType>
	inline UScriptStruct* TBoundMapCollectionIterator<ContainerType>::GetValueType() const
	{
		return EntryPtr->Key.GetValueType();
	}

	template<class ContainerType>
	inline typename TBoundMapCollectionIterator<ContainerType>::MapType TBoundMapCollectionIterator<ContainerType>::GetMap() const
	{
		return EntryPtr->Map;
	}
}

#undef UE_API
