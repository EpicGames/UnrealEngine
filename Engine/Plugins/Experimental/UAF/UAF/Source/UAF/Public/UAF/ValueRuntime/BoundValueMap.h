// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/MemoryOps.h"
#include "UAF/Attributes/AttributeTypedSet.h"
#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/AttributeMappingKey.h"
#include "UAF/ValueRuntime/IndirectAllocator.h"

#define UE_API UAF_API

namespace UE::UAF
{
	// Untyped base class for TBoundValueMap
	class FBoundValueMap
	{
	public:
		UE_NONCOPYABLE(FBoundValueMap);	// We disallow copy/move semantics

		struct FDefaultConstructArgs
		{
			const FAttributeTypedSetPtr& TypedSet;
			UScriptStruct* ValueType = nullptr;
			FReallocFun ReallocFun = nullptr;

			FDefaultConstructArgs(const FAttributeTypedSetPtr& InTypedSet, UScriptStruct* InValueType, FReallocFun InReallocFun)
				: TypedSet(InTypedSet)
				, ValueType(InValueType)
				, ReallocFun(InReallocFun)
			{
			}
		};

		using FConstructArgs = FDefaultConstructArgs;

		// Returns whether or not this map has any values
		[[nodiscard]] UE_API bool IsEmpty() const;

		// Returns the number of mapped values
		[[nodiscard]] UE_API int32 Num() const;

		// Returns the attribute typed set we map to
		[[nodiscard]] UE_API const FAttributeTypedSetPtr& GetTypedSet() const;

		// Returns the attribute type we map to
		[[nodiscard]] UE_API UScriptStruct* GetAttributeType() const;

		// Returns the mapped value type
		[[nodiscard]] UE_API UScriptStruct* GetValueType() const;

		// Returns the mapping key represented by this map
		[[nodiscard]] UE_API FAttributeMappingKey GetMappingKey() const;

		// Returns the total allocated size for this map
		[[nodiscard]] UE_API int32 GetAllocatedSize() const;

		// Returns the allocator function this map was initialized with
		[[nodiscard]] UE_API FReallocFun GetAllocator() const;

		// Returns a copy of this map using the specified allocator
		[[nodiscard]] virtual FBoundValueMap* Duplicate(FReallocFun InReallocFun) const = 0;

		// Initializes every value to its identity, optionally additive
		virtual void FillWithIdentity(bool bIsAdditive) = 0;

		// Copies the content of this map into another
		// Both must have matching typed sets and value types
		virtual void CopyTo(FBoundValueMap& Other) const = 0;

		// Sets a value at the specified index using a value setter function
		// The value setter (and/or caller) is responsible for validating the value type
		virtual void SetValueWithSetter(FAttributeSetIndex Index, const TFunctionRef<void(UScriptStruct* ValueType, uint8* OutValuePtr)>& ValueSetter) = 0;

		// Gets a value at the specified index using a value getter function
		// The value getter (and/or caller) is responsible for validating the value type
		virtual void GetValueWithGetter(FAttributeSetIndex Index, const TFunctionRef<void(UScriptStruct* ValueType, const uint8* ValuePtr)>& ValueGetter) const = 0;

		// Sets a value at the specified index
		// The called is responsible for validating the value type matches the one contained within this map
		template <class ValueType>
		void SetValue(FAttributeSetIndex Index, const ValueType& Value);

		// Gets a value from the specified index
		// The called is responsible for validating the value type matches the one contained within this map
		template <class ValueType>
		[[nodiscard]] ValueType GetValue(FAttributeSetIndex Index) const;

	protected:
		// Default construction for an empty map
		UE_API FBoundValueMap();

		// Constructs a map but does not allocate its memory, derived types are responsible for it
		UE_API FBoundValueMap(const FConstructArgs& Args, int32 AllocatedSize);

		// Destroys the map and frees its memory
		UE_API virtual ~FBoundValueMap();

		// Sets a value at the specified index
		virtual void SetValueImpl(FAttributeSetIndex Index, const void* ValuePtr) = 0;

		// Gets a value from the specified index
		virtual void GetValueImpl(FAttributeSetIndex Index, void* OutValuePtr) const = 0;

		// The set whose attributes we map values to
		FAttributeTypedSetPtr TypedSet;

		// The type of values this map contains
		UScriptStruct* ValueTypeStruct = nullptr;

		// The total allocated size of this map
		int32 AllocatedSize = 0;

		// The number of values allocated in our buffer
		int32 NumValues = 0;

		// The base untyped pointer for our values
		uint8* ValuesPtr = nullptr;

		// The indirect allocator function pointer
		FReallocFun ReallocFun = nullptr;

		friend class FValueBundle;
		friend class FValueRuntimeRegistry;
		friend void ReleaseBoundValueMap(FBoundValueMap* Map);
	};

	// Encapsulates a mapping of a value type to a specific attribute set (of any attribute type)
	// Our keys are held by the attribute set we point to (keys are shared) while we hold the
	// mapped values internally. Bound value maps and their values are allocated together in a contiguous buffer.
	// Value types must be a UStruct
	template<class InValueType>
	class TBoundValueMap : public FBoundValueMap
	{
	public:
		// The type of values held in this map
		using ValueType = InValueType;

		// Returns a copy of this map using the specified allocator
		[[nodiscard]] virtual FBoundValueMap* Duplicate(FReallocFun InReallocFun) const override;

		// Initializes every value to the specified one
		void FillWithValue(InValueType Value);

		// Initializes every value to its identity, optionally additive
		virtual void FillWithIdentity(bool bIsAdditive) override;

		// Copies the content of this map into another
		// Both must have matching typed sets and value types
		virtual void CopyTo(FBoundValueMap& Other) const override;

		// Value accessors by index
		[[nodiscard]] InValueType& operator[](FAttributeSetIndex Index);

		// Value accessors by index
		[[nodiscard]] const InValueType& operator[](FAttributeSetIndex Index) const;

		// Returns a pointer to the contiguous values held within
		[[nodiscard]] InValueType* GetData();

		// Returns a pointer to the contiguous values held within
		[[nodiscard]] const InValueType* GetData() const;

		// Returns the total size required for an instance of a map of this type
		[[nodiscard]] static int32 CalculateAllocationSize(int32 NumAttributes);

		// Typed equivalent for FBoundValueMap::SetValue
		void SetValue(FAttributeSetIndex Index, const InValueType& Value)
		{
			FBoundValueMap::SetValue<InValueType>(Index, Value);
		}

		// Typed equivalent for FBoundValueMap::GetValue
		[[nodiscard]] InValueType GetValue(FAttributeSetIndex Index) const
		{
			return FBoundValueMap::GetValue<InValueType>(Index);
		}
		
	protected:
		// Constructs a map
		TBoundValueMap(const FConstructArgs& Args, int32 AllocatedSize);

		// Destroys the map and frees its memory
		virtual ~TBoundValueMap() override;

		virtual void SetValueWithSetter(FAttributeSetIndex Index, const TFunctionRef<void(UScriptStruct* ValueType, uint8* OutValuePtr)>& ValueSetter) override;
		virtual void GetValueWithGetter(FAttributeSetIndex Index, const TFunctionRef<void(UScriptStruct* ValueType, const uint8* ValuePtr)>& ValueGetter) const override;
		virtual void SetValueImpl(FAttributeSetIndex Index, const void* ValuePtr) override;
		virtual void GetValueImpl(FAttributeSetIndex Index, void* OutValuePtr) const override;

		template<class T>
		friend TBoundValueMap<T>* MakeBoundValueMap(const typename TBoundValueMap<T>::FConstructArgs& Args);
	};

	// Allocates and constructs a new bound value map for the specified value type
	// Bound value maps and their values are contiguous in memory
	template<class ValueType>
	TBoundValueMap<ValueType>* MakeBoundValueMap(const typename TBoundValueMap<ValueType>::FConstructArgs& Args);

	// Releases and destroys the specified bound value map
	void ReleaseBoundValueMap(FBoundValueMap* Map);

	// Cast from an untyped map to a typed map
	template<class ValueType>
	UE_FORCEINLINE_HINT TBoundValueMap<ValueType>* Cast(FBoundValueMap* Src)
	{
		return Src != nullptr && ValueType::StaticStruct() == Src->GetValueType() ? static_cast<TBoundValueMap<ValueType>*>(Src) : nullptr;
	}

	// Cast from an untyped map to a typed map
	template<class ValueType>
	UE_FORCEINLINE_HINT const TBoundValueMap<ValueType>* Cast(const FBoundValueMap* Src)
	{
		return Src != nullptr && ValueType::StaticStruct() == Src->GetValueType() ? static_cast<const TBoundValueMap<ValueType>*>(Src) : nullptr;
	}

	// Cast from an untyped map to a typed map with an assert if the types mismatch
	template<class ValueType>
	inline TBoundValueMap<ValueType>* CastChecked(FBoundValueMap* Src)
	{
		TBoundValueMap<ValueType>* CastResult = Src != nullptr && ValueType::StaticStruct() == Src->GetValueType() ? static_cast<TBoundValueMap<ValueType>*>(Src) : nullptr;
		check(CastResult);
		return CastResult;
	}

	// Cast from an untyped map to a typed map with an assert if the types mismatch
	template<class ValueType>
	inline const TBoundValueMap<ValueType>* CastChecked(const FBoundValueMap* Src)
	{
		const TBoundValueMap<ValueType>* CastResult = Src != nullptr && ValueType::StaticStruct() == Src->GetValueType() ? static_cast<const TBoundValueMap<ValueType>*>(Src) : nullptr;
		check(CastResult);
		return CastResult;
	}

	//////////////////////////////////////////////////////////////////////////
	// Implementation

	template<class ValueType>
	inline void FBoundValueMap::SetValue(FAttributeSetIndex Index, const ValueType& Value)
	{
		checkf(ValueType::StaticStruct() == ValueTypeStruct, TEXT("Value type does not match the map"));
		SetValueImpl(Index, &Value);
	}

	template<class ValueType>
	inline ValueType FBoundValueMap::GetValue(FAttributeSetIndex Index) const
	{
		checkf(ValueType::StaticStruct() == ValueTypeStruct, TEXT("Value type does not match the map"));

		ValueType Result;
		GetValueImpl(Index, &Result);

		return Result;
	}

	template<class InValueType>
	inline TBoundValueMap<InValueType>::TBoundValueMap(const FConstructArgs& Args, int32 InAllocatedSize)
		: FBoundValueMap(Args, InAllocatedSize)
	{
		const int32 NumAttributes = Args.TypedSet->Num();

		check(InAllocatedSize == TBoundValueMap<InValueType>::CalculateAllocationSize(NumAttributes));

		if (NumAttributes != 0)
		{
			ValuesPtr = Align(reinterpret_cast<uint8*>(this) + sizeof(TBoundValueMap<InValueType>), alignof(InValueType));
			NumValues = NumAttributes;

			DefaultConstructItems<InValueType>(ValuesPtr, NumAttributes);
		}
	}

	template<class InValueType>
	inline TBoundValueMap<InValueType>::~TBoundValueMap()
	{
		if (ValuesPtr != nullptr)
		{
			DestructItems<InValueType>(GetData(), NumValues);

			ValuesPtr = nullptr;
			NumValues = 0;
		}
	}

	template<class InValueType>
	inline FBoundValueMap* TBoundValueMap<InValueType>::Duplicate(FReallocFun InReallocFun) const
	{
		TBoundValueMap<InValueType>* Copy = MakeBoundValueMap<InValueType>(FConstructArgs(TypedSet, ValueTypeStruct, InReallocFun));

		if (NumValues != 0)
		{
			ConstructItems<InValueType>(Copy->GetData(), GetData(), NumValues);
		}

		return Copy;
	}

	template<class InValueType>
	inline void TBoundValueMap<InValueType>::FillWithValue(InValueType Value)
	{
		if (NumValues == 0)
		{
			return;
		}

		InValueType* ValuePtr = GetData();
		InValueType* ValuePtrEnd = ValuePtr + NumValues;
		while (ValuePtr < ValuePtrEnd)
		{
			*ValuePtr = Value;

			ValuePtr++;
		}
	}

	template<class InValueType>
	inline void TBoundValueMap<InValueType>::FillWithIdentity(bool bIsAdditive)
	{
		InValueType Identity{};

		if constexpr (std::is_base_of_v<FTransformAnimationAttribute, InValueType>)
		{
			// Transforms have a special additive identity
			if (bIsAdditive)
			{
				Identity = FTransformAnimationAttribute{ FTransform(FQuat::Identity, FVector::ZeroVector, FVector::ZeroVector) };
			}
			else
			{
				Identity = FTransformAnimationAttribute{ FTransform(FQuat::Identity, FVector::ZeroVector, FVector::OneVector) };
			}
		}
		else
		{
			// Everything else uses the default value
		}

		FillWithValue(Identity);
	}

	template<class InValueType>
	inline void TBoundValueMap<InValueType>::CopyTo(FBoundValueMap& Other) const
	{
		checkf(TypedSet == Other.GetTypedSet() && ValueTypeStruct == Other.GetValueType(), TEXT("Typed set and value type must match to allow copying"));
		check(Other.Num() == Num());	// If the typed sets match, their sizes does too

		TBoundValueMap<InValueType>& Copy = reinterpret_cast<TBoundValueMap<InValueType>&>(Other);

		CopyAssignItems<InValueType>(Copy.GetData(), GetData(), NumValues);
	}

	template<class InValueType>
	inline InValueType& TBoundValueMap<InValueType>::operator[](FAttributeSetIndex Index)
	{
		checkf((Index.GetInt() >= 0) & (Index.GetInt() < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index.GetInt(), (long long)NumValues); // & for one branch
		return GetData()[Index.GetInt()];
	}

	template<class InValueType>
	inline const InValueType& TBoundValueMap<InValueType>::operator[](FAttributeSetIndex Index) const
	{
		checkf((Index.GetInt() >= 0) & (Index.GetInt() < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index.GetInt(), (long long)NumValues); // & for one branch
		return GetData()[Index.GetInt()];
	}

	template<class InValueType>
	inline InValueType* TBoundValueMap<InValueType>::GetData()
	{
		return reinterpret_cast<InValueType*>(ValuesPtr);
	}

	template<class InValueType>
	inline const InValueType* TBoundValueMap<InValueType>::GetData() const
	{
		return reinterpret_cast<const InValueType*>(ValuesPtr);
	}

	template<class InValueType>
	inline void TBoundValueMap<InValueType>::SetValueWithSetter(FAttributeSetIndex Index, const TFunctionRef<void(UScriptStruct* ValueType, uint8* OutValuePtr)>& ValueSetter)
	{
		checkf((Index.GetInt() >= 0) & (Index.GetInt() < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index.GetInt(), (long long)NumValues); // & for one branch
		InValueType* ValuePtr = GetData() + Index.GetInt();
		ValueSetter(ValueTypeStruct, reinterpret_cast<uint8*>(ValuePtr));
	}

	template<class InValueType>
	inline void TBoundValueMap<InValueType>::GetValueWithGetter(FAttributeSetIndex Index, const TFunctionRef<void(UScriptStruct* ValueType, const uint8* ValuePtr)>& ValueGetter) const
	{
		checkf((Index.GetInt() >= 0) & (Index.GetInt() < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index.GetInt(), (long long)NumValues); // & for one branch
		const InValueType* ValuePtr = GetData() + Index.GetInt();
		ValueGetter(ValueTypeStruct, reinterpret_cast<const uint8*>(ValuePtr));
	}

	template<class InValueType>
	inline void TBoundValueMap<InValueType>::SetValueImpl(FAttributeSetIndex Index, const void* ValuePtr)
	{
		checkf((Index.GetInt() >= 0) & (Index.GetInt() < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index.GetInt(), (long long)NumValues); // & for one branch
		const InValueType* Value = reinterpret_cast<const InValueType*>(ValuePtr);
		GetData()[Index.GetInt()] = *Value;
	}

	template<class InValueType>
	inline void TBoundValueMap<InValueType>::GetValueImpl(FAttributeSetIndex Index, void* OutValuePtr) const
	{
		checkf((Index.GetInt() >= 0) & (Index.GetInt() < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index.GetInt(), (long long)NumValues); // & for one branch
		InValueType* OutValue = reinterpret_cast<InValueType*>(OutValuePtr);
		*OutValue = GetData()[Index.GetInt()];
	}

	template<class InValueType>
	inline int32 TBoundValueMap<InValueType>::CalculateAllocationSize(int32 NumAttributes)
	{
		return Align(sizeof(TBoundValueMap<InValueType>), alignof(InValueType)) + sizeof(InValueType) * NumAttributes;
	}

	template<class ValueType>
	inline TBoundValueMap<ValueType>* MakeBoundValueMap(const typename TBoundValueMap<ValueType>::FConstructArgs& Args)
	{
		// We allocate the map and its values in a contiguous buffer part of a single allocation
		// As such, the map struct ends up being a small header
		const int32 AllocationSize = TBoundValueMap<ValueType>::CalculateAllocationSize(Args.TypedSet->Num());

		// Allocate our single buffer
		uint8* Ptr = (*Args.ReallocFun)(nullptr, 0, AllocationSize);

		// Construct our map over it, the specialized constructor is responsible for partitioning the rest of the buffer
		return new(Ptr) TBoundValueMap<ValueType>(Args, AllocationSize);
	}

	inline void ReleaseBoundValueMap(FBoundValueMap* Map)
	{
		if (Map == nullptr)
		{
			return;
		}

		const int32 AllocationSize = Map->AllocatedSize;
		const FReallocFun ReallocFun = Map->ReallocFun;

		Map->~FBoundValueMap();

		(*ReallocFun)(reinterpret_cast<uint8*>(Map), AllocationSize, 0);
	}
}

#undef UE_API
