// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Misc/MemStack.h"

namespace UE::UAF
{
	// Use allocator pointer instead of template argument
	// The allocator should behave as follows:
	//    - When Ptr is nullptr (and OldSize is 0), we are allocating memory
	//    - When Ptr is valid but NewSize is 0, we are freeing memory
	//    - When Ptr is valid and NewSize is not 0, we are re-allocating memory
	//    - It assumes that the elements that live in said memory can be trivially copyable
	//    - Allocations use default alignment
	typedef uint8* (*FReallocFun)(uint8* OldPtr, size_t OldSize, size_t NewSize);

	// This type can be specialized for template style allocators for easy interop
	// with the engine allocator types.
	// Note that we cannot support some allocators that require 'NeedsElementType'
	// such as the inline allocator.
	template<class AllocatorType>
	struct FAllocatorTypeTrait
	{
		// Specialize this struct for your allocator by implementing this function
		static uint8* Realloc(uint8* OldPtr, size_t OldSize, size_t NewSize) = delete;
	};

	template<int IndexSize>
	struct FAllocatorTypeTrait<TSizedDefaultAllocator<IndexSize>>
	{
		static uint8* Realloc(uint8* OldPtr, size_t OldSize, size_t NewSize)
		{
			if (OldPtr == nullptr && NewSize == 0)
			{
				// Nothing to do if we weren't previously allocated and we aren't requesting a valid size
				return nullptr;
			}

			return (uint8*)FMemory::Realloc(OldPtr, NewSize);
		}
	};

	template<uint32 Alignment>
	struct FAllocatorTypeTrait<TMemStackAllocator<Alignment>>
	{
		static uint8* Realloc(uint8* OldPtr, size_t OldSize, size_t NewSize)
		{
			if (NewSize == 0)
			{
				// Nothing to do if we are freeing memory
				return nullptr;
			}

			// Allocate memory from the stack.
			uint8* NewPtr = FMemStack::Get().PushBytes(NewSize, Alignment);

			// If we are growing an existing allocation, copy from the old
			if (OldPtr != nullptr)
			{
				const uint32 NumToCopy = FMath::Min(NewSize, OldSize);
				FMemory::Memcpy(NewPtr, OldPtr, NumToCopy);
			}

			return NewPtr;
		}
	};
}
