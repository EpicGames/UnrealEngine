// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF
{
	// [Join Operators]
	// Join operations take an operator argument to control how the join behaves.
	// A join operator is responsible for the following:
	//   - Return keys/values for each iterator type provided to the join function
	//   - Provide the largest key value
	//   - Provide fuzzy matching semantics (optional)
	//
	// Key Fetching:
	// Joins grab keys from iterators through the GetKey(const IterType&) functions.
	// Keys are sometimes cached within the join operation as such it makes little difference
	// whether they are returned by value or by const&. However, keys should be kept as small as possible
	// to help with performance.
	// 
	// The largest key is provided by the GetLargestKey() function. Joins operate on sorted
	// inputs which rely on the < (lower than) operator.
	//
	// Value Fetching:
	// Joins forward values from iterators through the GetValue(const IterType&) functions.
	// Join operations do not cache values: they are directly provided to the predicate function.
	// You can thus avoid value copying by using references (mutable or otherwise).
	// Mutability of values is determined by the predicate function signature.
	// 
	// Fuzzy Matching:
	// Fuzzy matching allows for mixed iteration as opposed to locked step iteration meaning
	// an fuzzy entry can match multiple values in other iterators. For example, a join operation with
	// a transformer list and an attribute collection will want the predicate called for each set map
	// within the collection that has a matching value type. This allows the same transformed to be
	// provided to multiple set maps that share the same value type but with a different attribute type
	// (through a partial key). Fuzzy matching is enabled when IsMatch(const ReferenceKey&, const TestKey&)
	// is defined on the join operator. When present, IsMatch is used to compare for equality in some
	// operations instead of == (direct equality).
	// The ReferenceKey is the smallest key our iterators currently point to. When the reference key
	// is a fuzzy key (smallest of its fuzzy group), it can match multiple TestKeys.
	//

	// An inner join will call the predicate for the union of all iterators
	// When an entry is missing from an iterator, it is skipped
	// This assumes that each iterator is sorted
	// The JoinOpType is used to control which key is used and what value is produced for the predicate
	// e.g. InnerJoinBy(.., A, B)
	// Where:
	//  - A: [(123, 'foo'), (456, 'bar')]
	//  - B: [(456, 'bar'), (888, 'wow')]
	// Predicate is invoked with:
	//  - [(456, 'bar'), (456, 'bar')]		(Both entries matching)
	// 
	// See [Join Operators] above for details on the Join Operator argument.
	template<class JoinOpType, class PredicateType, class... IteratorTypes>
	void InnerJoinBy(JoinOpType&& JoinOp, PredicateType&& Predicate, IteratorTypes&&... Iterators);

	// An outer join will call the predicate for the union of all iterators
	// When an entry is missing from an iterator, it is replaced by nullptr
	// This assumes that each iterator is sorted
	// The JoinOpType is used to control which key is used and what value is produced for the predicate
	// e.g. OuterJoinBy(.., A, B)
	// Where:
	//  - A: [(123, 'foo'), (456, 'bar')]
	//  - B: [(456, 'bar'), (888, 'wow')]
	// Predicate is invoked with:
	//  - [(123, 'foo'), null]				(No matching entry in B)
	//  - [(456, 'bar'), (456, 'bar')]		(Both entries matching)
	//  - [null, (888, 'wow')]				(No matching entry in A)
	// 
	// See [Join Operators] above for details on the Join Operator argument.
	template<class JoinOpType, class PredicateType, class... IteratorTypes>
	void OuterJoinBy(JoinOpType&& JoinOp, PredicateType&& Predicate, IteratorTypes&&... Iterators);

	//////////////////////////////////////////////////////////////////////////
	// Implementation

	namespace Private
	{
		/** Concept used to detect whether or not a Join Operator uses fuzzy matching */
		struct CHasFuzzyMatching
		{
			template <typename T>
			auto Requires(T& Val) -> decltype(
				Val.IsMatch({}, {})
				);
		};

		/** Concept used to detect whether or not a Join Operator specifies a custom LessThan operator */
		struct CHasCustomLessThan
		{
			template <typename T>
			auto Requires(T& Val) -> decltype(
				Val.IsLessThan({}, {})
				);
		};

		/** Concept used to detect whether or not a Join Operator specifies a HasPredicateWithKey function */
		struct CHasPredicateWithKey
		{
			template <typename T>
			auto Requires(T& Val) -> decltype(
				Val.HasPredicateWithKey()
				);
		};

		// Returns whether or not the predicate requires the Key as its first argument
		template<class JoinOpType>
		constexpr bool HasPredicateWithKey()
		{
			if constexpr (TModels_V<CHasPredicateWithKey, JoinOpType>)
			{
				return JoinOpType::HasPredicateWithKey();
			}
			else
			{
				return false;
			}
		}

		// Returns whether or not one of the specified iterators has reached the end and is empty
		template<class... IteratorTypes>
		inline bool AnyIteratorEmpty(const IteratorTypes&... Iterators)
		{
			return (... || !Iterators);
		}

		// Returns whether or not all iterators have reached the end and are empty
		template<class... IteratorTypes>
		inline bool AllIteratorsEmpty(const IteratorTypes&... Iterators)
		{
			return (... && !Iterators);
		}

		// Returns whether or not all iterators have a key with the specified value
		template<class JoinOpType, typename KeyType, class... IteratorTypes>
		inline bool AllIteratorEqual(JoinOpType& JoinOp, KeyType Key, const IteratorTypes&... Iterators)
		{
			if constexpr (TModels_V<CHasFuzzyMatching, JoinOpType>)
			{
				return (... && (JoinOp.IsMatch(Key, JoinOp.GetKey(Iterators))));
			}
			else
			{
				return (... && (Key == JoinOp.GetKey(Iterators)));
			}
		}

		// Returns the smallest iterator key from the specified iterator list
		// When bTestIfEmpty is enabled, empty iterators return the largest value otherwise we assume that at
		// all iterators are valid and can provide a key
		// Handles the last iterator in the template list
		template<bool bTestIfEmpty, class JoinOpType, class HeadIteratorType>
		inline auto IteratorMin(JoinOpType& JoinOp, const HeadIteratorType& HeadIterator)
		{
			if constexpr (bTestIfEmpty)
			{
				if (!HeadIterator)
				{
					// Our iterator isn't valid, use the largest key possible
					return JoinOp.GetLargestKey();
				}
			}

			checkf(HeadIterator, TEXT("Iterator should have remaining values"));
			return JoinOp.GetKey(HeadIterator);
		}

		// Returns the smallest iterator key from the specified iterator list
		// When bTestIfEmpty is enabled, empty iterators return the largest value otherwise we assume that at
		// all iterators are valid and can provide a key
		// Handles an intermediate iterator in the template list
		template<bool bTestIfEmpty, class JoinOpType, class HeadIteratorType, class... TailIteratorTypes>
		inline auto IteratorMin(JoinOpType& JoinOp, const HeadIteratorType& HeadIterator, const TailIteratorTypes&... TailIterators)
		{
			if constexpr (bTestIfEmpty)
			{
				if (!HeadIterator)
				{
					// Our iterator isn't valid, try the next one
					return IteratorMin<bTestIfEmpty>(JoinOp, TailIterators...);
				}
			}

			checkf(HeadIterator, TEXT("Iterator should have remaining values"));
			auto HeadKey = JoinOp.GetKey(HeadIterator);
			auto TailKey = IteratorMin<bTestIfEmpty>(JoinOp, TailIterators...);

			if constexpr (TModels_V<Private::CHasCustomLessThan, JoinOpType>)
			{
				return JoinOp.IsLessThan(HeadKey, TailKey) ? HeadKey : TailKey;
			}
			else
			{
				return HeadKey < TailKey ? HeadKey : TailKey;
			}
		}

		// Conditionally increment iterators that match the specified key
		// Returns how many iterators are empty
		// Handles the last iterator in the template list
		template<bool bTestIfEmpty, class JoinOpType, typename KeyType, class HeadIteratorType>
		inline uint32 ConditionalIncrement(JoinOpType& JoinOp, KeyType Key, HeadIteratorType& HeadIterator)
		{
			if constexpr (bTestIfEmpty)
			{
				// If we allow iterators to be empty, avoid attempting to increment them
				if (!HeadIterator)
				{
					// Already reached the end, we are empty
					return 1;
				}
			}

			// Incrementing requires an exact match
			checkf(HeadIterator, TEXT("Iterator should have remaining values"));
			if (Key == JoinOp.GetKey(HeadIterator))
			{
				++HeadIterator;
				return !HeadIterator ? 1 : 0;
			}

			// Our iterator didn't match the key, it isn't empty
			return 0;
		}

		// Conditionally increment iterators that match the specified key
		// Returns how many iterators are empty
		template<bool bTestIfEmpty, class JoinOpType, typename KeyType, class HeadIteratorType, class... TailIteratorTypes>
		inline uint32 ConditionalIncrement(JoinOpType& JoinOp, KeyType Key, HeadIteratorType& HeadIterator, TailIteratorTypes&... TailIterators)
		{
			if constexpr (bTestIfEmpty)
			{
				// If we allow iterators to be empty, avoid attempting to increment them
				if (!HeadIterator)
				{
					// Our iterator isn't valid, try the next one
					uint32 NumEmptyIterators = 1;
					NumEmptyIterators += ConditionalIncrement<bTestIfEmpty>(JoinOp, Key, TailIterators...);

					return NumEmptyIterators;
				}
			}

			uint32 NumEmptyIterators = 0;

			// Incrementing requires an exact match
			checkf(HeadIterator, TEXT("Iterator should have remaining values"));
			if (Key == JoinOp.GetKey(HeadIterator))
			{
				++HeadIterator;
				NumEmptyIterators = !HeadIterator ? 1 : 0;
			}

			NumEmptyIterators += ConditionalIncrement<bTestIfEmpty>(JoinOp, Key, TailIterators...);
			return NumEmptyIterators;
		}

		// Conditionally retrieves an iterator value when its key matches the specified value
		// The iterator value is returned when the key matches and the iterator isn't empty
		// otherwise we return nullptr
		template<class JoinOpType, typename KeyType, class IteratorType>
		inline auto ConditionalGetValue(JoinOpType& JoinOp, KeyType Key, IteratorType& Iterator)
		{
			if constexpr (TModels_V<Private::CHasFuzzyMatching, JoinOpType>)
				return Iterator && JoinOp.IsMatch(Key, JoinOp.GetKey(Iterator)) ? JoinOp.GetValue(Iterator) : nullptr;
			else
				return Iterator && Key == JoinOp.GetKey(Iterator) ? JoinOp.GetValue(Iterator) : nullptr;
		}
	}

	template<class JoinOpType, class PredicateType, class... IteratorTypes>
	inline void InnerJoinBy(JoinOpType&& JoinOp, PredicateType&& Predicate, IteratorTypes&&... Iterators)
	{
		static_assert(sizeof...(Iterators) >= 2, "To perform an InnerJoin, at least 2 iterators must be provided");

		// We know every iterator has at least one value
		constexpr bool bTestIfEmpty = false;

		// Whether or not to call the predicate with the first argument being the key
		constexpr bool bWithPredicateKey = Private::HasPredicateWithKey<JoinOpType>();

		// We loop as long as iterators still have values
		bool bAnyIteratorEmpty = Private::AnyIteratorEmpty(Iterators...);
		while (!bAnyIteratorEmpty)
		{
			// Find the key with the smallest value
			auto MinKey = Private::IteratorMin<bTestIfEmpty>(JoinOp, Iterators...);

			if (Private::AllIteratorEqual(JoinOp, MinKey, Iterators...))
			{
				// Our keys match, call the predicate
				// All our iterators match our key, return their value (even if the match is fuzzy)
				if constexpr (bWithPredicateKey)
				{
					Predicate(MinKey, JoinOp.GetValue(Iterators)...);
				}
				else
				{
					Predicate(JoinOp.GetValue(Iterators)...);
				}

				// Move our iterators forward
				if constexpr (TModels_V<Private::CHasFuzzyMatching, JoinOpType>)
				{
					// When we use wildcard matching, we have to conditionally increment because we might be a fuzzy match
					// Incrementing requires an exact match
					const uint32 NumEmptyIterators = Private::ConditionalIncrement<bTestIfEmpty>(JoinOp, MinKey, Iterators...);
					bAnyIteratorEmpty = NumEmptyIterators != 0;
				}
				else
				{
					(++Iterators, ...);

					// Continue iterating as long as iterators still have values left
					bAnyIteratorEmpty = Private::AnyIteratorEmpty(Iterators...);
				}
			}
			else
			{
				// At least one key doesn't match, increment iterators that match the smallest key, it isn't needed anymore
				const uint32 NumEmptyIterators = Private::ConditionalIncrement<bTestIfEmpty>(JoinOp, MinKey, Iterators...);
				bAnyIteratorEmpty = NumEmptyIterators != 0;
			}
		}

		// We are done, any remaining values are just partial matches
	}

	template<class JoinOpType, class PredicateType, class... IteratorTypes>
	inline void OuterJoinBy(JoinOpType&& JoinOp, PredicateType&& Predicate, IteratorTypes&&... Iterators)
	{
		static_assert(sizeof...(Iterators) >= 2, "To perform an OuterJoin, at least 2 iterators must be provided");

		// We might have iterators that have ran out of values
		constexpr bool bTestIfEmpty = true;

		// Whether or not to call the predicate with the first argument being the key
		constexpr bool bWithPredicateKey = Private::HasPredicateWithKey<JoinOpType>();

		// We loop as long as one iterator still has values
		bool bAllIteratorsEmpty = Private::AllIteratorsEmpty(Iterators...);
		while (!bAllIteratorsEmpty)
		{
			// Find the key with the smallest value
			auto MinKey = Private::IteratorMin<bTestIfEmpty>(JoinOp, Iterators...);

			// Call the predicate with a conditional fetch of our iterator values
			// If an iterator's key matches the smallest key, its value is used (even if the match is fuzzy) otherwise we use nullptr
			if constexpr (bWithPredicateKey)
			{
				Predicate(MinKey, Private::ConditionalGetValue(JoinOp, MinKey, Iterators)...);
			}
			else
			{
				Predicate(Private::ConditionalGetValue(JoinOp, MinKey, Iterators)...);
			}

			// Increment iterators that match the smallest key, it isn't needed anymore
			const uint32 NumEmptyIterators = Private::ConditionalIncrement<bTestIfEmpty>(JoinOp, MinKey, Iterators...);
			bAllIteratorsEmpty = NumEmptyIterators == sizeof...(Iterators);
		}
	}
}
