// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAF/ValueRuntime/ValueBundle.h"

namespace UE::UAF
{
	/**
	 * FPoseValueBundle
	 *
	 * A non-owning typed overlay on FValueBundle that adds convenient accessors for common
	 * animation pose data (bone transforms, float curves, etc.).
	 *
	 * This class is not meant to be constructed directly. It adds no data members and
	 * FValueBundle has no v-table, so it can be safely reinterpret-cast from any FValueBundle
	 * regardless of allocator (stack or heap). Use the static From(), AsMutable(), or AsConst()
	 * helpers to obtain a FPoseValueBundle reference from an existing FValueBundle.
	 */
	class FPoseValueBundle : public FValueBundle
	{
	public:
		// A helper which forwards the arguments to FValueBundle
		FPoseValueBundle(const FAttributeNamedSetPtr& NamedSet, FReallocFun ReallocFun);

		using FValueBundle::operator=;

		// -------------------------------------------------------------------
		//  Typed accessors for common pose data
		// -------------------------------------------------------------------

		/**
		 * Returns the bone transform map, or nullptr if this bundle doesn't contain bone transforms.
		 *
		 * Example: Read/write a single bone by index
		 * @code
		 *   TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Pose->FindBoneTransforms();
		 *
		 *   // Read
		 *   FTransform CurrentTransform = (*BoneTransforms)[Mapping.BoneIndex].Value;
		 *
		 *   // Write
		 *   (*BoneTransforms)[Mapping.BoneIndex].Value = NewTransform;
		 * @endcode
		 *
		 * Example: Look up a bone by name
		 * @code
		 *   TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Pose->FindBoneTransforms();
		 *   FAttributeTypedSetPtr BoneTypedSet = BoneTransforms->GetTypedSet();
		 *   FAttributeSetIndex HeadIndex = BoneTypedSet->FindIndex(FName("head"));
		 *   if (HeadIndex.IsValid())
		 *   {
		 *       FTransform HeadTransform = (*BoneTransforms)[HeadIndex].Value;
		 *   }
		 * @endcode
		 *
		 * Example: Iterate through all bones:
		 * @code
		 *   TBoundValueMap<FBoneTransformAnimationAttribute>* BoneTransforms = Pose->FindBoneTransforms();
		 *   const int32 NumBones = BoneTransforms->Num();
		 *   FBoneTransformAnimationAttribute* BoneData = BoneTransforms->GetData();
		 *   for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		 *   {
		 *       FTransform& BoneTransform = BoneData[BoneIndex].Value;
		 *       // Read or modify BoneTransform
		 *   }
		 * @endcode
		 */
		[[nodiscard]] TBoundValueMap<FBoneTransformAnimationAttribute>* FindBoneTransforms();
		[[nodiscard]] const TBoundValueMap<FBoneTransformAnimationAttribute>* FindBoneTransforms() const;

		/**
		 * Returns the float curve map (morph targets, control curves, material params), or nullptr if not present.
		 * To look up curves by name, use GetTypedSet() on the returned map for the name-to-index mapping.
		 *
		 * Example: Read/write a curve by name
		 * @code
		 *   TBoundValueMap<FFloatAnimationAttribute>* FloatCurves = Pose->FindFloatCurves();
		 *   FAttributeTypedSetPtr FloatTypedSet = FloatCurves->GetTypedSet();
		 *   FAttributeSetIndex Index = FloatTypedSet->FindIndex(FName("smile_left"));
		 *   if (Index.IsValid())
		 *   {
		 *       float Value = (*FloatCurves)[Index].Value;        // Read
		 *       (*FloatCurves)[Index].Value = 0.75f;              // Write
		 *   }
		 * @endcode
		 *
		 * Example: Iterate all curves and scale them:
		 * @code
		 *   TBoundValueMap<FFloatAnimationAttribute>* FloatCurves = Pose->FindFloatCurves();
		 *   const int32 NumCurves = FloatCurves->Num();
		 *   FFloatAnimationAttribute* CurveData = FloatCurves->GetData();
		 *   for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
		 *   {
		 *       CurveData[CurveIndex].Value *= 0.5f;
		 *   }
		 * @endcode
		 */
		[[nodiscard]] TBoundValueMap<FFloatAnimationAttribute>* FindFloatCurves();
		[[nodiscard]] const TBoundValueMap<FFloatAnimationAttribute>* FindFloatCurves() const;

		/**
		 * Returns whether this bundle contains bone transforms.
		 * @note Performs a map lookup internally (O(log N) where N is the number of bound value maps, typically very small).
		 *		 Cache the result rather than calling per-iteration in tight loops.
		 */
		[[nodiscard]] bool HasBoneTransforms() const;

		/**
		 * Returns whether this bundle contains float curves.
		 * @note Performs a map lookup internally (O(log N) where N is the number of bound value maps, typically very small).
		 *		 Cache the result rather than calling per-iteration in tight loops.
		 */
		[[nodiscard]] bool HasFloatCurves() const;

		// -------------------------------------------------------------------
		//  Cast helpers
		// -------------------------------------------------------------------

		/**
		 * Reinterpret a FValueBundle reference as a FPoseValueBundle.
		 * Safe because FPoseValueBundle adds no data members and FValueBundle has no vtable.
		 */
		[[nodiscard]] static FPoseValueBundle& From(FValueBundle& Bundle);

		/** @copydoc From(FValueBundle&) */
		[[nodiscard]] static const FPoseValueBundle& From(const FValueBundle& Bundle);

		/**
		 * Cast a CoW reference to a mutable FPoseValueBundle, returning nullptr if the bundle is empty.
		 * Checks mutability internally so the caller doesn't have to.
		 *
		 * Typical usage:
		 * @code
		 *   FValueBundleCoWRef InputRef = Evaluator.GetEvaluationStack().Pop();
		 *   FPoseValueBundle* Pose = FPoseValueBundle::AsMutable(InputRef);
		 * @endcode
		 */
		template<typename CoWRefType>
		[[nodiscard]] static FPoseValueBundle* AsMutable(CoWRefType& Ref);

		/**
		 * Cast a CoW reference to a const FPoseValueBundle, returning nullptr if the bundle is empty.
		 */
		template<typename CoWRefType>
		[[nodiscard]] static const FPoseValueBundle* AsConst(const CoWRefType& Ref);
	};

	static_assert(sizeof(FPoseValueBundle) == sizeof(FValueBundle), "Types must have the same size to allow casting usage in From(..)");

	// A heap allocated pose value bundle
	class FPoseValueBundleHeap : public FPoseValueBundle
	{
	public:
		// Constructs an empty pose bundle
		FPoseValueBundleHeap();

		// Constructs an empty pose bundle based on the specified named set
		explicit FPoseValueBundleHeap(const FAttributeNamedSetPtr& NamedSet);

		using FPoseValueBundle::operator=;
	};

	// A Mem-Stack allocated pose value bundle
	class FPoseValueBundleStack : public FPoseValueBundle
	{
	public:
		// Constructs an empty pose bundle
		FPoseValueBundleStack();

		// Constructs an empty pose bundle based on the specified named set
		explicit FPoseValueBundleStack(const FAttributeNamedSetPtr& NamedSet);

		using FPoseValueBundle::operator=;
	};

	using FPoseValueBundlePtr = TSharedPtr<FPoseValueBundle, ESPMode::ThreadSafe>;

	//////////////////////////////////////////////////////////////////////////
	// Inline implementation

	inline FPoseValueBundle::FPoseValueBundle(const FAttributeNamedSetPtr& InNamedSet, FReallocFun InReallocFun)
		: FValueBundle(InNamedSet, InReallocFun)
	{
	}

	inline TBoundValueMap<FBoneTransformAnimationAttribute>* FPoseValueBundle::FindBoneTransforms()
	{
		return GetBoundValueMaps().Find<FBoneTransformAnimationAttribute>(
			FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute>());
	}

	inline const TBoundValueMap<FBoneTransformAnimationAttribute>* FPoseValueBundle::FindBoneTransforms() const
	{
		return GetBoundValueMaps().Find<FBoneTransformAnimationAttribute>(
			FAttributeMappingKey::MakeFromTo<FBoneTransformAnimationAttribute>());
	}

	inline TBoundValueMap<FFloatAnimationAttribute>* FPoseValueBundle::FindFloatCurves()
	{
		return GetBoundValueMaps().Find<FFloatAnimationAttribute>(
			FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute>());
	}

	inline const TBoundValueMap<FFloatAnimationAttribute>* FPoseValueBundle::FindFloatCurves() const
	{
		return GetBoundValueMaps().Find<FFloatAnimationAttribute>(
			FAttributeMappingKey::MakeFromTo<FFloatAnimationAttribute>());
	}

	inline bool FPoseValueBundle::HasBoneTransforms() const
	{
		return FindBoneTransforms() != nullptr;
	}

	inline bool FPoseValueBundle::HasFloatCurves() const
	{
		return FindFloatCurves() != nullptr;
	}

	inline FPoseValueBundle& FPoseValueBundle::From(FValueBundle& Bundle)
	{
		return static_cast<FPoseValueBundle&>(Bundle);
	}

	inline const FPoseValueBundle& FPoseValueBundle::From(const FValueBundle& Bundle)
	{
		return static_cast<const FPoseValueBundle&>(Bundle);
	}

	template<typename CoWRefType>
	inline FPoseValueBundle* FPoseValueBundle::AsMutable(CoWRefType& Ref)
	{
		check(Ref.IsMutable());
		return &From(Ref.GetMutable());
	}

	template<typename CoWRefType>
	inline const FPoseValueBundle* FPoseValueBundle::AsConst(const CoWRefType& Ref)
	{
		return &From(Ref.Get());
	}

	inline FPoseValueBundleHeap::FPoseValueBundleHeap()
		: FPoseValueBundle(nullptr, &FAllocatorTypeTrait<FDefaultAllocator>::Realloc)
	{
	}

	inline FPoseValueBundleHeap::FPoseValueBundleHeap(const FAttributeNamedSetPtr& NamedSet)
		: FPoseValueBundle(NamedSet, &FAllocatorTypeTrait<FDefaultAllocator>::Realloc)
	{
	}

	inline FPoseValueBundleStack::FPoseValueBundleStack()
		: FPoseValueBundle(nullptr, &FAllocatorTypeTrait<TMemStackAllocator<>>::Realloc)
	{
	}

	inline FPoseValueBundleStack::FPoseValueBundleStack(const FAttributeNamedSetPtr& NamedSet)
		: FPoseValueBundle(NamedSet, &FAllocatorTypeTrait<TMemStackAllocator<>>::Realloc)
	{
	}
} // namespace UE::UAF
