// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"

#define UE_API UAF_API

namespace UE::UAF::Transformers
{
	// Attribute value transformer to accumulate values
	// Accumulate is the repeating step within an N-step interpolation: Overwrite, Accumulate0, Accumulate1, ...
	// Make sure to also implement variations for Overwrite and Interpolate
	struct FAccumulate final : public FValueTransformer
	{
		static UE_API FName TransformerName;

		// If the output buffer is nullptr, an allocator is provided and the implementation is responsible for allocating the output buffer and returning it
		// Per-value map is optional, when not present, the default weight should be used
		// Unbound values do not support per-value weights
		using FTransformBoundValueMapFunc = void (*)(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
		using FTransformUnboundValueMapFunc = void (*)(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);

		// Accumulate attribute values
		// Output = A + (B * WeightB)
		// This function supports outputting to either input
		static UE_API void Apply(const FValueTransformerMapPtr& TransformerMap, const FValueBundle& InputA, const FValueBundle& InputB, float WeightB, FValueBundle& Output);

		// Accumulate attribute values
		// If a weight isn't specified, the default weight will be used
		// Output = A + (B * (HasValueWeightB ? ValueWeightB : DefaultWeightB))
		// This function supports outputting to either input
		static UE_API void Apply(const FValueTransformerMapPtr& TransformerMap, const FValueBundle& InputA, const FValueBundle& InputB, const FValueBundle& PerValueWeightsB, float DefaultWeightB, FValueBundle& Output);
	};

	//////////////////////////////////////////////////////////////////////////
	// Engine built-in value type specializations

	namespace Private
	{
		struct FAccumulate_BoneTransformAttribute
		{
			using TransformerType = FAccumulate;
			using ValueType = FBoneTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FAccumulate_FloatAttribute
		{
			using TransformerType = FAccumulate;
			using ValueType = FFloatAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FAccumulate_IntegerAttribute
		{
			using TransformerType = FAccumulate;
			using ValueType = FIntegerAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FAccumulate_TransformAttribute
		{
			using TransformerType = FAccumulate;
			using ValueType = FTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FAccumulate_VectorAttribute
		{
			using TransformerType = FAccumulate;
			using ValueType = FVectorAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FAccumulate_QuaternionAttribute
		{
			using TransformerType = FAccumulate;
			using ValueType = FQuaternionAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};
	}
}

#undef UE_API
