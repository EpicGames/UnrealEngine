// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"

#define UE_API UAF_API

namespace UE::UAF::Transformers
{
	// Attribute value transformer to create additive values
	struct FMakeAdditiveSpace final : public FValueTransformer
	{
		static UE_API FName TransformerName;

		// If the output buffer is nullptr, an allocator is provided and the implementation is responsible for allocating the output buffer and returning it
		// If the base map is missing, transformers should use a default value
		using FTransformBoundValueMapFunc = void (*)(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator);
		using FTransformUnboundValueMapFunc = void (*)(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator);

		// Creates a new additive collection by subtracting the base from the specified Input
		static UE_API void Apply(
			const FValueTransformerMapPtr& TransformerMap,
			const FValueBundle& Base,
			const FValueBundle& Input,
			FValueBundle& AdditiveOutput);
	};

	// Attribute value transformer to apply additive values onto a base
	struct FApplyAdditiveSpace final : public FValueTransformer
	{
		static UE_API FName TransformerName;

		// If the output buffer is nullptr, an allocator is provided and the implementation is responsible for allocating the output buffer and returning it
		// If the base map is missing, transformers should use a default value
		// Per-value map is optional, when not present, the default weight should be used
		// Unbound values do not support per-value weights
		using FTransformBoundValueMapFunc = void (*)(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator);
		using FTransformUnboundValueMapFunc = void (*)(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator);

		// Applies an additive collection onto a base by adding it
		// Output = Base + Lerp(Identity, Additive, Weight)
		// This function supports outputting to either input
		static UE_API void Apply(
			const FValueTransformerMapPtr& TransformerMap,
			const FValueBundle& Base,
			const FValueBundle& Additive,
			float AdditiveWeight,
			FValueBundle& Output);

		// Applies an additive collection onto a base by adding it using per value weights (must be of type FFloatAnimationAttribute)
		// If a weight isn't specified, the default weight will be used
		// Output = Base + Lerp(Identity, Additive, HasValueWeight ? ValueWeight : DefaultWeight)
		// This function supports outputting to either input
		static UE_API void Apply(
			const FValueTransformerMapPtr& TransformerMap,
			const FValueBundle& Base,
			const FValueBundle& Additive,
			const FValueBundle& PerValueWeights,
			float DefaultWeight,
			FValueBundle& Output);
	};

	//////////////////////////////////////////////////////////////////////////
	// Engine built-in value type specializations

	namespace Private
	{
		struct FMakeAdditiveSpace_BoneTransformAttribute
		{
			using TransformerType = FMakeAdditiveSpace;
			using ValueType = FBoneTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FMakeAdditiveSpace_FloatAttribute
		{
			using TransformerType = FMakeAdditiveSpace;
			using ValueType = FFloatAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FMakeAdditiveSpace_IntegerAttribute
		{
			using TransformerType = FMakeAdditiveSpace;
			using ValueType = FIntegerAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FMakeAdditiveSpace_TransformAttribute
		{
			using TransformerType = FMakeAdditiveSpace;
			using ValueType = FTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FMakeAdditiveSpace_VectorAttribute
		{
			using TransformerType = FMakeAdditiveSpace;
			using ValueType = FVectorAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FMakeAdditiveSpace_QuaternionAttribute
		{
			using TransformerType = FMakeAdditiveSpace;
			using ValueType = FQuaternionAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Input, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Input, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FApplyAdditiveSpace_BoneTransformAttribute
		{
			using TransformerType = FApplyAdditiveSpace;
			using ValueType = FBoneTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FApplyAdditiveSpace_FloatAttribute
		{
			using TransformerType = FApplyAdditiveSpace;
			using ValueType = FFloatAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FApplyAdditiveSpace_IntegerAttribute
		{
			using TransformerType = FApplyAdditiveSpace;
			using ValueType = FIntegerAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FApplyAdditiveSpace_TransformAttribute
		{
			using TransformerType = FApplyAdditiveSpace;
			using ValueType = FTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FApplyAdditiveSpace_VectorAttribute
		{
			using TransformerType = FApplyAdditiveSpace;
			using ValueType = FVectorAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};

		struct FApplyAdditiveSpace_QuaternionAttribute
		{
			using TransformerType = FApplyAdditiveSpace;
			using ValueType = FQuaternionAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Additive, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& Output, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Additive, float AdditiveWeight, FUnboundValueMap*& Output, FReallocFun OutputAllocator);
		};
	}
}

#undef UE_API
