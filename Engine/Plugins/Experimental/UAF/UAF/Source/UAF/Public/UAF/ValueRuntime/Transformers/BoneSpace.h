// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API UAF_API

namespace UE::UAF
{
	class FPoseValueBundle;
}

namespace UE::UAF::Transformers
{
	// Bone space conversion utility functions
	// Not a transformer, but transformer-like
	struct FBoneSpace final
	{
		// Converts bone transforms between Local Space and Component Space
		// These functions support outputting to their input in place
		static UE_API void LocalToComponent(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues);
		static UE_API void ComponentToLocal(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues);

		// Converts bone transforms between Local Space and Mesh Rotation Space
		// Mesh Rotation Space assumes that bone transform scales are uniform and will yield incorrect results
		// if non-uniform (aka 3D) scale is used. It transforms the rotations into component space by ignoring
		// translation and scale. The output values will thus be in Mixed Space.
		// These functions support outputting to their input in place
		static UE_API void LocalToMeshRotation(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues);
		static UE_API void MeshRotationToLocal(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues);

		// Converts bone transforms between Local Space and Root Rotation Space
		// Root Rotation Space is similar to Mesh Rotation space except that the root bone transform is ignored
		// and assumed to be the identity transform (root corrections are ignored).
		// These functions support outputting to their input in place
		static UE_API void LocalToRootRotation(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues);
		static UE_API void RootRotationToLocal(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues);

		// Converts bone transforms between Local Space and Mesh Scale Space
		// Mesh Scale Space assumes that bone transform scales are uniform and will yield incorrect results
		// if non-uniform (aka 3D) scale is used. It transforms the scales into component space by ignoring
		// rotation and translation. The output values will thus be in a Mixed Space.
		// These functions support outputting to their input in place
		static UE_API void LocalToMeshScale(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues);
		static UE_API void MeshScaleToLocal(const FPoseValueBundle& InputValues, FPoseValueBundle& OutputValues);
	};
}

#undef UE_API
