// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"

#define UE_API UAF_API

namespace UE::UAF::Transformers
{
	// Attribute value transformer to interpolate values
	// Make sure to also implement variations for Overwrite and Accumulate
	struct FInterpolate final : public FValueTransformer
	{
		static UE_API FName TransformerName;

		// If the output buffer is nullptr, an allocator is provided and the implementation is responsible for allocating the output buffer and returning it
		// Note that the input/output buffers can be the same buffer
		// Per-value map is optional, when not present, the default weight should be used
		// Unbound values do not support per-value weights
		using FTransformBoundValueMapFunc = void (*)(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
		using FTransformUnboundValueMapFunc = void (*)(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);

		// Interpolate attribute values
		// Output = lerp(A, B, WeightB)
		// Output = (A * (1.0 - WeightB)) + (B * WeightB)
		// This function supports outputting to either input
		static UE_API void Apply(const FValueTransformerMapPtr& TransformerMap, const FValueBundle& InputA, const FValueBundle& InputB, float WeightB, FValueBundle& Output);

		// Interpolate attribute values
		// If a weight isn't specified, the default weight will be used
		// Output = lerp(A, B, HasValueWeightB ? ValueWeightB : DefaultWeightB)
		// This function supports outputting to either input
		static UE_API void Apply(const FValueTransformerMapPtr& TransformerMap, const FValueBundle& InputA, const FValueBundle& InputB, const FValueBundle& PerValueWeightsB, float DefaultWeightB, FValueBundle& Output);
	};

	//////////////////////////////////////////////////////////////////////////
	// Engine built-in value type specializations

	namespace Private
	{
		struct FInterpolate_BoneTransformAttribute
		{
			using TransformerType = FInterpolate;
			using ValueType = FBoneTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FInterpolate_FloatAttribute
		{
			using TransformerType = FInterpolate;
			using ValueType = FFloatAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FInterpolate_IntegerAttribute
		{
			using TransformerType = FInterpolate;
			using ValueType = FIntegerAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FInterpolate_TransformAttribute
		{
			using TransformerType = FInterpolate;
			using ValueType = FTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FInterpolate_VectorAttribute
		{
			using TransformerType = FInterpolate;
			using ValueType = FVectorAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FInterpolate_QuaternionAttribute
		{
			using TransformerType = FInterpolate;
			using ValueType = FQuaternionAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* MapA, const FBoundValueMap* MapB, const FBoundValueMap* PerValueWeightMapB, float DefaultWeightB, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* MapA, const FUnboundValueMap* MapB, float WeightB, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};
	}
}

#undef UE_API
