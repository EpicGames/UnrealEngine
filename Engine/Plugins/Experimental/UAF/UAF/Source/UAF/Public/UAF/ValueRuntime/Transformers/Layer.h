// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"

#define UE_API UAF_API

namespace UE::UAF::Transformers
{
	// Attribute value transformer to layer values
	struct FLayer final : public FValueTransformer
	{
		static UE_API FName TransformerName;

		// If the output buffer is nullptr, an allocator is provided and the implementation is responsible for allocating the output buffer and returning it
		// Note that the input/output buffers can be the same buffer
		// Per-value map is optional, when not present, the default weight should be used
		// Unbound values do not support per-value weights
		using FTransformBoundValueMapFunc = void (*)(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
		using FTransformUnboundValueMapFunc = void (*)(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);

		// Layer values onto a base
		// Output = layer(Base, Layer, LayerWeight)
		// Output = ValueInLayer ? lerp(BaseValue, LayerValue, LayerWeight) : BaseValue
		// Layer values do not have to be present in the Base and extra values are dropped
		// This function supports outputting to either input
		static UE_API void Apply(
			const FValueTransformerMapPtr& TransformerMap,
			const FValueBundle& Base,
			const FValueBundle& Layer,
			float LayerWeight,
			FValueBundle& Output);

		// Layer values onto a base
		// Output = ValueInLayer ? lerp(BaseValue, LayerValue, HasLayerValueWeight ? LayerValueWeight : DefaultLayerWeight) : BaseValue
		// Layer values do not have to be present in the Base and extra values are dropped
		// This function supports outputting to either input
		static UE_API void Apply(
			const FValueTransformerMapPtr& TransformerMap,
			const FValueBundle& Base,
			const FValueBundle& Layer,
			const FValueBundle& PerValueLayerWeights,
			float DefaultLayerWeight,
			FValueBundle& Output);
	};

	//////////////////////////////////////////////////////////////////////////
	// Engine built-in value type specializations

	namespace Private
	{
		struct FLayer_BoneTransformAttribute
		{
			using TransformerType = FLayer;
			using ValueType = FBoneTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FLayer_FloatAttribute
		{
			using TransformerType = FLayer;
			using ValueType = FFloatAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FLayer_IntegerAttribute
		{
			using TransformerType = FLayer;
			using ValueType = FIntegerAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FLayer_TransformAttribute
		{
			using TransformerType = FLayer;
			using ValueType = FTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FLayer_VectorAttribute
		{
			using TransformerType = FLayer;
			using ValueType = FVectorAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FLayer_QuaternionAttribute
		{
			using TransformerType = FLayer;
			using ValueType = FQuaternionAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* Base, const FBoundValueMap* Layer, const FBoundValueMap* PerValueWeights, float DefaultLayerWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* Base, const FUnboundValueMap* Layer, float LayerWeight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};
	}
}

#undef UE_API
