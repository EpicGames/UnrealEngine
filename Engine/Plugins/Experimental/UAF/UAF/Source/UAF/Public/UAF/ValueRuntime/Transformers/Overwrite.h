// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"

#define UE_API UAF_API

namespace UE::UAF::Transformers
{
	// Attribute value transformer to overwrite values
	// Overwrite is the initial step within an N-step interpolation: Overwrite, Accumulate0, Accumulate1, ...
	// Make sure to also implement variations for Accumulate and Interpolate
	struct FOverwrite final : public FValueTransformer
	{
		static UE_API FName TransformerName;

		// If the output buffer is nullptr, an allocator is provided and the implementation is responsible for allocating the output buffer and returning it
		// Note that the input/output buffers can be the same buffer
		// Per-value map is optional, when not present, the default weight should be used
		// Unbound values do not support per-value weights
		using FTransformBoundValueMapFunc = void (*)(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
		using FTransformUnboundValueMapFunc = void (*)(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);

		// Overwrite attribute values
		// Output = (A * Weight)
		// This function supports outputting to its input
		static UE_API void Apply(const FValueTransformerMapPtr& TransformerMap, const FValueBundle& Input, float Weight, FValueBundle& Output);

		// Overwrite attribute values using per value weights (must be of type FFloatAnimationAttribute)
		// If a weight isn't specified, the default weight will be used
		// Output = A * (HasValueWeight ? ValueWeight : DefaultWeight)
		// This function supports outputting to its input
		static UE_API void Apply(const FValueTransformerMapPtr& TransformerMap, const FValueBundle& Input, const FValueBundle& PerValueWeights, float DefaultWeight, FValueBundle& Output);
	};

	//////////////////////////////////////////////////////////////////////////
	// Engine built-in value type specializations

	namespace Private
	{
		struct FOverwrite_BoneTransformAttribute
		{
			using TransformerType = FOverwrite;
			using ValueType = FBoneTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FOverwrite_FloatAttribute
		{
			using TransformerType = FOverwrite;
			using ValueType = FFloatAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FOverwrite_IntegerAttribute
		{
			using TransformerType = FOverwrite;
			using ValueType = FIntegerAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FOverwrite_TransformAttribute
		{
			using TransformerType = FOverwrite;
			using ValueType = FTransformAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FOverwrite_VectorAttribute
		{
			using TransformerType = FOverwrite;
			using ValueType = FVectorAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};

		struct FOverwrite_QuaternionAttribute
		{
			using TransformerType = FOverwrite;
			using ValueType = FQuaternionAnimationAttribute;

			static void TransformBoundValueMap(const FBoundValueMap* InputMap, const FBoundValueMap* PerValueWeightMap, float DefaultWeight, FBoundValueMap*& OutputMap, FReallocFun OutputAllocator);
			static void TransformUnboundValueMap(const FUnboundValueMap* InputMap, float Weight, FUnboundValueMap*& OutputMap, FReallocFun OutputAllocator);
		};
	}
}

#undef UE_API
