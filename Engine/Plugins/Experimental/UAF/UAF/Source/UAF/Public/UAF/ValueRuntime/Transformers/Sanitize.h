// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UAF/Attributes/EngineAttributes.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"

// We validate mapped values in development and debug builds
#define UE_UAF_VALIDATE_MAPPED_VALUES !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#define UE_API UAF_API

namespace UE::UAF::Transformers
{
	// Attribute value transformer to sanitize values
	// This can be used to normalize rotations and other like values as well as check for NaN/Inf values
	struct FSanitize final : public FValueTransformer
	{
		static UE_API FName TransformerName;

		using FTransformBoundValueMapFunc = void (*)(FBoundValueMap* Map);
		using FTransformUnboundValueMapFunc = void (*)(FUnboundValueMap* Map);

		// Sanitize attribute values
		// This can be used to normalize rotations (and other like values) as well as check for NaN/Inf values
		static UE_API void Apply(const FValueTransformerMapPtr& TransformerMap, FValueBundle& Collection);
	};

	//////////////////////////////////////////////////////////////////////////
	// Engine built-in value type specializations

	namespace Private
	{
		// Sanitize bone transforms (normalize rotation, ensure finite)
		struct FSanitize_BoneTransformAttribute
		{
			using TransformerType = FSanitize;
			using ValueType = FBoneTransformAnimationAttribute;

			static void TransformBoundValueMap(FBoundValueMap* Map);
			static void TransformUnboundValueMap(FUnboundValueMap* Map);
		};

#if UE_UAF_VALIDATE_MAPPED_VALUES
		// Ensure values are not NaN (but Inf is valid)
		struct FSanitize_FloatAttribute
		{
			using TransformerType = FSanitize;
			using ValueType = FFloatAnimationAttribute;

			static void TransformBoundValueMap(FBoundValueMap* Map);
			static void TransformUnboundValueMap(FUnboundValueMap* Map);
		};
#endif

		struct FSanitize_TransformAttribute
		{
			using TransformerType = FSanitize;
			using ValueType = FTransformAnimationAttribute;

			static void TransformBoundValueMap(FBoundValueMap* Map);
			static void TransformUnboundValueMap(FUnboundValueMap* Map);
		};

		struct FSanitize_QuaternionAttribute
		{
			using TransformerType = FSanitize;
			using ValueType = FQuaternionAnimationAttribute;

			static void TransformBoundValueMap(FBoundValueMap* Map);
			static void TransformUnboundValueMap(FUnboundValueMap* Map);
		};
	}
}

#undef UE_API
