// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/ValueRuntime/ValueBundle.h"

#define UE_API UAF_API

namespace UE::UAF
{
	class FValueRuntimeRegistry;
	class FValueTransformerList;

	// A raw function pointer for all transformers (static functions)
	// Only used for storage purposes, NEVER CALL THIS!
	using FRawTransformerFunc = void(*)();

	// Value transformer base type
	// All transformer types must derive from this
	struct FValueTransformer
	{
		// The name of the transformer (exposed in derived types)
		//static FName TransformerName;

		// An optional function signature of the transformer for bound value maps (exposed in derived types)
		//using FTransformBoundValueMapFunc = void (*)(...);

		// An optional function signature of the transformer for unbound value maps (exposed in derived types)
		//using FTransformUnboundValueMapFunc = void (*)(...);
	};

	// An iterator over a transformer list within the value runtime registry
	class FValueTransformerListIterator
	{
	public:
		[[nodiscard]] FValueTransformerListIterator(const FValueTransformerList& TransformerList, int32 StartOffset, int32 Count);

		// Increments and moves the iterator to the next transformer entry
		FValueTransformerListIterator& operator++();

		// Returns whether or not the iterator still contains values
		[[nodiscard]] explicit operator bool() const;

		// Returns the value type that the transformer operates on for the current entry
		[[nodiscard]] UScriptStruct* GetValueType() const;

		// Returns the transformer for the current entry
		[[nodiscard]] FRawTransformerFunc GetTransformerFunc() const;

	private:
		UScriptStruct* const* ValueTypePtr = nullptr;
		const FRawTransformerFunc* TransformerFuncPtr = nullptr;
		UScriptStruct* const* ValueTypeEndPtr = nullptr;
	};

	// A list of transformer specializations that operators over various value types.
	// Transformers have a base type (e.g. interpolate, sanitize) and can be specialized to operate on arbitrary
	// value types (e.g. FFloatAnimationAttribute). They can also transform two types of containers: bound value maps
	// and unbound value maps.
	class FValueTransformerList
	{
	public:
		// Creates an empty transformer list
		UE_API FValueTransformerList();

		// Creates a transformer list for the specified transformer type
		[[nodiscard]] UE_API explicit FValueTransformerList(FName TransformerName);

		// Returns the transformer name
		[[nodiscard]] UE_API FName GetTransformerName() const;

		// Returns the number of bound value map transformers in this list
		[[nodiscard]] int32 NumBoundValueMapTransformers() const { return BoundValueMapTransformerCount; }

		// Returns the number of unbound value map transformers in this list
		[[nodiscard]] int32 NumUnboundValueMapTransformers() const { return ValueTypes.Num() - BoundValueMapTransformerCount; }

		// Adds a new bound value map transformer entry
		UE_API bool AddBoundValueMapTransformer(UScriptStruct* ValueType, FRawTransformerFunc TransformerFunc);

		// Adds a new unbound value map transformer entry
		UE_API bool AddUnboundValueMapTransformer(UScriptStruct* ValueType, FRawTransformerFunc TransformerFunc);

		// Returns an iterator over all bound value map transformer specializations within
		[[nodiscard]] FValueTransformerListIterator CreateBoundValueMapTransformerIterator() const { return FValueTransformerListIterator(*this, 0, NumBoundValueMapTransformers()); }

		// Returns an iterator over all unbound value map transformer specializations within
		[[nodiscard]] FValueTransformerListIterator CreateUnboundValueMapTransformerIterator() const { return FValueTransformerListIterator(*this, NumBoundValueMapTransformers(), NumUnboundValueMapTransformers()); }

	private:
		// The type of the transformer (e.g. interpolate, sanitize)
		FName TransformerName;

		// The number of bound value map transformers in this list
		int32 BoundValueMapTransformerCount = 0;

		// List of value types for which we have transformers registered (e.g. FTransformAnimationAttribute, FFloatAnimationAttribute) (sorted by container type, then by value type)
		// This list is split in two, bound value map transformers come first, then unbound value map transformers
		// Each sub-list is sorted by value type
		TArray<UScriptStruct*> ValueTypes;

		// List of transformer functions for each value type (sorted by container type, then by value type)
		// This list is split in two, bound value map transformers come first, then unbound value map transformers
		// Each sub-list is sorted by value type
		TArray<FRawTransformerFunc> Transformers;

		friend FValueRuntimeRegistry;
		friend FValueTransformerListIterator;
	};

	using FValueTransformerMapPtr = TSharedPtr<TMap<FName, FValueTransformerList>>;

	// An operator structure to join value transformer lists and attribute collections using InnerJoin/OuterJoin/etc
	// Ensures that the predicate is called for each bound value map with a value type that has a corresponding transformer
	struct FValueTransformerListWithBoundMapCollectionJoinOp
	{
		// Key sorting uses signed comparison, use the largest signed value regardless of pointer size
		static constexpr intptr_t WILDCARD_ATTRIBUTE_TYPE_VALUE = static_cast<intptr_t>(static_cast<uintptr_t>(~0ULL) >> 1);

		// We enable fuzzy matching by implementing its API
		bool IsMatch(FAttributeMappingKey ReferenceKey, FAttributeMappingKey TestKey) const
		{
			// Our wildcard key ensures that it is larger than all other keys meaning it will only ever be the
			// reference key if we've matched all other smaller values (all valid ones)
			if (reinterpret_cast<intptr_t>(TestKey.GetAttributeType()) == WILDCARD_ATTRIBUTE_TYPE_VALUE)
			{
				// If our test value is our wildcard value, we match if our value types match
				return ReferenceKey.GetValueType() == TestKey.GetValueType();
			}

			// Otherwise we test normally
			return ReferenceKey == TestKey;
		}

		FAttributeMappingKey GetLargestKey() const { return FAttributeMappingKey::LARGEST_VALUE; }

		// FAttributeValueTransformerListIterator entries
		FAttributeMappingKey GetKey(const FValueTransformerListIterator& It) const
		{
			// We use a wildcard entry for the attribute type within our key to ensure it is always larger than all other valid values
			return FAttributeMappingKey(reinterpret_cast<UScriptStruct*>(WILDCARD_ATTRIBUTE_TYPE_VALUE), It.GetValueType());
		}

		FRawTransformerFunc GetValue(const FValueTransformerListIterator& It) const { return It.GetTransformerFunc(); }

		// Normal FBoundMapCollectionJoinOp
		template<class IteratorType>
		FAttributeMappingKey GetKey(const IteratorType& It) const { return It.GetKey(); }

		template<class IteratorType>
			requires requires (const IteratorType& It) { It.GetMap(); }
		typename IteratorType::MapType GetValue(const IteratorType& It) const { return It.GetMap(); }
	};

	// An operator structure to join value transformer lists and attribute collections using InnerJoin/OuterJoin/etc
	// Ensures that the predicate is called for each unbound value map with a value type that has a corresponding transformer
	struct FValueTransformerListWithUnboundMapCollectionJoinOp
	{
		// Key sorting uses signed comparison, use the largest signed value regardless of pointer size
		static constexpr intptr_t KEY_LARGEST_VALUE = static_cast<intptr_t>(static_cast<uintptr_t>(~0ULL) >> 1);

		UScriptStruct* GetLargestKey() const { return reinterpret_cast<UScriptStruct*>(KEY_LARGEST_VALUE); }

		// FAttributeValueTransformerListIterator entries
		UScriptStruct* GetKey(const FValueTransformerListIterator& It) const { return It.GetValueType(); }
		FRawTransformerFunc GetValue(const FValueTransformerListIterator& It) const { return It.GetTransformerFunc(); }

		// Normal FUnboundMapCollectionJoinOp
		template<class IteratorType>
		UScriptStruct* GetKey(const IteratorType& It) const { return It.GetValueType(); }

		template<class IteratorType>
			requires requires (const IteratorType& It) { It.GetMap(); }
		typename IteratorType::MapType GetValue(const IteratorType& It) const { return It.GetMap(); }
	};

	//////////////////////////////////////////////////////////////////////////
	// Implementation

	inline FValueTransformerListIterator::FValueTransformerListIterator(const FValueTransformerList& TransformerList, int32 StartOffset, int32 Count)
		: ValueTypePtr(TransformerList.ValueTypes.GetData() + StartOffset)
		, TransformerFuncPtr(TransformerList.Transformers.GetData() + StartOffset)
		, ValueTypeEndPtr(ValueTypePtr + Count)
	{
	}

	inline FValueTransformerListIterator& FValueTransformerListIterator::operator++()
	{
		++ValueTypePtr;
		++TransformerFuncPtr;

		return *this;
	}

	inline FValueTransformerListIterator::operator bool() const
	{
		return ValueTypePtr < ValueTypeEndPtr;
	}

	inline UScriptStruct* FValueTransformerListIterator::GetValueType() const
	{
		return *ValueTypePtr;
	}

	inline FRawTransformerFunc FValueTransformerListIterator::GetTransformerFunc() const
	{
		return *TransformerFuncPtr;
	}
}

#undef UE_API
