// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/ValueRuntime/IndirectAllocator.h"
#include "UAF/ValueRuntime/UnboundValueMap.h"

#define UE_API UAF_API

namespace UE::UAF
{
	// A iterator that iterates over all unbound value maps within an unbound map collection
	template<class ContainerType>
	class TUnboundMapCollectionIterator
	{
	public:
		static constexpr bool bIsContainerConst = std::is_const_v<ContainerType>;

		using EntryPtrType = typename std::conditional<bIsContainerConst, const typename ContainerType::FMapEntry*, typename ContainerType::FMapEntry*>::type;
		using MapType = typename std::conditional<bIsContainerConst, const FUnboundValueMap*, FUnboundValueMap*>::type;

		// Creates an empty iterator
		[[nodiscard]] TUnboundMapCollectionIterator() = default;

		// Creates an iterator over the map entries within the specified collection
		[[nodiscard]] TUnboundMapCollectionIterator(ContainerType& Collection);

		// Increments and moves the iterator to the next map entry
		TUnboundMapCollectionIterator& operator++();

		// Returns whether or not the iterator still contains values
		[[nodiscard]] explicit operator bool() const;

		// Returns the value type of the current map entry
		[[nodiscard]] UScriptStruct* GetValueType() const;

		// Returns the map from the current entry
		[[nodiscard]] MapType GetMap() const;

	private:
		// Current map entry
		EntryPtrType EntryPtr = nullptr;

		// One past the last map entry
		EntryPtrType EntryEndPtr = nullptr;
	};

	/*
	 * Unbound Value Map Collection
	 *
	 * Holds a sorted collection of unbound value maps.
	 * Entries are sorted by value type.
	 */
	class FUnboundMapCollection
	{
	private:
		struct FMapEntry
		{
			// The value type (key) that identifies this entry
			UScriptStruct* ValueType = nullptr;

			// The map containing our values
			FUnboundValueMap* Map = nullptr;
		};

	public:
		UE_API FUnboundMapCollection(FReallocFun InReallocFun);
		UE_API FUnboundMapCollection(const FUnboundMapCollection& Other);
		UE_API FUnboundMapCollection(FUnboundMapCollection&& Other);
		UE_API ~FUnboundMapCollection();

		UE_API FUnboundMapCollection& operator=(const FUnboundMapCollection& Other);
		UE_API FUnboundMapCollection& operator=(FUnboundMapCollection&& Other);

		// Returns whether or not we are empty
		[[nodiscard]] UE_API bool IsEmpty() const;

		// Resets the collection while retaining our capacity
		UE_API void Reset();

		// Clears the collection of any content
		UE_API void Empty();

		// Returns the allocator function this collection was initialized with
		[[nodiscard]] UE_API FReallocFun GetAllocator() const;

		// Returns the number of maps contained within
		[[nodiscard]] UE_API int32 Num() const;

		// Constructs and adds a new map to this collection.
		// Returns nullptr if we fail to add an entry (e.g. duplicate)
		template <class ValueType> [[nodiscard]] TUnboundValueMap<ValueType>* Add();

		// Constructs and adds a new map to this collection.
		// Returns nullptr if we fail to add an entry (e.g. duplicate)
		[[nodiscard]] UE_API FUnboundValueMap* Add(UScriptStruct* ValueType);

		// Adds a new map to this collection.
		// Returns false if we fail to add an entry (e.g. duplicate, mismatched allocator)
		UE_API bool Add(FUnboundValueMap* Map);

		// Appends a new map to this collection (the input must belong at the end of the sorted map list).
		// Returns false if we fail to add an entry (e.g. duplicate, mismatched allocator, not sorted)
		UE_API bool Append(FUnboundValueMap* Map);

		// Removes the specified map from this collection
		// Returns whether or not the entry was successfully removed
		UE_API bool Remove(FUnboundValueMap* Map);

		// Removes the specified map from this collection
		// Returns whether or not the entry was successfully removed
		UE_API bool Remove(UScriptStruct* ValueType);

		// Finds a map of the specified ValueType in this collection
		// Returns nullptr if this collection does not contain such a mapping
		template <class ValueType> [[nodiscard]] TUnboundValueMap<ValueType>* Find();

		// Finds a map of the specified ValueType in this collection
		// Returns nullptr if this collection does not contain such a mapping
		template <class ValueType> [[nodiscard]] const TUnboundValueMap<ValueType>* Find() const;

		// Finds a map of the specified ValueType in this collection
		// Returns nullptr if this collection does not contain such a mapping
		[[nodiscard]] UE_API FUnboundValueMap* Find(UScriptStruct* ValueType);

		// Finds a map of the specified ValueType in this collection
		// Returns nullptr if this collection does not contain such a mapping
		[[nodiscard]] UE_API const FUnboundValueMap* Find(UScriptStruct* ValueType) const;

		// Finds or adds a map of the specified ValueType in this collection
		// Returns nullptr if we fail to add an entry (e.g. duplicate)
		template <class ValueType> [[nodiscard]] TUnboundValueMap<ValueType>* FindOrAdd();

		// Finds or adds a map of the specified ValueType in this collection
		// Returns nullptr if we fail to add an entry (e.g. duplicate)
		[[nodiscard]] UE_API FUnboundValueMap* FindOrAdd(UScriptStruct* ValueType);

		// Iterators
		using FIterator = TUnboundMapCollectionIterator<FUnboundMapCollection>;
		using FConstIterator = TUnboundMapCollectionIterator<const FUnboundMapCollection>;

		// Returns an iterator over all maps contained within, sorted by their key (value type)
		[[nodiscard]] FIterator CreateIterator();
		[[nodiscard]] FConstIterator CreateConstIterator() const;

	private:
		UE_API FUnboundValueMap** AddImpl(UScriptStruct* ValueType, bool bConstructMap);

		// An array of maps (sorted by value type)
		FMapEntry* Maps = nullptr;

		// How many maps are contained within this collection
		int32 MapCount = 0;

		// How many maps we can contain within this collection
		int32 MapCapacity = 0;

		// The indirect allocator function pointer
		FReallocFun ReallocFun = nullptr;

		friend FIterator;
		friend FConstIterator;
	};

	// A heap allocated unbound map collection
	class FUnboundMapCollectionHeap : public FUnboundMapCollection
	{
	public:
		// Constructs an empty collection
		FUnboundMapCollectionHeap();

		using FUnboundMapCollection::operator=;
	};

	// A Mem-Stack allocated unbound map collection
	class FUnboundMapCollectionStack : public FUnboundMapCollection
	{
	public:
		// Constructs an empty collection
		FUnboundMapCollectionStack();

		using FUnboundMapCollection::operator=;
	};

	// An operator structure to join unbound map collections using InnerJoin/OuterJoin/etc
	struct FUnboundMapCollectionJoinOp
	{
		// Key sorting uses signed comparison, use the largest signed value regardless of pointer size
		static constexpr intptr_t LARGEST_KEY_VALUE = static_cast<intptr_t>(static_cast<uintptr_t>(~0ULL) >> 1);

		UScriptStruct* GetLargestKey() const { return reinterpret_cast<UScriptStruct*>(LARGEST_KEY_VALUE); }

		UScriptStruct* GetKey(const FUnboundMapCollection::FIterator& It) const { return It.GetValueType(); }
		UScriptStruct* GetKey(const FUnboundMapCollection::FConstIterator& It) const { return It.GetValueType(); }

		FUnboundValueMap* GetValue(const FUnboundMapCollection::FIterator& It) const { return It.GetMap(); }
		const FUnboundValueMap* GetValue(const FUnboundMapCollection::FConstIterator& It) const { return It.GetMap(); }
	};
}

//////////////////////////////////////////////////////////////////////////
// Implementation

namespace UE::UAF
{
	template <class ValueType>
	inline TUnboundValueMap<ValueType>* FUnboundMapCollection::Add()
	{
		UScriptStruct* ValueTypeStruct = ValueType::StaticStruct();

		// Don't construct the map internally, it'll have to use indirection to do so and here we know the concrete type
		constexpr bool bConstructMap = false;

		FUnboundValueMap** MapPtr = AddImpl(ValueTypeStruct, bConstructMap);
		if (MapPtr == nullptr)
		{
			return nullptr;
		}

		TUnboundValueMap<ValueType>* Map = MakeUnboundValueMap<ValueType>(ReallocFun);

		check(*MapPtr == nullptr);
		*MapPtr = Map;

		return Map;
	}

	template <class ValueType>
	inline TUnboundValueMap<ValueType>* FUnboundMapCollection::Find()
	{
		return Cast<ValueType>(Find(ValueType::StaticStruct()));
	}

	template <class ValueType>
	inline const TUnboundValueMap<ValueType>* FUnboundMapCollection::Find() const
	{
		return Cast<ValueType>(Find(ValueType::StaticStruct()));
	}

	template <class ValueType>
	inline TUnboundValueMap<ValueType>* FUnboundMapCollection::FindOrAdd()
	{
		if (TUnboundValueMap<ValueType>* Map = Find<ValueType>())
		{
			return Map;
		}

		return Add<ValueType>();
	}

	inline typename FUnboundMapCollection::FIterator FUnboundMapCollection::CreateIterator()
	{
		return FIterator(*this);
	}

	inline typename FUnboundMapCollection::FConstIterator FUnboundMapCollection::CreateConstIterator() const
	{
		return FConstIterator(*this);
	}

	inline FUnboundMapCollectionHeap::FUnboundMapCollectionHeap()
		: FUnboundMapCollection(&FAllocatorTypeTrait<FDefaultAllocator>::Realloc)
	{
	}

	inline FUnboundMapCollectionStack::FUnboundMapCollectionStack()
		: FUnboundMapCollection(&FAllocatorTypeTrait<TMemStackAllocator<>>::Realloc)
	{
	}

	template<class ContainerType>
	inline TUnboundMapCollectionIterator<ContainerType>::TUnboundMapCollectionIterator(ContainerType& Collection)
		: EntryPtr(Collection.Maps)
		, EntryEndPtr(Collection.Maps + Collection.MapCount)
	{
	}

	template<class ContainerType>
	inline TUnboundMapCollectionIterator<ContainerType>& TUnboundMapCollectionIterator<ContainerType>::operator++()
	{
		++EntryPtr;
		return *this;
	}

	template<class ContainerType>
	inline TUnboundMapCollectionIterator<ContainerType>::operator bool() const
	{
		return EntryPtr != EntryEndPtr;
	}

	template<class ContainerType>
	inline UScriptStruct* TUnboundMapCollectionIterator<ContainerType>::GetValueType() const
	{
		return EntryPtr->ValueType;
	}

	template<class ContainerType>
	inline typename TUnboundMapCollectionIterator<ContainerType>::MapType TUnboundMapCollectionIterator<ContainerType>::GetMap() const
	{
		return EntryPtr->Map;
	}
}

#undef UE_API
