// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/BinarySearch.h"
#include "Algo/Sort.h"
#include "Templates/MemoryOps.h"
#include "UAF/ValueRuntime/IndirectAllocator.h"

#define UE_API UAF_API

namespace UE::UAF
{
	// Untyped base class for TUnboundValueMap
	class FUnboundValueMap
	{
	public:
		UE_NONCOPYABLE(FUnboundValueMap);	// We disallow copy/move semantics

		// Returns whether or not this map has any values
		[[nodiscard]] UE_API bool IsEmpty() const;

		// Returns the number of mapped values
		[[nodiscard]] UE_API int32 Num() const;

		// Returns the maximum number of values we can map
		[[nodiscard]] UE_API int32 Max() const;

		// Returns the mapped value type
		[[nodiscard]] UE_API UScriptStruct* GetValueType() const;

		// Returns the total allocated size for this map
		[[nodiscard]] UE_API int32 GetAllocatedSize() const;

		// Returns the allocator function this map was initialized with
		[[nodiscard]] UE_API FReallocFun GetAllocator() const;

		// Returns a copy of this map using the specified allocator
		[[nodiscard]] virtual FUnboundValueMap* Duplicate(FReallocFun InReallocFun) const = 0;

		// Copies the content of this map into another
		// Both must have matching value types
		virtual void CopyTo(FUnboundValueMap& Other) const = 0;

		// Moves the content of this map to another
		// Both must have matching value types
		// The content of this map after the move is undefined, just like with move assignment
		UE_API virtual void MoveTo(FUnboundValueMap& Other);

		// Finds a mapping and returns its index or INDEX_NONE if not found, O(logN)
		[[nodiscard]] int32 IndexOf(FName Name) const;

		// Returns whether or not this map contains the specified mapped value, O(logN)
		[[nodiscard]] bool Contains(FName Name) const;

		// Retrieves a mapping name by index, O(1)
		[[nodiscard]] FName GetName(int32 Index) const;

		// Adds a new mapping, sets the value using the provided setter, O(logN)
		// Returns whether or not we inserted (we can fail if a value with that name already exists)
		// The value setter (and/or caller) is responsible for validating the value type
		virtual bool AddWithSetter(FName Name, const TFunctionRef<void(UScriptStruct* ValueType, uint8* OutValuePtr)>& ValueSetter) = 0;

		// Gets a value at the specified index using a value getter function, O(1)
		// The value getter (and/or caller) is responsible for validating the value type
		virtual void GetValueWithGetter(int32 Index, const TFunctionRef<void(UScriptStruct* ValueType, const uint8* ValuePtr)>& ValueGetter) const = 0;

	protected:
		// Default construction for an empty map
		UE_API FUnboundValueMap();

		// Constructs a map but does not allocate its memory, derived types are responsible for it
		UE_API FUnboundValueMap(UScriptStruct* ValueType, FReallocFun ReallocFun, int32 ContainerSize);

		// Destroys the map and frees its memory
		UE_API virtual ~FUnboundValueMap();

		// Returns a pointer to the start of the names
		[[nodiscard]] FName* GetNames();
		[[nodiscard]] const FName* GetNames() const;

		// The type of values this map contains
		UScriptStruct* ValueTypeStruct = nullptr;

		// The number of values mapped in our buffer
		int32 NumValues = 0;

		// The maximum number of values that we can map in our buffer
		int32 MaxValues = 0;

		// The base untyped pointer for our names/values
		// Names and values are allocated together within the same contiguous memory buffer
		// Names are first within the buffer followed by value types using default alignment
		uint8* DataPtr = nullptr;

		// The indirect allocator function pointer
		FReallocFun ReallocFun = nullptr;

		// The size in bytes of this container (excluding any externally allocated data)
		int32 ContainerSize = 0;

		friend class FValueBundle;
		friend class FValueRuntimeRegistry;
		friend void ReleaseUnboundValueMap(FUnboundValueMap* Map);
	};

	// A iterator that iterates over all name/value pairs within an unbound value map
	template<class ContainerType>
	class TUnboundValueMapIterator
	{
	public:
		static constexpr bool bIsContainerConst = std::is_const_v<ContainerType>;

		using NamePtrType = typename std::conditional<bIsContainerConst, const FName*, FName*>::type;
		using ValuePtrType = typename std::conditional<bIsContainerConst, const typename ContainerType::ValueType*, typename ContainerType::ValueType*>::type;

		// Creates an empty iterator
		[[nodiscard]] TUnboundValueMapIterator() = default;

		// Creates an iterator over the name/value map entries within the specified unbound value map
		[[nodiscard]] TUnboundValueMapIterator(ContainerType& Map);

		// Increments and moves the iterator to the next name/value pair
		TUnboundValueMapIterator& operator++();

		// Returns whether or not the iterator still contains values
		[[nodiscard]] explicit operator bool() const;

		// Returns the name of the current name/value pair
		[[nodiscard]] FName GetName() const;

		// Returns the value of the current name/value pair
		[[nodiscard]] ValuePtrType GetValue() const;

	private:
		// Current name/value pair
		NamePtrType NamePtr = nullptr;
		ValuePtrType ValuePtr = nullptr;

		// One past the last name/value pair
		NamePtrType NameEndPtr = nullptr;
	};

	// Encapsulates a mapping of FName to value type
	// Value types must be a UStruct
	template<class InValueType>
	class TUnboundValueMap : public FUnboundValueMap
	{
	public:
		// The type of values held in this map
		using ValueType = InValueType;

		// Returns a copy of this map using the specified allocator
		[[nodiscard]] virtual FUnboundValueMap* Duplicate(FReallocFun InReallocFun) const override;

		// Copies the content of this map into another
		// Both must have matching value types
		virtual void CopyTo(FUnboundValueMap& Other) const override;

		// Adds a new mapping, sets the value using the provided setter, O(logN)
		// Returns whether or not we inserted (we can fail if a value with that name already exists)
		// The value setter (and/or caller) is responsible for validating the value type
		virtual bool AddWithSetter(FName Name, const TFunctionRef<void(UScriptStruct* ValueType, uint8* OutValuePtr)>& ValueSetter) override;

		// Gets a value at the specified index using a value getter function, O(1)
		// The value getter (and/or caller) is responsible for validating the value type
		virtual void GetValueWithGetter(int32 Index, const TFunctionRef<void(UScriptStruct* ValueType, const uint8* ValuePtr)>& ValueGetter) const override;

		// Adds a new mapping, O(logN)
		// Returns whether or not we inserted (we can fail if a value with that name already exists)
		bool Add(FName Name, const InValueType& Value);

		// Adds a new mapping at the end of the map, must be sorted, O(1)
		// Returns whether or not we inserted (we can fail if the value isn't sorted to land at the end of the map)
		bool Append(FName Name, const InValueType& Value);

		// Removes a mapping, O(logN)
		// Returns whether or not we removed the value (we can fail if the name isn't found)
		bool Remove(FName Name);

		// Removes a mapping, O(1)
		void RemoveAt(int32 Index);

		// Finds a mapping and returns a pointer to it or nullptr if not found, O(logN)
		[[nodiscard]] InValueType* Find(FName Name);
		[[nodiscard]] const InValueType* Find(FName Name) const;

		// Retrieves a mapping value by index, O(1)
		[[nodiscard]] InValueType& GetValue(int32 Index);
		[[nodiscard]] const InValueType& GetValue(int32 Index) const;

		// Initializes the mapping using sorted inputs, O(N)
		template<class GetNamePredicateType, class GetValuePredicateType>
		void SetSorted(int32 NumValues, GetNamePredicateType GetNamePredicate, GetValuePredicateType GetValuePredicate);

		// Initializes the mapping using unsorted inputs, O(NlogN)
		template<class GetNamePredicateType, class GetValuePredicateType>
		void SetUnsorted(int32 NumValues, GetNamePredicateType GetNamePredicate, GetValuePredicateType GetValuePredicate);

		// Iterators
		using FIterator = TUnboundValueMapIterator<TUnboundValueMap<InValueType>>;
		using FConstIterator = TUnboundValueMapIterator<const TUnboundValueMap<InValueType>>;

		// Returns an iterator over all name/value pairs contained within, sorted by their name
		[[nodiscard]] FIterator CreateIterator();
		[[nodiscard]] FConstIterator CreateConstIterator() const;

	protected:
		static const int32 ValueTypeAlignment = sizeof(InValueType) >= 16 ? 16 : 8;

		// Constructs a map
		TUnboundValueMap(UScriptStruct* ValueType, FReallocFun ReallocFun, int32 ContainerSize);

		// Destroys the map and frees its memory
		virtual ~TUnboundValueMap() override;

		void Grow();

		// Returns a pointer to the start of the values
		InValueType* GetValues();
		const InValueType* GetValues() const;

		template<class T>
		friend TUnboundValueMap<T>* MakeUnboundValueMap(FReallocFun ReallocFun);
		friend FIterator;
		friend FConstIterator;
	};

	// An operator structure to join unbound value maps using InnerJoin/OuterJoin/etc
	struct FUnboundValueMapJoinOp
	{
		// We generally want the key in our predicate since the values by themselves have little meaning
		static constexpr bool HasPredicateWithKey() { return true; }

		bool IsLessThan(FName LHS, FName RHS) const
		{
			return LHS.FastLess(RHS);
		}

		// Special dummy FName that is larger than all other FNames in fast sorted order
		FName GetLargestKey() const { return FName(FNameEntryId::FromUnstableInt(MAX_int32), FNameEntryId::FromUnstableInt(MAX_int32), MAX_int32); }

		template<typename IteratorType>
		FName GetKey(const IteratorType& It) const { return It.GetName(); }

		template<typename IteratorType>
		typename IteratorType::ValuePtrType GetValue(const IteratorType& It) const { return It.GetValue(); }
	};

	// Allocates and constructs a new unbound value map for the specified value type
	template<class ValueType>
	TUnboundValueMap<ValueType>* MakeUnboundValueMap(FReallocFun ReallocFun);

	// Releases and destroys the specified unbound value map
	void ReleaseUnboundValueMap(FUnboundValueMap* Map);

	// Cast from an untyped name/value map to a typed name/value map
	template<class ValueType>
	UE_FORCEINLINE_HINT TUnboundValueMap<ValueType>* Cast(FUnboundValueMap* Src)
	{
		return Src != nullptr && ValueType::StaticStruct() == Src->GetValueType() ? static_cast<TUnboundValueMap<ValueType>*>(Src) : nullptr;
	}

	// Cast from an untyped name/value map to a typed name/value map
	template<class ValueType>
	UE_FORCEINLINE_HINT const TUnboundValueMap<ValueType>* Cast(const FUnboundValueMap* Src)
	{
		return Src != nullptr && ValueType::StaticStruct() == Src->GetValueType() ? static_cast<const TUnboundValueMap<ValueType>*>(Src) : nullptr;
	}

	// Cast from an untyped name/value map to a typed name/value map with an assert if the types mismatch
	template<class ValueType>
	inline TUnboundValueMap<ValueType>* CastChecked(FUnboundValueMap* Src)
	{
		TUnboundValueMap<ValueType>* CastResult = Src != nullptr && ValueType::StaticStruct() == Src->GetValueType() ? static_cast<TUnboundValueMap<ValueType>*>(Src) : nullptr;
		check(CastResult);
		return CastResult;
	}

	// Cast from an untyped name/value map to a typed name/value map with an assert if the types mismatch
	template<class ValueType>
	inline const TUnboundValueMap<ValueType>* CastChecked(const FUnboundValueMap* Src)
	{
		const TUnboundValueMap<ValueType>* CastResult = Src != nullptr && ValueType::StaticStruct() == Src->GetValueType() ? static_cast<const TUnboundValueMap<ValueType>*>(Src) : nullptr;
		check(CastResult);
		return CastResult;
	}

	//////////////////////////////////////////////////////////////////////////
	// Implementation

	inline int32 FUnboundValueMap::IndexOf(FName Name) const
	{
		const FName* Names = GetNames();

		const int32 Index = Algo::LowerBound(TArrayView<const FName>(Names, NumValues), Name, FNameFastLess());
		if (Index >= NumValues || Names[Index] != Name)
		{
			// No value is mapped to that name
			return INDEX_NONE;
		}

		return Index;
	}

	inline bool FUnboundValueMap::Contains(FName Name) const
	{
		return IndexOf(Name) != INDEX_NONE;
	}

	inline FName FUnboundValueMap::GetName(int32 Index) const
	{
		checkf((Index >= 0) & (Index < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index, (long long)NumValues); // & for one branch
		return GetNames()[Index];
	}

	inline FName* FUnboundValueMap::GetNames()
	{
		return reinterpret_cast<FName*>(DataPtr);
	}

	inline const FName* FUnboundValueMap::GetNames() const
	{
		return reinterpret_cast<const FName*>(DataPtr);
	}

	template<class InValueType>
	inline TUnboundValueMap<InValueType>::TUnboundValueMap(UScriptStruct* ValueTypeStruct, FReallocFun InReallocFun, int32 InContainerSize)
		: FUnboundValueMap(ValueTypeStruct, InReallocFun, InContainerSize)
	{
		check(ValueTypeStruct == InValueType::StaticStruct());
		check(InContainerSize == sizeof(TUnboundValueMap<InValueType>));
	}

	template<class InValueType>
	inline TUnboundValueMap<InValueType>::~TUnboundValueMap()
	{
		if (DataPtr != nullptr)
		{
			// FNames don't need to be destroyed, only destroy the values
			DestructItems<InValueType>(GetValues(), NumValues);

			DataPtr = nullptr;
			NumValues = MaxValues = 0;
		}
	}

	template<class InValueType>
	inline FUnboundValueMap* TUnboundValueMap<InValueType>::Duplicate(FReallocFun InReallocFun) const
	{
		TUnboundValueMap<InValueType>* Copy = MakeUnboundValueMap<InValueType>(InReallocFun);

		if (NumValues != 0)
		{
			check(Copy->MaxValues == 0 && Copy->DataPtr == nullptr);

			const int32 NewSize = Align(sizeof(FName) * NumValues, ValueTypeAlignment) + (sizeof(InValueType) * NumValues);

			Copy->DataPtr = (*InReallocFun)(nullptr, 0, NewSize);
			Copy->NumValues = NumValues;
			Copy->MaxValues = NumValues;

			ConstructItems<FName>(Copy->GetNames(), GetNames(), NumValues);
			ConstructItems<InValueType>(Copy->GetValues(), GetValues(), NumValues);
		}

		return Copy;
	}

	template<class InValueType>
	inline void TUnboundValueMap<InValueType>::CopyTo(FUnboundValueMap& Other) const
	{
		checkf(ValueTypeStruct == Other.GetValueType(), TEXT("Value types must match to allow copying"));

		TUnboundValueMap<InValueType>& Copy = reinterpret_cast<TUnboundValueMap<InValueType>&>(Other);

		// Destroy old values
		if (Copy.NumValues != 0)
		{
			// FNames don't need to be destroyed, only destroy the values
			DestructItems<InValueType>(Copy.GetValues(), Copy.NumValues);
		}

		// Resize
		if (Copy.MaxValues != NumValues)
		{
			const int32 OldSize = Align(sizeof(FName) * Copy.MaxValues, ValueTypeAlignment) + (sizeof(InValueType) * Copy.MaxValues);
			const int32 NewSize = Align(sizeof(FName) * NumValues, ValueTypeAlignment) + (sizeof(InValueType) * NumValues);

			Copy.DataPtr = (*Copy.ReallocFun)(Copy.DataPtr, OldSize, NewSize);
			Copy.MaxValues = NumValues;
		}

		// Copy construct our new values
		if (NumValues != 0)
		{
			ConstructItems<FName>(Copy.GetNames(), GetNames(), NumValues);
			ConstructItems<InValueType>(Copy.GetValues(), GetValues(), NumValues);
		}

		Copy.NumValues = NumValues;
	}

	template<class InValueType>
	inline bool TUnboundValueMap<InValueType>::AddWithSetter(FName Name, const TFunctionRef<void(UScriptStruct* ValueType, uint8* OutValuePtr)>& ValueSetter)
	{
		FName* Names = GetNames();

		const int32 Index = Algo::LowerBound(TArrayView<const FName>(Names, NumValues), Name, FNameFastLess());
		if (Index < NumValues && Names[Index] == Name)
		{
			// A mapping already exists that matches the specified name
			return false;
		}

		if (NumValues >= MaxValues)
		{
			// Not enough space, grow our buffer
			Grow();

			Names = GetNames();
		}

		InValueType* Values = GetValues();

		// Insert our value at the right position, making space for it
		const int32 NumToMove = NumValues - Index;
		RelocateConstructItems<FName>(Names + Index + 1, Names + Index, NumToMove);
		RelocateConstructItems<InValueType>(Values + Index + 1, Values + Index, NumToMove);
		new(Names + Index) FName(Name);
		ValueSetter(ValueTypeStruct, reinterpret_cast<uint8*>(Values + Index));
		NumValues++;

		return true;
	}

	template<class InValueType>
	inline void TUnboundValueMap<InValueType>::GetValueWithGetter(int32 Index, const TFunctionRef<void(UScriptStruct* ValueType, const uint8* ValuePtr)>& ValueGetter) const
	{
		checkf((Index >= 0) & (Index < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index, (long long)NumValues); // & for one branch
		const InValueType* ValuePtr = GetValues() + Index;
		ValueGetter(ValueTypeStruct, reinterpret_cast<const uint8*>(ValuePtr));
	}

	template<class InValueType>
	inline bool TUnboundValueMap<InValueType>::Add(FName Name, const InValueType& Value)
	{
		FName* Names = GetNames();

		const int32 Index = Algo::LowerBound(TArrayView<const FName>(Names, NumValues), Name, FNameFastLess());
		if (Index < NumValues && Names[Index] == Name)
		{
			// A mapping already exists that matches the specified name
			return false;
		}

		if (NumValues >= MaxValues)
		{
			// Not enough space, grow our buffer
			Grow();
			
			Names = GetNames();
		}

		InValueType* Values = GetValues();

		// Insert our value at the right position, making space for it
		const int32 NumToMove = NumValues - Index;
		RelocateConstructItems<FName>(Names + Index + 1, Names + Index, NumToMove);
		RelocateConstructItems<InValueType>(Values + Index + 1, Values + Index, NumToMove);
		new(Names + Index) FName(Name);
		new(Values + Index) InValueType(Value);
		NumValues++;

		return true;
	}

	template<class InValueType>
	inline bool TUnboundValueMap<InValueType>::Append(FName Name, const InValueType& Value)
	{
		FName* Names = GetNames();

		if (NumValues != 0 && !Names[NumValues - 1].FastLess(Name))
		{
			// The name doesn't belong at the end, it isn't sorted
			return false;
		}

		if (NumValues >= MaxValues)
		{
			// Not enough space, grow our buffer
			Grow();

			Names = GetNames();
		}

		InValueType* Values = GetValues();

		// Append our new value at the end
		new(Names + NumValues) FName(Name);
		new(Values + NumValues) InValueType(Value);
		NumValues++;

		return true;
	}

	template<class InValueType>
	inline bool TUnboundValueMap<InValueType>::Remove(FName Name)
	{
		const FName* Names = GetNames();

		const int32 Index = Algo::LowerBound(TArrayView<const FName>(Names, NumValues), Name, FNameFastLess());
		if (Index >= NumValues || Names[Index] != Name)
		{
			// No value is mapped to that name
			return false;
		}

		RemoveAt(Index);

		return true;
	}

	template<class InValueType>
	inline void TUnboundValueMap<InValueType>::RemoveAt(int32 Index)
	{
		checkf((Index >= 0) & (Index < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index, (long long)NumValues); // & for one branch

		FName* Names = GetNames();
		InValueType* Values = GetValues();

		// FNames don't need to be destroyed, only destroy the values
		DestructItem<InValueType>(Values + Index);

		const int32 NumToMove = NumValues - Index - 1;
		RelocateConstructItems<FName>(Names + Index, Names + Index + 1, NumToMove);
		RelocateConstructItems<InValueType>(Values + Index, Values + Index + 1, NumToMove);
		NumValues--;
	}

	template<class InValueType>
	inline InValueType* TUnboundValueMap<InValueType>::Find(FName Name)
	{
		const FName* Names = GetNames();

		const int32 Index = Algo::LowerBound(TArrayView<const FName>(Names, NumValues), Name, FNameFastLess());
		if (Index >= NumValues || Names[Index] != Name)
		{
			// No value is mapped to that name
			return nullptr;
		}

		return GetValues() + Index;
	}

	template<class InValueType>
	inline const InValueType* TUnboundValueMap<InValueType>::Find(FName Name) const
	{
		const FName* Names = GetNames();

		const int32 Index = Algo::LowerBound(TArrayView<const FName>(Names, NumValues), Name, FNameFastLess());
		if (Index >= NumValues || Names[Index] != Name)
		{
			// No value is mapped to that name
			return nullptr;
		}

		return GetValues() + Index;
	}

	template<class InValueType>
	inline InValueType& TUnboundValueMap<InValueType>::GetValue(int32 Index)
	{
		checkf((Index >= 0) & (Index < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index, (long long)NumValues); // & for one branch

		return GetValues()[Index];
	}

	template<class InValueType>
	inline const InValueType& TUnboundValueMap<InValueType>::GetValue(int32 Index) const
	{
		checkf((Index >= 0) & (Index < NumValues), TEXT("Map index out of bounds: %lld into an map of size %lld"), (long long)Index, (long long)NumValues); // & for one branch

		return GetValues()[Index];
	}

	template<class InValueType>
	template<class GetNamePredicateType, class GetValuePredicateType>
	inline void TUnboundValueMap<InValueType>::SetSorted(int32 InNumValues, GetNamePredicateType GetNamePredicate, GetValuePredicateType GetValuePredicate)
	{
		// Destroy old values
		if (NumValues != 0)
		{
			// FNames don't need to be destroyed, only destroy the values
			DestructItems<InValueType>(GetValues(), NumValues);
		}

		// Resize
		if (MaxValues != InNumValues)
		{
			const int32 OldSize = Align(sizeof(FName) * MaxValues, ValueTypeAlignment) + (sizeof(InValueType) * MaxValues);
			const int32 NewSize = Align(sizeof(FName) * InNumValues, ValueTypeAlignment) + (sizeof(InValueType) * InNumValues);

			DataPtr = (*ReallocFun)(DataPtr, OldSize, NewSize);
			MaxValues = InNumValues;
		}

		FName* Names = GetNames();
		InValueType* Values = GetValues();

		// Set now to make debugger easier
		NumValues = InNumValues;

		// Set our values
		for (int32 Index = 0; Index < InNumValues; ++Index)
		{
			const FName Name = GetNamePredicate(Index);
			checkf(Index == 0 || Names[Index - 1].FastLess(Name), TEXT("Attempting to initialize an unbound value map from unsorted data"));

			new(Names + Index) FName(Name);
			new(Values + Index) InValueType(GetValuePredicate(Index));
		}
	}

	template<class InValueType>
	template<class GetNamePredicateType, class GetValuePredicateType>
	inline void TUnboundValueMap<InValueType>::SetUnsorted(int32 InNumValues, GetNamePredicateType GetNamePredicate, GetValuePredicateType GetValuePredicate)
	{
		// Destroy old values
		if (NumValues != 0)
		{
			// FNames don't need to be destroyed, only destroy the values
			DestructItems<InValueType>(GetValues(), NumValues);
		}

		// Resize
		if (MaxValues != InNumValues)
		{
			const int32 OldSize = Align(sizeof(FName) * MaxValues, ValueTypeAlignment) + (sizeof(InValueType) * MaxValues);
			const int32 NewSize = Align(sizeof(FName) * InNumValues, ValueTypeAlignment) + (sizeof(InValueType) * InNumValues);

			DataPtr = (*ReallocFun)(DataPtr, OldSize, NewSize);
			MaxValues = InNumValues;
		}

		FName* Names = GetNames();
		InValueType* Values = GetValues();

		// Set now to make debugger easier
		NumValues = InNumValues;

#if 1
		// TODO: We don't appear to have a version of Algo::Sort that works on iterators
		// and so we can't sort our two arrays in tandem, use a temporary array instead
		using FTmpElementType = TPair<FName, InValueType>;
		TArray<FTmpElementType> TmpElements;
		TmpElements.Reserve(InNumValues);
		for (int32 Index = 0; Index < InNumValues; ++Index)
		{
			TmpElements.Emplace(FTmpElementType{ FName(GetNamePredicate(Index)), InValueType(GetValuePredicate(Index)) });
		}

		// Sort our temporary elements
		Algo::Sort(TmpElements, [](const FTmpElementType& LHS, const FTmpElementType& RHS) { return LHS.Key.FastLess(RHS.Key); });

		// Set our values
		for (int32 Index = 0; Index < InNumValues; ++Index)
		{
			const FTmpElementType& Element = TmpElements[Index];

			new(Names + Index) FName(Element.Key);
			new(Values + Index) InValueType(Element.Value);
		}
#else
		// Set our values
		for (int32 Index = 0; Index < InNumValues; ++Index)
		{
			new(Names + Index) FName(GetNamePredicate(Index));
			new(Values + Index) InValueType(GetValuePredicate(Index));
		}

		// Sort our values
		//Algo::Sort(Elements, FElementSortPredicate());	// TODO
#endif
	}

	template<class InValueType>
	inline typename TUnboundValueMap<InValueType>::FIterator TUnboundValueMap<InValueType>::CreateIterator()
	{
		return FIterator(*this);
	}

	template<class InValueType>
	inline typename TUnboundValueMap<InValueType>::FConstIterator TUnboundValueMap<InValueType>::CreateConstIterator() const
	{
		return FConstIterator(*this);
	}

	template<class InValueType>
	FORCENOINLINE void TUnboundValueMap<InValueType>::Grow()
	{
		const bool bAllowQuantize = false;
		const int32 NewCapacity = DefaultCalculateSlackGrow(MaxValues + 1, MaxValues, 1, bAllowQuantize);

		const int32 OldValuesOffset = Align(sizeof(FName) * MaxValues, ValueTypeAlignment);
		const int32 NewValuesOffset = Align(sizeof(FName) * NewCapacity, ValueTypeAlignment);
		const int32 OldSize = OldValuesOffset + (sizeof(InValueType) * MaxValues);
		const int32 NewSize = NewValuesOffset + (sizeof(InValueType) * NewCapacity);

		// Our names and values live within the same buffer but when we re-size, they need to end up at
		// different offsets since both halves grow. To avoid reallocate moving our names/values and
		// then having another pass to offset our values, we handle it manually.

		// Cache our old pointers
		FName* OldNames = GetNames();
		InValueType* OldValues = GetValues();

		// Allocate our new buffer
		uint8* NewDataPtr = (*ReallocFun)(nullptr, 0, NewSize);

		// Cache our new pointers
		FName* NewNames = reinterpret_cast<FName*>(NewDataPtr);
		InValueType* NewValues = reinterpret_cast<InValueType*>(NewDataPtr + NewValuesOffset);

		// Relocate our name/value pairs
		RelocateConstructItems<FName>(NewNames, OldNames, NumValues);
		RelocateConstructItems<InValueType>(NewValues, OldValues, NumValues);

		// Deallocate our old buffer
		(*ReallocFun)(DataPtr, OldSize, 0);

		// Update our state
		DataPtr = NewDataPtr;

		MaxValues = NewCapacity;
	}

	template<class InValueType>
	inline InValueType* TUnboundValueMap<InValueType>::GetValues()
	{
		const int32 ValuesOffset = Align(MaxValues * sizeof(FName), ValueTypeAlignment);
		return reinterpret_cast<InValueType*>(DataPtr + ValuesOffset);
	}

	template<class InValueType>
	inline const InValueType* TUnboundValueMap<InValueType>::GetValues() const
	{
		const int32 ValuesOffset = Align(MaxValues * sizeof(FName), ValueTypeAlignment);
		return reinterpret_cast<const InValueType*>(DataPtr + ValuesOffset);
	}

	template<class ContainerType>
	inline TUnboundValueMapIterator<ContainerType>::TUnboundValueMapIterator(ContainerType& Map)
		: NamePtr(Map.GetNames())
		, ValuePtr(Map.GetValues())
		, NameEndPtr(NamePtr + Map.Num())
	{
	}

	template<class ContainerType>
	inline TUnboundValueMapIterator<ContainerType>& TUnboundValueMapIterator<ContainerType>::operator++()
	{
		++NamePtr;
		++ValuePtr;
		return *this;
	}

	template<class ContainerType>
	inline TUnboundValueMapIterator<ContainerType>::operator bool() const
	{
		return NamePtr != NameEndPtr;
	}

	template<class ContainerType>
	inline FName TUnboundValueMapIterator<ContainerType>::GetName() const
	{
		return *NamePtr;
	}

	template<class ContainerType>
	inline typename TUnboundValueMapIterator<ContainerType>::ValuePtrType TUnboundValueMapIterator<ContainerType>::GetValue() const
	{
		return ValuePtr;
	}

	template<class ValueType>
	inline TUnboundValueMap<ValueType>* MakeUnboundValueMap(FReallocFun ReallocFun)
	{
		const int32 ContainerSize = sizeof(TUnboundValueMap<ValueType>);

		// Allocate our map container buffer
		uint8* Ptr = (*ReallocFun)(nullptr, 0, ContainerSize);

		// Construct our map over it
		return new(Ptr) TUnboundValueMap<ValueType>(ValueType::StaticStruct(), ReallocFun, ContainerSize);
	}

	inline void ReleaseUnboundValueMap(FUnboundValueMap* Map)
	{
		if (Map == nullptr)
		{
			return;
		}

		const int32 ContainerSize = Map->ContainerSize;
		const FReallocFun ReallocFun = Map->ReallocFun;

		Map->~FUnboundValueMap();

		(*ReallocFun)(reinterpret_cast<uint8*>(Map), ContainerSize, 0);
	}
}

#undef UE_API
