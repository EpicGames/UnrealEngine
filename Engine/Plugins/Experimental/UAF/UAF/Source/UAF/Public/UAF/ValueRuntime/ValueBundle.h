// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/MemStack.h"
#include "UAF/Attributes/AttributeNamedSet.h"
#include "UAF/ValueRuntime/BoundMapCollection.h"
#include "UAF/ValueRuntime/IndirectAllocator.h"
#include "UAF/ValueRuntime/UnboundMapCollection.h"
#include "UAF/ValueRuntime/ValueSpace.h"

#include "ValueBundle.generated.h"

#define UE_API UAF_API

namespace UE::UAF
{
	struct FInputValueTrait;
	class IVirtualValueBundle;
};

namespace UE::UAF
{
	/*
	 * Value Bundle
	 *
	 * This struct holds values produced by UAF. They come in two flavors: bound and unbound.
	 * Bound values are fast and defined up front in the skeleton set binding asset.
	 * Unbound values are slower but produced on demand during evaluation.
	 */
	class FValueBundle
	{
	public:
		UE_API FValueBundle(const FAttributeNamedSetPtr& NamedSet, FReallocFun InReallocFun);
		UE_API FValueBundle(const FValueBundle& Other);
		UE_API FValueBundle(FValueBundle&& Other);
		UE_API ~FValueBundle();

		UE_API FValueBundle& operator=(const FValueBundle& Other);
		UE_API FValueBundle& operator=(FValueBundle&& Other);

		// Returns whether or not we are empty
		[[nodiscard]] UE_API bool IsEmpty() const;

		// Returns the named set this collection maps to
		[[nodiscard]] UE_API const FAttributeNamedSetPtr& GetNamedSet() const;

		// Returns the space our values live in
		[[nodiscard]] UE_API FValueSpace GetValueSpace() const;

		// Sets the space our values live in
		// Note that care must be taken when changing the value space as all values within the collection must be migrated accordingly
		UE_API void SetValueSpace(FValueSpace ValueSpace);

		// Sets the space our values line in and default initializes this collection
		// For local/component space, we initialize with the reference pose
		// For additive space, we initialize with the additive identity
		// Other spaces initialize to their identity
		// Note that only bound values are default initialized
		UE_API void InitWithValueSpace(FValueSpace ValueSpace);

		// Returns whether or not the space our values live in is additive
		[[nodiscard]] UE_API bool IsAdditive() const;

		// Resets the collection with the specified named set
		UE_API void Reset(const FAttributeNamedSetPtr& NamedSet);

		// Clears the collection of any content
		UE_API void Empty();

		// Returns the allocator function this collection was initialized with
		[[nodiscard]] UE_API FReallocFun GetAllocator() const;

		// Returns a collection of bound value maps
		[[nodiscard]] UE_API FBoundMapCollection& GetBoundValueMaps();
		[[nodiscard]] UE_API const FBoundMapCollection& GetBoundValueMaps() const;

		// Returns a collection of unbound value maps
		[[nodiscard]] UE_API FUnboundMapCollection& GetUnboundValueMaps();
		[[nodiscard]] UE_API const FUnboundMapCollection& GetUnboundValueMaps() const;

	private:
		// The named set we are based on, every set mapping uses a typed set within it
		FAttributeNamedSetPtr NamedSet;

		// A collection of bound value maps
		FBoundMapCollection BoundMaps;

		// A collection of unbound value maps
		FUnboundMapCollection UnboundMaps;

		// The space in which our values live
		FValueSpace ValueSpace;

		// The indirect allocator function pointer
		FReallocFun ReallocFun = nullptr;
	};

	// A heap allocated value bundle
	class FValueBundleHeap : public FValueBundle
	{
	public:
		// Constructs an empty bundle
		FValueBundleHeap();

		// Constructs an empty bundle based on the specified named set
		explicit FValueBundleHeap(const FAttributeNamedSetPtr& NamedSet);

		using FValueBundle::operator=;
	};

	// A Mem-Stack allocated value bundle
	class FValueBundleStack : public FValueBundle
	{
	public:
		// Constructs an empty bundle
		FValueBundleStack();

		// Constructs an empty bundle based on the specified named set
		explicit FValueBundleStack(const FAttributeNamedSetPtr& NamedSet);

		using FValueBundle::operator=;
	};

	using FValueBundlePtr = TSharedPtr<FValueBundle, ESPMode::ThreadSafe>;
}

/*
 * Value Bundle
 *
 * 
 * This struct holds attributes produced by UAF. They come in two flavors: static (set map) and dynamic (map).
 * Static attributes are fast and defined up front in the skeleton set binding asset.
 * Dynamic attributes are slower but produced on demand during evaluation.
 * This struct can also represent a 'virtual value', for instance an inlined, or owned graph instance 
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Value Bundle"))
struct FUAFValueBundle
{
	GENERATED_BODY()

	// Set as a reference pose (the assumed value of an empty value bundle)
	void SetAsRefPose()
	{
		Impl.Reset();
	}

	// Get whether this is a reference pose
	bool IsRefPose() const
	{
		return !Impl.IsValid();
	}

	// Set as an implementation - @see UE::UAF::IVirtualValueBundle
	template<typename ImplType, typename... ArgsType>
	void SetAs(ArgsType&&... InArgs)
	{
		static_assert(std::is_base_of_v<UE::UAF::IVirtualValueBundle, ImplType>, "Value Bundle Implementation must be a child of UE::UAF::IVirtualValueBundle");
		Impl = MakeShared<ImplType>(Forward<ArgsType>(InArgs)...);
	}

	// Access the implementation
	const UE::UAF::IVirtualValueBundle* GetImpl() const
	{
		return Impl.Get();
	}

	// Access the implementation, asserting if it is invalid/refpose
	UE::UAF::IVirtualValueBundle& GetImplChecked() const
	{
		check(Impl.IsValid());
		return *Impl.Get();
	}

private:
	// Implementation
	TSharedPtr<UE::UAF::IVirtualValueBundle> Impl;
};

//////////////////////////////////////////////////////////////////////////
// Implementation

namespace UE::UAF
{
	inline FValueBundleHeap::FValueBundleHeap()
		: FValueBundle(nullptr, &FAllocatorTypeTrait<FDefaultAllocator>::Realloc)
	{
	}

	inline FValueBundleHeap::FValueBundleHeap(const FAttributeNamedSetPtr& NamedSet)
		: FValueBundle(NamedSet, &FAllocatorTypeTrait<FDefaultAllocator>::Realloc)
	{
	}

	inline FValueBundleStack::FValueBundleStack()
		: FValueBundle(nullptr, &FAllocatorTypeTrait<TMemStackAllocator<>>::Realloc)
	{
	}

	inline FValueBundleStack::FValueBundleStack(const FAttributeNamedSetPtr& NamedSet)
		: FValueBundle(NamedSet, &FAllocatorTypeTrait<TMemStackAllocator<>>::Realloc)
	{
	}
}

#undef UE_API
