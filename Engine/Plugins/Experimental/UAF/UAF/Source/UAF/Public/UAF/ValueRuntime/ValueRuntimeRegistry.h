// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UAF/ValueRuntime/BoundValueMap.h"
#include "UAF/ValueRuntime/Transformers/Transformer.h"
#include "UAF/ValueRuntime/UnboundValueMap.h"

#define UE_API UAF_API

namespace UE::UAF
{
	// A global registry to store value runtime metadata
	class FValueRuntimeRegistry
	{
	public:
		UE_NONCOPYABLE(FValueRuntimeRegistry);	// We disallow copy/move semantics

		using BoundValueMapConstructorFun = TFunction<FBoundValueMap*(const FBoundValueMap::FConstructArgs& Args)>;
		using UnboundValueMapConstructorFun = TFunction<FUnboundValueMap*(FReallocFun ReallocFun)>;

		// Returns the value runtime registry instance
		[[nodiscard]] static UE_API FValueRuntimeRegistry& Get();

		// Constructs a bound value map from its UStruct value type
		[[nodiscard]] UE_API FBoundValueMap* ConstructBoundValueMap(const FBoundValueMap::FConstructArgs& Args) const;

		// Constructs a name/value map from its UStruct value type
		[[nodiscard]] UE_API FUnboundValueMap* ConstructUnboundValueMap(UScriptStruct* ValueType, FReallocFun ReallocFun) const;

		// Returns the map of known value transformers
		[[nodiscard]] UE_API FValueTransformerMapPtr GetTransformerMap() const;

		// Registers a bound value map of the specified value type
		template<class ValueType>
		void RegisterBoundValueMapInitializer();

		// Registers a name/value map of the specified value type
		template<class ValueType>
		void RegisterUnboundValueMapInitializer();

		template<class TransformerSpecializationType>
		bool RegisterValueTransformer();

	private:
		UE_API FValueRuntimeRegistry();
		UE_API void RegisterBoundValueMapInitializer(UScriptStruct* ValueType, BoundValueMapConstructorFun Initializer);
		UE_API void RegisterUnboundValueMapInitializer(UScriptStruct* ValueType, UnboundValueMapConstructorFun Initializer);
		UE_API bool RegisterBouneValueMapTransformer(FName TransformerName, UScriptStruct* ValueType, FRawTransformerFunc TransformerFunc);
		UE_API bool RegisterUnboundValueMapTransformer(FName TransformerName, UScriptStruct* ValueType, FRawTransformerFunc TransformerFunc);

		// Initialize/destroy the global registry
		static UE_API void Init();
		static UE_API void Destroy();

		// A map of ValueType -> Bound Value Map constructors
		// This is used for indirect construction when we have a UScriptStruct value type
		TMap<UScriptStruct*, BoundValueMapConstructorFun> BoundValueMapInitializers;
		mutable FRWLock BoundValueMapInitializerLock;

		// A map of ValueType -> Map constructors
		// This is used for indirect construction when we have a UScriptStruct value type
		TMap<UScriptStruct*, UnboundValueMapConstructorFun> UnboundValueMapInitializers;
		mutable FRWLock UnboundValueMapInitializerLock;

		// A shared pointer to a map of transformer name -> transformer list (a transformer per value type)
		FValueTransformerMapPtr TransformerMap;
		mutable FRWLock TransformerMapLock;

		friend class FAnimNextModuleImpl;
	};

	//////////////////////////////////////////////////////////////////////////
	// Implementation

	template<class ValueType>
	inline void FValueRuntimeRegistry::RegisterBoundValueMapInitializer()
	{
		RegisterBoundValueMapInitializer(ValueType::StaticStruct(), [](const FBoundValueMap::FConstructArgs& Args)
			{
				typename TBoundValueMap<ValueType>::FConstructArgs TypedArgs(Args.TypedSet, Args.ValueType, Args.ReallocFun);

				return MakeBoundValueMap<ValueType>(Args);
			});
	}

	template<class ValueType>
	inline void FValueRuntimeRegistry::RegisterUnboundValueMapInitializer()
	{
		RegisterUnboundValueMapInitializer(ValueType::StaticStruct(), [](FReallocFun ReallocFun)
			{
				return MakeUnboundValueMap<ValueType>(ReallocFun);
			});
	}

	template<class TransformerSpecializationType>
	inline bool FValueRuntimeRegistry::RegisterValueTransformer()
	{
		using TransformerCppType = typename TransformerSpecializationType::TransformerType;
		using ValueType = typename TransformerSpecializationType::ValueType;

		static_assert(std::is_base_of_v<FValueTransformer, TransformerCppType>, "Specified transformer must derive from FValueTransformer and provide the necessary API");

		const FName TransformerName = TransformerCppType::TransformerName;
		UScriptStruct* ValueTypeStruct = ValueType::StaticStruct();

		bool bSuccess = false;

		if constexpr (requires { &TransformerSpecializationType::TransformBoundValueMap; })
		{
			// Signature must match our transformer
			using FTransformBoundValueMapFunc = typename TransformerCppType::FTransformBoundValueMapFunc;
			FTransformBoundValueMapFunc SpecializedTransformFunc = &TransformerSpecializationType::TransformBoundValueMap;

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

			// Cast to a raw dummy function pointer type that we exclusively use for storage
			// The transformer is responsible for casting it back to the original type it specifies
			auto RawSpecializedTransformFunc = reinterpret_cast<FRawTransformerFunc>(SpecializedTransformFunc);

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

			bSuccess |= RegisterBouneValueMapTransformer(TransformerName, ValueTypeStruct, RawSpecializedTransformFunc);
		}

		if constexpr (requires { &TransformerSpecializationType::TransformUnboundValueMap; })
		{
			// Signature must match our transformer
			using FTransformUnboundValueMapFunc = typename TransformerCppType::FTransformUnboundValueMapFunc;
			FTransformUnboundValueMapFunc SpecializedTransformFunc = &TransformerSpecializationType::TransformUnboundValueMap;

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

			// Cast to a raw dummy function pointer type that we exclusively use for storage
			// The transformer is responsible for casting it back to the original type it specifies
			auto RawSpecializedTransformFunc = reinterpret_cast<FRawTransformerFunc>(SpecializedTransformFunc);

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

			bSuccess |= RegisterUnboundValueMapTransformer(TransformerName, ValueTypeStruct, RawSpecializedTransformFunc);
		}

		// Returns true if we registered with at least one transformer type
		return bSuccess;
	}
}

#undef UE_API
