// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API UAF_API

namespace UE::UAF
{
	// Encodes the possible spaces values can live in
	enum class EValueSpaceType : uint16
	{
		// Unknown value space
		None		= 0x0000,

		// Bone transforms: relative to their parent bone
		// Everything else: normal space (no parent/child relationships)
		Local		= 0x0001,

		// Bone transforms: relative to the owning component
		// Everything else: normal space (no parent/child relationships)
		Component	= 0x0002,

		// Bone transforms: rotations/translations/scales can be in individual spaces, @see EMixedSpaceFlags
		//                  Only safe with uniform scale
		// Everything else: normal space (no parent/child relationships)
		Mixed		= 0x0003,

		// Values are in machine learning latent space
		// Value types will dictate which space they are in
		AI			= 0x0004,
	};

	// Encodes the possible permutations of mixed spaces that bone transforms can live in
	enum class EMixedSpaceFlags : uint16
	{
		// No mixed space, equivalent to Local Space
		None			= 0x0000,

		// Rotations are relative to the owning component
		// Only safe with uniform scale
		MeshRotation	= 0x0010,

		// Scales are relative to the owning component
		// Only safe with uniform scale
		MeshScale		= 0x0020,

		// Rotations are relative to the root bone
		// Only safe with uniform scale
		RootRotation	= 0x0040,
	};

	ENUM_CLASS_FLAGS(EMixedSpaceFlags);

	// Encapsulates the possible spaces that values can live in
	class FValueSpace
	{
	public:
		// Constructs an unknown space
		FValueSpace() = default;

		// Constructs with the specified space type and additive flag
		explicit FValueSpace(EValueSpaceType Type, bool bIsAdditive = false);

		// Constructs a mixed space with the specified flags and additive flag
		explicit FValueSpace(EMixedSpaceFlags Flags, bool bIsAdditive = false);

		// Returns the value space type
		[[nodiscard]] EValueSpaceType GetType() const;

		// Sets the value space type, preserving whether we are additive or not
		void SetType(EValueSpaceType Type);

		// Returns the mixed space flags
		[[nodiscard]] EMixedSpaceFlags GetMixedSpaceFlags() const;

		// Sets the value space type to mixed along with the specified flags, preserving whether we are additive or not
		void SetMixedSpaceFlags(EMixedSpaceFlags Flags);

		// Returns whether we are additive or not
		[[nodiscard]] bool IsAdditive() const;

		// Sets whether we are additive or not, preserving the type and mixed flags
		void SetIsAdditive(bool bIsAdditive);

		// Equality operators
		bool operator==(FValueSpace Other) const;
		bool operator!=(FValueSpace Other) const;

		// Converts this value space into a string representation
		[[nodiscard]] UE_API FString ToString() const;

	private:
		static constexpr uint16 SPACE_CATEGORY_MASK		= 0x000F;
		static constexpr uint16 MIXED_SPACE_FLAGS_MASK	= 0x00F0;
		static constexpr uint16 IS_ADDITIVE_MASK		= 0x8000;

		// Our internal value is opaque and packed
		uint16 Packed = static_cast<uint16>(EValueSpaceType::None);
	};

	//////////////////////////////////////////////////////////////////////////
	// Implementation

	inline FValueSpace::FValueSpace(EValueSpaceType Type, bool bIsAdditive)
		: Packed(static_cast<uint16>(Type) | (bIsAdditive ? IS_ADDITIVE_MASK : 0))
	{
	}

	inline FValueSpace::FValueSpace(EMixedSpaceFlags Flags, bool bIsAdditive)
		: Packed(static_cast<uint16>(EValueSpaceType::Mixed) | static_cast<uint16>(Flags) | (bIsAdditive ? IS_ADDITIVE_MASK : 0))
	{
	}

	inline EValueSpaceType FValueSpace::GetType() const
	{
		return static_cast<EValueSpaceType>(Packed & SPACE_CATEGORY_MASK);
	}

	inline void FValueSpace::SetType(EValueSpaceType Type)
	{
		// Only preserve the additive flag
		Packed = (Packed & IS_ADDITIVE_MASK) | static_cast<uint16>(Type);
	}

	inline EMixedSpaceFlags FValueSpace::GetMixedSpaceFlags() const
	{
		return static_cast<EMixedSpaceFlags>(Packed & MIXED_SPACE_FLAGS_MASK);
	}

	inline void FValueSpace::SetMixedSpaceFlags(EMixedSpaceFlags Flags)
	{
		// Only preserve the additive flag and make sure our category is mixed
		Packed = (Packed & IS_ADDITIVE_MASK) | static_cast<uint16>(EValueSpaceType::Mixed) | static_cast<uint16>(Flags);
	}

	inline bool FValueSpace::IsAdditive() const
	{
		return (Packed & IS_ADDITIVE_MASK) != 0;
	}

	inline void FValueSpace::SetIsAdditive(bool bIsAdditive)
	{
		// Preserve everything else
		Packed = (Packed & ~IS_ADDITIVE_MASK) | (bIsAdditive ? IS_ADDITIVE_MASK : 0);
	}

	inline bool FValueSpace::operator==(FValueSpace Other) const
	{
		return Packed == Other.Packed;
	}

	inline bool FValueSpace::operator!=(FValueSpace Other) const
	{
		return Packed != Other.Packed;
	}
}

#undef UE_API
