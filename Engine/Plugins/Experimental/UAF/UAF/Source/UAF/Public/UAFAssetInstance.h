// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFAssetInstanceComponent.h"
#include "Script/UAFScriptComponent.h"
#include "Param/ParamType.h"
#include "StructUtils/PropertyBag.h"
#include "RewindDebugger/UAFTrace.h"
#include "Templates/SubScriptStructOf.h"
#include "Variables/UAFInstanceVariableData.h"

#include "UAFAssetInstance.generated.h"

#define UE_API UAF_API

class UUAFSharedVariables;
struct FAnimNextFunctionReference;
struct FAnimNextModuleInjectionComponent;
struct FUAFAssetInstanceComponent;
struct FUAFScriptComponent;
struct FAnimNextModuleInstance;
struct FAnimNextModuleContextData;

namespace UE::UAF::Tests { struct FUAFBindingTestFixture; }

namespace UE::UAF
{
	struct FPoolHandle;
	struct FInjectionInfo;
	struct FUAFTrace;
	struct FInstanceTaskContext;
	struct FVariableOverridesCollection;
	struct FLayerStackTrait;
}

// Base struct for instances of UAF assets
USTRUCT()
struct FUAFAssetInstance
{
	GENERATED_BODY()

	FUAFAssetInstance() = default;

	UE_API explicit FUAFAssetInstance(UScriptStruct* InScriptStruct);

	// Get the asset that this instance represents
	template<typename AssetType>
	const AssetType* GetAsset() const
	{
		return CastChecked<AssetType>(Asset);
	}

	// Safely get the name of the asset that this host provides
	FName GetAssetName() const
	{
		return Asset ? Asset->GetFName() : NAME_None;
	}

	// Get a variable's value given its name, copying the value to OutResult.
	// If the type does not match exactly then a conversion will be attempted.
	// @param	InVariable			The variable to get the value of
	// @param	OutResult			Result that will be filled if no errors occur
	// @return see EPropertyBagResult
	template<typename ValueType>
	EPropertyBagResult GetVariable(const FAnimNextVariableReference& InVariable, ValueType& OutResult) const
	{
		return Variables.GetVariable(InVariable, FAnimNextParamType::GetType<ValueType>(), TArrayView<uint8>(reinterpret_cast<uint8*>(&OutResult), sizeof(ValueType)));
	}

	FUAFInstanceVariableData& GetVariables() { return Variables;};
	const FUAFInstanceVariableData& GetVariables() const { return Variables;};

	// Access a variable's value given its name.
	// Type must match strictly, no conversions are performed.
	// @param	InVariable			The variable to get the value of
	// @param	InFunction			Function that will be called if no errors occur
	// @return see EPropertyBagResult
	template<typename ValueType>
	EPropertyBagResult AccessVariable(const FAnimNextVariableReference& InVariable, TFunctionRef<void(ValueType&)> InFunction) const
	{
		TArrayView<uint8> Data;
		EPropertyBagResult Result = Variables.AccessVariable(InVariable, FAnimNextParamType::GetType<ValueType>(), Data);
		if (Result == EPropertyBagResult::Success)
		{
			InFunction(*reinterpret_cast<ValueType*>(Data.GetData()));
		}
		return Result;
	}
	
	// Set a variable's value given a reference.
	// If the type does not match exactly then a conversion will be attempted.
	// @param	InVariable			The variable to set the value of
	// @param	InNewValue			The value to set the variable to
	// @return see EPropertyBagResult
	template<typename ValueType>
	EPropertyBagResult SetVariable(const FAnimNextVariableReference& InVariable, const ValueType& InNewValue) const
	{
		return SetVariable(InVariable, FAnimNextParamType::GetType<ValueType>(), TConstArrayView<uint8>(reinterpret_cast<const uint8*>(&InNewValue), sizeof(ValueType)));
	}

	// Get a variable's value given its name, copying the value to OutResult.
	// If the type does not match exactly then a conversion will be attempted.
	// @param	InVariable			The variable to get the value of
	// @param	InType				The type to read as (conversions will be attempted)
	// @param	OutResult			Raw byte view to receive the value
	// @return see EPropertyBagResult
	EPropertyBagResult GetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8> OutResult) const
	{
		return Variables.GetVariable(InVariable, InType, OutResult);
	}

	// Set a variable's value given a reference.
	// If the type does not match exactly then a conversion will be attempted.
	// @param	InVariable			The variable to set the value of
	// @param	InType				The type of the variable
	// @param	InNewValue			The value to set the variable to
	// @return see EPropertyBagResult
	EPropertyBagResult SetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue) const
	{
		return Variables.SetVariable(InVariable, InType, InNewValue);
	}

	// Access the memory of the shared variable struct directly.
	// @param	InFunction			Function called with a reference to the variable's struct 
	// @return true if the variables could be accessed. Variables can exist but be unable to be accessed if user overrides are set
	template<typename StructType>
	bool AccessVariablesStruct(TFunctionRef<void(StructType&)> InFunction) const
	{
		return Variables.AccessVariablesStructInternal(TBaseStructure<StructType>::Get(), [&InFunction](FStructView InStructView)
		{
			InFunction(InStructView.Get<StructType>());
		});
	}

	// Access the memory of the shared variable struct directly.
	// @param	InStruct			The variable struct type to access
	// @param	InFunction			Function called with a reference to the variable's struct 
	// @return true if the variables could be accessed. Variables can exist but be unable to be accessed if user overrides are set
	bool AccessVariablesStruct(const UScriptStruct* InStruct, TFunctionRef<void(FStructView)> InFunction) const
	{
		return Variables.AccessVariablesStructInternal(InStruct, [&InFunction](FStructView InStructView)
		{
			InFunction(InStructView);
		});
	}

	// Access the memory of the shared variable struct directly.
	// @param	InFunction			Function called with a reference to the variable's struct 
	// @return true if the variables could be accessed. Variables can exist but be unable to be accessed if user overrides are set
	void ForEachVariablesStruct(TFunctionRef<void(FStructView)> InFunction) const
	{
		return Variables.ForEachVariablesStructInternal(InFunction);
	}

	// Find references to all variables of the specified type
	// @param	OutVariables		The variable references we found
	template<typename ValueType>
	void GetAllVariablesOfType(TArray<FAnimNextVariableReference>& OutVariables) const
	{
		return Variables.GetAllVariablesOfType(FAnimNextParamType::GetType<ValueType>(), OutVariables);
	}

	// Execute a bound parameterless function and copy the return value into OutReturnValue.
	// Resolves the function across the instance hierarchy by matching the function reference's
	// owning asset, so functions on a parent System asset can be called from a child Graph instance.
	// Returns true on success
	UE_API bool ExecuteParameterlessFunction(const FAnimNextFunctionReference& InFunction, void* OutReturnValue, int32 ReturnValueSize);

	// Returns a typed instance component, creating it lazily the first time it is queried.
	// When querying for structs of a base type, existing derived types will be returned, rather than a new base struct created.
	template<typename ComponentType>
	[[nodiscard]] ComponentType& GetOrAddComponent()
	{
		static_assert(std::is_base_of<FUAFAssetInstanceComponent, ComponentType>::value, "ComponentType type must derive from FUAFAssetInstanceComponent");

		if (FUAFAssetInstanceComponent* ExistingComponent = TryGetComponent(ComponentType::StaticStruct()))
		{
			return static_cast<ComponentType&>(*ExistingComponent);
		}

		TInstancedStruct<ComponentType> NewInstancedStruct = TInstancedStruct<ComponentType>::Make();
		TInstancedStruct<FUAFAssetInstanceComponent>& InstancedStruct = Components.Add_GetRef(MoveTemp(NewInstancedStruct));
		ComponentType& NewComponent = InstancedStruct.GetMutable<ComponentType>();
		FUAFAssetInstanceComponent& BaseComponent = static_cast<FUAFAssetInstanceComponent&>(NewComponent);
		BaseComponent.Instance = this;
		BaseComponent.OnBindToInstance();
		return NewComponent;
	}

	// Returns an instance component of the requested type, creating it lazily the first time it is queried.
	// When querying for structs of a base type, existing derived types will be returned, rather than a new base struct created.
	[[nodiscard]] FUAFAssetInstanceComponent& GetOrAddComponent(TSubScriptStructOf<FUAFAssetInstanceComponent> InScriptStruct)
	{
		check(InScriptStruct.Get() != nullptr);
		if (FUAFAssetInstanceComponent* ExistingComponent = TryGetComponent(InScriptStruct))
		{
			return *ExistingComponent;
		}

		TInstancedStruct<FUAFAssetInstanceComponent> NewInstancedStruct;
		NewInstancedStruct.InitializeAsScriptStruct(InScriptStruct);
		TInstancedStruct<FUAFAssetInstanceComponent>& InstancedStruct = Components.Add_GetRef(MoveTemp(NewInstancedStruct));
		FUAFAssetInstanceComponent& NewComponent = InstancedStruct.GetMutable<FUAFAssetInstanceComponent>();
		NewComponent.Instance = this;
		NewComponent.OnBindToInstance();
		return NewComponent;
	}

	// Returns a typed instance component pointer if found or nullptr otherwise.
	// When querying for structs of a base type, existing derived types will be returned.
	template<typename ComponentType>
	ComponentType* TryGetComponent()
	{
		static_assert(std::is_base_of<FUAFAssetInstanceComponent, ComponentType>::value, "ComponentType type must derive from FUAFAssetInstanceComponent");
		return static_cast<ComponentType*>(TryGetComponent(ComponentType::StaticStruct()));
	}

	// Returns a typed instance component pointer if found or nullptr otherwise (const).
	// When querying for structs of a base type, existing derived types will be returned.
	template<typename ComponentType>
	const ComponentType* TryGetComponent() const
	{
		static_assert(std::is_base_of<FUAFAssetInstanceComponent, ComponentType>::value, "ComponentType type must derive from FUAFAssetInstanceComponent");
		return static_cast<const ComponentType*>(TryGetComponent(ComponentType::StaticStruct()));
	}

	// Returns a typed instance component pointer if found or nullptr otherwise.
	// When querying for structs of a base type, existing derived types will be returned.
	UE_API FUAFAssetInstanceComponent* TryGetComponent(const UScriptStruct* InStruct);

	// Iterates all components, calling the supplied predicate. If the predicate returns false, iteration stops.
	template<typename ComponentType>
	void ForEachComponent(TFunctionRef<bool(ComponentType&)> InFunction)
	{
		static_assert(std::is_base_of<FUAFAssetInstanceComponent, ComponentType>::value, "ComponentType type must derive from FUAFAssetInstanceComponent");

		for (TInstancedStruct<FUAFAssetInstanceComponent>& Pair : Components)
		{
			if (ComponentType* ComponentPtr = Pair.GetMutablePtr<ComponentType>())
			{
				if (!InFunction(*ComponentPtr))
				{
					return;
				}
			}
		}
	}

	// Get the instance (graph, module etc.) that owns/hosts us
	FUAFAssetInstance* GetHost() const
	{
		return HostInstance.Pin().Get();
	}

	// Get the root module instance that owns us. Can be nullptr in some cases, such as automated tests.
	UE_API FAnimNextModuleInstance* GetRootInstance();

	uint64 GetUniqueId() const
	{
#if UAF_TRACE_ENABLED
		return UniqueId;
#else
		return 0;
#endif 
	}

#if DO_CHECK
	// Validate that the supplied property bag is of the same layout as this instance's variables
	UE_API bool LayoutMatches(const FInstancedPropertyBag& InPropertyBag) const;
#endif

	// Get the struct type of this instance 
	const UScriptStruct* GetScriptStruct() const
	{
		return ScriptStruct;
	}

	// Returns this instance as the specified derived type, if our struct type is a child type
	template<typename DerivedType>
	DerivedType* AsPtr()
	{
		static_assert(std::is_base_of_v<FUAFAssetInstance, DerivedType>, "Type must derive from FUAFInstance");
		if(ScriptStruct && ScriptStruct->IsChildOf(DerivedType::StaticStruct()))
		{
			return static_cast<DerivedType*>(this);
		}
		return nullptr;
	}

	// Returns this instance as the specified derived type, if our struct type is a child type
	template<typename DerivedType>
	const DerivedType* AsPtr() const
	{
		static_assert(std::is_base_of_v<FUAFAssetInstance, DerivedType>, "Type must derive from FUAFInstance");
		if(ScriptStruct && ScriptStruct->IsChildOf(DerivedType::StaticStruct()))
		{
			return static_cast<const DerivedType*>(this);
		}
		return nullptr;
	}

	// Returns this instance as the specified derived type. Asserts if our type is not a child type.
	template<typename DerivedType>
	DerivedType& As()
	{
		static_assert(std::is_base_of_v<FUAFAssetInstance, DerivedType>, "Type must derive from FUAFInstance");
		check(ScriptStruct && ScriptStruct->IsChildOf(DerivedType::StaticStruct()));
		return static_cast<DerivedType&>(*this);
	}

	// Returns this instance as the specified derived type. Asserts if our type is not a child type.
	template<typename DerivedType>
	const DerivedType& As() const
	{
		static_assert(std::is_base_of_v<FUAFAssetInstance, DerivedType>, "Type must derive from FUAFInstance");
		check(ScriptStruct && ScriptStruct->IsChildOf(DerivedType::StaticStruct()));
		return static_cast<const DerivedType&>(*this);
	}

	// Sets the debug name for the instance, used for tracing etc.
	void SetDebugName(FName Name)
	{
#if UAF_TRACE_ENABLED
		DebugName = Name;
#endif
	}

protected:
	// Used during initialization. Creates variables or references those of the outer host and applies any overrides
	UE_API void InitializeVariables(const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides = nullptr);

#if WITH_EDITOR
	// Used during compilation, migrate all variables to new bags according to new defaults. Reapplies any cached overrides.
	UE_API void MigrateVariables();
#endif

	// Used on instance initialization to copy default components from the asset
	UE_API void CopyDefaultComponents();

	// Binds all components during initialization (once instance has been set up, variables created etc.)
	UE_API void BindDefaultComponents();

	// Releases any components this instance hosts
	UE_API void ReleaseComponents();

	// Access the variable by index directly.
	void AccessVariablePropertyByIndex(int32 InIndex, TFunctionRef<void(const FProperty*, TArrayView<uint8>)> InFunction)
	{
		return Variables.AccessVariablePropertyByIndex(InIndex, InFunction);
	}

protected:
	// Array of components
	UPROPERTY(Transient)
	TArray<TInstancedStruct<FUAFAssetInstanceComponent>> Components;

	// Container for all variables and overrides
	UPROPERTY(Transient)
	FUAFInstanceVariableData Variables;

	// Hard reference to the asset used to create this instance to ensure we can release it safely
	UPROPERTY(Transient)
	TObjectPtr<const UObject> Asset;

	// The struct type of this instance
	UPROPERTY(Transient)
	TObjectPtr<const UScriptStruct> ScriptStruct;

	// The instance (graph, module etc.) that owns/hosts us
	TWeakPtr<FUAFAssetInstance> HostInstance;

	friend UE::UAF::FInjectionInfo;
	friend FAnimNextModuleInjectionComponent;
	friend FAnimNextModuleContextData;
	friend FUAFInstanceVariableData;
	friend UE::UAF::FUAFTrace;
	friend UE::UAF::FInstanceTaskContext;
	friend FUAFRigVMComponent;
	friend UE::UAF::FLayerStackTrait;
	friend struct UE::UAF::Tests::FUAFBindingTestFixture;

	FName GetDebugName() const
	{
#if UAF_TRACE_ENABLED
		if (!DebugName.IsNone())
		{
			return DebugName;
		}
#endif
		return GetAssetName();
	}

#if UAF_TRACE_ENABLED
	FName DebugName;
	uint64 UniqueId;
	UE_API volatile static int64 NextUniqueId;
#endif 
};

#undef UE_API