// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFAssetInstanceComponent.generated.h"

struct FUAFAssetInstance;
struct FAnimNextGraphInstance;
struct FAnimNextModuleInstance;
class UUAFAnimGraph;

#define UE_API UAF_API

// Macro used to provide basic RTTI via UScriptStruct for asset instance components
#define DECLARE_UAF_ASSET_INSTANCE_COMPONENT() \
	virtual UScriptStruct* GetScriptStruct() const override { return StaticStruct(); } \

/** An asset instance component is attached and owned by an asset instance. */
USTRUCT(meta=(Hidden, Abstract))
struct FUAFAssetInstanceComponent
{
	GENERATED_BODY()

	using ContainerType = FUAFAssetInstance;

	FUAFAssetInstanceComponent() = default;

	virtual ~FUAFAssetInstanceComponent() = default;

	// Returns the owning asset instance this component lives on
	// Can return nullptr if an owning asset instance has not been created (such as a default component on an asset)
	FUAFAssetInstance* GetAssetInstancePtr()
	{
		return Instance;
	}

	// Returns the owning asset instance this component lives on
	// Will assert if an owning asset instance has not been created (such as a default component on an asset)
	FUAFAssetInstance& GetAssetInstance()
	{
		check(Instance != nullptr);
		return *Instance;
	}

	// Returns the owning asset instance this component lives on
	// Will assert if an owning asset instance has not been created (such as a default component on an asset)
	const FUAFAssetInstance& GetAssetInstance() const
	{
		check(Instance != nullptr);
		return *Instance;
	}

	// Get the UScriptStruct of this component, must be overriden in derived structs (use DECLARE_UAF_ASSET_INSTANCE_COMPONENT)
	virtual const UScriptStruct* GetScriptStruct() const PURE_VIRTUAL(FUAFAssetInstanceComponent::GetScriptStruct, checkNoEntry(); return nullptr; )

private:
	// Override point for derived components to hook into instance-binding.
	// Called after the instance back-ptr is set:
	// - When an instance first copies the default components from an asset
	// - When an instance lazily creates a component on the fly
	virtual void OnBindToInstance() {}

protected:
	// The owning asset instance this component lives on
	FUAFAssetInstance* Instance = nullptr;

	friend FUAFAssetInstance;
	friend FAnimNextGraphInstance;
	friend FAnimNextModuleInstance;
	friend UUAFAnimGraph;
};

template<>
struct TStructOpsTypeTraits<FUAFAssetInstanceComponent> : public TStructOpsTypeTraitsBase2<FUAFAssetInstanceComponent>
{
	enum
	{
		WithPureVirtual = true,
	};
};

#undef UE_API