// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextSharedVariables.generated.h"

#define UE_API UAF_API

class UUAFSharedVariablesFactory;
class UUAFSharedVariables;

namespace UE::UAF
{
struct FInjectionInfo;
}

namespace UE::UAF::UncookedOnly
{
struct FUtils;
}

// Shared variables are just bundles of variables and their default values
UCLASS(MinimalAPI, BlueprintType)
class UUAFSharedVariables : public UUAFRigVMAsset
{
	GENERATED_BODY()

public:
	UE_API UUAFSharedVariables(const FObjectInitializer& ObjectInitializer);

private:
	friend class UUAFSharedVariablesFactory;
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend UE::UAF::FInjectionInfo;
};

#undef UE_API
