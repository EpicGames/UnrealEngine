// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextVariableReference.h"
#include "UObject/SoftObjectPath.h"
#include "Param/ParamType.h"

class UUAFRigVMAsset;
struct FAnimNextVariableReference;

// A soft reference to an AnimNext variable
struct FAnimNextSoftVariableReference
{
	FAnimNextSoftVariableReference() = default;

	// Construct a soft reference from a FName
	static FAnimNextSoftVariableReference FromName(FName InName, const FSoftObjectPath& InSoftObjectPath)
	{
		return FAnimNextSoftVariableReference(InName, InSoftObjectPath, TOptional<FAnimNextParamType>());
	}

#if WITH_EDITOR
	static FAnimNextSoftVariableReference FromNameAndType(FName InName, const FSoftObjectPath& InSoftObjectPath, FAnimNextParamType InType)
	{
		return FAnimNextSoftVariableReference(InName, InSoftObjectPath, InType);
	}
#endif // WITH_EDITOR

	explicit FAnimNextSoftVariableReference(const FAnimNextVariableReference& VariableReference) : Name(VariableReference.GetName()), SoftObjectPath(VariableReference.GetObject()) 
	{
		ensureAlwaysMsgf(!UE::UAF::IsInternalVariableName(Name), TEXT("Attempting to create soft variable reference using a variable name indicative of a programmatic variable %s"), *Name.ToString());
	}

	// Get the name of the variable
	FName GetName() const
	{
		return Name;
	}

	// Get the asset or struct that the variable reference is contained in
	const FSoftObjectPath& GetSoftObjectPath() const
	{
		return SoftObjectPath;
	}

	// Check whether this can ever refer to a valid variable
	bool IsNone() const
	{
		return Name.IsNone() || SoftObjectPath.IsNull();
	}

	// Set this reference to None
	void Reset()
	{
		Name = NAME_None; 
		SoftObjectPath.Reset();
		OptionalType.Reset();
	}
	
	// Whether parameter type was set
	bool HasType() const
	{
		return OptionalType.IsSet();
	}
	
	// Return reference parameter type, only when set otherwise will assert (use HasType for checking validity)
	FAnimNextParamType GetType() const
	{
		return OptionalType.GetValue();
	}

	bool operator==(const FAnimNextSoftVariableReference& InOther) const
	{
		return Name == InOther.Name && SoftObjectPath == InOther.SoftObjectPath && OptionalType == InOther.OptionalType;
	}

	bool operator!=(const FAnimNextSoftVariableReference& InOther) const
	{
		return !(*this == InOther);
	}

	friend uint32 GetTypeHash(const FAnimNextSoftVariableReference& InKey)
	{
		return HashCombineFast(GetTypeHash(InKey.Name), GetTypeHash(InKey.SoftObjectPath), GetTypeHash(InKey.OptionalType));
	}
private:
	FAnimNextSoftVariableReference(FName InName, const FSoftObjectPath& InSoftObjectPath, const TOptional<FAnimNextParamType> InType = TOptional<FAnimNextParamType>())
		   : Name(InName)
		   , SoftObjectPath(InSoftObjectPath)
		   , OptionalType(InType)
	{}

private:
	// The name of the variable
	FName Name;

	// The asset or struct that the variable reference is contained in
	FSoftObjectPath SoftObjectPath;
	
	// (Expected) type when used for searching references
	TOptional<FAnimNextParamType> OptionalType;
};

