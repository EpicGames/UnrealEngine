// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

#include "AnimNextVariableReference.generated.h"

class UUAFRigVMAsset;
struct FAnimNextSoftVariableReference;

namespace UE::UAF::Tests
{
class FVariableReferenceTest;
}

namespace UE::UAF
{
static const FString InternalVariablePrefix(TEXT("__InternalVar_"));
// Whether the variable is a function internal one
static bool IsInternalVariableName(FName InVariableName) { return InVariableName.ToString().StartsWith(InternalVariablePrefix); }
}

// A reference to a specific variable (UAnimNextVariableEntry) on UUAFRigVMAsset derived class or FProperty on UScriptStruct (NativeSharedVariables).
USTRUCT(BlueprintType)
struct FAnimNextVariableReference
{
	GENERATED_BODY()

	FAnimNextVariableReference() = default;

	// Construct a reference from an existing soft reference. NOTE: can load the object
	//UAF_API static FAnimNextVariableReference FromSoftReference(const FAnimNextSoftVariableReference& InSoftReference);

	// Construct a reference to a struct variable.
	UAF_API static FAnimNextVariableReference FromProperty(const FProperty* InProperty, const UScriptStruct* InStruct);

	// Construct a reference to an asset/struct variable using its name.
	UAF_API static FAnimNextVariableReference FromName(FName InName, const UObject* InObject);
	
#if WITH_EDITOR
	// Construct a reference to an asset/struct variable using its guid and name. (Editor only)
	UAF_API static FAnimNextVariableReference FromNameAndGuid(FName InName, FGuid InGuid, const UObject* InObject);
	
	UAF_API bool Serialize(FArchive& Ar);
	UAF_API void PostSerialize(const FArchive& Ar);
#endif // WITH_EDITOR

	UAF_API bool Identical(const FAnimNextVariableReference* Other, uint32 PortFlags) const;
	
	// Legacy constructor from an FName
	UE_DEPRECATED(5.6, "Constructor should no longer be used, please use constructor that takes an asset or a struct")
	explicit FAnimNextVariableReference(FName InName)
		: Name(InName)
	{
	}

	UE_DEPRECATED(5.8, "FAnimNextVariableReference constructor will become private please use FAnimNextVariableReference::From* APIs instead")
	UAF_API explicit FAnimNextVariableReference(FName InName, const UUAFRigVMAsset* InAsset);

	UE_DEPRECATED(5.8, "FAnimNextVariableReference constructor will become private please use FAnimNextVariableReference::From* APIs instead")
	UAF_API explicit FAnimNextVariableReference(FName InName, const UScriptStruct* InStruct);

	UAF_API explicit FAnimNextVariableReference(const FAnimNextSoftVariableReference& InSoftReference);

	// Get the name of the variable
	FName GetName() const
	{
		return Name;
	}

	// Get the asset or struct that the variable reference is contained in
	TObjectPtr<const UObject> GetObject() const
	{
		return Object;
	}

	// Check whether this can ever refer to a valid variable
	bool IsNone() const
	{
		// TODO: Once we deprecate the name-based lookup path, we can expand this to check the object ptr too
		return Name.IsNone();
	}

	// Set this reference to None
	void Reset()
	{
		Name = NAME_None;
		Object = nullptr;
		CachedProperty = nullptr;
#if WITH_EDITORONLY_DATA 
		CachedGuid.Invalidate();		
#endif // WITH_EDITORONLY_DATA
	}

	// Check whether refers to a valid variable
	UAF_API bool IsValid() const;

	bool operator==(const FAnimNextVariableReference& InOther) const
	{
		// Exclude FGuid value as its purpose is editor-time validation, and will be filtered out of non-editor serialization
		return Name == InOther.Name && Object == InOther.Object;
	}

	bool operator!=(const FAnimNextVariableReference& InOther) const
	{
		return !(*this == InOther);
	}

	friend uint32 GetTypeHash(const FAnimNextVariableReference& InKey)
	{
		// Exclude FGuid value as its purpose is editor-time validation, and will be filtered out of non-editor serialization
		return HashCombineFast(GetTypeHash(InKey.Name), GetTypeHash(InKey.Object));
	}

	// Get the property associated with this variable. Returns null for invalid variables.
	UAF_API const FProperty* ResolveProperty() const;

private:
	FAnimNextVariableReference(FName InName, FGuid InGuid, const UObject* InObject);

#if WITH_EDITOR
	static UAF_API void ValidateVariableNameAndGuid(FName& InOutName, FGuid& InOutGuid, const UObject* InSourceObject, const FString& OwnerName);
#endif // WITH_EDITOR

private:
	// The name of the variable
	UPROPERTY(EditAnywhere, Category = Variable)
	FName Name;

	// The asset or struct that the variable reference is contained in
	// Note: Only deprecated paths allow this to be empty, so all variables in a context will be searched
	UPROPERTY(EditAnywhere, Category = Variable)
	TObjectPtr<const UObject> Object;

	// Cached property used for resolving in GetProperty
	mutable TFieldPath<const FProperty> CachedProperty;

#if WITH_EDITORONLY_DATA
	// GUID of the variable
	UPROPERTY(VisibleAnywhere, Category = Variable)
	FGuid CachedGuid;
#endif // WITH_EDITORONLY_DATA

	friend class UE::UAF::Tests::FVariableReferenceTest;
};

template<>
struct TStructOpsTypeTraits<FAnimNextVariableReference> : public TStructOpsTypeTraitsBase2<FAnimNextVariableReference>
{
#if WITH_EDITOR
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
		WithIdentical = true,
	};
#endif
};
