// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextVariableReference.h"
#include "StructUtils/PropertyBag.h"
#include "UAFInstanceVariableData.generated.h"

struct FRigVMExternalVariableDef;
class UUAFRigVMAsset;
class UUAFSharedVariables;
struct FUAFAssetInstance;
struct FAnimNextModuleInstance;
struct FAnimNextGraphInstance;
struct FAnimNextParamType;
struct FUAFInstanceVariableData;
struct FUAFInstanceVariableContainerProxy;
struct FUAFInstanceVariableDataProxy;
struct FRigVMExternalVariableRuntimeData;
struct FAnimNextRigVMAssetStructData;
struct FUAFInstanceVariableContainer;
struct FUAFRigVMComponent;
class IRigVMRuntimeAssetInterface;

namespace UE::UAF::Tests { struct FUAFBindingTestFixture; }

namespace UE::UAF
{
template<typename TSharedData> struct TControlRigTraitBase;
struct FVariableOverrides;
	struct FVariableOverridesCollection;
	struct FUAFTrace;
	struct FInstanceTaskContext;
}


USTRUCT()
struct FUAFInstanceVariableData
{
	GENERATED_BODY()

	FUAFInstanceVariableData() = default;

	// GC support
	void AddStructReferencedObjects(FReferenceCollector& Collector);

private:
	friend FUAFAssetInstance;
	friend FAnimNextModuleInstance;
	friend FAnimNextGraphInstance;
	friend FUAFInstanceVariableDataProxy;
	friend UE::UAF::FUAFTrace;
	friend UE::UAF::FInstanceTaskContext;
	friend FUAFRigVMComponent;
	template<typename T> friend struct UE::UAF::TControlRigTraitBase;
	friend struct UE::UAF::Tests::FUAFBindingTestFixture;

	// Initialize from a host's asset and its references
	void Initialize(const FUAFAssetInstance& InHostInstance, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides);

	void AddVariablesContainerForAsset(const UUAFRigVMAsset* InAsset, const FUAFAssetInstance& InInstance, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides);
	void AddVariablesContainerForStruct(const UScriptStruct* InStruct, const FUAFAssetInstance& InInstance, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides);
	void AddVariablesContainerForRigVMAsset(const TScriptInterface<const IRigVMRuntimeAssetInterface>& InRigVMAsset, const FUAFAssetInstance& InInstance, const TSharedPtr<const UE::UAF::FVariableOverridesCollection>& InOverrides);

	// Reset to the default state
	void Reset();

	// Helper function used during initialization & migration
	void RebuildNameMaps();

#if WITH_EDITOR
	// Used during compilation, migrate all variables to new bags according to new defaults.
	void Migrate(const FUAFAssetInstance& InHostInstance);
#endif

	// Resolve any VariableOverrides to FUAFInstanceVariableContainer::ResolvedOverrides
	void ResolveOverrides();

	// Gets the value of the specified variable. If the type does not match exactly then a conversion will be attempted.
	UAF_API EPropertyBagResult GetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8> OutResult) const;

	// Access the memory of the variable directly. Type must match strictly, no conversions are performed.
	UAF_API EPropertyBagResult AccessVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TArrayView<uint8>& OutResult) const;

	// Sets the value of the specified variable. If the type does not match exactly then a conversion will be attempted.
	UAF_API EPropertyBagResult SetVariable(const FAnimNextVariableReference& InVariable, const FAnimNextParamType& InType, TConstArrayView<uint8> InNewValue) const;

	// Access the memory of the shared variable struct directly.
	template<typename StructType>
	bool AccessVariablesStruct(TFunctionRef<void(StructType&)> InFunction) const
	{
		return AccessVariablesStructInternal(TBaseStructure<StructType>::Get(), [&InFunction](FStructView InStructView)
		{
			InFunction(InStructView.Get<StructType>());
		});
	}

	// Access the memory of the shared variable container directly.
	// @return false if the struct was not present in our set of variable containers
	UAF_API bool AccessVariablesStructInternal(const UScriptStruct* InStruct, TFunctionRef<void(FStructView)> InFunction) const;

	// Access the memory of each shared variable struct directly.
	UAF_API void ForEachVariablesStructInternal(TFunctionRef<void(FStructView)> InFunction) const;

	// Find references to all variables of the specified type
	UAF_API void GetAllVariablesOfType(const FAnimNextParamType& InType, TArray<FAnimNextVariableReference>& OutVariables) const;

	// Iterate over all variables calling InFunction
	UAF_API void ForEachVariable(TFunctionRef<bool(FName, const FAnimNextParamType&, int32)> InFunction) const;

	// Access the variable by index directly.
	UAF_API void AccessVariablePropertyByIndex(int32 InIndex, TFunctionRef<void(const FProperty*, TArrayView<uint8>)> InFunction);

	// Access a variable via a FAnimNextVariableReference
	UAF_API EPropertyBagResult AccessVariableProperty(const FAnimNextVariableReference& InVariable, TFunctionRef<void(const FProperty*, TArrayView<uint8>)> InFunction);

	// Variable containers owned by the containing instance
	TArray<TSharedRef<FUAFInstanceVariableContainer>> OwnedVariableContainers;

	// User variables used to operate the graph/module (i.e. does not include variables overriden at this level in the instance hierarchy) 
	TArray<TWeakPtr<FUAFInstanceVariableContainer>> InternalSharedVariableContainers;

	// All shared variables at this scope in the instance hierarchy
	TArray<TWeakPtr<FUAFInstanceVariableContainer>> AllSharedVariableContainers;

	// Map from asset/struct to index in SharedVariableSets
	UPROPERTY(Transient)
	TMap<TObjectPtr<const UObject>, int32> AllSharedVariableContainersMap;

	struct FVariableMapping
	{
		FVariableMapping(uint16 InAllSharedVariableContainersIndex, uint16 InVariableIndex)
			: AllSharedVariableContainersIndex(InAllSharedVariableContainersIndex)
			, VariableIndex(InVariableIndex)
		{}

		// AllSharedVariableContainers index
		uint16 AllSharedVariableContainersIndex = 0;

		// Index into the FUAFInstanceVariableContainer property bag/struct/overrides
		uint16 VariableIndex = 0;
	};

	// Helper function for variable access
	const FVariableMapping* FindMapping(const FAnimNextVariableReference& InVariable) const;

	// Legacy name-based mappings of variables
	TMap<FName, FVariableMapping> LegacyVariableNameMap;

	// Map of variable ref -> FVariableMapping
	TMap<FAnimNextVariableReference, FVariableMapping> VariablesMap;

	// Total count of all variables
	int32 NumAllVariables = 0;

	// Total count of all variables used to operate the graph/module
	int32 NumInternalVariables = 0;

#if WITH_EDITOR
	// Whether Initialize() has been called at least once
	bool bHasBeenInitialized = false;
#endif
};

template<>
struct TStructOpsTypeTraits<FUAFInstanceVariableData> : public TStructOpsTypeTraitsBase2<FUAFInstanceVariableData>
{
	enum
	{
		WithAddStructReferencedObjects = true,
	};
};
