// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAF : ModuleRules
	{
		public UAF(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"RigVM",
					"Engine",
					"RenderCore",
					"HierarchyTableRuntime",
					"HierarchyTableAnimationRuntime",
					"RewindDebuggerRuntimeInterface",
					"PropertyBindingUtils",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"UniversalObjectLocator",
					"TraceLog",
				}
			);

			if(Target.bWithLiveCoding)
			{
				PrivateDependencyModuleNames.Add("LiveCoding");
			}

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"RigVMDeveloper",
						"ToolMenus",
						"Slate",
						"SlateCore",
						"UnrealEd"
					}
				);
				
				PrivateIncludePathModuleNames.AddRange(
					new string[]
					{
						"UAFUncookedOnly",
					}
				);

				DynamicallyLoadedModuleNames.AddRange(
					new string[]
					{
						"UAFUncookedOnly"
					}
				);
			}
		}
	}
}
