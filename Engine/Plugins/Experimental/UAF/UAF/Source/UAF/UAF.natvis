<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!--
  *
  * UAF Visualizers
  *
  -->

  <Type Name="UE::UAF::FConstExprUID">
    <DisplayString Condition="UID == 0">Invalid</DisplayString>
    <DisplayString Condition="UID != 0 &amp;&amp; DebugName == nullptr">UID={UID,x}</DisplayString>
    <DisplayString Condition="UID != 0 &amp;&amp; DebugName != nullptr">UID={UID,x}, Name={DebugName,su}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <!-- Attribute visualizers -->

  <Type Name="UE::UAF::FAttributeDescription">
    <DisplayString Condition="ParentName.ComparisonIndex.Value == 0">Name={Name}, Type={Type->NamePrivate}, LOD={LOD}</DisplayString>
    <DisplayString>Name={Name}, ParentName={ParentName}, Type={Type->NamePrivate}, LOD={LOD}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="UE::UAF::FAttributeSetKey">
    <DisplayString Condition="Value == 0">Invalid</DisplayString>
    <DisplayString Condition="((Value >> kParentIndexShift) &amp; kIndexMask) == 0">BindingIndex={(Value &amp; kIndexMask) - 1}, LOD={(Value >> kLODShift) &amp; kLODMask}</DisplayString>
    <DisplayString>BindingIndex={(Value &amp; kIndexMask) - 1}, ParentBindingIndex={((Value >> kParentIndexShift) &amp; kIndexMask) - 1}, LOD={(Value >> kLODShift) &amp; kLODMask}</DisplayString>
    <Expand HideRawView="true">
      <Item Name="BindingIndex">(int32)((Value &amp; kIndexMask) - 1)</Item>
      <Item Name="ParentBindingIndex">(int32)(((Value >> kParentIndexShift) &amp; kIndexMask) - 1)</Item>
      <Item Name="LOD">(int32)((Value >> kLODShift) &amp; kLODMask)</Item>
      <Item Name="RawValue">Value,x</Item>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FAttributeBindingIndex">
    <DisplayString Condition="BoneIndex == -1">Invalid</DisplayString>
    <DisplayString>{BoneIndex}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="UE::UAF::FAttributeSetIndex">
    <DisplayString Condition="BoneIndex == -1">Invalid</DisplayString>
    <DisplayString>{BoneIndex}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="UE::UAF::FAttributeType">
    <DisplayString>{UID}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="FFloatAnimationAttribute">
    <DisplayString>{Value}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="FIntegerAnimationAttribute">
    <DisplayString>{Value}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="FStringAnimationAttribute">
    <DisplayString>{Value}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="FTransformAnimationAttribute">
    <DisplayString>{{Translation={{X={Value.Translation.XY.m128d_f64[0]} Y={Value.Translation.XY.m128d_f64[1]} Z={Value.Translation.ZW.m128d_f64[0]}}} Rotation={{X={Value.Rotation.XY.m128d_f64[0]} Y={Value.Rotation.XY.m128d_f64[1]} Z={Value.Rotation.ZW.m128d_f64[0]} W={Value.Rotation.ZW.m128d_f64[1]}}}}}</DisplayString>
    <Expand HideRawView="true">
      <Synthetic Name="Translation">
        <DisplayString>{{X={Value.Translation.XY.m128d_f64[0]} Y={Value.Translation.XY.m128d_f64[1]} Z={Value.Translation.ZW.m128d_f64[0]}}}</DisplayString>
        <Expand>
          <Item Name="X">Value.Translation.XY.m128d_f64[0]</Item>
          <Item Name="Y">Value.Translation.XY.m128d_f64[1]</Item>
          <Item Name="Z">Value.Translation.ZW.m128d_f64[0]</Item>
          <Item Name="||V||&#xB2;">Value.Translation.XY.m128d_f64[0]*Value.Translation.XY.m128d_f64[0] + Value.Translation.XY.m128d_f64[1]*Value.Translation.XY.m128d_f64[1] + Value.Translation.ZW.m128d_f64[0]*Value.Translation.ZW.m128d_f64[0]</Item>
        </Expand>
      </Synthetic>
      <Synthetic Name="Rotation">
        <DisplayString>{{X={Value.Rotation.XY.m128d_f64[0]} Y={Value.Rotation.XY.m128d_f64[1]} Z={Value.Rotation.ZW.m128d_f64[0]} W={Value.Rotation.ZW.m128d_f64[1]}}}</DisplayString>
        <Expand>
          <Item Name="X">Value.Rotation.XY.m128d_f64[0]</Item>
          <Item Name="Y">Value.Rotation.XY.m128d_f64[1]</Item>
          <Item Name="Z">Value.Rotation.ZW.m128d_f64[0]</Item>
          <Item Name="W">Value.Rotation.ZW.m128d_f64[1]</Item>
          <Item Name="||Q||&#xB2;">Value.Rotation.XY.m128d_f64[0]*Value.Rotation.XY.m128d_f64[0] + Value.Rotation.XY.m128d_f64[1]*Value.Rotation.XY.m128d_f64[1] + Value.Rotation.ZW.m128d_f64[0]*Value.Rotation.ZW.m128d_f64[0] + Value.Rotation.ZW.m128d_f64[1]*Value.Rotation.ZW.m128d_f64[1]</Item>
        </Expand>
      </Synthetic>
      <Synthetic Name="Scale3D">
        <DisplayString>{{X={Value.Scale3D.XY.m128d_f64[0]} Y={Value.Scale3D.XY.m128d_f64[1]} Z={Value.Scale3D.ZW.m128d_f64[0]}}}</DisplayString>
        <Expand>
          <Item Name="X">Value.Scale3D.XY.m128d_f64[0]</Item>
          <Item Name="Y">Value.Scale3D.XY.m128d_f64[1]</Item>
          <Item Name="Z">Value.Scale3D.ZW.m128d_f64[0]</Item>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <Type Name="FBoneTransformAnimationAttribute">
    <DisplayString>{{Translation={{X={Value.Translation.XY.m128d_f64[0]} Y={Value.Translation.XY.m128d_f64[1]} Z={Value.Translation.ZW.m128d_f64[0]}}} Rotation={{X={Value.Rotation.XY.m128d_f64[0]} Y={Value.Rotation.XY.m128d_f64[1]} Z={Value.Rotation.ZW.m128d_f64[0]} W={Value.Rotation.ZW.m128d_f64[1]}}}}}</DisplayString>
    <Expand HideRawView="true">
      <Synthetic Name="Translation">
        <DisplayString>{{X={Value.Translation.XY.m128d_f64[0]} Y={Value.Translation.XY.m128d_f64[1]} Z={Value.Translation.ZW.m128d_f64[0]}}}</DisplayString>
        <Expand>
          <Item Name="X">Value.Translation.XY.m128d_f64[0]</Item>
          <Item Name="Y">Value.Translation.XY.m128d_f64[1]</Item>
          <Item Name="Z">Value.Translation.ZW.m128d_f64[0]</Item>
          <Item Name="||V||&#xB2;">Value.Translation.XY.m128d_f64[0]*Value.Translation.XY.m128d_f64[0] + Value.Translation.XY.m128d_f64[1]*Value.Translation.XY.m128d_f64[1] + Value.Translation.ZW.m128d_f64[0]*Value.Translation.ZW.m128d_f64[0]</Item>
        </Expand>
      </Synthetic>
      <Synthetic Name="Rotation">
        <DisplayString>{{X={Value.Rotation.XY.m128d_f64[0]} Y={Value.Rotation.XY.m128d_f64[1]} Z={Value.Rotation.ZW.m128d_f64[0]} W={Value.Rotation.ZW.m128d_f64[1]}}}</DisplayString>
        <Expand>
          <Item Name="X">Value.Rotation.XY.m128d_f64[0]</Item>
          <Item Name="Y">Value.Rotation.XY.m128d_f64[1]</Item>
          <Item Name="Z">Value.Rotation.ZW.m128d_f64[0]</Item>
          <Item Name="W">Value.Rotation.ZW.m128d_f64[1]</Item>
          <Item Name="||Q||&#xB2;">Value.Rotation.XY.m128d_f64[0]*Value.Rotation.XY.m128d_f64[0] + Value.Rotation.XY.m128d_f64[1]*Value.Rotation.XY.m128d_f64[1] + Value.Rotation.ZW.m128d_f64[0]*Value.Rotation.ZW.m128d_f64[0] + Value.Rotation.ZW.m128d_f64[1]*Value.Rotation.ZW.m128d_f64[1]</Item>
        </Expand>
      </Synthetic>
      <Synthetic Name="Scale3D">
        <DisplayString>{{X={Value.Scale3D.XY.m128d_f64[0]} Y={Value.Scale3D.XY.m128d_f64[1]} Z={Value.Scale3D.ZW.m128d_f64[0]}}}</DisplayString>
        <Expand>
          <Item Name="X">Value.Scale3D.XY.m128d_f64[0]</Item>
          <Item Name="Y">Value.Scale3D.XY.m128d_f64[1]</Item>
          <Item Name="Z">Value.Scale3D.ZW.m128d_f64[0]</Item>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FBoundMapCollection">
    <Intrinsic Name="_GetMapsData" Expression="(UnrealEditor-UAF!UE::UAF::FBoundMapCollection::FMapEntry*)Maps"/>
    <DisplayString Condition="MapCount == 0">Empty</DisplayString>
    <DisplayString>SetName={NamedSet.Reference->Name}, Num={MapCount}</DisplayString>
    <Expand>
      <Item Name="NamedSet">NamedSet</Item>
      <CustomListItems>
        <Variable Name="Index" InitialValue="0" />
        <Size>MapCount</Size>
        <Loop Condition="MapCount > Index">
          <Item Name="[{(*this)._GetMapsData()[Index].Key.AttributeType->NamePrivate}]">*(*this)._GetMapsData()[Index].Map</Item>
          <Exec>++Index</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FUnboundMapCollection">
    <Intrinsic Name="_GetMapsData" Expression="(UnrealEditor-UAF!UE::UAF::FUnboundMapCollection::FMapEntry*)Maps"/>
    <DisplayString Condition="MapCount == 0">Empty</DisplayString>
    <DisplayString>Num={MapCount}</DisplayString>
    <Expand>
      <CustomListItems>
        <Variable Name="Index" InitialValue="0" />
        <Size>MapCount</Size>
        <Loop Condition="MapCount > Index">
          <Item Name="[{(*this)._GetMapsData()[Index].ValueType->NamePrivate}]">*(*this)._GetMapsData()[Index].Map</Item>
          <Exec>++Index</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FAttributeMappingKey">
    <DisplayString Condition="AttributeType == nullptr &amp;&amp; ValueType == nullptr">Invalid</DisplayString>
    <DisplayString Condition="AttributeType == 0x7fffffffffffffff">* -> {ValueType->NamePrivate}</DisplayString>
    <DisplayString>{AttributeType->NamePrivate} -> {ValueType->NamePrivate}</DisplayString>
    <Expand HideRawView="true">
    </Expand>
  </Type>

  <Type Name="UE::UAF::FBoundValueMap">
    <DisplayString Condition="NumValues == 0">Empty</DisplayString>
    <DisplayString>Num={NumValues}, Type={ValueTypeStruct->NamePrivate}</DisplayString>
    <Expand>
      <Item Name="TypedSet">TypedSet</Item>
      <Item Name="ValueType">ValueTypeStruct->NamePrivate</Item>
      <Item Name="Num">NumValues</Item>
    </Expand>
  </Type>

  <Type Name="UE::UAF::TBoundValueMap&lt;*&gt;">
    <DisplayString Condition="NumValues == 0">Empty</DisplayString>
    <DisplayString>Num={NumValues}</DisplayString>
    <Expand>
      <Item Name="TypedSet">TypedSet</Item>
      <Item Name="ValueType">ValueTypeStruct->NamePrivate</Item>
      <Item Name="Num">NumValues</Item>
      <ArrayItems>
        <Size>NumValues</Size>
        <ValuePointer>($T1*)ValuesPtr</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FUnboundValueMap">
    <DisplayString Condition="NumValues == 0">Empty</DisplayString>
    <DisplayString>Num={NumValues}, Type={ValueTypeStruct->NamePrivate}</DisplayString>
    <Expand>
      <Item Name="ValueType">ValueTypeStruct->NamePrivate</Item>
      <Item Name="Num">NumValues</Item>
      <Item Name="Max">MaxValues</Item>
    </Expand>
  </Type>

  <Type Name="UE::UAF::TUnboundValueMap&lt;*&gt;">
    <Intrinsic Name="_GetValueAlignment" Expression="sizeof($T1) >= 16 ? 16 : 8"/>
    <Intrinsic Name="_GetNames" Expression="(FName*)DataPtr"/>
    <Intrinsic Name="_GetValues" Expression="($T1*)((intptr_t)(DataPtr + MaxValues * sizeof(FName) + (*this)._GetValueAlignment() - 1) &amp; ~((*this)._GetValueAlignment() - 1))"/>
    <DisplayString Condition="NumValues == 0">Empty</DisplayString>
    <DisplayString>Num={NumValues}</DisplayString>
    <Expand>
      <Item Name="ValueType">ValueTypeStruct->NamePrivate</Item>
      <Item Name="Num">NumValues</Item>
      <Item Name="Max">MaxValues</Item>
      <CustomListItems>
        <Variable Name="Index" InitialValue="0" />
        <Size>NumValues</Size>
        <Loop Condition="NumValues > Index">
          <Item Name="[{(*this)._GetNames()[Index]}]">(*this)._GetValues()[Index]</Item>
          <Exec>++Index</Exec>
        </Loop>
      </CustomListItems>
    </Expand>
  </Type>

  <Type Name="UE::UAF::FValueSpace">
    <Intrinsic Name="_GetType" Expression="(UE::UAF::EValueSpaceType)(Packed &amp; SPACE_CATEGORY_MASK)"/>
    <Intrinsic Name="_GetMixedFlags" Expression="(UE::UAF::EMixedSpaceFlags)(Packed &amp; MIXED_SPACE_FLAGS_MASK)"/>
    <Intrinsic Name="_IsAdditive" Expression="!!(Packed &amp; IS_ADDITIVE_MASK)"/>
    <DisplayString Condition="(*this)._GetType() == UE::UAF::EValueSpaceType::Mixed">Type=Mixed, Flags={(*this)._GetMixedFlags(),en}, Additive={(*this)._IsAdditive()}</DisplayString>
    <DisplayString>Type={(*this)._GetType(),en}, Additive={(*this)._IsAdditive()}</DisplayString>
  </Type>

</AutoVisualizer>
