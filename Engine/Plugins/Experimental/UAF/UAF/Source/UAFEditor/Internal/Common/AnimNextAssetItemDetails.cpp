// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAssetItemDetails.h"

#include "ClassIconFinder.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Entries/AnimNextEventGraphEntry.h"
#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "ISceneOutlinerTreeItem.h"
#include "RigVMModel/RigVMGraph.h"
#include "WorkspaceItemMenuContext.h"
#include "IWorkspaceEditor.h"
#include "PersonaModule.h"
#include "PersonaUtils.h"
#include "ScopedTransaction.h"
#include "RigVMModel/RigVMClient.h"
#include "ToolMenus.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Module/AnimNextModule_EditorData.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "RigVMFunctions/Execution/RigVMFunction_UserDefinedEvent.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Variables/SVariablesView.h"

#define LOCTEXT_NAMESPACE "FAnimNextGraphItemDetails"

namespace UE::UAF::Editor
{

const FSlateBrush* FAnimNextAssetItemDetails::GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const
{
	const FSoftObjectPath& AssetPath = Export.GetTopLevelAssetPath();

	// Deal with in-memory objects directly
	if (UObject* LoadedObject = AssetPath.ResolveObject())
	{
		return FSlateIconFinder::FindIconForClass(LoadedObject->GetClass()).GetIcon();
	}

	// Otherwise force on-disk only query of cached AssetRegistry state
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	FAssetData AssetData;
	if (AssetRegistry.TryGetAssetByObjectPath(AssetPath, AssetData) == UE::AssetRegistry::EExists::Exists)
	{
		const UClass* AssetClass = FClassIconFinder::GetIconClassForAssetData(AssetData);
		return FSlateIconFinder::FindIconForClass(AssetClass).GetIcon();
	}

	return nullptr;
}

bool FAnimNextAssetItemDetails::HandleDoubleClick(const FToolMenuContext& ToolMenuContext) const
{
	UWorkspaceItemMenuContext* WorkspaceItemContext = ToolMenuContext.FindContext<UWorkspaceItemMenuContext>();
	const UAssetEditorToolkitMenuContext* AssetEditorContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>();
	if (WorkspaceItemContext == nullptr || AssetEditorContext == nullptr)
	{
		return false;
	}

	const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorContext->Toolkit.Pin());
	if(!WorkspaceEditor.IsValid())
	{
		return false;
	}

	// Double-clicking a SharedVariables entry will open it directly in the variables view
	if (WorkspaceItemContext->SelectedExports.Num() == 1)
	{
		const FWorkspaceOutlinerItemExport& SharedVariablesExport = WorkspaceItemContext->SelectedExports[0];
		const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = SharedVariablesExport.GetResolvedExport().GetData();
		if (Data.IsValid() && Data.GetScriptStruct()->IsChildOf(FAnimNextSharedVariablesOutlinerData::StaticStruct()))
		{
			const FTabId VariablesTabId(VariablesTabName);
			TSharedPtr<SDockTab> VariableDockTab = WorkspaceEditor->GetTabManager()->FindExistingLiveTab(VariablesTabId);
			if (VariableDockTab.IsValid())
			{
				const TSharedRef<SVariablesView> VariablesView = StaticCastSharedRef<SVariablesView>(VariableDockTab->GetContent());
				VariablesView->SetExportDirectly(SharedVariablesExport);

				return false;
			}
		}
	}

	return false;
}

void FAnimNextAssetItemDetails::RegisterToolMenuExtensions()
{
	FToolMenuOwnerScoped OwnerScoped(TEXT("FAnimNextModuleItemDetails"));
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("WorkspaceOutliner.ItemContextMenu");
	if (Menu == nullptr)
	{
		return;
	}

	Menu->AddDynamicSection(TEXT("AnimNextAssetItem"), FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		UWorkspaceItemMenuContext* WorkspaceItemContext = InMenu->FindContext<UWorkspaceItemMenuContext>();
		const UAssetEditorToolkitMenuContext* AssetEditorContext = InMenu->FindContext<UAssetEditorToolkitMenuContext>();
		if (WorkspaceItemContext == nullptr || AssetEditorContext == nullptr)
		{
			return;
		}

		const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorContext->Toolkit.Pin());
		if(!WorkspaceEditor.IsValid())
		{
			return;
		}

		if (WorkspaceItemContext->SelectedExports.Num() != 1)
		{
			return;
		}

		TInstancedStruct<FWorkspaceOutlinerItemData>& Data = WorkspaceItemContext->SelectedExports[0].GetResolvedExport().GetData();
		if (!Data.IsValid() || !Data.GetScriptStruct()->IsChildOf(FAnimNextRigVMAssetOutlinerData::StaticStruct()))
		{
			return;
		}

		const FAnimNextRigVMAssetOutlinerData& OutlinerData = Data.Get<FAnimNextRigVMAssetOutlinerData>();
		UUAFRigVMAsset* Asset = OutlinerData.GetAsset();
		if(Asset == nullptr)
		{
			return;
		}

		UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
		if(EditorData == nullptr)
		{
			return;
		}

		FToolMenuSection& AssetSection = InMenu->AddSection("AnimNextAsset", LOCTEXT("AnimNextAssetSectionLabel", "AnimNext Asset"));

		// Per sub-item type addition
		TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> AllEntryClasses = EditorData->GetEntryClasses();
		for(UClass* SubEntryClass : AllEntryClasses)
		{
			static FTextFormat AddEntryLabelFormat(LOCTEXT("AddEntryLabelFormat", "Add {0}"));
			static FTextFormat AddEntryTooltipFormat(LOCTEXT("AddEntryTooltipFormat", "Adds a new {0} to this asset"));

			if(!EditorData->CanAddNewEntry(SubEntryClass))
			{
				continue;
			}

			if(SubEntryClass == UAnimNextEventGraphEntry::StaticClass())
			{
				AssetSection.AddSubMenu(
					SubEntryClass->GetFName(),
					FText::Format(AddEntryLabelFormat, SubEntryClass->GetDisplayNameText()),
					FText::Format(AddEntryTooltipFormat, SubEntryClass->GetDisplayNameText()),
					FNewToolMenuDelegate::CreateLambda([SubEntryClass, WorkspaceItemContext, EditorData, Asset](UToolMenu* InToolMenu)
					{
						FToolMenuSection& EventsSection = InToolMenu->AddSection(SubEntryClass->GetFName());

						static FTextFormat AddEventTooltipFormat(LOCTEXT("AddEventGraphTooltipFormat", "Adds a {0} event graph to this asset"));

						auto AddEventMenuEntry = [&EventsSection, SubEntryClass, WorkspaceItemContext, EditorData, Asset](FName InEventName, const FText& InLabel, const FText& InTooltip, UScriptStruct* InStruct)
						{
							EventsSection.AddMenuEntry(
								InEventName,
								InLabel,
								InTooltip,
								FSlateIconFinder::FindIconForClass(SubEntryClass, "ClassIcon.Object"),
								FUIAction(
									FExecuteAction::CreateWeakLambda(WorkspaceItemContext, [InEventName, SubEntryClass, EditorData, InStruct]()
									{
										FScopedTransaction Transaction(LOCTEXT("AddEventGraph", "Add Event Graph"));
										EditorData->AddEventGraph(InEventName, InStruct);
									}),
									FCanExecuteAction::CreateWeakLambda(WorkspaceItemContext, [Asset, InEventName]()
									{
										return !Asset->GetVM()->ContainsEntry(InEventName);
									})
								));
						};

						if(EditorData->IsA<UUAFSystem_EditorData>())
						{
							for(TObjectIterator<UScriptStruct> It; It; ++It)
							{
								UScriptStruct* Struct = *It;
								if(!Struct->IsChildOf(FRigUnit_AnimNextModuleEventBase::StaticStruct()) || Struct == FRigUnit_AnimNextModuleEventBase::StaticStruct())
								{
									continue;
								}
								if(Struct->HasMetaData(FRigVMStruct::HiddenMetaName) || Struct->HasMetaData(FRigVMStruct::AbstractMetaName))
								{
									continue;
								}
								
								TInstancedStruct<FRigUnit_AnimNextModuleEventBase> StructInstance;
								StructInstance.InitializeAsScriptStruct(Struct);
								FName EventName = StructInstance.Get<FRigUnit_AnimNextModuleEventBase>().GetEventName();
								FText EventNameText = FText::FromName(EventName);
								AddEventMenuEntry(EventName, EventNameText, FText::Format(AddEventTooltipFormat, EventNameText), Struct);
							}
						}

						AddEventMenuEntry(TEXT("CustomEvent"), LOCTEXT("CustomEventLabel", "Custom Event"), FText::Format(AddEventTooltipFormat, LOCTEXT("CustomEventDisplayNameInline", "custom")), FRigVMFunction_UserDefinedEvent::StaticStruct());

						EventsSection.AddSubMenu(
							TEXT("NotifyEvents"),
							LOCTEXT("NotifyEventsLabel", "Notify Event"),
							LOCTEXT("NotifyEventsTooltip", "Add an event graph to handle a notify event"),
							FNewToolMenuDelegate::CreateLambda([SubEntryClass, WorkspaceItemContext, EditorData, Asset](UToolMenu* InToolMenu)
							{
								FToolMenuSection& NotifySection = InToolMenu->AddSection(TEXT("NotifyEvents"));

								auto CreateNewEventGraph = [Asset, EditorData](FName InEventGraphName)
								{
									if(!Asset->GetVM()->ContainsEntry(InEventGraphName))
									{
										FScopedTransaction Transaction(LOCTEXT("AddEventGraph", "Add Event Graph"));
										EditorData->AddEventGraph(InEventGraphName, FRigVMFunction_UserDefinedEvent::StaticStruct());
									}
								};

								NotifySection.AddEntry(FToolMenuEntry::InitMenuEntry(
									TEXT("NewNotifyEvent"),
									LOCTEXT("AddNewNotifyEventLabel", "Add New Notify Event..."),
									LOCTEXT("AddNewNotifyEventTooltip", "Add a new notify event as a custom event"),
									FSlateIcon(),
									FExecuteAction::CreateLambda([CreateNewEventGraph]()
									{ 
										// Show dialog to enter new track name
										TSharedRef<STextEntryPopup> TextEntry =
											SNew(STextEntryPopup)
											.Label(LOCTEXT("NewNotifyLabel", "Notify Name"))
											.OnTextCommitted_Lambda([CreateNewEventGraph](const FText& InText, ETextCommit::Type InCommitType)
											{
												FSlateApplication::Get().DismissAllMenus();
												FName NotifyName = *InText.ToString();
												CreateNewEventGraph(NotifyName);
											});

										// Show dialog to enter new event name
										FSlateApplication::Get().PushMenu(
											FSlateApplication::Get().GetInteractiveTopLevelWindows()[0],
											FWidgetPath(),
											TextEntry,
											FSlateApplication::Get().GetCursorPos(),
											FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup));
									}),
									EUserInterfaceActionType::Button
								));

								FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
								TSharedRef<SWidget> PickerWidget = PersonaModule.CreateSkeletonNotifyPicker(FOnNotifyPicked::CreateLambda([CreateNewEventGraph](FName InNotifyName)
								{
									FSlateApplication::Get().DismissAllMenus();
									CreateNewEventGraph(InNotifyName);
								}));
								NotifySection.AddEntry(FToolMenuEntry::InitWidget(
									TEXT("NotifyEventPicker"),
									SNew(SBox)
									.WidthOverride(300.0f)
									.HeightOverride(400.0f)
									[
										PickerWidget
									],
									FText::GetEmpty(),
									true, false, true));
							}));
					}),
					false,
					FSlateIconFinder::FindIconForClass(SubEntryClass, "ClassIcon.Object"));
			}
		}
	}));
}

void FAnimNextAssetItemDetails::UnregisterToolMenuExtensions()
{
	if(UToolMenus* ToolMenus = UToolMenus::Get())
	{
		ToolMenus->UnregisterOwnerByName("FAnimNextAssetItemDetails");
	}
}

}

#undef LOCTEXT_NAMESPACE // "FAnimNextGraphItemDetails"