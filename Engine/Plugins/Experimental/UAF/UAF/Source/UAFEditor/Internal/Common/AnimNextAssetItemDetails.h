// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWorkspaceOutlinerItemDetails.h"

class UUAFRigVMAssetEditorData;
enum class EAnimNextEditorDataNotifType : uint8;

namespace UE::UAF::Editor
{

class FAnimNextAssetItemDetails : public UE::Workspace::IWorkspaceOutlinerItemDetails
{
public:
	FAnimNextAssetItemDetails() = default;

	UAFEDITOR_API virtual const FSlateBrush* GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const override;
	UAFEDITOR_API virtual bool HandleDoubleClick(const FToolMenuContext& ToolMenuContext) const override;

	static void RegisterToolMenuExtensions();
	static void UnregisterToolMenuExtensions();
};

}
