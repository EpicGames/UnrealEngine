// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "GraphEditorSchemaActions.generated.h"

#define UE_API UAFEDITOR_API

struct FAnimNextParamType;
struct FSlateBrush;
class URigVMLibraryNode;
class URigVMUnitNode;

USTRUCT()
struct FAnimNextSchemaAction : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	FAnimNextSchemaAction() = default;
	
	FAnimNextSchemaAction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, FText InKeywords = FText::GetEmpty())
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), 0, MoveTemp(InKeywords))
	{
	}

	UE_API virtual const FSlateBrush* GetIconBrush() const;

	UE_API virtual const FLinearColor& GetIconColor() const;
};

USTRUCT()
struct FAnimNextSchemaAction_RigUnit : public FAnimNextSchemaAction
{
	GENERATED_BODY()

	FAnimNextSchemaAction_RigUnit() = default;
	
	FAnimNextSchemaAction_RigUnit(const TSubclassOf<URigVMUnitNode>& InNodeClass, UScriptStruct* InStructTemplate, FText InNodeCategory, FText InMenuDesc, FText InToolTip, FText InKeywords = FText::GetEmpty())
		: FAnimNextSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InKeywords)
		, StructTemplate(InStructTemplate)
		, NodeClass(InNodeClass)
	{}

	// FEdGraphSchemaAction Interface
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override { return nullptr; }

private:
	// The script struct for our rig unit
	UScriptStruct* StructTemplate = nullptr;
	TSubclassOf<URigVMUnitNode> NodeClass;
};


USTRUCT()
struct FAnimNextSchemaAction_DispatchFactory : public FAnimNextSchemaAction
{
	GENERATED_BODY()

	FAnimNextSchemaAction_DispatchFactory() = default;

	FAnimNextSchemaAction_DispatchFactory(FName InNotation, FText InNodeCategory, FText InMenuDesc, FText InToolTip, FText InKeywords = FText::GetEmpty())
		: FAnimNextSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InKeywords)
		, Notation(InNotation)
	{}

	virtual const FSlateBrush* GetIconBrush() const override;
	
	// FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override { return nullptr; }

private:
	// Notation for dispatch factory
	FName Notation;
};


USTRUCT()
struct FAnimNextSchemaAction_Variable : public FAnimNextSchemaAction
{
	GENERATED_BODY()

	enum class EVariableAccessorChoice
	{
		Set,
		Get,
		Deferred
	};

	FAnimNextSchemaAction_Variable() = default;

	UE_API FAnimNextSchemaAction_Variable(FName InName, const UObject* InSourceObject, const FAnimNextParamType& InType, const EVariableAccessorChoice InVariableAccessorChoice);

	// FEdGraphSchemaAction Interface
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override { return nullptr; }

	UE_API virtual const FSlateBrush* GetIconBrush() const override;

	UE_API virtual const FLinearColor& GetIconColor() const override;
	
	UE_API void SetVariableAccessorChoice(EVariableAccessorChoice InVariableAccessorChoice, bool bUpdateSearchData = false);

private:
	FName Name;
	FString TypeObjectPath;
	FString SourceObjectPath;
	FString TypeName;
	EVariableAccessorChoice VariableAccessorChoice = EVariableAccessorChoice::Set;
	FLinearColor VariableColor;
};

USTRUCT()
struct FAnimNextSchemaAction_AddComment : public FAnimNextSchemaAction
{
	GENERATED_BODY()

	FAnimNextSchemaAction_AddComment();

	// FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override { return nullptr; }

	virtual const FSlateBrush* GetIconBrush() const override;
};

USTRUCT()
struct FAnimNextSchemaAction_Function : public FAnimNextSchemaAction
{
	GENERATED_BODY()

	FAnimNextSchemaAction_Function() = default;

	FAnimNextSchemaAction_Function(const FRigVMGraphFunctionHeader& InReferencedPublicFunctionHeader, const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, const FText& InKeywords = FText::GetEmpty());
	FAnimNextSchemaAction_Function(const URigVMLibraryNode* InFunctionLibraryNode, const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, const FText& InKeywords = FText::GetEmpty());
	
	FAnimNextSchemaAction_Function(const URigVMLibraryNode* InFunctionLibraryNode);

	virtual const FSlateBrush* GetIconBrush() const override;
	
	// FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override { return nullptr; }

private:
	/** The public function definition we will spawn from [optional] */
	UPROPERTY(Transient)
	FRigVMGraphFunctionHeader ReferencedPublicFunctionHeader;

	/** Marked as true for local function definitions */
	UPROPERTY(Transient)
	bool bIsLocalFunction = false;

	/** Holds the node type that this spawner will instantiate. */
	UPROPERTY(Transient)
	TSubclassOf<UEdGraphNode> NodeClass;
};

USTRUCT()
struct FAnimNextSchemaAction_PromoteToVariable : public FAnimNextSchemaAction
{
	GENERATED_BODY()

	FAnimNextSchemaAction_PromoteToVariable();

	// FEdGraphSchemaAction Interface
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override { return nullptr; }

	virtual const FSlateBrush* GetIconBrush() const override;
};

#undef UE_API
