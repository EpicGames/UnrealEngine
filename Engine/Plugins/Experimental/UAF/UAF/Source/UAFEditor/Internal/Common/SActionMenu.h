// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GraphEditor.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Layout/Visibility.h"
#include "SGraphActionMenu.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SWidget.h"

struct FWorkspaceOutlinerItemExport;
class UUAFRigVMAssetEditorData;
class URigVMSchema;
class SEditableTextBox;
class SGraphActionMenu;
class UEdGraph;
class IRigVMClientHost;
class URigVMController;
class URigVMHost;

namespace UE::UAF::Editor
{

struct FActionMenuContextData
{
	TArray<UObject*> SelectedObjects;

	UEdGraph* Graph = nullptr;
	FWorkspaceOutlinerItemExport Export;
	const URigVMSchema* RigVMSchema = nullptr;
	URigVMHost* RigVMHost = nullptr;
	IRigVMClientHost* RigVMClientHost = nullptr;
	URigVMController* RigVMController = nullptr;
	UUAFRigVMAssetEditorData* EditorData = nullptr;
	bool bShowGlobalManifestNodes = true;
};

class SActionMenu : public SBorder
{
public:
	/** Delegate to retrieve the action list for the graph */
	DECLARE_DELEGATE_TwoParams(FCollectAllGraphActions, FGraphContextMenuBuilder& /*MenuBuilder*/, const FActionMenuContextData& /*ActionMenuContextData*/);

	/** Delegate for the OnCloseReason event which is always raised when the SActionMenu closes */
	DECLARE_DELEGATE_ThreeParams(FClosedReason, bool /*bActionExecuted*/, bool /*bContextSensitiveChecked*/, bool /*bGraphPinContext*/);

	SLATE_BEGIN_ARGS(SActionMenu)
		: _NewNodePosition(FVector2D::ZeroVector)
		, _AutoExpandActionMenu(false)
	{}
	
	SLATE_ARGUMENT(FVector2D, NewNodePosition)
	SLATE_ARGUMENT(TArray<UEdGraphPin*>, DraggedFromPins)
	SLATE_ARGUMENT(SGraphEditor::FActionMenuClosed, OnClosedCallback)
	SLATE_ARGUMENT(bool, AutoExpandActionMenu)
	SLATE_EVENT(FClosedReason, OnCloseReason)
	SLATE_EVENT(FCollectAllGraphActions, OnCollectGraphActionsCallback)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraph* InGraph, const FWorkspaceOutlinerItemExport& InExport);

	virtual ~SActionMenu() override;

	TSharedRef<SEditableTextBox> GetFilterTextBox();

protected:
	void OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedAction, ESelectInfo::Type InSelectionType);

	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);

	/** Callback used to populate all actions list in SGraphActionMenu */
	void CollectAllActions(FGraphActionListBuilderBase& OutAllActions);

	void CollectAllAnimNextGraphActions(FGraphContextMenuBuilder& MenuBuilder) const;
	
private:
	FActionMenuContextData ContextData;

	bool bAutoExpandActionMenu = false;
	bool bActionExecuted = false;

	TArray<UEdGraphPin*> DraggedFromPins;
	FDeprecateSlateVector2D NewNodePosition = FVector2f::ZeroVector;

	SGraphEditor::FActionMenuClosed OnClosedCallback;
	FClosedReason OnCloseReasonCallback;

	FCollectAllGraphActions OnCollectGraphActionsCallback;

	TSharedPtr<SGraphActionMenu> GraphActionMenu;
};

}
