// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Margin.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableRow.h"

namespace UE::UAF::Editor
{

// An entry category displayed in a tree view
template<typename ItemType>
class SCategoryHeaderTableRow : public STableRow<ItemType>
{
public:
	SLATE_BEGIN_ARGS(SCategoryHeaderTableRow)
		: _Padding(FMargin(0.0f, 2.0f, .0f, 0.0f))
		{}
		SLATE_DEFAULT_SLOT(typename SCategoryHeaderTableRow::FArguments, Content)
		SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		STableRow<ItemType>::ChildSlot
			.Padding(InArgs._Padding)
			[
				SAssignNew(ContentBorder, SBorder)
					.BorderImage(this, &SCategoryHeaderTableRow::GetBackgroundImage)
					.Padding(FMargin(3.0f, 5.0f))
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(5.0f)
							.AutoWidth()
							[
								SNew(SExpanderArrow, STableRow< ItemType >::SharedThis(this))
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								InArgs._Content.Widget
							]
					]
			];

		STableRow < ItemType >::ConstructInternal(
			typename STableRow< ItemType >::FArguments()
			.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
			.ShowSelection(false),
			InOwnerTableView
		);
	}

	const FSlateBrush* GetBackgroundImage() const
	{
		if (STableRow<ItemType>::IsHovered())
		{
			return FAppStyle::Get().GetBrush("Brushes.Secondary");
		}
		else
		{
			return FAppStyle::Get().GetBrush("Brushes.Header");
		}
	}

	virtual void SetContent(TSharedRef< SWidget > InContent) override
	{
		ContentBorder->SetContent(InContent);
	}

	virtual void SetRowContent(TSharedRef< SWidget > InContent) override
	{
		ContentBorder->SetContent(InContent);
	}

	virtual const FSlateBrush* GetBorder() const
	{
		return nullptr;
	}

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			STableRow<ItemType>::ToggleExpansion();
			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}
private:
	TSharedPtr<SBorder> ContentBorder;
};

} // end namespace UE::UAF::Editor
