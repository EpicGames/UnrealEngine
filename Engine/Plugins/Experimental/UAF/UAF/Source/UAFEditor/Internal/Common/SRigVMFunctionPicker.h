// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class UUAFRigVMAsset;
class FTextFilterExpressionEvaluator;
class SSearchBox;

namespace UE::UAF::Editor
{
	class SRigVMFunctionRowWidget;
}

namespace UE::UAF::Editor
{

// Delegate called when the user picks a function from the dropdown menu
// The header passed when selecting 'None' will be invalid
using FOnRigVMFunctionPicked = TDelegate<void(const FRigVMGraphFunctionHeader&)>;

// Optional filter predicate: return true to include the function, false to exclude it
using FOnFilterRigVMFunction = TDelegate<bool(const FRigVMGraphFunctionHeader&)>;

// Picker widget the allows choosing a RigVM function
class SRigVMFunctionPicker : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SRigVMFunctionPicker)
		: _AllowNew(true)
		, _AllowClear(true)
		, _ContentOnly(false)
		, _IsContextSensitive(true)
	{}

	// The 'current' asset, used for accessing private functions
	SLATE_ARGUMENT(FAssetData, CurrentAsset)

	// Delegate called when the user picks a function from the dropdown menu.
	// The header passed when selecting 'None' will be invalid 
	SLATE_EVENT(FOnRigVMFunctionPicked, OnRigVMFunctionPicked)

	// Delegate called when the user chooses 'New Function...'
	SLATE_EVENT(FSimpleDelegate, OnNewFunction)

	// Attribute used to display the currently-picked function in a combo button 
	SLATE_ATTRIBUTE(FText, FunctionName)

	// Attribute used to display tooltip information about the currently-picked function
	SLATE_ATTRIBUTE(FText, FunctionToolTip)

	// Whether to show the 'New Function...' item
	SLATE_ARGUMENT(bool, AllowNew)

	// Whether to show the 'None' item
	SLATE_ARGUMENT(bool, AllowClear)

	// When true, renders only the search box + tree view (no combo button wrapper).
	// Use this when embedding the picker inside another popup or tab.
	SLATE_ARGUMENT(bool, ContentOnly)

	// Optional filter for narrowing which functions appear. Return true to include, false to exclude.
	SLATE_EVENT(FOnFilterRigVMFunction, OnFilterFunction)

	// Assets to filter by when context-sensitive. Empty = no context filtering available.
	SLATE_ARGUMENT(TArray<TWeakObjectPtr<const UUAFRigVMAsset>>, FilterAssets)

	// Whether context sensitivity is initially enabled (only relevant when FilterAssets is non-empty).
	SLATE_ARGUMENT(bool, IsContextSensitive)

	SLATE_END_ARGS()

	UAFEDITOR_API void Construct(const FArguments& InArgs);

private:
	friend UE::UAF::Editor::SRigVMFunctionRowWidget;

	// Requests a refresh of the dropdown menu contents
	void RequestRefreshEntries();

	// Refreshes the dropdown menu contents
	void RefreshEntries();

	enum class EEntryType
	{
		None,
		Asset,
		Function,
		NewFunction,
	};

	struct FEntry
	{
		explicit FEntry(EEntryType InType)
			: Type(InType)
		{}

		EEntryType Type;
		FText Name;
		FText ToolTip;
		const FSlateBrush* Icon = nullptr;
	};

	struct FFunctionEntry : FEntry
	{
		FFunctionEntry()
			: FEntry(EEntryType::Function)
		{}

		FRigVMGraphFunctionHeader FunctionHeader;
		const FSlateBrush* ReturnTypeIcon = nullptr;
		FSlateColor ReturnTypeColor = FLinearColor::Gray;
		FText ReturnTypeName;
	};

	struct FAssetEntry : FEntry
	{
		FAssetEntry()
			: FEntry(EEntryType::Asset)
		{}

		FAssetData Asset;
		TArray<TSharedPtr<FEntry>> Functions;
		TArray<TSharedPtr<FEntry>> FilteredFunctions;
	};

	struct FNewFunctionEntry : FEntry
	{
		FNewFunctionEntry()
			: FEntry(EEntryType::NewFunction)
		{}
	};

	struct FNoneEntry : FEntry
	{
		FNoneEntry()
			: FEntry(EEntryType::None)
		{}
	};

	FAssetData CurrentAsset;
	TWeakObjectPtr<UUAFRigVMAsset> WeakCurrentAsset;
	FOnRigVMFunctionPicked OnRigVMFunctionPicked;
	FSimpleDelegate OnNewFunction;
	TAttribute<FText> FunctionName;
	TAttribute<FText> FunctionToolTip;
	TSharedPtr<SSearchBox> SearchBox;
	FText FilterText;
	TSharedPtr<STreeView<TSharedPtr<FEntry>>> TreeView;
	TArray<TSharedPtr<FEntry>> Entries;
	TArray<TSharedPtr<FEntry>> FilteredEntries;
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;
	FOnFilterRigVMFunction OnFilterFunction;
	TArray<TWeakObjectPtr<const UUAFRigVMAsset>> FilterAssets;
	bool bIsContextSensitive = true;
	bool bAllowClear = false;
	bool bAllowNew = false;
};

}
