// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFVariableDragDropContext.generated.h"

class UEdGraph;

UCLASS()
class UUAFVariableDragDropContext : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UEdGraph> ParentGraph;

	FVector2f Location;
	FName Name;
	FString TypeObjectPath;
	FString SourceObjectPath;
	FString TypeName;
};