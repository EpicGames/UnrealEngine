// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Param/ParamType.h"
#include "EdGraphSchema_K2.h"

struct FAnimNextVariableBindingData;
class UUAFSystem;
class UUAFRigVMAssetEditorData;
struct FAnimNextParamType;
class URigVMController;
struct FAnimNextWorkspaceAssetRegistryExports;
class SWidget;
class URigVMUnitNode;
struct FRigVMFunction;
struct FGraphContextMenuBuilder;

struct FAnimNextAssetRegistryExports;
class UAnimNextVariableEntry;

namespace UE::UAF::Editor
{

struct FUtils
{
	static UAFEDITOR_API FName ValidateName(const UObject* InObject, const FString& InName);

	static UAFEDITOR_API void GetAllEntryNames(const UUAFRigVMAssetEditorData* InEditorData, TSet<FName>& OutNames);

	static UAFEDITOR_API FAnimNextParamType GetParameterTypeFromMetaData(const FStringView& InStringView);

	static UAFEDITOR_API void GetFilteredVariableTypeTree(TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& TypeTree, ETypeTreeFilter TypeTreeFilter);

	static UAFEDITOR_API bool IsValidParameterNameString(FStringView InStringView, FText& OutErrorText);

	static UAFEDITOR_API bool IsValidParameterName(const FName InName, FText& OutErrorText);
	
	static UAFEDITOR_API bool IsValidParameterNameForEntry(const UAnimNextVariableEntry* Entry, const FText& InNewName, FText& OutErrorText);

	static UAFEDITOR_API bool AddSchemaRigUnitAction(const TSubclassOf<URigVMUnitNode>& UnitNodeClass, UScriptStruct* Struct, const FRigVMFunction& Function, FGraphContextMenuBuilder& InContexMenuBuilder);

	static UAFEDITOR_API void GetRigUnitStructMetadata(const UScriptStruct* Struct, FString& OutCategoryMetadata, FString& OutDisplayNameMetadata, FString& OutMenuDescSuffixMetadata);
};

}
