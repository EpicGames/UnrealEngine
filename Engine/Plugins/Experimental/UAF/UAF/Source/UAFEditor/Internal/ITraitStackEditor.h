// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "UObject/WeakObjectPtr.h"

class UAnimNextEdGraphNode;
class SWidget;
class IDetailPropertyRow;
class IDetailCategoryBuilder;
class URigVMController;

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{

struct FTraitStackData
{
	FTraitStackData() = default;

	explicit FTraitStackData(const TWeakObjectPtr<UAnimNextEdGraphNode>& InEdGraphNodeWeak)
		: EdGraphNodeWeak(InEdGraphNodeWeak)
	{}

	TWeakObjectPtr<UAnimNextEdGraphNode> EdGraphNodeWeak = nullptr;
};

// Interface used to talk to the trait stack editor embedded in a workspace
class ITraitStackEditor : public IModularFeature
{
public:
	static inline FLazyName ModularFeatureName = FLazyName(TEXT("TraitStackEditor"));

	virtual ~ITraitStackEditor() = default;

	// Sets the trait data to be displayed for the specified workspace editor instance
	virtual void SetTraitData(const TSharedRef<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor, const FTraitStackData& InTraitStackData) = 0;

	// Create a header widget for the trait in a trait stack node corresponding to InPropertyRow
	virtual TSharedRef<SWidget> CreateTraitHeaderWidget(IDetailCategoryBuilder& InCategory, IDetailPropertyRow& InPropertyRow, URigVMController* InController) = 0;

	// Create a header widget for the stack in a trait stack node 
	virtual TSharedRef<SWidget> CreateTraitStackHeaderWidget(IDetailCategoryBuilder& InCategory, URigVMController* InController) = 0;
};

}