// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "AnimNextModuleFactory.generated.h"

UCLASS(MinimalAPI)
class UUAFSystemFactory : public UFactory
{
	GENERATED_BODY()

	UUAFSystemFactory();
	
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override
	{
		return FactoryCreateNew(Class, InParent, Name, Flags, Context, Warn, NAME_None);
	}
};