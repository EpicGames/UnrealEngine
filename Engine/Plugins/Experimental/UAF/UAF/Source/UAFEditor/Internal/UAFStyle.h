// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API UAFEDITOR_API

class FUAFStyle final : public FSlateStyleSet
{
public:
	static UE_API FUAFStyle& Get();

private:
	UE_API FUAFStyle();
	UE_API ~FUAFStyle();

	// Resolve path relative to plugin's resources directory
	static UE_API FString InResources(const FString& RelativePath);

	UE_API void SetClassIcons();

	UE_API void SetVariablesOutlinerStyles();
	
	UE_API void SetTraitStyles();
	
	UE_API void SetAssetWizardStyles();

	UE_API void SetSetBindingStyles();
	
	UE_API void SetBlendProfileStyles();
};

#undef UE_API
