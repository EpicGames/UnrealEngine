// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextExports.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Param/ParamType.h"

struct FAnimNextVariableReference;
struct FAnimNextSoftVariableReference;

namespace UE::UAF::Editor
{

// Result of a filter operation via FOnFilterVariableType
enum class EFilterVariableResult : int32
{
	Include,
	Exclude
};

// Delegate called to filter variables by type for display to the user
using FOnFilterVariableType = TDelegate<EFilterVariableResult(const FAnimNextParamType& /*InType*/)>;

// Delegate called when a variable has been picked. Graph argument is invalid when an unbound variable is chosen.
using FOnVariablePicked = TDelegate<void(const FAnimNextSoftVariableReference& /*InSoftVariableReference*/, const FAnimNextParamType& /*InType*/)>;

// Delegate called to filter variables for display to the user
using FOnFilterVariable = TDelegate<EFilterVariableResult(const FAnimNextSoftVariableReference& /*InSoftVariableReference*/)>;

// Delegate called to determined context sensitivity
using FOnIsContextSensitive = TDelegate<bool()>;

// Delegate called when context sensitivity changes
using FOnContextSensitivityChanged = TDelegate<void(bool)>;

// Delegate called to get the inclusion/exclusion flags
using FOnGetIncludeExcludeFilter = TDelegate<void(EAnimNextExportedVariableFlags& OutFlagInclusionFilter, EAnimNextExportedVariableFlags& OutFlagExclusionFilter)>;

// Fired when the user picks a sub-property leaf from inside a struct variable.
// SubPropertyPath contains the chain of FNames from the top-level variable to the leaf.
using FOnSubPropertyVariablePicked = TDelegate<void(
	const FAnimNextSoftVariableReference& /*TopLevelVar*/,
	const TArray<FName>&                  /*SubPropertyPath*/,
	const FAnimNextParamType&             /*LeafType*/)>;

struct FVariablePickerArgs
{
	FVariablePickerArgs() = default;

	// Delegate used to signal whether selection has changed
	FSimpleDelegate OnSelectionChanged;

	// Delegate called when a single variable has been picked
	FOnVariablePicked OnVariablePicked;

	// Delegate called to filter variables for display to the user
	FOnFilterVariable OnFilterVariable;

	// Delegate called to filter variables by type for display to the user (RigVM registry checks, pin constraints, etc.)
	// Applied as an additional gate on top of CompatibleTypes when both are set.
	FOnFilterVariableType OnFilterVariableType;

	// If non-empty, the widget handles type-compat filtering and struct expansion internally.
	// A variable passes if its type is directly compatible with any entry, or (with struct expansion)
	// if it contains a compatible leaf. bAllowStructExpansion is auto-enabled when this is set.
	TArray<FAnimNextParamType> CompatibleTypes;

	// Delegate called to determined context sensitivity
	FOnIsContextSensitive* OnIsContextSensitive = nullptr;

	// Delegate called when context sensitivity changes
	FOnContextSensitivityChanged OnContextSensitivityChanged;

	// Delegate called to get the inclusion/exclusion flags
	FOnGetIncludeExcludeFilter OnGetIncludeExcludeFilter;

	// Default filter for AnimNext export flags to be used for variable list population (used if OnGetIncludeExcludeFilter is not overriden)
	EAnimNextExportedVariableFlags FlagInclusionFilter = EAnimNextExportedVariableFlags::Declared | EAnimNextExportedVariableFlags::Public;
	EAnimNextExportedVariableFlags FlagExclusionFilter = EAnimNextExportedVariableFlags::Referenced;

	// Whether the search box should be focused on widget creation
	bool bFocusSearchWidget = true;

	// Fired when the user selects a sub-property leaf from inside a struct variable.
	FOnSubPropertyVariablePicked OnSubPropertyVariablePicked;

	// If true, struct-type variables that contain at least one compatible sub-property
	// appear in the list as expandable nodes.
	bool bAllowStructExpansion = false;
};

}
