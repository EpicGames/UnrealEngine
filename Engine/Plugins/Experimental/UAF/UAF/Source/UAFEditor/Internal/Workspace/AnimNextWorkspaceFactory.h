// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkspaceFactory.h"
#include "AnimNextWorkspaceFactory.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UAnimNextWorkspaceFactory : public UWorkspaceFactory
{
	GENERATED_BODY()

	UAnimNextWorkspaceFactory();

	// UFactory interface
	virtual bool ConfigureProperties() override;
	virtual FText GetDisplayName() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual const TArray<FText>& GetMenuCategorySubMenus() const override;
	virtual bool ShouldShowInNewMenu() const override;
};