// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextEditorModule.h"

#include "AnimNextConfig.h"
#include "EdGraphNode_Comment.h"
#include "ISettingsModule.h"
#include "IWorkspaceEditor.h"
#include "SSimpleButton.h"
#include "UncookedOnlyUtils.h"
#include "Editor/RigVMEditorTools.h"
#include "FindInAnimNextRigVMAsset.h"
#include "Framework/Application/SlateApplication.h"
#include "Module/AnimNextModule.h"
#include "Common/AnimNextEdGraphNodeCustomization.h"
#include "Module/AnimNextModule_EditorData.h"
#include "AnimNextEdGraphNode.h"
#include "Common/AnimNextCompilerResultsTabSummoner.h"
#include "Common/AnimNextFindTabSummoner.h"
#include "Common/UAFBrowserTabSummoner.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Variables/VariableCustomization.h"
#include "Variables/VariableBindingPropertyCustomization.h"
#include "Param/ParamTypePropertyCustomization.h"
#include "IWorkspaceEditorModule.h"
#include "Common/SActionMenu.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "Editor/RigVMGraphDetailCustomization.h"
#include "FileHelpers.h"
#include "IUniversalObjectLocatorEditorModule.h"
#include "Param/AnimNextComponentLocatorEditor.h"
#include "Param/AnimNextLocatorContext.h"
#include "Param/ObjectCastLocatorEditor.h"
#include "Param/ObjectFunctionLocatorEditor.h"
#include "Param/ObjectPropertyLocatorEditor.h"
#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "AssetCompilationHandler.h"
#include "ContentBrowserModule.h"
#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "Common/AnimNextGraphItemDetails.h"
#include "Common/AnimNextCollapseNodeItemDetails.h"
#include "Common/AnimNextRigVMAssetCommands.h"
#include "Param/AnimNextActorLocatorEditor.h"
#include "RigVMCommands.h"
#include "ToolMenus.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Common/AnimNextAssetItemDetails.h"
#include "Common/AssetPreview/AnimSequenceAssetPreview.h"
#include "Common/AssetPreview/IUAFAssetPreview.h"
#include "Common/AssetPreview/UAFAssetPreviewFactorySubsystem.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Entries/AnimNextEventGraphEntry.h"
#include "Logging/MessageLog.h"
#include "Graph/AnimNextEdGraphCustomization.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Variables/SVariablesView.h"
#include "Variables/VariableOverrideCommands.h"
#include "Workspace/AnimNextWorkspaceEditorMode.h"
#include "ExternalPackageHelper.h"
#include "Common/GraphEditorSchemaActions.h"
#include "EditorUtils.h"
#include "PersonaModule.h"
#include "Developer/TraceServices/Public/TraceServices/ModuleService.h"
#include "Editor/RewindDebuggerInterface/Public/IRewindDebuggerExtension.h"
#include "Editor/RewindDebuggerInterface/Public/IRewindDebuggerTrackCreator.h"
#include "Graph/AnimNextGraphPanelPinFactory.h"
#include "Module/ModuleEventPropertyCustomization.h"
#include "Widgets/Docking/SDockTab.h"
#include "RewindDebugger/RewindDebuggerAnimNext.h"
#include "RewindDebugger/AnimNextModuleTrack.h"
#include "RewindDebugger/AnimNextTraceModule.h"
#include "Settings/UAFEditorUserSettings.h"
#include "Variables/VariableReferencePropertyCustomization.h"
#include "AnimNextSharedVariableNode.h"
#include "UAFStyle.h"
#include "Common/AnimNextAssetFindReplaceVariables.h"
#include "Common/Outliner/OutlinerCommands.h"
#include "Functions/Outliner/FunctionsOutlinerView.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "UAF/Viewport/SystemViewportController.h"
#include "Variables/Outliner/VariablesOutlinerCommands.h"
#include "WidgetDrawerConfig.h"
#include "Widgets/Docking/SDockTab.h"
#include "Interfaces/IPluginManager.h"
#include "AssetToolsModule.h"
#include "UAF/Debug/VariableWatchTabSummoner.h"
#include "WorkspaceSchema.h"
#include "Styling/SlateIconFinder.h"
#include "UAF/AbstractSkeleton/Sets/Commands.h"
#include "Variables/UAFAssetDataPropertyCustomization.h"
#include "UAF/Wizard/AssetWizard.h"
#include "BindableValue/UAFBindableValueCustomization.h"

#define LOCTEXT_NAMESPACE "AnimNextEditorModule"

#if UAF_TRACE_ENABLED
namespace
{
	FRewindDebuggerUAF GRewindDebuggerUAF;
#if UAF_TRACE_ENABLED
	UE::UAF::Editor::FAnimNextModuleTrackCreator GAnimNextModulesTrackCreator;
#endif
	FUAFTraceModule GUAFTraceModule;
}
#endif

namespace UE::UAF::Editor
{
	const auto ToggleUAFContentHidden = FConsoleCommandDelegate::CreateStatic([]()
    {
    	if (TSharedPtr<IPlugin> UAFPlugin = IPluginManager::Get().FindPlugin("UAF"))
    	{
    		TSharedRef<FPathPermissionList>& FolderPermissions = FAssetToolsModule::GetModule().Get().GetFolderPermissionList();

			static const TCHAR* DenyItemKey = TEXT("UAFContent");
			const FString DenyItemValue = UAFPlugin->GetMountedAssetPath();

    		if (FolderPermissions->ContainsDenyListItem(DenyItemValue))
    		{
    			FolderPermissions->RemoveDenyListItem(DenyItemKey, DenyItemValue);
    		}
		    else
		    {
		    	FolderPermissions->AddDenyListItem(DenyItemKey, DenyItemValue);
		    }
    	}

		if (TSharedPtr<IPlugin> UAFPlugin = IPluginManager::Get().FindPlugin("UAFAnimNode"))
		{
			TSharedRef<FPathPermissionList>& FolderPermissions = FAssetToolsModule::GetModule().Get().GetFolderPermissionList();

			static const TCHAR* DenyItemKey = TEXT("UAFAnimNodeContent");
			const FString DenyItemValue = UAFPlugin->GetMountedAssetPath();

			if (FolderPermissions->ContainsDenyListItem(DenyItemValue))
			{
				FolderPermissions->RemoveDenyListItem(DenyItemKey, DenyItemValue);
			}
			else
			{
				FolderPermissions->AddDenyListItem(DenyItemKey, DenyItemValue);
			}
		}
    });

static FAutoConsoleCommand ToggleHiddenUAFContent(
    TEXT("UAF.ToggleHiddenContent"),
    TEXT("Toggles whether internal UAF content is shown in the editor"),
    ToggleUAFContentHidden);

static bool bEnableUAFBrowser = true;
static FAutoConsoleVariableRef CVarEnableUAFBrowser(
	TEXT("a.UAF.EnableUAFBrowser"),
	bEnableUAFBrowser,
	TEXT("Experimental. If true enables the UAF Browser."));

void FAnimNextEditorModule::StartupModule()
{
	FVariableOverrideCommands::Register();
	FRigVMCommands::Register();
	FAnimNextRigVMAssetCommands::Register();
	FVariablesOutlinerCommands::Register();
	FSetBindingEditorCommands::Register();
	FOutlinerCommands::Register();
	FUAFStyle::Get();

	// Register settings for user editing
	ISettingsModule& SettingsModule = FModuleManager::Get().LoadModuleChecked<ISettingsModule>("Settings");
	SettingsModule.RegisterSettings("Editor", "General", "UAF",
		LOCTEXT("SettingsName", "UAF"),
		LOCTEXT("SettingsDescription", "Customize AnimNext Settings."),
		GetMutableDefault<UAnimNextConfig>()
	);
	
	EnableAssetWizard();

#if UAF_TRACE_ENABLED
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &GRewindDebuggerUAF);
#if UAF_TRACE_ENABLED
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GAnimNextModulesTrackCreator);
#endif
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &GUAFTraceModule);
#endif // UAF_TRACE_ENABLED

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomPropertyTypeLayout(
		"AnimNextParamType",
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FParamTypePropertyTypeCustomization>(); }));
	
	PropertyModule.RegisterCustomPropertyTypeLayout(
		"AnimNextVariableBinding",
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FVariableBindingPropertyCustomization>(); }));

	PropertyModule.RegisterCustomPropertyTypeLayout(
		"AnimNextVariableReference",
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FVariableReferencePropertyCustomization>(); }));

	UAFAssetDataPropertyIdentifier = MakeShared<FUAFAssetDataPropertyIdentifier>();
	PropertyModule.RegisterCustomPropertyTypeLayout(
		"InstancedStruct",
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FUAFAssetDataPropertyCustomization>(); }),
		UAFAssetDataPropertyIdentifier);

	ModuleEventPropertyTypeIdentifier = MakeShared<FModuleEventPropertyTypeIdentifier>();
	PropertyModule.RegisterCustomPropertyTypeLayout(
		"NameProperty",
		FOnGetPropertyTypeCustomizationInstance::CreateLambda([] { return MakeShared<FModuleEventPropertyCustomization>(); }),
		ModuleEventPropertyTypeIdentifier);

	PropertyModule.RegisterCustomClassLayout("AnimNextVariableEntry",
		FOnGetDetailCustomizationInstance::CreateLambda([] { return MakeShared<FVariableCustomization>(); }));

	static const FName BindableTypes[] = {
		"BindableBool",      "BindableFloat",  "BindableDouble",
		"BindableInt32",     "BindableInt64",  "BindableByte",
		"BindableName",      "BindableVector", "BindableQuat",
		"BindableTransform", "BindableEnum",   "BindableStruct",
		"BindableObject"
	};
	for (FName TypeName : BindableTypes)
	{
		PropertyModule.RegisterCustomPropertyTypeLayout(
			TypeName,
			FOnGetPropertyTypeCustomizationInstance::CreateLambda(
				[] { return MakeShared<UE::UAF::Editor::FBindableValueCustomization>(); }));
	}

	GraphPanelPinFactory = MakeShared<FAnimNextGraphPanelPinFactoryEditor>();
	FEdGraphUtilities::RegisterVisualPinFactory(GraphPanelPinFactory);

	// Temp register asset previews while we still use the UUAFAssetPreviewFactorySubsystem
	FCoreDelegates::GetOnPostEngineInit().AddLambda([]()
	{
		if (GEditor)
		{
			GEditor->GetEditorSubsystem<UUAFAssetPreviewFactorySubsystem>()->AddAssetPreviewFactory(MakeShared<FAnimSequenceAssetPreviewFactory>());
		}
	});

	Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");

	WorkspaceEditorModule.OnRegisterTabsForEditor().AddLambda([](FWorkflowAllowedTabSet& TabFactories, const TSharedRef<FTabManager>& InTabManager, TSharedPtr<UE::Workspace::IWorkspaceEditor> InEditorPtr)
	{
		TSharedRef<FAnimNextCompilerResultsTabSummoner> CompilerResultsTabSummoner = MakeShared<FAnimNextCompilerResultsTabSummoner>(InEditorPtr);
		TabFactories.RegisterFactory(CompilerResultsTabSummoner);
		CompilerResultsTabSummoner->RegisterTabSpawner(InTabManager, nullptr);

		TSharedRef<FAnimNextFindTabSummoner> FindTabSummoner = MakeShared<FAnimNextFindTabSummoner>(InEditorPtr);
		TabFactories.RegisterFactory(FindTabSummoner);
		FindTabSummoner->RegisterTabSpawner(InTabManager, nullptr);

		TSharedRef<FAnimNextVariablesTabSummoner> VariablesTabSummoner = MakeShared<FAnimNextVariablesTabSummoner>(InEditorPtr);
		TabFactories.RegisterFactory(VariablesTabSummoner);
		VariablesTabSummoner->RegisterTabSpawner(InTabManager, nullptr);

		TSharedRef<FAnimNextFunctionsTabSummoner> FunctionsTabSummoner = MakeShared<FAnimNextFunctionsTabSummoner>(InEditorPtr);
		TabFactories.RegisterFactory(FunctionsTabSummoner);
		FunctionsTabSummoner->RegisterTabSpawner(InTabManager, nullptr);

		TSharedRef<FAnimNextFindAndReplaceTabSummoner> FindAndReplaceTabSummoner = MakeShared<FAnimNextFindAndReplaceTabSummoner>(InEditorPtr);
		TabFactories.RegisterFactory(FindAndReplaceTabSummoner);
		FindAndReplaceTabSummoner->RegisterTabSpawner(InTabManager, nullptr);

		TSharedRef<FVariableWatchTabSummoner> VariableWatchTabSummoner = MakeShared<FVariableWatchTabSummoner>(InEditorPtr);
		TabFactories.RegisterFactory(VariableWatchTabSummoner);
		VariableWatchTabSummoner->RegisterTabSpawner(InTabManager, nullptr);

		if (bEnableUAFBrowser)
		{
			TSharedRef<FUAFBrowserTabSummoner> DockedBrowserTabSummoner = MakeShared<FUAFBrowserTabSummoner>(InEditorPtr
				, ETabState::ClosedTab
				, FUAFBrowserTabSummoner::DockedTabID
				, LOCTEXT("UAFBrowserDockedLabel", "UAF Browser (Docked)"));
			TabFactories.RegisterFactory(DockedBrowserTabSummoner);
			DockedBrowserTabSummoner->RegisterTabSpawner(InTabManager, nullptr);

			TSharedRef<FUAFBrowserTabSummoner> SideBarBrowserTabSummoner = MakeShared<FUAFBrowserTabSummoner>(InEditorPtr
				, ETabState::SidebarTab
				, FUAFBrowserTabSummoner::SidebarTabID
				, LOCTEXT("UAFBrowserSidebarLabel", "UAF Browser (Sidebar)"));
			TabFactories.RegisterFactory(SideBarBrowserTabSummoner);
			SideBarBrowserTabSummoner->RegisterTabSpawner(InTabManager, nullptr);
		}
	});


	WorkspaceEditorModule.OnExtendTabs().AddLambda([](FLayoutExtender& InLayoutExtender, TSharedPtr<UE::Workspace::IWorkspaceEditor> InEditorPtr)
	{
		FTabManager::FTab CompilerResultsTab(FTabId(CompilerResultsTabName), ETabState::ClosedTab);
		InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::BottomMiddleDocumentArea), ELayoutExtensionPosition::After, CompilerResultsTab);

		FTabManager::FTab FindTab(FTabId(FindTabName), ETabState::ClosedTab);
		InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::BottomMiddleDocumentArea), ELayoutExtensionPosition::After, FindTab);

		FTabManager::FTab FindAndReplaceTab(FTabId(FindAndReplaceTabName), ETabState::ClosedTab);
		InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::BottomMiddleDocumentArea), ELayoutExtensionPosition::After, FindAndReplaceTab);

		FTabManager::FTab VariablesTab(FTabId(VariablesTabName), ETabState::OpenedTab);
		InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::BottomLeftDocumentArea), ELayoutExtensionPosition::After, VariablesTab);

		FTabManager::FTab FunctionsTab(FTabId(FunctionsTabName), ETabState::OpenedTab);
		InLayoutExtender.ExtendStack(Workspace::WorkspaceTabs::BottomLeftDocumentArea, ELayoutExtensionPosition::After, FunctionsTab);

		if (bEnableUAFBrowser)
		{
			FTabManager::FTab UAFBrowserDockedTab(FTabId(FUAFBrowserTabSummoner::DockedTabID), ETabState::ClosedTab);
			InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::BottomMiddleDocumentArea), ELayoutExtensionPosition::After, UAFBrowserDockedTab);

			const float SidebarSizeCoefficient = 0.2f;
			const bool bPinnedInSidebar = false;
			FTabManager::FTab UAFBrowserSidebarTab(FTabId(FUAFBrowserTabSummoner::SidebarTabID)
				, ETabState::SidebarTab
				, ESidebarLocation::Right
				, SidebarSizeCoefficient
				, bPinnedInSidebar);
			InLayoutExtender.ExtendLayout(FTabId(Workspace::WorkspaceTabs::RightSidebarDocumentArea), ELayoutExtensionPosition::After, UAFBrowserSidebarTab);
		}
	});

	UToolMenu* AnimNextEdGraphContextMenu = UToolMenus::Get()->ExtendMenu( "GraphEditor.GraphNodeContextMenu.AnimNextEdGraphNode");
	AnimNextEdGraphContextMenu->AddDynamicSection("FindAndReplace", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		if (UGraphNodeContextMenuContext* GraphNodeContextMenuContext = InMenu->Context.FindContext<UGraphNodeContextMenuContext>())
		{
			if (const URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNodeContextMenuContext->Node))
			{
				if (UAssetEditorToolkitMenuContext* AssetEditorToolkitMenuContext = InMenu->Context.FindContext<UAssetEditorToolkitMenuContext>())
				{
					if (TSharedPtr<FAssetEditorToolkit> SharedToolkit = AssetEditorToolkitMenuContext->Toolkit.Pin())
					{
						if (SharedToolkit->GetToolkitFName() == Workspace::IWorkspaceEditor::GetWorkspaceEditorToolkitName())
						{
							FAnimNextSoftVariableReference SearchReference;
							FAnimNextParamType SearchType;

							if (UUAFSharedVariableNode* SharedVariableNode = Cast<UUAFSharedVariableNode>(RigVMEdGraphNode->GetModelNode()))
							{
								const UUAFSharedVariables* Asset = SharedVariableNode->Asset.Get();
								const UScriptStruct* Struct = SharedVariableNode->Struct.Get();

								check(SharedVariableNode->Type == EAnimNextSharedVariablesType::Asset && Asset || SharedVariableNode->Type == EAnimNextSharedVariablesType::Struct && Struct);

								const UObject* Object = Asset ? Cast<UObject>(Asset) : Cast<UObject>(Struct);
								SearchReference = FAnimNextSoftVariableReference::FromName(SharedVariableNode->GetVariableName(), Object);
								if (SharedVariableNode->Type == EAnimNextSharedVariablesType::Asset)
								{
									const UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<const UUAFRigVMAssetEditorData, const UUAFSharedVariables>(Asset);						
									const UAnimNextVariableEntry* Entry = Cast<UAnimNextVariableEntry>(EditorData->FindEntry(SharedVariableNode->GetVariableName()));
									SearchType = Entry ? Entry->GetType() : FAnimNextParamType();
								}
								else if (SharedVariableNode->Type == EAnimNextSharedVariablesType::Struct)
								{
									if (const FProperty* Property = SharedVariableNode->Struct->FindPropertyByName(SharedVariableNode->GetVariableName()))
									{
										SearchType = FAnimNextParamType::FromProperty(Property);
									}
								}
							}
							else if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(RigVMEdGraphNode->GetModelNode()))
							{
								if (const UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(VariableNode->GetTypedOuter(UUAFRigVMAsset::StaticClass())))
								{
									const UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<const UUAFRigVMAssetEditorData, const UUAFRigVMAsset>(Asset); 
									if (const UAnimNextVariableEntry* Entry = Cast<UAnimNextVariableEntry>(EditorData->FindEntry(VariableNode->GetVariableName())))
									{
										SearchReference = FAnimNextSoftVariableReference::FromName(VariableNode->GetVariableName(), Asset);
										SearchType = Entry->GetType();
									}
									else
									{
										SearchReference.Reset();
										SearchType = FAnimNextParamType();
									}
								}
							}

							if (!SearchReference.IsNone() && SearchType.IsValid())
							{
								FToolMenuSection& FindReplaceSection = InMenu->AddSection("FindAndReplace", LOCTEXT("FindReferencesLabel", "Find References"));			
								FindReplaceSection.AddSubMenu("FindReferencesMenu", LOCTEXT("FindReferencesMenuLabel", "Find References..."), LOCTEXT("FindReferencesMenuToolTip", "Find references to selected variables by different means."),
								FNewToolMenuDelegate::CreateLambda(
									[SearchReference, SearchType](UToolMenu* InSubmenu)
									{									
										auto InvokeFindVariables = [SearchReference, SearchType](const FToolMenuContext& Context, ESearchScope Scope)
										{
											UAssetEditorToolkitMenuContext* AssetEditorToolkitMenuContext = Context.FindContext<UAssetEditorToolkitMenuContext>();										 
											if (TSharedPtr<Workspace::IWorkspaceEditor> SharedWorkspaceEditor = StaticCastSharedPtr<Workspace::IWorkspaceEditor>(AssetEditorToolkitMenuContext->Toolkit.Pin()))
											{
												if (TSharedPtr<SDockTab> FindAndReplaceTab = SharedWorkspaceEditor->GetTabManager()->TryInvokeTab(FTabId("FindAndReplaceTab")))
												{
													TSharedRef<IAnimAssetFindReplace> FindAndReplace = StaticCastSharedRef<IAnimAssetFindReplace>(FindAndReplaceTab->GetContent());
													FindAndReplace->SetCurrentProcessor(UAnimNextAssetFindReplaceVariables::StaticClass());
													UAnimNextAssetFindReplaceVariables* AnimNextFindAndReplaceProcessor = FindAndReplace->GetProcessor<UAnimNextAssetFindReplaceVariables>();
													
													AnimNextFindAndReplaceProcessor->SetWorkspaceEditor(SharedWorkspaceEditor.ToSharedRef());
													
													AnimNextFindAndReplaceProcessor->SetSearchScope(Scope);
													AnimNextFindAndReplaceProcessor->SetFindReference(SearchReference, SearchType);					
												}
											}
										};

										InSubmenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntry("FindGlobalReferences", FOutlinerCommands::Get().FindReferences->GetLabel(), FOutlinerCommands::Get().FindReferences->GetDescription(), UAnimNextAssetFindReplaceVariables::GetIconFromSearchScope(ESearchScope::Global), FToolUIAction(FToolMenuExecuteAction::CreateLambda(InvokeFindVariables, ESearchScope::Global))));

										InSubmenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntry("FindWorkspaceReferences", FOutlinerCommands::Get().FindReferencesInWorkspace->GetLabel(), FOutlinerCommands::Get().FindReferencesInWorkspace->GetDescription(), UAnimNextAssetFindReplaceVariables::GetIconFromSearchScope(ESearchScope::Workspace), FToolUIAction(FToolMenuExecuteAction::CreateLambda(InvokeFindVariables, ESearchScope::Workspace))));

										InSubmenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntry("FindAssetReferences", FOutlinerCommands::Get().FindReferencesInAsset->GetLabel(), FOutlinerCommands::Get().FindReferencesInAsset->GetDescription(), UAnimNextAssetFindReplaceVariables::GetIconFromSearchScope(ESearchScope::Asset), FToolUIAction(FToolMenuExecuteAction::CreateLambda(InvokeFindVariables, ESearchScope::Asset))));
									}),
									FToolUIActionChoice(),
									EUserInterfaceActionType::Button,
									false,
									FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Search"))
								);	
							}				
						}
					}
				}
			}			
		}
	}));

	UToolMenu* WorkspaceEditorToolbar = UToolMenus::Get()->ExtendMenu("AssetEditor.WorkspaceEditor.Toolbar");
	{
		const auto Handler = FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if (UAssetEditorToolkitMenuContext* Context = InMenu->FindContext<UAssetEditorToolkitMenuContext>())
			{
				if (TSharedPtr<FAssetEditorToolkit> ToolkitShared = Context->Toolkit.Pin())
				{
					TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<Workspace::IWorkspaceEditor>(ToolkitShared);

					auto GetIcon = [WeakWorkspaceEditor = TWeakPtr<UE::Workspace::IWorkspaceEditor>(WorkspaceEditor)]()
					{
						static const FName CompileStatusBackground("AssetEditor.CompileStatus.Background");
						static const FName CompileStatusUnknown("AssetEditor.CompileStatus.Overlay.Unknown");
						static const FName CompileStatusError("AssetEditor.CompileStatus.Overlay.Error");
						static const FName CompileStatusGood("AssetEditor.CompileStatus.Overlay.Good");
						static const FName CompileStatusWarning("AssetEditor.CompileStatus.Overlay.Warning");

						TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin();
						if(!WorkspaceEditor.IsValid())
						{
							return FSlateIcon();
						}

						UAnimNextWorkspaceEditorMode* EditorMode = Cast<UAnimNextWorkspaceEditorMode>(WorkspaceEditor->GetEditorModeManager().GetActiveScriptableMode(UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace));
						if(EditorMode == nullptr)
						{
							return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
						}

						ECompileStatus Status = EditorMode->GetLatestCompileStatus();
						switch (Status)
						{
						default:
						case ECompileStatus::Unknown:
						case ECompileStatus::Dirty:
							return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
						case ECompileStatus::Error:
							return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusError);
						case ECompileStatus::UpToDate:
							return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusGood);
						case ECompileStatus::Warning:
							return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusWarning);
						}
					};

					auto GetTooltip = [WeakWorkspaceEditor = TWeakPtr<UE::Workspace::IWorkspaceEditor>(WorkspaceEditor)]()
					{
						TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin();
						if(!WorkspaceEditor.IsValid())
						{
							return LOCTEXT("CompileGenericTooltip", "Compile all relevant assets");
						}

						UAnimNextWorkspaceEditorMode* EditorMode = Cast<UAnimNextWorkspaceEditorMode>(WorkspaceEditor->GetEditorModeManager().GetActiveScriptableMode(UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace));
						if(EditorMode == nullptr)
						{
							return LOCTEXT("CompileGenericTooltip", "Compile all relevant assets");
						}

						FTextBuilder TextBuilder;
						TextBuilder.AppendLine(LOCTEXT("CompileCurrentTooltip", "Compile the current asset"));

						if(EditorMode->IsAutoCompileChecked())
						{
							TextBuilder.AppendLine(LOCTEXT("AutoCompileEnabledTooltip", "Auto-compilation is enabled"));
						}
						else
						{
							TextBuilder.AppendLine(LOCTEXT("AutoCompileDisabledTooltip", "Auto-compilation is disabled"));
						}

						return TextBuilder.ToText();
					};
			
					FToolMenuEntry CompileEntry = FToolMenuEntry::InitToolBarButton(FRigVMCommands::Get().Compile, TAttribute<FText>(), MakeAttributeLambda(GetTooltip), MakeAttributeLambda(GetIcon));

					FToolMenuEntry CompileOptionsEntry = FToolMenuEntry::InitComboButton(
						"CompileOptionsCombo",
						FUIAction(),
						FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
						{
							FToolMenuSection& CompileOptionsSection = InToolMenu->AddSection("CompileOptionsSection", LOCTEXT("CompileOptionsComboLabel", "Compilation Options"));
							CompileOptionsSection.AddEntry(FToolMenuEntry::InitMenuEntry(FRigVMCommands::Get().AutoCompile, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon("RigVMEditorStyle", "RigVM.AutoCompileGraph")));
						}),
						FText::GetEmpty(),
						LOCTEXT("CompileOptionsComboTooltip", "Compilation Options"),
						FSlateIcon(),
						true);

					FToolMenuSection& CompileSection = InMenu->FindOrAddSection("CompileOptions");
					CompileSection.AddEntry(CompileEntry);
					CompileSection.AddEntry(CompileOptionsEntry);
				}
			}
		});
	
		WorkspaceEditorToolbar->AddDynamicSection("RigVMOperations", Handler, FToolMenuInsert("WorkspaceOperations", EToolMenuInsertType::After));
	}

	SystemTopLevelAssetPath = UUAFSystem::StaticClass()->GetClassPathName();
	UAFRigVMAssetTopLevelAssetPath = UUAFRigVMAsset::StaticClass()->GetClassPathName();
	
	RegisterWorkspaceDocumentTypes(WorkspaceEditorModule);

	WorkspaceEditorModule.OnRegisterWorkspaceDetailsCustomization().AddLambda([](const TWeakPtr<Workspace::IWorkspaceEditor>& InWorkspaceEditor, TSharedPtr<IDetailsView>& InDetailsView)
		{
			InDetailsView->RegisterInstancedCustomPropertyLayout(UAnimNextEdGraphNode::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([InWorkspaceEditor]()
				{
					return MakeShared<FAnimNextEdGraphNodeCustomization>(InWorkspaceEditor);
				}));

			InDetailsView->RegisterInstancedCustomPropertyLayout(UAnimNextEdGraph::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([InWorkspaceEditor]()
				{
					return MakeShared<FAnimNextEdGraphCustomization>();
				}));
		
			TArray<UScriptStruct*> StructsToCustomize = {
				TBaseStructure<FVector>::Get(),
				TBaseStructure<FVector2D>::Get(),
				TBaseStructure<FVector4>::Get(),
				TBaseStructure<FRotator>::Get(),
				TBaseStructure<FQuat>::Get(),
				TBaseStructure<FTransform>::Get(),
				TBaseStructure<FEulerTransform>::Get(),
			};
			for (UScriptStruct* StructToCustomize : StructsToCustomize)
			{
				InDetailsView->RegisterInstancedCustomPropertyTypeLayout(StructToCustomize->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([]()
					{
						return FRigVMGraphMathTypeDetailCustomization::MakeInstance();
					}));
			}
		});

	UE::UniversalObjectLocator::IUniversalObjectLocatorEditorModule& UolEditorModule = FModuleManager::LoadModuleChecked<UE::UniversalObjectLocator::IUniversalObjectLocatorEditorModule>("UniversalObjectLocatorEditor");
	UolEditorModule.RegisterLocatorEditor("AnimNextObjectFunction", MakeShared<FObjectFunctionLocatorEditor>());
	UolEditorModule.RegisterLocatorEditor("AnimNextObjectProperty", MakeShared<FObjectPropertyLocatorEditor>());
	UolEditorModule.RegisterLocatorEditor("AnimNextObjectCast", MakeShared<FObjectCastLocatorEditor>());
	UolEditorModule.RegisterLocatorEditor("UAFComponent", MakeShared<FComponentLocatorEditor>());
	UolEditorModule.RegisterLocatorEditor("AnimNextActor", MakeShared<FActorLocatorEditor>());

	UolEditorModule.RegisterEditorContext("UAFContext", MakeShared<FLocatorContext>());

	RegisterLocatorFragmentEditorType("Actor");
	RegisterLocatorFragmentEditorType("Asset");
	RegisterLocatorFragmentEditorType("AnimNextScope");
	RegisterLocatorFragmentEditorType("AnimNextGraph");
	RegisterLocatorFragmentEditorType("AnimNextObjectFunction");
	RegisterLocatorFragmentEditorType("AnimNextObjectProperty");
	RegisterLocatorFragmentEditorType("AnimNextObjectCast");
	RegisterLocatorFragmentEditorType("UAFComponent");
	RegisterLocatorFragmentEditorType("AnimNextActor");

	Workspace::IWorkspaceEditorModule& WorkspaceModule = FModuleManager::Get().LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
	WorkspaceModule.RegisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextGraphOutlinerData::StaticStruct()->GetFName()), MakeShared<FAnimNextGraphItemDetails>());
	WorkspaceModule.RegisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextCollapseGraphOutlinerData::StaticStruct()->GetFName()), MakeShared<FAnimNextCollapseNodeItemDetails>());

	FAnimNextGraphItemDetails::RegisterToolMenuExtensions();
	FAnimNextCollapseNodeItemDetails::RegisterToolMenuExtensions();

	const TSharedPtr<FAnimNextAssetItemDetails> AssetItemDetails = MakeShared<FAnimNextAssetItemDetails>();
	WorkspaceModule.RegisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextModuleOutlinerData::StaticStruct()->GetFName()), AssetItemDetails);
	WorkspaceModule.RegisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextSharedVariablesOutlinerData::StaticStruct()->GetFName()), AssetItemDetails);

	FAnimNextGraphItemDetails::RegisterToolMenuExtensions();
	FAnimNextAssetItemDetails::RegisterToolMenuExtensions();

	SupportedAssetClasses.Append(
		{
			UUAFSystem::StaticClass()->GetClassPathName(),
			UUAFSharedVariables::StaticClass()->GetClassPathName()
		});

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	FPersonaModule::FNotifyHostAssetParameters NotifyHostParameters;
	NotifyHostParameters.OnRemoveNotify = FPersonaModule::FNotifyHostAssetParameters::FOnRemoveNotify::CreateStatic(&UUAFRigVMAssetEditorData::HandleRemoveNotify);
	NotifyHostParameters.OnReplaceNotify = FPersonaModule::FNotifyHostAssetParameters::FOnReplaceNotify::CreateStatic(&UUAFRigVMAssetEditorData::HandleReplaceNotify);
	PersonaModule.RegisterNotifyHostAsset(UAFRigVMAssetTopLevelAssetPath, NotifyHostParameters);

	RegisterAssetCompilationHandler(UAFRigVMAssetTopLevelAssetPath, FAssetCompilationHandlerFactoryDelegate::CreateLambda([](UObject* InAsset)
	{
		return MakeShared<FAssetCompilationHandler>(InAsset);
	}));

	WorkspaceEditorModule.RegisterViewportControllerFactory(UUAFSystem::StaticClass(), []()
	{
		return MakeUnique<FSystemViewportController>();
	});
	
	// Hide the UAF content directory and its contents
	ToggleUAFContentHidden.Execute();

	IModularFeatures::Get().RegisterModularFeature(UE::PIE::IOutOfDateAssetCompilation::GetModularFeatureName(), &PIE_UAFCompilationHandler);
}

void FAnimNextEditorModule::ShutdownModule()
{
	if(FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout("AnimNextParamType");
		PropertyModule.UnregisterCustomPropertyTypeLayout("AnimNextVariableBinding");
		PropertyModule.UnregisterCustomPropertyTypeLayout("AnimNextVariableReference");
		PropertyModule.UnregisterCustomPropertyTypeLayout("NameProperty", ModuleEventPropertyTypeIdentifier);
		PropertyModule.UnregisterCustomPropertyTypeLayout("UAFSystemReference");
		PropertyModule.UnregisterCustomPropertyTypeLayout("InstancedStruct", UAFAssetDataPropertyIdentifier);
		PropertyModule.UnregisterCustomClassLayout("AnimNextVariableEntry");
		PropertyModule.UnregisterCustomClassLayout("UAFRigVMAssetEditorData");

		static const FName BindableTypes[] = {
			"BindableBool",      "BindableFloat",  "BindableDouble",
			"BindableInt32",     "BindableInt64",  "BindableByte",
			"BindableName",      "BindableVector", "BindableQuat",
			"BindableTransform", "BindableEnum",   "BindableStruct",
			"BindableObject"
		};
		for (FName TypeName : BindableTypes)
		{
			PropertyModule.UnregisterCustomPropertyTypeLayout(TypeName);
		}
	}

	UnregisterWorkspaceDocumentTypes();

	if(FModuleManager::Get().IsModuleLoaded("UniversalObjectLocatorEditor"))
	{
		UE::UniversalObjectLocator::IUniversalObjectLocatorEditorModule& UolEditorModule = FModuleManager::GetModuleChecked<UE::UniversalObjectLocator::IUniversalObjectLocatorEditorModule>("UniversalObjectLocatorEditor");
		UolEditorModule.UnregisterLocatorEditor("AnimNextObjectCast");
		UolEditorModule.UnregisterLocatorEditor("AnimNextObjectFunction");
		UolEditorModule.UnregisterLocatorEditor("AnimNextObjectProperty");
		UolEditorModule.UnregisterLocatorEditor("UAFComponent");
		UolEditorModule.UnregisterLocatorEditor("AnimNextActor");

		UolEditorModule.UnregisterEditorContext("UAFContext");
	}
	
	if (UObjectInitialized())
	{
		Workspace::IWorkspaceEditorModule& WorkspaceModule = FModuleManager::Get().LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
		WorkspaceModule.UnregisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextGraphOutlinerData::StaticStruct()->GetFName()));
		WorkspaceModule.UnregisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextCollapseGraphOutlinerData::StaticStruct()->GetFName()));
		FAnimNextGraphItemDetails::UnregisterToolMenuExtensions();
		WorkspaceModule.UnregisterWorkspaceItemDetails(Workspace::FOutlinerItemDetailsId(FAnimNextModuleOutlinerData::StaticStruct()->GetFName()));
		FAnimNextAssetItemDetails::UnregisterToolMenuExtensions();
	}

	UnregisterLocatorFragmentEditorType("Actor");
	UnregisterLocatorFragmentEditorType("Asset");
	UnregisterLocatorFragmentEditorType("AnimNextScope");
	UnregisterLocatorFragmentEditorType("AnimNextGraph");
	UnregisterLocatorFragmentEditorType("AnimNextObjectFunction");
	UnregisterLocatorFragmentEditorType("AnimNextObjectProperty");
	UnregisterLocatorFragmentEditorType("AnimNextObjectCast");
	UnregisterLocatorFragmentEditorType("UAFComponent");
	UnregisterLocatorFragmentEditorType("AnimNextActor");

	if(FModuleManager::Get().IsModuleLoaded("Persona"))
	{
		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		PersonaModule.UnregisterNotifyHostAsset(UAFRigVMAssetTopLevelAssetPath);
	}
	UnregisterAssetCompilationHandler(SystemTopLevelAssetPath);

#if UAF_TRACE_ENABLED
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &GRewindDebuggerUAF);
	IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GAnimNextModulesTrackCreator);
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &GUAFTraceModule);
#endif // UAF_TRACE_ENABLED

	FEdGraphUtilities::UnregisterVisualPinFactory(GraphPanelPinFactory);

	DisableAssetWizard();
	
	IModularFeatures::Get().UnregisterModularFeature(UE::PIE::IOutOfDateAssetCompilation::GetModularFeatureName(), &PIE_UAFCompilationHandler);
}

void FAnimNextEditorModule::RegisterLocatorFragmentEditorType(FName InLocatorFragmentEditorName)
{
	LocatorFragmentEditorNames.Add(InLocatorFragmentEditorName);
}

void FAnimNextEditorModule::UnregisterLocatorFragmentEditorType(FName InLocatorFragmentEditorName)
{
	LocatorFragmentEditorNames.Remove(InLocatorFragmentEditorName);
}

void FAnimNextEditorModule::AddWorkspaceSupportedAssetClass(const FTopLevelAssetPath& InClassAssetPath)
{
	if (InClassAssetPath.IsValid())
	{
		SupportedAssetClasses.AddUnique(InClassAssetPath);
	}	
}

void FAnimNextEditorModule::RemoveWorkspaceSupportedAssetClass(const FTopLevelAssetPath& InClassAssetPath)
{
	if (InClassAssetPath.IsValid())
	{
		SupportedAssetClasses.Remove(InClassAssetPath);
	}	
}

FDelegateHandle FAnimNextEditorModule::RegisterGraphMenuActionsProvider(const FOnCollectGraphMenuActionsDelegate& CollectDelegate)
{
	return OnCollectGraphMenuActionsDelegateImpl.Add(CollectDelegate);
}

void FAnimNextEditorModule::UnregisterGraphMenuActionsProvider(const FDelegateHandle& DelegateHandle)
{
	OnCollectGraphMenuActionsDelegateImpl.Remove(DelegateHandle);
}

void FAnimNextEditorModule::RegisterAssetCompilationHandler(const FTopLevelAssetPath& InClassPath, FAssetCompilationHandlerFactoryDelegate InAssetCompilationHandlerFactory)
{
	check(InAssetCompilationHandlerFactory.IsBound());
	AssetCompilationHandlerFactories.Add(InClassPath, InAssetCompilationHandlerFactory);
}

void FAnimNextEditorModule::UnregisterAssetCompilationHandler(const FTopLevelAssetPath& InClassPath)
{
	AssetCompilationHandlerFactories.Remove(InClassPath);
}

FDelegateHandle FAnimNextEditorModule::RegisterNodeDblClickHandler(const FNodeDblClickNotificationDelegate& InNodeDblClickNotificationDelegate)
{
	return OnNodeDblClickHandlerMulticast.Add(InNodeDblClickNotificationDelegate);
}

void FAnimNextEditorModule::UnregisterNodeDblClickHandler(const FDelegateHandle& InDelegateHandle)
{
	OnNodeDblClickHandlerMulticast.Remove(InDelegateHandle);
}

void FAnimNextEditorModule::CollectGraphMenuActions(const TWeakPtr<UE::Workspace::IWorkspaceEditor>& WorkspaceEditor, FGraphContextMenuBuilder& InContextMenuBuilder, const FActionMenuContextData& InActionMenuContextData)
{
	if (OnCollectGraphMenuActionsDelegateImpl.IsBound())
	{
		OnCollectGraphMenuActionsDelegateImpl.Broadcast(WorkspaceEditor, InContextMenuBuilder, InActionMenuContextData);
	}

	if (const URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(InContextMenuBuilder.CurrentGraph))
	{
		const URigVMGraph* Graph = RigVMEdGraph->GetModel();
		check(Graph);

		const IRigVMClientHost* RigVMClientHost = InActionMenuContextData.RigVMClientHost;
		check(RigVMClientHost);
		const URigVMHost* RigVMHost = InActionMenuContextData.RigVMHost;
		check(RigVMHost);
		URigVMController* RigVMController = InActionMenuContextData.RigVMController;
		check(RigVMController);
		const URigVMSchema* RigVMSchema = InActionMenuContextData.RigVMSchema;
		check(RigVMSchema);
		const UUAFRigVMAssetEditorData* EditorData = InActionMenuContextData.EditorData;
		check(EditorData);

		for (const FRigVMFunction& Function : FRigVMRegistry::Get().GetFunctions())
		{
			if (RigVMSchema == nullptr || !RigVMSchema->SupportsUnitFunction(RigVMController, &Function))
			{
				continue;
			}

			UScriptStruct* Struct = Function.Struct;
			if (Struct == nullptr)
			{
				continue;
			}

			// skip deprecated units
			if (Function.Struct->HasMetaData(FRigVMStruct::DeprecatedMetaName))
			{
				continue;
			}

			// skip hidden units
			if (Function.Struct->HasMetaData(FRigVMStruct::HiddenMetaName))
			{
				continue;
			}

			// Disallow trait stacks to be added here, as it will be added at AnimNextAnimGraph with a custom node class
			static const UScriptStruct* TraitStackStruct = FindObjectChecked<UScriptStruct>(nullptr, TEXT("/Script/UAFAnimGraph.RigUnit_AnimNextTraitStack"));
			if (Struct && Struct->IsChildOf(TraitStackStruct))
			{
				continue;
			}

			UE::UAF::Editor::FUtils::AddSchemaRigUnitAction(URigVMUnitNode::StaticClass(), Struct, Function, InContextMenuBuilder);
		}

		for (const FRigVMDispatchFactory* Factory : FRigVMRegistry::Get().GetFactories())
		{
			if (RigVMSchema == nullptr || !RigVMSchema->SupportsDispatchFactory(RigVMController, Factory))
			{
				continue;
			}

			const FRigVMTemplate* Template = Factory->GetTemplate();
			if (Template == nullptr)
			{
				continue;
			}

			// skip deprecated factories
			if (Factory->GetScriptStruct()->HasMetaData(FRigVMStruct::DeprecatedMetaName))
			{
				continue;
			}

			// skip hidden factories
			if (Factory->GetScriptStruct()->HasMetaData(FRigVMStruct::HiddenMetaName))
			{
				continue;
			}

			FText NodeCategory = FText::FromString(Factory->GetCategory());
			FText MenuDesc = FText::FromString(Factory->GetNodeTitle(FRigVMTemplateTypeMap()));
			FText ToolTip = Factory->GetNodeTooltip(FRigVMTemplateTypeMap());

			InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_DispatchFactory>(Template->GetNotation(), NodeCategory, MenuDesc, ToolTip));
		};

		if (URigVMFunctionLibrary* LocalFunctionLibrary = RigVMClientHost->GetLocalFunctionLibrary())
		{
			const FSoftObjectPath LocalLibrarySoftPath = LocalFunctionLibrary->GetFunctionHostObjectPath();

			TArray<URigVMLibraryNode*> Functions = LocalFunctionLibrary->GetFunctions();
			for (URigVMLibraryNode* FunctionLibraryNode : Functions)
			{
				if (LocalFunctionLibrary->IsFunctionPublic(FunctionLibraryNode->GetFName()))	// Public functions will be added when processing asset registry exports
				{
					continue;
				}
				const FText NodeCategory = FText::FromString(FunctionLibraryNode->GetNodeCategory());
				const FText MenuDesc = FText::FromString(FunctionLibraryNode->GetName());
				const FText ToolTip = FunctionLibraryNode->GetToolTipText();

				InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_Function>(FunctionLibraryNode, NodeCategory, MenuDesc, ToolTip));
			}
		}

		TMap<FAssetData, FRigVMGraphFunctionHeaderArray> FunctionExports;
		UncookedOnly::FUtils::GetExportedFunctionsFromAssetRegistry(UE::UAF::AnimNextPublicGraphFunctionsExportsRegistryTag, FunctionExports);
		// TODO: Ideally we can filter functions by schema or execute context, but right now we dont expose the schema and function execute contexts are
		// all FRigVMExecuteContext, rather than the 'most derived' context in the function.
		//	UncookedOnly::FUtils::GetExportedFunctionsFromAssetRegistry(UE::UAF::ControlRigAssetPublicGraphFunctionsExportsRegistryTag, FunctionExports);

		for (const auto& Export : FunctionExports.Array())
		{
			for (const FRigVMGraphFunctionHeader& FunctionHeader : Export.Value.Headers)
			{
				if (FunctionHeader.LibraryPointer.IsValid())
				{
					const FText NodeCategory = FText::FromString(FunctionHeader.Category);
					const FText MenuDesc = FText::FromString(FunctionHeader.NodeTitle);
					const FText ToolTip = FunctionHeader.GetTooltip();

					InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_Function>(FunctionHeader, NodeCategory, MenuDesc, ToolTip));
				}
			}
		}

		TArray<UUAFRigVMAssetEditorData::FVariableInfo> VariableInfoArray;
		EditorData->GetAllVariables(VariableInfoArray, UUAFRigVMAssetEditorData::EVariableRecursion::IncludeShared, UUAFRigVMAssetEditorData::EVariableAccessFilter::All);

		TArray<const UUAFRigVMAssetEditorData::FVariableInfo> ProcessedVariables;
		
		for (const UUAFRigVMAssetEditorData::FVariableInfo& VariableInfo : VariableInfoArray)
		{
			InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_Variable>(VariableInfo.Name, VariableInfo.SourceObject, VariableInfo.Type, FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Set));
			InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_Variable>(VariableInfo.Name, VariableInfo.SourceObject, VariableInfo.Type, FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Get));

			ProcessedVariables.AddUnique(VariableInfo);
		}

		InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_AddComment>());

		// Collect all variables available within the current working context (asset export chain recursing upwards to module)
		if(FWorkspaceOutlinerAssetReferenceItemData::IsAssetReference(InActionMenuContextData.Export))
		{			
			TArray<FSoftObjectPath> AssetPaths;
			InActionMenuContextData.Export.GetAssetPaths(AssetPaths);
		
			for (const FSoftObjectPath& SoftObjectPath : AssetPaths)
			{
				if(UUAFRigVMAsset* RigVMAsset = Cast<UUAFRigVMAsset>(SoftObjectPath.TryLoad()))
				{
					if(UUAFRigVMAssetEditorData* AssetEditorData = UncookedOnly::FUtils::GetEditorData(RigVMAsset))
					{
						// Skip editor-data if its already been handled
						if (AssetEditorData != EditorData)
						{
							VariableInfoArray.Reset();
							AssetEditorData->GetAllVariables(VariableInfoArray, UUAFRigVMAssetEditorData::EVariableRecursion::IncludeShared, UUAFRigVMAssetEditorData::EVariableAccessFilter::All);
							for (const UUAFRigVMAssetEditorData::FVariableInfo& VariableInfo : VariableInfoArray)
							{
								if (!ProcessedVariables.Contains(VariableInfo))
								{
									InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_Variable>(VariableInfo.Name, VariableInfo.SourceObject, VariableInfo.Type, FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Set));
									InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_Variable>(VariableInfo.Name, VariableInfo.SourceObject, VariableInfo.Type, FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Get));
								}
							}
						}
					}
				}
			}
		}

		// Promotion to variable
		if (InContextMenuBuilder.FromPin != nullptr)
		{
			const FAnimNextParamType ParamType = UE::UAF::UncookedOnly::FUtils::GetParamTypeFromPinType(InContextMenuBuilder.FromPin->PinType);
			if (ParamType.IsValid())
			{
				InContextMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_PromoteToVariable>());
			}
		}
	}	
}

void FAnimNextEditorModule::RegisterWorkspaceDocumentTypes(Workspace::IWorkspaceEditorModule& WorkspaceEditorModule)
{
	// --- AnimNextModule ---
	Workspace::FObjectDocumentArgs AnimNextModuleDocumentArgs(
			Workspace::FOnRedirectWorkspaceContext::CreateLambda([](const Workspace::FWorkspaceDocument& Document) -> UObject*
			{
				UObject* InObject = Document.GetObject();
				if(UUAFSystem* Module = Cast<UUAFSystem>(InObject))
				{
				    UUAFSystem_EditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFSystem_EditorData>(Module);

					const TInstancedStruct<FWorkspaceOutlinerItemData>& ExportData = Document.Export.GetData();
					if (ExportData.GetScriptStruct() == FAnimNextGraphOutlinerData::StaticStruct())
					{
						const FAnimNextGraphOutlinerData& TypeData = ExportData.Get<FAnimNextGraphOutlinerData>();
						if (UAnimNextEventGraphEntry* Entry = Cast<UAnimNextEventGraphEntry>(TypeData.GetEntry()))
						{
							return Entry->GetEdGraph();
						}
					}
					
				    // Redirect to the inner graph, if any
				    UAnimNextEventGraphEntry* EventGraphEntry = EditorData->FindFirstEntryOfType<UAnimNextEventGraphEntry>();
				    if(EventGraphEntry)
				    {
					    return EventGraphEntry->GetEdGraph();
				    }
    
				    return nullptr;
				}
				
				return InObject;
			}));

	AnimNextModuleDocumentArgs.DocumentEditorMode = UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace;

	WorkspaceEditorModule.RegisterObjectDocumentType(SystemTopLevelAssetPath, AnimNextModuleDocumentArgs);

	// --- AnimNextEdGraph ---
	Workspace::FGraphDocumentWidgetArgs GraphArgs;
	GraphArgs.SpawnLocation = Workspace::WorkspaceTabs::TopMiddleDocumentArea;
	GraphArgs.OnCreateActionMenu = Workspace::FOnCreateActionMenu::CreateLambda([this](const Workspace::FWorkspaceEditorContext& InContext, UEdGraph* InGraph, const FVector2D& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
	{
		const FString WorkspaceAssetPath = FExternalPackageHelper::GetExternalObjectsPath(InContext.WorkspaceEditor->GetPackageName());
		TWeakPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditorWeak = InContext.WorkspaceEditor.ToWeakPtr();

		TSharedRef<SActionMenu> ActionMenu = SNew(SActionMenu, InGraph, InContext.Document.Export)
			.AutoExpandActionMenu(bAutoExpand)
			.NewNodePosition(InNodePosition)
			.DraggedFromPins(InDraggedPins)
			.OnClosedCallback(InOnMenuClosed)
			.OnCollectGraphActionsCallback(SActionMenu::FCollectAllGraphActions::CreateLambda([this, WorkspaceEditorWeak](FGraphContextMenuBuilder& InContextMenuBuilder, const FActionMenuContextData& InActionMenuContextData)
				{
					CollectGraphMenuActions(WorkspaceEditorWeak, InContextMenuBuilder, InActionMenuContextData);
				}));

		TSharedPtr<SWidget> FilterTextBox = StaticCastSharedRef<SWidget>(ActionMenu->GetFilterTextBox());
		return FActionMenuContent(StaticCastSharedRef<SWidget>(ActionMenu), FilterTextBox);
	});
	GraphArgs.OnNodeTextCommitted = Workspace::FOnNodeTextCommitted::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
	{
		URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(NodeBeingChanged->GetGraph());
		if (RigVMEdGraph == nullptr)
		{
			return;
		}

		UEdGraphNode_Comment* CommentBeingChanged = Cast<UEdGraphNode_Comment>(NodeBeingChanged);
		if (CommentBeingChanged != nullptr)
		{
			RigVMEdGraph->GetController()->SetCommentTextByName(CommentBeingChanged->GetFName(), NewText.ToString(), CommentBeingChanged->FontSize, CommentBeingChanged->bCommentBubbleVisible, CommentBeingChanged->bColorCommentBubble, true, true);
			return;
		}
		
		URigVMEdGraphNode* RigVMNodeBeingChanged = Cast<URigVMEdGraphNode>(NodeBeingChanged);
		if (RigVMNodeBeingChanged != nullptr)
		{
			RigVMEdGraph->GetController()->SetNodeTitle(RigVMNodeBeingChanged->GetModelNode(), NewText.ToString(), true, true, true);
			return;
		}
	});
	GraphArgs.OnCanDeleteSelectedNodes = Workspace::FOnCanPerformActionOnSelectedNodes::CreateLambda([this](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& InSelectedNodes)
	{
		if(InSelectedNodes.Num() > 0)
		{
			for(UObject* NodeObject : InSelectedNodes)
			{
				// If any nodes allow deleting, then do not disable the delete option
				const UEdGraphNode* Node = Cast<UEdGraphNode>(NodeObject);
				if(Node && Node->CanUserDeleteNode())
				{
					return true;
				}
			}
		}
		return false;
	});
	GraphArgs.OnDeleteSelectedNodes = Workspace::FOnPerformActionOnSelectedNodes::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& InSelectedNodes)
	{
		if(InSelectedNodes.IsEmpty())
		{
			return;
		}

		URigVMController* Controller = nullptr;
		
		bool bRelinkPins = false;
		TArray<URigVMNode*> NodesToRemove;

		for (FGraphPanelSelectionSet::TConstIterator NodeIt(InSelectedNodes); NodeIt; ++NodeIt)
		{
			if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
			{
				URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(Node->GetGraph());
				if (RigVMEdGraph == nullptr)
				{
					continue;
				}

				if(Controller == nullptr)
				{
					Controller = RigVMEdGraph->GetController();
				}
				
				if (Node->CanUserDeleteNode())
				{
					if (const URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(Node))
					{
						bRelinkPins = bRelinkPins || FSlateApplication::Get().GetModifierKeys().IsShiftDown();

						if(URigVMGraph* Model = RigVMEdGraph->GetModel())
						{
							if(URigVMNode* ModelNode = Model->FindNodeByName(*RigVMEdGraphNode->GetModelNodePath()))
							{
								NodesToRemove.Add(ModelNode);
							}
						}
					}
					else if (const UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(Node))
					{
						if(URigVMGraph* Model = RigVMEdGraph->GetModel())
						{
							if(URigVMNode* ModelNode = Model->FindNodeByName(CommentNode->GetFName()))
							{
								NodesToRemove.Add(ModelNode);
							}
						}
					}
					else
					{
						Node->GetGraph()->RemoveNode(Node);
					}
				}
			}
		}

		if(NodesToRemove.IsEmpty() || Controller == nullptr)
		{
			return;
		}

		Controller->OpenUndoBracket(TEXT("Delete selected nodes"));
		if(bRelinkPins && NodesToRemove.Num() == 1)
		{
			Controller->RelinkSourceAndTargetPins(NodesToRemove[0], true);;
		}
		Controller->RemoveNodes(NodesToRemove, true, true);
		Controller->CloseUndoBracket();
		InContext.WorkspaceEditor->SetDetailsObjects({});
	});
	GraphArgs.OnCanCopySelectedNodes = Workspace::FOnCanPerformActionOnSelectedNodes::CreateLambda([this](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& InSelectedNodes)
	{
		if(InSelectedNodes.Num() > 0)
		{
			return true;
		}
		return false;
	});
	GraphArgs.OnCopySelectedNodes = Workspace::FOnPerformActionOnSelectedNodes::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& InSelectedNodes)
	{
		if(InSelectedNodes.IsEmpty())
		{
			return;
		}

		URigVMEdGraph* RigVMEdGraph = InContext.Document.GetTypedObject<URigVMEdGraph>();
		if (RigVMEdGraph == nullptr)
		{
			return;
		}

		URigVMController* Controller = RigVMEdGraph->GetController();

		FString ExportedText = Controller->ExportSelectedNodesToText();
		FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
	});
	GraphArgs.OnCanPasteNodes = Workspace::FOnCanPasteNodes::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, const FString& InImportData)
	{
		bool bCanUserImportNodes = false;

		if (!InImportData.IsEmpty())
		{
			bCanUserImportNodes = true;
		}

		return bCanUserImportNodes;
	});
	GraphArgs.OnPasteNodes = Workspace::FOnPasteNodes::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, const FVector2D& InPasteLocation, const FString& InImportData)
	{
		if(InImportData.IsEmpty())
		{
			return;
		}

		URigVMEdGraph* RigVMEdGraph = InContext.Document.GetTypedObject<URigVMEdGraph>();
		if (RigVMEdGraph == nullptr)
		{
			return;
		}

		if (IRigVMClientHost* RigVMClientHost = RigVMEdGraph->GetImplementingOuter<IRigVMClientHost>())
		{
			FString TextToImport;
			FPlatformApplicationMisc::ClipboardPaste(TextToImport);
			URigVMController* Controller = RigVMEdGraph->GetController();

			Controller->OpenUndoBracket(TEXT("Pasted Nodes."));

			if (UE::RigVM::Editor::Tools::PasteNodes(InPasteLocation, TextToImport, Controller, RigVMEdGraph->GetModel(), RigVMClientHost->GetLocalFunctionLibrary(), RigVMClientHost->GetRigVMGraphFunctionHost().GetInterface(), true, true))
			{
				Controller->CloseUndoBracket();
			}
			else
			{
				Controller->CancelUndoBracket();
			}
		}
	});
	UE::Workspace::FOnCanPerformActionOnSelectedNodes& OnCanCopySelectedNodes = GraphArgs.OnCanCopySelectedNodes;
	UE::Workspace::FOnCanPerformActionOnSelectedNodes& OnCanDeleteSelectedNodes = GraphArgs.OnCanDeleteSelectedNodes;
	GraphArgs.OnCanCutSelectedNodes = Workspace::FOnCanPerformActionOnSelectedNodes::CreateLambda([OnCanCopySelectedNodes, OnCanDeleteSelectedNodes](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& InSelectedNodes)
		{
			bool bCanUserCopyNode = false;

			if (OnCanCopySelectedNodes.IsBound() && OnCanDeleteSelectedNodes.IsBound())
			{
				bCanUserCopyNode = OnCanCopySelectedNodes.Execute(InContext, InSelectedNodes) && OnCanDeleteSelectedNodes.Execute(InContext, InSelectedNodes);
			}

			return bCanUserCopyNode;
		});
	UE::Workspace::FOnPerformActionOnSelectedNodes& OnCopySelectedNodes = GraphArgs.OnCopySelectedNodes;
	UE::Workspace::FOnPerformActionOnSelectedNodes& OnDeleteSelectedNodes = GraphArgs.OnDeleteSelectedNodes;
	GraphArgs.OnCutSelectedNodes = Workspace::FOnPerformActionOnSelectedNodes::CreateLambda([OnCopySelectedNodes, OnDeleteSelectedNodes](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& InSelectedNodes)
		{
			if (InSelectedNodes.IsEmpty())
			{
				return;
			}

			URigVMEdGraph* RigVMEdGraph = InContext.Document.GetTypedObject<URigVMEdGraph>();
			if (RigVMEdGraph == nullptr)
			{
				return;
			}


			if (OnCopySelectedNodes.IsBound() && OnDeleteSelectedNodes.IsBound())
			{
				URigVMController* Controller = RigVMEdGraph->GetController();

				OnCopySelectedNodes.Execute(InContext, InSelectedNodes);

				Controller->OpenUndoBracket(TEXT("Cut Nodes."));
				OnDeleteSelectedNodes.Execute(InContext, InSelectedNodes);
				Controller->CloseUndoBracket();

				InContext.WorkspaceEditor->SetDetailsObjects({});
			}
		});
	UE::Workspace::FOnCanPasteNodes& OnCanPasteNodes = GraphArgs.OnCanPasteNodes;
	GraphArgs.OnCanDuplicateSelectedNodes = Workspace::FOnCanPerformActionOnSelectedNodes::CreateLambda([OnCanCopySelectedNodes, OnCanPasteNodes](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& InSelectedNodes)
		{
			bool bCanUserCopyNode = false;

			if (OnCanCopySelectedNodes.IsBound() && OnCanPasteNodes.IsBound())
			{
				FString TextToImport;
				FPlatformApplicationMisc::ClipboardPaste(TextToImport);

				bCanUserCopyNode = OnCanCopySelectedNodes.Execute(InContext, InSelectedNodes) && OnCanPasteNodes.Execute(InContext, TextToImport);
			}

			return bCanUserCopyNode;
		});
	UE::Workspace::FOnPasteNodes& OnPasteNodes = GraphArgs.OnPasteNodes;
	GraphArgs.OnDuplicateSelectedNodes = Workspace::FOnDuplicateSelectedNodes::CreateLambda([this, OnCopySelectedNodes, OnPasteNodes](const Workspace::FWorkspaceEditorContext& InContext, const FVector2D& InPasteLocation, const FGraphPanelSelectionSet& InSelectedNodes)
		{
			if (InSelectedNodes.IsEmpty())
			{
				return;
			}

			URigVMEdGraph* RigVMEdGraph = InContext.Document.GetTypedObject<URigVMEdGraph>();
			if (RigVMEdGraph == nullptr)
			{
				return;
			}

			if (OnCopySelectedNodes.IsBound() && OnPasteNodes.IsBound())
			{
				OnCopySelectedNodes.Execute(InContext, InSelectedNodes);

				FString TextToImport;
				FPlatformApplicationMisc::ClipboardPaste(TextToImport);

				URigVMController* Controller = RigVMEdGraph->GetController();

				Controller->OpenUndoBracket(TEXT("Duplicate Nodes."));
				OnPasteNodes.Execute(InContext, InPasteLocation, TextToImport);
				Controller->CloseUndoBracket();
			}
		});
	GraphArgs.OnGraphSelectionChanged = Workspace::FOnGraphSelectionChanged::CreateLambda([this](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& NewSelection)
	{
		URigVMEdGraph* RigVMEdGraph = InContext.Document.GetTypedObject<URigVMEdGraph>();
		if (RigVMEdGraph == nullptr)
		{
			return;
		}

		if (RigVMEdGraph->bIsSelecting || GIsTransacting)
		{
			return;
		}

		TGuardValue<bool> SelectGuard(RigVMEdGraph->bIsSelecting, true);

		TArray<FName> NodeNamesToSelect;
		TArray<UObject*> ObjectsToSelect;
		for (UObject* Object : NewSelection)
		{
			if (const URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(Object))
			{
				NodeNamesToSelect.Add(RigVMEdGraphNode->GetModelNodeName());
				if (const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(RigVMEdGraphNode->GetModelNode()))
				{
					if (const URigVMLibraryNode* LibraryNode = FunctionReferenceNode->LoadReferencedNode())
					{
						if (const URigVMGraph* ContainedGraph = LibraryNode->GetContainedGraph())
						{					
							if (const IRigVMClientHost* Host = LibraryNode->GetImplementingOuter<IRigVMClientHost>())
							{
								if (URigVMEdGraph* ContainedEdGraph = Cast<URigVMEdGraph>(Host->GetEditorObjectForRigVMGraph(ContainedGraph)))
								{							
									ObjectsToSelect.Add(ContainedEdGraph);
									continue;
								}
							}						
						}
					}
				}
				else if (const URigVMFunctionInterfaceNode* InterfaceNode = Cast<URigVMFunctionInterfaceNode>(RigVMEdGraphNode->GetModelNode()))
				{
					if (const URigVMCollapseNode* CollapseNodeOwner = InterfaceNode->FindReferencedNode())
					{
						const URigVMGraph* ContainedGraph = CollapseNodeOwner->GetContainedGraph();
						const IRigVMClientHost* RigVMClientHost = CollapseNodeOwner->GetImplementingOuter<IRigVMClientHost>();
						if (ContainedGraph && RigVMClientHost)
						{
							if (URigVMEdGraph* ContainedEdGraph = Cast<URigVMEdGraph>(RigVMClientHost->GetEditorObjectForRigVMGraph(ContainedGraph)))
							{							
								ObjectsToSelect.Add(ContainedEdGraph);
								continue;
							}
						}
					}
				}
				else if (const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(RigVMEdGraphNode->GetModelNode()))
				{
					const UUAFRigVMAssetEditorData* EditorData = nullptr;
					FGuid VariableGUID;
					if (const UUAFSharedVariableNode* SharedVarNode = Cast<UUAFSharedVariableNode>(VariableNode))
					{
						EditorData = SharedVarNode->Asset ? UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(SharedVarNode->Asset.Get()) : nullptr;
						VariableGUID = SharedVarNode->CachedGuid;
					}
					else
					{
						EditorData = VariableNode->GetTypedOuter<UUAFRigVMAssetEditorData>();
						VariableGUID = VariableNode->GetVariableGuid();
					}

					if (VariableGUID.IsValid())
					{
						if (EditorData)
						{
							UAnimNextVariableEntry* Entry = nullptr;
							EditorData->ForEachEntryOfType<UAnimNextVariableEntry>([VariableGUID, &Entry](UAnimNextVariableEntry* VariableEntry)
							{
								if (VariableEntry->GetGuid() == VariableGUID)
								{
									Entry = VariableEntry;
									return false;
								}
						
								return true;
							});

							if (Entry)
							{
								ObjectsToSelect.Add(Entry);
								continue;
							}
							else
							{
								UE_LOGFMT(LogAnimation, Warning, "Could not find a valid entry for [{0}] in [{1}]", VariableNode->GetVariableName(), EditorData->GetFName());
							}
						}
						else
						{
							UE_LOGFMT(LogAnimation, Warning, "Could not resolve valid editor data for [{0}]", VariableNode->GetVariableName());
						}
					}
					else
					{
						UE_LOGFMT(LogAnimation, Warning, "Could not resolve variable GUID for [{0}]", VariableNode->GetVariableName());
					}
					
				}
			}
			else if (const UEdGraphNode* Node = Cast<UEdGraphNode>(Object))
			{
				NodeNamesToSelect.Add(Node->GetFName());
			}
			
			ObjectsToSelect.Add(Object);
		}
		RigVMEdGraph->GetController()->SetNodeSelection(NodeNamesToSelect, true, true);
			
		InContext.WorkspaceEditor->SetDetailsObjects(ObjectsToSelect);
	});
	GraphArgs.OnNodeDoubleClicked = Workspace::FOnNodeDoubleClicked::CreateLambda([this](const Workspace::FWorkspaceEditorContext& InContext, const UEdGraphNode* InNode)
	{
		if (const UAnimNextEdGraphNode* RigVMEdGraphNode = Cast<UAnimNextEdGraphNode>(InNode))
		{
			const URigVMNode* ModelNode = RigVMEdGraphNode->GetModelNode();

			if (OnNodeDblClickHandlerMulticast.IsBound())
			{
				OnNodeDblClickHandlerMulticast.Broadcast(InContext, InNode);
			}

			if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(ModelNode))
			{
				URigVMGraph* ContainedGraph = LibraryNode->GetContainedGraph();

				if (const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(LibraryNode))
				{
					if (URigVMLibraryNode* ReferencedNode = FunctionReferenceNode->LoadReferencedNode())
					{
						ContainedGraph = ReferencedNode->GetContainedGraph();
					}
				}

				if (ContainedGraph)
				{
					if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = InContext.WorkspaceEditor)
					{
						if (IRigVMClientHost* RigVMClientHost = ContainedGraph->GetImplementingOuter<IRigVMClientHost>())
						{
							if (const UObject* EditorObject = RigVMClientHost->GetEditorObjectForRigVMGraph(ContainedGraph))
							{
								const FSoftObjectPath GraphSoftPath = EditorObject;
								if (GraphSoftPath.IsValid() && GraphSoftPath.IsSubobject())
								{
									FDocumentTracker::EOpenDocumentCause OpenCause = FDocumentTracker::NavigatingCurrentDocument;
									if (const UUAFEditorUserSettings* UAFEditorUserSettings = GetMutableDefault<UUAFEditorUserSettings>())
									{
										if (UAFEditorUserSettings->bAutoOpenFunctionsInSeparateTabs)
										{
											OpenCause = FDocumentTracker::OpenNewDocument;
										}
									}
									
									FWorkspaceOutlinerItemExport Export = FWorkspaceOutlinerItemExport(*GraphSoftPath.GetSubPathUtf8String(), GraphSoftPath.GetWithoutSubPath());
									WorkspaceEditor->OpenExports({ Export }, OpenCause);
								}
							}
						}
					}
				}
			}
		}
	});
	GraphArgs.OnGraphDocumentCreated = Workspace::FOnGraphDocumentCreated::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, TSharedPtr<SWidget> InGraphWidget)
	{
		if (UAnimNextEdGraph* EdGraph = InContext.Document.GetTypedObject<UAnimNextEdGraph>())
		{
			if(UUAFRigVMAssetEditorData* EditorData = EdGraph->GetTypedOuter<UUAFRigVMAssetEditorData>())
			{
				EditorData->RigVMGraphModifiedEvent.AddSPLambda(InGraphWidget.Get(), [EditorData, WeakWorkspaceEditor = TWeakPtr<Workspace::IWorkspaceEditor>(InContext.WorkspaceEditor)](ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
				{
					switch (InNotifType)
					{
						case ERigVMGraphNotifType::NodeRemoved:
						{
							if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
							{
								if (UEdGraph* EdGraph = Cast<UEdGraph>(EditorData->GetEditorObjectForRigVMGraph(CollapseNode->GetContainedGraph())))
								{
									if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin())
									{
										WorkspaceEditor->CloseObjects({ EdGraph });
										WorkspaceEditor->SetDetailsObjects({});
									}
								}
							}
							else if(URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(InSubject))
							{
								if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin())
								{
									WorkspaceEditor->SetDetailsObjects({});
								}
							}
							break;
						}
						default:
						{
							break;
						}
					}
				});
			}
		}
	});
	GraphArgs.OnCanOpenInNewTab = Workspace::FOnCanPerformActionOnSelectedNodes::CreateLambda([this](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& InSelectedNodes)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(InSelectedNodes); NodeIt; ++NodeIt)
		{
			URigVMGraph* ContainedGraph = nullptr;
			if (const UAnimNextEdGraphNode* RigVMEdGraphNode = Cast<UAnimNextEdGraphNode>(*NodeIt))
			{
				const URigVMNode* ModelNode = RigVMEdGraphNode->GetModelNode();
				if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(ModelNode))
				{
					ContainedGraph = LibraryNode->GetContainedGraph();
					if (const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(LibraryNode))
					{
						if (const URigVMLibraryNode* ReferencedNode = FunctionReferenceNode->LoadReferencedNode())
						{
							ContainedGraph = ReferencedNode->GetContainedGraph();
						}
					}
				}

				if (ContainedGraph)
				{
					return true;
				}
			}
		}

		return false;
	});
	GraphArgs.OnOpenInNewTab = Workspace::FOnPerformActionOnSelectedNodes::CreateLambda([this](const Workspace::FWorkspaceEditorContext& InContext, const FGraphPanelSelectionSet& InSelectedNodes)
	{		
		TArray<FWorkspaceOutlinerItemExport> ExportsToOpen;
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(InSelectedNodes); NodeIt; ++NodeIt)
		{
			if (const UAnimNextEdGraphNode* RigVMEdGraphNode = Cast<UAnimNextEdGraphNode>(*NodeIt))
			{
				const URigVMNode* ModelNode = RigVMEdGraphNode->GetModelNode();
				if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(ModelNode))
				{
					const URigVMGraph* ContainedGraph = nullptr;
					if (const URigVMCollapseNode* CollapsedNode = Cast<URigVMCollapseNode>(ModelNode))
					{
						ContainedGraph = CollapsedNode->GetContainedGraph();
					}
					else if(const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(ModelNode))
					{
						if (const URigVMLibraryNode* ReferencedNode = FunctionReferenceNode->LoadReferencedNode())
						{
							ContainedGraph = ReferencedNode->GetContainedGraph();
						}
					}

					if (ContainedGraph)
					{
						if (const IRigVMClientHost* RigVMClientHost = ContainedGraph->GetImplementingOuter<IRigVMClientHost>())
						{
							if (const UObject* EditorObject = RigVMClientHost->GetEditorObjectForRigVMGraph(ContainedGraph))
							{
								const FSoftObjectPath GraphSoftPath = EditorObject;
								if (GraphSoftPath.IsValid() && GraphSoftPath.IsSubobject())
								{
									const FWorkspaceOutlinerItemExport ParentExport = InContext.Document.Export;
									FWorkspaceOutlinerItemExport& Export = ExportsToOpen.Add_GetRef(FWorkspaceOutlinerItemExport(*GraphSoftPath.GetSubPathUtf8String(), ParentExport));
								}
							}
						}
					}
				}
			}
		}

		if (ExportsToOpen.Num() > 0)
		{
			if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = InContext.WorkspaceEditor)
			{
				WorkspaceEditor->OpenExports(ExportsToOpen, FDocumentTracker::EOpenDocumentCause::ForceOpenNewDocument);
			}
		}
	});
	GraphArgs.OnCreateComment = Workspace::FOnCreateComment::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, const FVector2f& GraphLocation)
		{
			if (UAnimNextEdGraph* EdGraph = InContext.Document.GetTypedObject<UAnimNextEdGraph>())
			{
				if (URigVMController* Controller = EdGraph->GetController())
				{
					Controller->OpenUndoBracket(TEXT("Create Comment"));
					TArray<UEdGraphPin*> Pins;
					FAnimNextSchemaAction_AddComment CommentAction;
					UEdGraphNode* EdNode = CommentAction.PerformAction(EdGraph, Pins, GraphLocation);
					Controller->CloseUndoBracket();
				}
			}
		});

	Workspace::FObjectDocumentArgs GraphDocumentArgs = WorkspaceEditorModule.CreateGraphDocumentArgs(GraphArgs);
	Workspace::FOnMakeDocumentWidget WorkspaceMakeDocumentWidgetDelegate = GraphDocumentArgs.OnMakeDocumentWidget;
	GraphDocumentArgs.OnMakeDocumentWidget = Workspace::FOnMakeDocumentWidget::CreateLambda([WorkspaceMakeDocumentWidgetDelegate](const Workspace::FWorkspaceEditorContext& InContext)
	{
		TWeakPtr<Workspace::IWorkspaceEditor> WeakWorkspaceEditor = InContext.WorkspaceEditor;

		if (UAnimNextEdGraph* EdGraph = InContext.Document.GetTypedObject<UAnimNextEdGraph>())
		{
			UUAFRigVMAssetEditorData* EditorData = EdGraph->GetTypedOuter<UUAFRigVMAssetEditorData>();
			if(EditorData)
			{
				// If we are dirty, make sure to reconstruct our nodes and recompile as needed
				if (EditorData->bVMRecompilationRequired)
				{
					EditorData->ReconstructAllNodes();
					EditorData->RecompileVMIfRequired();
				}

				EditorData->InteractionBracketFinished.RemoveAll(&InContext.WorkspaceEditor.Get());
				EditorData->InteractionBracketFinished.AddSPLambda(&InContext.WorkspaceEditor.Get(), [WeakWorkspaceEditor](UUAFRigVMAssetEditorData* InEditorData)
				{
					if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin())
					{
						WorkspaceEditor->RefreshDetails();
					}
				});

				EditorData->RigVMCompiledEvent.RemoveAll(&InContext.WorkspaceEditor.Get());
				EditorData->RigVMCompiledEvent.AddSPLambda(&InContext.WorkspaceEditor.Get(), [WeakWorkspaceEditor](UObject*, URigVM*, FRigVMExtendedExecuteContext&)
				{
					if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin())
					{
						int32 NumEntries = FMessageLog("AnimNextCompilerResults").NumMessages(EMessageSeverity::Warning);
						if(NumEntries > 0)
						{
							WorkspaceEditor->GetTabManager()->TryInvokeTab(FTabId(CompilerResultsTabName));
						}
					}
				});
				
				// Register any general AnimNext commands on document make
				if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin())
				{
					const TSharedRef<FUICommandList>& CommandList = WorkspaceEditor->GetToolkitCommands();
					{
						const FAnimNextRigVMAssetCommands& Commands = FAnimNextRigVMAssetCommands::Get();

						auto TryFindAnimNextRigVMAsset = [WeakWorkspaceEditor]()
						{
							if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditorPinned = WeakWorkspaceEditor.Pin())
							{
								if (TSharedPtr<SDockTab> Tab = WorkspaceEditorPinned->GetTabManager()->TryInvokeTab(FTabId(FindTabName)))
								{
									TSharedRef<SFindInAnimNextRigVMAsset> FindInAnimNextRigVMAssetWidget = StaticCastSharedRef<SFindInAnimNextRigVMAsset>(Tab->GetContent());
									FindInAnimNextRigVMAssetWidget->FocusForUse();
								}
							}
						};

						CommandList->MapAction(Commands.FindInAnimNextRigVMAsset,
							FExecuteAction::CreateLambda(TryFindAnimNextRigVMAsset));

						auto TryFindAndReplaceAnimNextRigVMAsset = [WeakWorkspaceEditor]()
						{
							if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditorPinned = WeakWorkspaceEditor.Pin())
							{
								if (TSharedPtr<SDockTab> Tab = WorkspaceEditorPinned->GetTabManager()->TryInvokeTab(FTabId(FindAndReplaceTabName)))
								{
									TSharedRef<IAnimAssetFindReplace> FindInAnimNextRigVMAssetWidget = StaticCastSharedRef<IAnimAssetFindReplace>(Tab->GetContent());
									Tab->DrawAttention();
								}
							}
						};

						CommandList->MapAction(Commands.FindAndReplaceInAnimNextRigVMAsset,
							FExecuteAction::CreateLambda(TryFindAndReplaceAnimNextRigVMAsset));

						if (bEnableUAFBrowser)
						{
							auto OpenUAFBrowser = [WeakWorkspaceEditor]()
							{
								if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditorPinned = WeakWorkspaceEditor.Pin())
								{
									if (const UUAFEditorUserSettings* UAFEditorUserSettings = GetDefault<UUAFEditorUserSettings>())
									{
										if (UAFEditorUserSettings->bSummonUAFBrowserHotkeyOpensSidebar)
										{
											WorkspaceEditorPinned->GetTabManager()->TryToggleSidebarTab(FUAFBrowserTabSummoner::SidebarTabID);
										}
										else
										{
											FUAFBrowserTabSummoner::ToggleDrawer();
										}
									}
								}
							};

							CommandList->MapAction(Commands.OpenUAFBrowser,
								FExecuteAction::CreateLambda(OpenUAFBrowser));
						}

					}
				}
			}
		}

		if (WorkspaceMakeDocumentWidgetDelegate.IsBound())
		{
			return WorkspaceMakeDocumentWidgetDelegate.Execute(InContext);
		}

		return SNullWidget::NullWidget;
	});

	GraphDocumentArgs.OnGetDocumentBreadcrumbTrail = Workspace::FOnGetDocumentBreadcrumbTrail::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, TArray<TSharedPtr<Workspace::FWorkspaceBreadcrumb>>& OutBreadcrumbs)
	{
		if (URigVMEdGraph* EdGraph = InContext.Document.GetTypedObject<UAnimNextEdGraph>())
		{
			if (UUAFRigVMAssetEditorData* EditorData = EdGraph->GetTypedOuter<UUAFRigVMAssetEditorData>())
			{
				// Iterate model tree, so we display all graph parents until we reach the Entry
				URigVMGraph* ModelGraph = EdGraph->GetModel();
				while (ModelGraph != nullptr)
				{
					URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(EditorData->GetEditorObjectForRigVMGraph(ModelGraph));

					if (RigVMEdGraph != nullptr && EditorData->GetLocalFunctionLibrary() != RigVMEdGraph->GetModel())
					{
						const TSharedPtr<Workspace::FWorkspaceBreadcrumb>& GraphCrumb = OutBreadcrumbs.Add_GetRef(MakeShared<Workspace::FWorkspaceBreadcrumb>());

						TWeakObjectPtr<URigVMEdGraph> WeakEdGraph = RigVMEdGraph;
						TWeakPtr<Workspace::IWorkspaceEditor> WeakWorkspaceEditor = InContext.WorkspaceEditor;
						TWeakObjectPtr<const UUAFRigVMAssetEditorData> WeakEditorData = EditorData;

						FText GraphName;
						if (WeakEdGraph.IsValid())
						{
							if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(WeakEdGraph->GetModel()->GetOuter()))
							{
								GraphName = FText::FromName(CollapseNode->GetFName());
							}
							else if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(WeakEdGraph->GetModel()->GetOuter()))
							{
								if (URigVMLibraryNode* ReferencedNode = Cast<URigVMLibraryNode>(FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer.GetNodeSoftPath().ResolveObject()))
								{
									GraphName = FText::FromName(ReferencedNode->GetFName());
								}
							}

							if (GraphName.IsEmpty())
							{
								if (WeakEditorData.IsValid() && WeakEditorData->GetLocalFunctionLibrary() == WeakEdGraph->GetModel())
								{
									GraphName = UE::UAF::UncookedOnly::FUtils::GetFunctionLibraryDisplayName();
								}
								else if(UUAFRigVMAssetEntry* Entry = WeakEdGraph->GetTypedOuter<UUAFRigVMAssetEntry>())
								{
									GraphName = Entry->GetDisplayName();
								}
								else
								{
									GraphName = FText::FromName(WeakEdGraph->GetFName());
								}
							}
						}

						GraphCrumb->OnGetLabel = Workspace::FWorkspaceBreadcrumb::FOnGetBreadcrumbLabel::CreateLambda(
							[GraphName]
							{
								return GraphName;
							});
						GraphCrumb->CanSave = Workspace::FWorkspaceBreadcrumb::FCanSaveBreadcrumb::CreateLambda(
							[WeakEdGraph]
							{
								if (const URigVMEdGraph* Graph = WeakEdGraph.Get())
								{
									return Graph->GetPackage()->IsDirty();
								}
								return false;
							}
						);
						GraphCrumb->OnClicked = Workspace::FWorkspaceBreadcrumb::FOnBreadcrumbClicked::CreateLambda(
							[WeakEditorData, WeakEdGraph, WeakWorkspaceEditor, Export = InContext.Document.Export]
							{
								if (const TSharedPtr<Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
								{
									SharedWorkspaceEditor->OpenExports({ Export });
								}
							}
						);
						GraphCrumb->OnSave = Workspace::FWorkspaceBreadcrumb::FOnSaveBreadcrumb::CreateLambda(
							[WeakEdGraph]
							{
								if (const URigVMEdGraph* Graph = WeakEdGraph.Get())
								{
									FEditorFileUtils::PromptForCheckoutAndSave({ Graph->GetPackage() }, false, /*bPromptToSave=*/ false);
								}
							}
						);
					}

					ModelGraph = ModelGraph->GetTypedOuter<URigVMGraph>();
				}

				// Display the Asset
				if(UUAFRigVMAsset* OuterAsset = UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(EditorData))
				{
					const TSharedPtr<Workspace::FWorkspaceBreadcrumb>& OuterGraphCrumb = OutBreadcrumbs.Add_GetRef(MakeShared<Workspace::FWorkspaceBreadcrumb>());
					TWeakObjectPtr<UUAFRigVMAsset> WeakOuterAsset = OuterAsset;
					TWeakPtr<Workspace::IWorkspaceEditor> WeakWorkspaceEditor = InContext.WorkspaceEditor;
					OuterGraphCrumb->OnGetLabel = Workspace::FWorkspaceBreadcrumb::FOnGetBreadcrumbLabel::CreateLambda([AssetName = OuterAsset->GetFName()]{ return FText::FromName(AssetName); });
					OuterGraphCrumb->OnClicked = Workspace::FWorkspaceBreadcrumb::FOnBreadcrumbClicked::CreateLambda(
						[WeakOuterAsset, WeakWorkspaceEditor, Export = InContext.Document.Export]
						{
							if (const TSharedPtr<Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
							{
								SharedWorkspaceEditor->OpenExports({Export});
							}
						}
					);
					OuterGraphCrumb->CanSave = Workspace::FWorkspaceBreadcrumb::FCanSaveBreadcrumb::CreateLambda(
						[WeakOuterAsset]
						{
							if (UUAFRigVMAsset* Asset = WeakOuterAsset.Get())
							{
								return Asset->GetPackage()->IsDirty();
							}

							return false;
						}
					);
					OuterGraphCrumb->OnSave = Workspace::FWorkspaceBreadcrumb::FOnSaveBreadcrumb::CreateLambda(
						[WeakOuterAsset]
							{
								if (UUAFRigVMAsset* Asset = WeakOuterAsset.Get())
								{
									FEditorFileUtils::PromptForCheckoutAndSave({Asset->GetPackage()}, false, /*bPromptToSave=*/ false);
								}
							}
						);
				}
			}
		}
	});

	GraphDocumentArgs.DocumentEditorMode = UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace;

	WorkspaceEditorModule.RegisterObjectDocumentType(FTopLevelAssetPath(TEXT("/Script/UAFUncookedOnly.AnimNextEdGraph")), GraphDocumentArgs);

	Workspace::FDocumentSubObjectArgs GraphNodeSubObjectArgs;
	GraphNodeSubObjectArgs.OnGetDocumentForSubObject = Workspace::FOnGetDocumentForSubObject::CreateLambda([](const UObject* InObject) -> UObject*
	{
		if(const UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(InObject))
		{
			return EdGraphNode->GetTypedOuter<UAnimNextEdGraph>();
		}
		return nullptr;
	});
	GraphNodeSubObjectArgs.OnPostDocumentOpenedForSubObject = Workspace::FOnPostDocumentOpenedForSubObject::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, TSharedRef<SWidget> InWidget, UObject* InObject)
	{
		if(UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(InObject))
		{
			check(InWidget->GetType() == TEXT("SGraphEditor"));
			TSharedPtr<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(InWidget);
			GraphEditor->JumpToNode(EdGraphNode, false);
		}
	});

	WorkspaceEditorModule.RegisterDocumentSubObjectType(FTopLevelAssetPath(TEXT("/Script/UAFUncookedOnly.AnimNextEdGraphNode")), GraphNodeSubObjectArgs);

	// Shared variables
	{
		Workspace::FObjectDocumentArgs DocumentArgs;
		DocumentArgs.DocumentEditorMode = UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace;

		DocumentArgs.OnMakeDocumentWidget = Workspace::FOnMakeDocumentWidget::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext) -> TSharedRef<SWidget>
			{
				return SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SharedVariablesDocument_HintText", "Edit variables inside the 'Variables' tab."))
					.TextStyle(FAppStyle::Get(), "HintText")
				];
			});

		DocumentArgs.OnGetTabIcon = Workspace::FOnGetTabIcon::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext) -> const FSlateBrush*
			{
				return FSlateIconFinder::FindIconForClass(UUAFSharedVariables::StaticClass()).GetIcon();
			});

		DocumentArgs.OnGetTabName = Workspace::FOnGetTabName::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext) -> FText
			{
				if (UUAFSharedVariables* SharedVariables = InContext.Document.GetTypedObject<UUAFSharedVariables>())
				{
					return FText::FromName(SharedVariables->GetFName());
				}
				else
				{
					return LOCTEXT("SharedVariablesDocument_EmptyTabName", "Shared Variables");
				}
			});

		DocumentArgs.OnGetDocumentBreadcrumbTrail = Workspace::FOnGetDocumentBreadcrumbTrail::CreateLambda([](const Workspace::FWorkspaceEditorContext& InContext, TArray<TSharedPtr<Workspace::FWorkspaceBreadcrumb>>& OutBreadcrumbs)
			{
				if (UUAFSharedVariables* SharedVariables = InContext.Document.GetTypedObject<UUAFSharedVariables>())
				{
					const TSharedPtr<Workspace::FWorkspaceBreadcrumb>& Crumb = OutBreadcrumbs.Add_GetRef(MakeShared<Workspace::FWorkspaceBreadcrumb>());
					
					Crumb->OnGetLabel = Workspace::FWorkspaceBreadcrumb::FOnGetBreadcrumbLabel::CreateLambda([SharedVariablesName = SharedVariables->GetFName()]
						{
							return FText::FromName(SharedVariablesName);
						});

					Crumb->CanSave = Workspace::FWorkspaceBreadcrumb::FCanSaveBreadcrumb::CreateLambda([SharedVariables]
						{
							return SharedVariables->GetPackage()->IsDirty();
						}
					);
				}
			});

		SharedVariablesTopLevelAssetPath = UUAFSharedVariables::StaticClass()->GetClassPathName();
		WorkspaceEditorModule.RegisterObjectDocumentType(SharedVariablesTopLevelAssetPath, DocumentArgs);
	}
}

void FAnimNextEditorModule::UnregisterWorkspaceDocumentTypes()
{
	if(FModuleManager::Get().IsModuleLoaded("WorkspaceEditor"))
	{
		Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
		WorkspaceEditorModule.UnregisterObjectDocumentType(SystemTopLevelAssetPath);
		WorkspaceEditorModule.UnregisterObjectDocumentType(FTopLevelAssetPath(TEXT("/Script/UAFUncookedOnly.AnimNextEdGraph")));
		WorkspaceEditorModule.UnregisterObjectDocumentType(SharedVariablesTopLevelAssetPath);
		WorkspaceEditorModule.UnregisterDocumentSubObjectType(FTopLevelAssetPath(TEXT("/Script/UAFUncookedOnly.AnimNextEdGraphNode")));
	}
}

const FAssetCompilationHandlerFactoryDelegate* FAnimNextEditorModule::FindAssetCompilationHandlerFactory(UClass* InAssetClass) const
{
	UClass* AssetClass = InAssetClass;
	while(AssetClass)
	{
		if(const FAssetCompilationHandlerFactoryDelegate* FoundDelegate = AssetCompilationHandlerFactories.Find(AssetClass->GetClassPathName()))
		{
			return FoundDelegate;
		}
		AssetClass = AssetClass->GetSuperClass();
	}

	return nullptr;
}

void FAnimNextEditorModule::EnableAssetWizard()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.RegisterWizard({
		"UAFAssetWizard",
		LOCTEXT("UAFAssetWizard_DisplayName","UAF Asset Wizard"),
		LOCTEXT("UAFAssetWizard_Description", "Wizard for creating UAF assets from templates"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Animation_16x"),
		FAdvancedAssetCategory(EAssetTypeCategories::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")),
		FSimpleDelegate::CreateLambda([]()
		{
			UAF::Editor::FAssetWizard::Launch();
		})});
}

void FAnimNextEditorModule::DisableAssetWizard()
{
	if (FModuleManager::Get().IsModuleLoaded("ContentBrowser"))
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		ContentBrowserModule.UnregisterWizard("UAFAssetWizard");
	}
}

}

IMPLEMENT_MODULE(UE::UAF::Editor::FAnimNextEditorModule, UAFEditor);

#undef LOCTEXT_NAMESPACE
