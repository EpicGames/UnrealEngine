// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAnimNextEditorModule.h"
#include "Module/ModuleEventPropertyCustomization.h"
#include "UObject/TopLevelAssetPath.h"
#include "Variables/UAFAssetDataPropertyCustomization.h"
#include "PIEAssetCompilationHandler.h"

class UAnimNextWorkspaceSchema;
class FAnimNextGraphPanelNodeFactory;
class UAnimNextWorkspaceEditorMode;
class FAnimNextGraphPanelPinFactoryEditor;

namespace UE::UAF::Editor
{
	class FParamNamePropertyTypeIdentifier;
	class FLocatorContext;
}

namespace UE::Workspace
{
	class IWorkspaceEditorModule;
}

namespace UE::UAF::Editor
{

class FAnimNextEditorModule : public IAnimNextEditorModule
{
	friend UAnimNextWorkspaceSchema;
	friend UAnimNextWorkspaceEditorMode;

private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// IAnimNextEditorModule interface
	virtual void RegisterLocatorFragmentEditorType(FName InLocatorFragmentEditorName) override;
	virtual void UnregisterLocatorFragmentEditorType(FName InLocatorFragmentEditorName) override;
	virtual void AddWorkspaceSupportedAssetClass(const FTopLevelAssetPath& InClassAssetPath) override;
	virtual void RemoveWorkspaceSupportedAssetClass(const FTopLevelAssetPath& InClassAssetPath) override;
	virtual FDelegateHandle RegisterGraphMenuActionsProvider(const FOnCollectGraphMenuActionsDelegate& InCollectDelegate) override;
	virtual void UnregisterGraphMenuActionsProvider(const FDelegateHandle& InDelegateHandle) override;
	virtual void RegisterAssetCompilationHandler(const FTopLevelAssetPath& InClassPath, FAssetCompilationHandlerFactoryDelegate InAssetCompilationHandlerFactory) override;
	virtual void UnregisterAssetCompilationHandler(const FTopLevelAssetPath& InClassPath) override;
	virtual FDelegateHandle RegisterNodeDblClickHandler(const FNodeDblClickNotificationDelegate& InNodeDblClickNotificationDelegate) override;
	virtual void UnregisterNodeDblClickHandler(const FDelegateHandle& InDelegateHandle) override;
	virtual void EnableAssetWizard() override;
	virtual void DisableAssetWizard() override;

private:
	void CollectGraphMenuActions(const TWeakPtr<UE::Workspace::IWorkspaceEditor>& WorkspaceEditor, FGraphContextMenuBuilder& InContextMenuBuilder, const FActionMenuContextData& InActionMenuContextData);

	void RegisterWorkspaceDocumentTypes(Workspace::IWorkspaceEditorModule& WorkspaceEditorModule);
	void UnregisterWorkspaceDocumentTypes();

	const FAssetCompilationHandlerFactoryDelegate* FindAssetCompilationHandlerFactory(UClass* InAssetClass) const;

private:
	/** Type identifier for parameter names */
	TSharedPtr<FParamNamePropertyTypeIdentifier> Identifier;

	/** Registered names for locator fragments */
	TSet<FName> LocatorFragmentEditorNames;

	TArray<FTopLevelAssetPath> SupportedAssetClasses;

	FCollectGraphMenuActionsMulticast OnCollectGraphMenuActionsDelegateImpl;
	FNodeDblClickNotificationMulticast OnNodeDblClickHandlerMulticast;

	TMap<FTopLevelAssetPath, FAssetCompilationHandlerFactoryDelegate> AssetCompilationHandlerFactories;

	TSharedPtr<FAnimNextGraphPanelPinFactoryEditor> GraphPanelPinFactory;
	TSharedPtr<FModuleEventPropertyTypeIdentifier> ModuleEventPropertyTypeIdentifier;
	TSharedPtr<FUAFAssetDataPropertyIdentifier> UAFAssetDataPropertyIdentifier;

	FTopLevelAssetPath UAFRigVMAssetTopLevelAssetPath;
	FTopLevelAssetPath SystemTopLevelAssetPath;
	FTopLevelAssetPath SharedVariablesTopLevelAssetPath;
	
	FOutOfDateAssetCompilation PIE_UAFCompilationHandler;

	friend class FLocatorContext;
};

}
