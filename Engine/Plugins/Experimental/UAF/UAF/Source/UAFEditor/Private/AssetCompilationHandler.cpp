// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetCompilationHandler.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Editor.h"
#include "UAFCompilationScope.h"
#include "UncookedOnlyUtils.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "PIEAssetCompilationHandler.h"
#include "Logging/MessageLog.h"
#include "Settings/UAFEditorUserSettings.h"
#include "UObject/UObjectIterator.h"

namespace UE::UAF::Editor
{

FAssetCompilationHandler::FAssetCompilationHandler(const UObject* InAsset)
{
	check(InAsset && InAsset->IsA<UUAFRigVMAsset>());
}

void FAssetCompilationHandler::Compile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, UObject* InAsset)
{
	UUAFRigVMAsset* AnimNextRigVMAsset = CastChecked<UUAFRigVMAsset>(InAsset);
	UncookedOnly::Compilation::RequestAssetCompilation(AnimNextRigVMAsset);
	OnCompileStatusChanged().ExecuteIfBound();
}

void FAssetCompilationHandler::SetAutoCompile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, UObject* InAsset, bool bInAutoCompile)
{
	UUAFRigVMAsset* AnimNextRigVMAsset = CastChecked<UUAFRigVMAsset>(InAsset);
	UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(AnimNextRigVMAsset);
	EditorData->SetAutoVMRecompile(bInAutoCompile);
}

bool FAssetCompilationHandler::GetAutoCompile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, const UObject* InAsset) const
{
	const UUAFRigVMAsset* AnimNextRigVMAsset = CastChecked<UUAFRigVMAsset>(InAsset);
	UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(AnimNextRigVMAsset);
	return EditorData->GetAutoVMRecompile();
}

ECompileStatus FAssetCompilationHandler::GetCompileStatus(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, const UObject* InAsset) const
{
	const UUAFRigVMAsset* AnimNextRigVMAsset = CastChecked<UUAFRigVMAsset>(InAsset);
	UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(AnimNextRigVMAsset);
	if(EditorData->bErrorsDuringCompilation)
	{
		return ECompileStatus::Error;
	}

	if(EditorData->bWarningsDuringCompilation)
	{
		return ECompileStatus::Warning;
	}

	if(EditorData->IsDirtyForRecompilation())
	{
		return ECompileStatus::Dirty;
	}

	return ECompileStatus::UpToDate;
}

bool FOutOfDateAssetCompilation::HandleOutOfDateAssets_Internal(const FRequestPlaySessionParams& InPlaySessionParams, TArray<UObject*>& OutAssetsWithErrors) const
{
	// Ignore if user opted not to auto compile on PIE
	if (!GetDefault<UUAFEditorUserSettings>()->bAutoCompileOutOfDateAssets)
	{
		return true;
	}

	OutAssetsWithErrors.Reset();

	for (TObjectIterator<UUAFRigVMAssetEditorData> UAFAssetEditorData; UAFAssetEditorData; ++UAFAssetEditorData)
	{
		if (UUAFRigVMAssetEditorData* EditorData = *UAFAssetEditorData)
		{
			if (UUAFRigVMAsset* Asset = UncookedOnly::FUtils::GetAsset(EditorData))
			{
				EditorData->RecompileVMIfRequired();

				if (EditorData->bErrorsDuringCompilation && EditorData->bDisplayCompilePIEWarning)
				{
					OutAssetsWithErrors.Add(Asset);
				}
			}
		}
	}

	if (OutAssetsWithErrors.Num())
	{
		if (!ShowCompilationErrorsDialog(OutAssetsWithErrors, NSLOCTEXT("UAF", "UAFCompilationTypeText", "UAF"), [](const TWeakObjectPtr<UObject>& InAssetToOpen)
		{
			check(InAssetToOpen.IsValid());
			GEditor->EditObject(InAssetToOpen.Get());
		}))
		{
			// TODO we do not have an equivalent of BP errors showing the compilation log
			return false;
		}
		
		// Mark asset to ignore asset errors until manually compiled
		for (UObject* Asset : OutAssetsWithErrors)
		{
			UncookedOnly::FUtils::GetEditorData(CastChecked<UUAFRigVMAsset>(Asset))->bDisplayCompilePIEWarning = false;
		}
	}

	return true;
}

}
