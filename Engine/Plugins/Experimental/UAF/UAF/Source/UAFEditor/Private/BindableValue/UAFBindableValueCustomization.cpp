// Copyright Epic Games, Inc. All Rights Reserved.

#include "BindableValue/UAFBindableValueCustomization.h"

#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/UnrealType.h"
#include "IPropertyUtilities.h"

#include "AnimNextFunctionReference.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Common/SRigVMFunctionPicker.h"
#include "Variables/SVariablePicker.h"
#include "UncookedOnlyUtils.h"
#include "Variables/SVariablePickerCombo.h"
#include "Variables/VariablePickerArgs.h"
#include "Variables/AnimNextVariableReference.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "Param/ParamType.h"

#include "BindableValue/UAFBindableTypes.h"
#include "BindableValue/UAFPropertyBinding.h"
#include "SEnumCombo.h"
#include "IDetailChildrenBuilder.h"
#include "InstancedStructDetails.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "PropertyCustomizationHelpers.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Param/ParamCompatibility.h"
#include "Param/ParamUtils.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"

#define LOCTEXT_NAMESPACE "UAFBindableValueCustomization"

namespace
{

/**
 * Returns the FAnimNextParamType that corresponds to the concrete FBindableXxx subtype.
 * For FBindableEnum, reads EnumClass from the first raw data element of InPropertyHandle.
 */
FAnimNextParamType GetBindableParamType(const UScriptStruct* BindableStruct, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	using EValueType     = EPropertyBagPropertyType;
	using EContainerType = EPropertyBagContainerType;

	if (!BindableStruct)
	{
		return FAnimNextParamType();
	}

	if (BindableStruct == FBindableBool::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Bool);
	}
	if (BindableStruct == FBindableFloat::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Float);
	}
	if (BindableStruct == FBindableDouble::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Double);
	}
	if (BindableStruct == FBindableInt32::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Int32);
	}
	if (BindableStruct == FBindableInt64::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Int64);
	}
	if (BindableStruct == FBindableByte::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Byte);
	}
	if (BindableStruct == FBindableName::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Name);
	}
	if (BindableStruct == FBindableVector::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Struct, EContainerType::None, TBaseStructure<FVector>::Get());
	}
	if (BindableStruct == FBindableQuat::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Struct, EContainerType::None, TBaseStructure<FQuat>::Get());
	}
	if (BindableStruct == FBindableTransform::StaticStruct())
	{
		return FAnimNextParamType(EValueType::Struct, EContainerType::None, TBaseStructure<FTransform>::Get());
	}

	if (BindableStruct == FBindableEnum::StaticStruct())
	{
		UEnum* EnumClass = nullptr;
		if (InPropertyHandle.IsValid())
		{
			InPropertyHandle->EnumerateRawData(
				[&EnumClass](void* RawData, const int32, const int32) -> bool
				{
					const FBindableEnum* Bindable = static_cast<const FBindableEnum*>(RawData);
					EnumClass = Bindable->GetEnumClass();
					return false; // first instance only
				});
		}
		if (EnumClass)
		{
			return FAnimNextParamType(EValueType::Enum, EContainerType::None, EnumClass);
		}
		return FAnimNextParamType();
	}

	if (BindableStruct == FBindableStruct::StaticStruct())
	{
		UScriptStruct* InnerStruct = nullptr;
		if (InPropertyHandle.IsValid())
		{
			InPropertyHandle->EnumerateRawData(
				[&InnerStruct](void* RawData, const int32, const int32) -> bool
				{
					const FBindableStruct* Bindable = static_cast<const FBindableStruct*>(RawData);
					// Prefer StructClass (explicit type restriction) over ConstantValue's runtime type
					if (Bindable->GetStructClass())
					{
						InnerStruct = Bindable->GetStructClass();
					}
					else
					{
						InnerStruct = const_cast<UScriptStruct*>(Bindable->GetConstantValue().GetScriptStruct());
					}
					return false;
				});
		}
		if (InnerStruct)
		{
			return FAnimNextParamType(EValueType::Struct, EContainerType::None, InnerStruct);
		}
		return FAnimNextParamType();
	}

	if (BindableStruct == FBindableObject::StaticStruct())
	{
		UClass* ObjClass = nullptr;
		if (InPropertyHandle.IsValid())
		{
			InPropertyHandle->EnumerateRawData(
				[&ObjClass](void* RawData, const int32, const int32) -> bool
				{
					ObjClass = static_cast<const FBindableObject*>(RawData)->GetObjectClass();
					return false;
				});
		}
		return FAnimNextParamType(EValueType::Object, EContainerType::None,
			ObjClass ? ObjClass : UObject::StaticClass());
	}

	return FAnimNextParamType();
}

/**
 * Returns true if the function is bindable: 0 caller-provided inputs, exactly 1 output.
 * (0 caller-provided inputs, at least 1 output).
 * Input variable arguments (bIsInputVariable) are excluded — they are read
 * from system variable storage by the wrapper event, not provided by the caller.
 */
bool IsFunctionBindable(const FRigVMGraphFunctionHeader& InHeader)
{
	int32 InputCount = 0;
	int32 OutputCount = 0;
	for (const FRigVMGraphFunctionArgument& Arg : InHeader.Arguments)
	{
		if (Arg.IsExecuteContext())
		{
			continue;
		}
		if (Arg.bIsInputVariable)
		{
			continue;
		}
		if (Arg.Direction == ERigVMPinDirection::Input)
		{
			InputCount++;
		}
		if (Arg.Direction == ERigVMPinDirection::Output)
		{
			OutputCount++;
		}
	}
	return InputCount == 0 && OutputCount == 1;
}

/**
 * Returns true if the function is bindable and its return type is compatible
 * with InTargetType (supporting promotion/demotion with data loss, e.g. float↔double).
 */
bool IsFunctionCompatibleWithBindableType(const FRigVMGraphFunctionHeader& InHeader, const FAnimNextParamType& InTargetType)
{
	if (!IsFunctionBindable(InHeader))
	{
		return false;
	}
	if (!InTargetType.IsValid())
	{
		return true;
	}
	for (const FRigVMGraphFunctionArgument& Arg : InHeader.Arguments)
	{
		if (!Arg.IsExecuteContext() && !Arg.bIsInputVariable && Arg.Direction == ERigVMPinDirection::Output)
		{
			FRigVMTemplateArgumentType RigVMArgType(Arg.CPPType, Arg.CPPTypeObject.Get());
			FAnimNextParamType ReturnType = FAnimNextParamType::FromRigVMTemplateArgument(RigVMArgType);
			return UE::UAF::FParamUtils::GetCompatibility(InTargetType, ReturnType).IsCompatibleWithDataLoss();
		}
	}
	return false;
}

enum class EFunctionBindingStatus { Valid, FunctionNotFound, ReturnTypeIncompatible };

EFunctionBindingStatus ValidateFunctionBinding(
	const FAnimNextFunctionReference& InFunction,
	const FAnimNextParamType& InTargetType,
	const UUAFRigVMAsset* InAsset,
	FName* OutResolvedName = nullptr)
{
	// Collect public function headers to validate against.
	FRigVMGraphFunctionHeaderArray AllHeaders;
	TMap<FAssetData, FRigVMGraphFunctionHeaderArray> PublicExports;
	UE::UAF::UncookedOnly::FUtils::GetExportedFunctionsFromAssetRegistry(UE::UAF::AnimNextPublicGraphFunctionsExportsRegistryTag, PublicExports);

	if (InAsset)
	{
		// Prefer the specific asset's exports when available.
		if (const FRigVMGraphFunctionHeaderArray* AssetPublicHeaders = PublicExports.Find(FAssetData(InAsset)))
		{
			AllHeaders.Headers.Append(AssetPublicHeaders->Headers);
		}
	}
	else
	{
		// No asset context (e.g. chooser table) -- search all public exports.
		for (const auto& Pair : PublicExports)
		{
			AllHeaders.Headers.Append(Pair.Value.Headers);
		}
	}

	// Match by GUID first (rename-robust), fall back to name if GUID is invalid (legacy data)
	const FGuid& FunctionGuid = InFunction.GetFunctionGuid();
	FString FuncName = InFunction.GetEventName().ToString();
	FuncName.RemoveFromStart(TEXT("__InternalCall_"));

	for (const FRigVMGraphFunctionHeader& Header : AllHeaders.Headers)
	{
		const bool bMatches = FunctionGuid.IsValid()
			? (Header.Variant.Guid == FunctionGuid)
			: (Header.Name == FName(*FuncName));

		if (bMatches)
		{
			if (OutResolvedName)
			{
				*OutResolvedName = Header.Name;
			}
			if (IsFunctionCompatibleWithBindableType(Header, InTargetType))
			{
				return EFunctionBindingStatus::Valid;
			}
			return EFunctionBindingStatus::ReturnTypeIncompatible;
		}
	}
	return EFunctionBindingStatus::FunctionNotFound;
}

} // anonymous namespace

namespace UE::UAF::Editor
{

void FBindableValueCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle>      InPropertyHandle,
	FDetailWidgetRow&                InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	const FProperty*       Property    = PropertyHandle->GetProperty();
	const FStructProperty* StructProp  = CastField<FStructProperty>(Property);
	if (!StructProp)
	{
		return;
	}

	const UScriptStruct* BindableStruct = StructProp->Struct;

	// For FBindableStruct: read StructClass and auto-initialize ConstantValue via the property system.
	// We must NOT modify raw data directly — use copy+export+SetValueFromFormattedString so that
	// undo, dirty marking, and OnPropertyChanged callbacks all fire correctly.
	if (BindableStruct == FBindableStruct::StaticStruct())
	{
		bIsBindableStruct = true;
		bool bNeedsInitialization = false;
		PropertyHandle->EnumerateConstRawData(
			[this, &bNeedsInitialization](const void* RawData, const int32, const int32) -> bool
			{
				const FBindableStruct* Bindable = static_cast<const FBindableStruct*>(RawData);
				if (Bindable->GetStructClass())
				{
					ResolvedStructClass = Bindable->GetStructClass();
					const FInstancedStruct& Current = Bindable->GetConstantValue();
					if (!Current.IsValid() || Current.GetScriptStruct() != Bindable->GetStructClass())
					{
						bNeedsInitialization = true;
					}
				}
				return false;
			});

		if (bNeedsInitialization)
		{
			FString TextValue;
			PropertyHandle->EnumerateConstRawData(
				[&TextValue, BindableStruct, ResolvedClass = ResolvedStructClass](const void* RawData, const int32, const int32) -> bool
				{
					const int32 StructSize = BindableStruct->GetStructureSize();

					TArray<uint8> CopyBuf;
					CopyBuf.SetNumUninitialized(StructSize);
					BindableStruct->InitializeStruct(CopyBuf.GetData());
					BindableStruct->CopyScriptStruct(CopyBuf.GetData(), RawData);

					FInstancedStruct NewValue;
					NewValue.InitializeAs(ResolvedClass);
					static_cast<FBindableStruct*>(static_cast<void*>(CopyBuf.GetData()))->SetConstantValue(MoveTemp(NewValue));

					TArray<uint8> DefaultBuf;
					DefaultBuf.SetNumUninitialized(StructSize);
					BindableStruct->InitializeStruct(DefaultBuf.GetData());

					BindableStruct->ExportText(TextValue, CopyBuf.GetData(), DefaultBuf.GetData(), nullptr, 0, nullptr, true);

					BindableStruct->DestroyStruct(DefaultBuf.GetData());
					BindableStruct->DestroyStruct(CopyBuf.GetData());
					return false;
				});

			PropertyHandle->SetValueFromFormattedString(TextValue);
		}
	}

	const FAnimNextParamType TargetParamType = GetBindableParamType(BindableStruct, PropertyHandle);

	// -----------------------------------------------------------------------
	// Shared helpers (captured by value into lambdas below)
	// -----------------------------------------------------------------------

	// Returns true if the first element has an active binding.
	auto HasBinding = [Handle = PropertyHandle]() -> bool
	{
		bool bResult = false;
		Handle->EnumerateConstRawData([&bResult](const void* RawData, const int32, const int32) -> bool
		{
			bResult = static_cast<const FBindableValueBase*>(RawData)->HasBinding();
			return false; // first only
		});
		return bResult;
	};

	// Clears the binding by modifying raw data directly and firing a property change notification.
	// Direct modification is used instead of SetValueFromFormattedString because the latter
	// fails silently for property handles backed by FStructOnScope (e.g. trait properties).
	auto ClearBindingDirect = [](TSharedPtr<IPropertyHandle> Handle)
	{
		Handle->NotifyPreChange();
		Handle->EnumerateRawData(
			[](void* RawData, const int32, const int32) -> bool
			{
				static_cast<FBindableValueBase*>(RawData)->ClearBinding();
				return true;
			});
		Handle->NotifyPostChange(EPropertyChangeType::ValueSet);
	};

	// -----------------------------------------------------------------------
	// Collect filter assets from selected objects (context sensitivity)
	// -----------------------------------------------------------------------

	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		if (const TSharedPtr<IPropertyUtilities> PropertyUtilities = InCustomizationUtils.GetPropertyUtilities())
		{
			Objects = PropertyUtilities->GetSelectedObjects();
		}
		
		for (const TWeakObjectPtr<UObject>& WeakObject : Objects)
		{
			UObject* Object = WeakObject.Get();
			if (!Object)
			{
				continue;
			}

			if (const UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(Object))
			{
				if (!CurrentRigVMAsset.IsValid())
				{
					CurrentRigVMAsset = Asset;
				}
				FilterAssets.AddUnique(Asset);
			}
			if (const UUAFRigVMAsset* OuterAsset = Object->GetTypedOuter<UUAFRigVMAsset>())
			{
				if (!CurrentRigVMAsset.IsValid())
				{
					CurrentRigVMAsset = OuterAsset;
				}
				FilterAssets.AddUnique(OuterAsset);
			}
		}

		TArray<TWeakObjectPtr<const UUAFRigVMAsset>> ReferencedAssets;
		for (const TWeakObjectPtr<const UUAFRigVMAsset>& WeakAsset : FilterAssets)
		{
			if (const UUAFRigVMAsset* Asset = WeakAsset.Get())
			{
				for (const UUAFRigVMAsset* Referenced : Asset->GetReferencedVariableAssets())
				{
					ReferencedAssets.Add(Referenced);
				}
			}
		}
		FilterAssets.Append(ReferencedAssets);
	}

	// -----------------------------------------------------------------------
	// Build FVariablePickerArgs
	// -----------------------------------------------------------------------

	FVariablePickerArgs PickerArgs;

	if (TargetParamType.IsValid())
	{
		PickerArgs.CompatibleTypes = { TargetParamType };
	}

	// Helper: sets a binding by modifying raw data directly and firing a property change notification.
	// Direct modification is used instead of SetValueFromFormattedString because the latter
	// fails silently for property handles backed by FStructOnScope (e.g. trait properties).
	// The notification triggers SetOnPropertyValueChangedWithData callbacks which export the
	// full struct value (including binding via custom ExportTextItem) to the RigVM pin.
	auto SetBindingDirect = [](TSharedPtr<IPropertyHandle> Handle, const FUAFPropertyBinding& InBinding)
	{
		Handle->NotifyPreChange();
		Handle->EnumerateRawData(
			[&InBinding](void* RawData, const int32, const int32) -> bool
			{
				static_cast<FBindableValueBase*>(RawData)->SetBinding(InBinding);
				return true;
			});
		Handle->NotifyPostChange(EPropertyChangeType::ValueSet);
	};

	// Write a Variable binding.
	PickerArgs.OnVariablePicked = FOnVariablePicked::CreateLambda(
		[this, SetBindingDirect](const FAnimNextSoftVariableReference& SoftRef, const FAnimNextParamType& /*Type*/)
		{
			FUAFPropertyBinding NewBinding;
			NewBinding.SourceType     = EUAFBindingSourceType::Variable;
			NewBinding.SourceVariable = FAnimNextVariableReference(SoftRef);
			SetBindingDirect(PropertyHandle, NewBinding);
		});

	// Write a SubProperty binding.
	PickerArgs.OnSubPropertyVariablePicked = FOnSubPropertyVariablePicked::CreateLambda(
		[this, SetBindingDirect](
			const FAnimNextSoftVariableReference& TopVarRef,
			const TArray<FName>&                  SubPropertyPath,
			const FAnimNextParamType& /*LeafType*/)
		{
			FUAFPropertyBinding NewBinding;
			NewBinding.SourceType     = EUAFBindingSourceType::SubProperty;
			NewBinding.SourceVariable = FAnimNextVariableReference(TopVarRef);
			for (const FName& Seg : SubPropertyPath)
			{
				NewBinding.SubPropertyPath.AddPathSegment(Seg);
			}
			SetBindingDirect(PropertyHandle, NewBinding);

			FSlateApplication::Get().DismissAllMenus();
		});

	// -----------------------------------------------------------------------
	// Context sensitivity
	// -----------------------------------------------------------------------

	if (FilterAssets.Num() > 0)
	{
		OnIsContextSensitiveDelegate = FOnIsContextSensitive::CreateLambda([this]() -> bool
		{
			return bIsContextSensitive;
		});

		PickerArgs.OnIsContextSensitive = &OnIsContextSensitiveDelegate;

		PickerArgs.OnContextSensitivityChanged = FOnContextSensitivityChanged::CreateLambda(
			[this](bool bInIsContextSensitive)
			{
				bIsContextSensitive = bInIsContextSensitive;
			});

		PickerArgs.OnGetIncludeExcludeFilter = FOnGetIncludeExcludeFilter::CreateLambda(
			[this](EAnimNextExportedVariableFlags& OutFlagInclusionFilter, EAnimNextExportedVariableFlags& OutFlagExclusionFilter)
			{
				if (bIsContextSensitive && FilterAssets.Num() > 0)
				{
					OutFlagInclusionFilter = EAnimNextExportedVariableFlags::Declared;
					OutFlagExclusionFilter = EAnimNextExportedVariableFlags::Referenced;
				}
				else
				{
					OutFlagInclusionFilter = EAnimNextExportedVariableFlags::Declared | EAnimNextExportedVariableFlags::Public;
					OutFlagExclusionFilter = EAnimNextExportedVariableFlags::Referenced;
				}
			});

		PickerArgs.OnFilterVariable = FOnFilterVariable::CreateLambda(
			[this](const FAnimNextSoftVariableReference& InVariableReference) -> EFilterVariableResult
			{
				if (!bIsContextSensitive || FilterAssets.Num() == 0)
				{
					return EFilterVariableResult::Include;
				}
				for (const TWeakObjectPtr<const UUAFRigVMAsset>& WeakFilterAsset : FilterAssets)
				{
					if (const UUAFRigVMAsset* FilterAsset = WeakFilterAsset.Get())
					{
						if (InVariableReference.GetSoftObjectPath() == FSoftObjectPath(FilterAsset))
						{
							return EFilterVariableResult::Include;
						}
					}
				}
				return EFilterVariableResult::Exclude;
			});
	}

	// -----------------------------------------------------------------------
	// Build row: [ConstantValue (greyed when bound)] [SVariablePickerCombo] [×]
	// -----------------------------------------------------------------------

	ConstantValueHandle = PropertyHandle->GetChildHandle(FName("ConstantValue"));

	TSharedRef<SWidget> ConstantWidget = SNullWidget::NullWidget;
	if (ConstantValueHandle.IsValid())
	{
		// Forwards any property level meta data to the constant value (meta = .. on UPROPERTY) 
		if (const FProperty* OuterProperty = PropertyHandle->GetMetaDataProperty())
		{
			if (const TMap<FName, FString>* MetaDataMap = OuterProperty->GetMetaDataMap())
			{
				for (const TPair<FName, FString>& Entry : *MetaDataMap)
				{
					ConstantValueHandle->SetInstanceMetaData(Entry.Key, Entry.Value);
				}
			}
		}

		// Forwards any runtime level meta data overrides on the property to the constant value 
		if (const TMap<FName, FString>* InstanceMap = PropertyHandle->GetInstanceMetaDataMap())
		{
			for (const TPair<FName, FString>& Entry : *InstanceMap)
			{
				ConstantValueHandle->SetInstanceMetaData(Entry.Key, Entry.Value);
			}
		}
		
		UEnum* EnumClass = nullptr;
		if (BindableStruct == FBindableEnum::StaticStruct())
		{
			PropertyHandle->EnumerateRawData([&EnumClass](void* RawData, const int32, const int32) -> bool
			{
				EnumClass = static_cast<const FBindableEnum*>(RawData)->GetEnumClass();
				return false;
			});
		}

		if (EnumClass)
		{
			ConstantWidget = SNew(SEnumComboBox, EnumClass)
				.CurrentValue_Lambda([Handle = ConstantValueHandle]() -> int32
				{
					int32 Value = 0;
					Handle->GetValue(Value);
					return Value;
				})
				.OnEnumSelectionChanged_Lambda([Handle = PropertyHandle, StructType = BindableStruct](int32 NewValue, ESelectInfo::Type)
				{
					// Use copy+export+SetValueFromFormattedString on the parent handle so
					// the value change and pin default text update happen atomically.
					// Direct SetValue on the child handle gets overwritten by the workspace
					// re-importing the old pin default text during the change notification.
					FString TextValue;
					Handle->EnumerateConstRawData(
						[&TextValue, NewValue, StructType](const void* RawData, const int32, const int32) -> bool
						{
							const int32 StructSize = StructType->GetStructureSize();

							TArray<uint8> CopyBuf;
							CopyBuf.SetNumUninitialized(StructSize);
							StructType->InitializeStruct(CopyBuf.GetData());
							StructType->CopyScriptStruct(CopyBuf.GetData(), RawData);
							static_cast<FBindableEnum*>(static_cast<void*>(CopyBuf.GetData()))->SetConstantValue(NewValue);

							StructType->ExportText(TextValue, CopyBuf.GetData(), CopyBuf.GetData(), nullptr, 0, nullptr, /*bAllowNativeOverride=*/true);

							StructType->DestroyStruct(CopyBuf.GetData());
							return false;
						});

					Handle->SetValueFromFormattedString(TextValue);
				});
		}
		else if (bIsBindableStruct && ResolvedStructClass)
		{
			// FBindableStruct with known type: show type name label; child properties exposed in CustomizeChildren
			ConstantWidget = SNew(STextBlock)
				.Text(ResolvedStructClass->GetDisplayNameText());
		}
		else if (BindableStruct == FBindableObject::StaticStruct())
		{
			UClass* ObjClass = nullptr;
			PropertyHandle->EnumerateRawData([&ObjClass](void* RawData, const int32, const int32) -> bool
			{
				ObjClass = static_cast<const FBindableObject*>(RawData)->GetObjectClass();
				return false;
			});

			ConstantWidget = SNew(SObjectPropertyEntryBox)
				.PropertyHandle(ConstantValueHandle)
				.AllowedClass(ObjClass ? ObjClass : UObject::StaticClass());
		}
		else if (BindableStruct == FBindableTransform::StaticStruct())
		{
			bIsBindableTransform = true;
		}
		else
		{
			ConstantWidget = ConstantValueHandle->CreatePropertyValueWidgetWithCustomization(nullptr);
		}
	}

	// When the constant value changes through the widget, also notify the parent FBindableXxx
	// handle. This ensures that external callbacks (e.g. trait pin default update) that are
	// registered on the parent handle see the change and export the full struct value.
	if (ConstantValueHandle.IsValid())
	{
		ConstantValueHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this]()
		{
			PropertyHandle->NotifyPreChange();
			PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		}));
	}

	// Validate function binding once at widget creation time (not per-frame).
	// Also resolve the current display name from GUID (handles function renames).
	EFunctionBindingStatus CachedFunctionStatus = EFunctionBindingStatus::Valid;
	FName CachedFunctionDisplayName;
	PropertyHandle->EnumerateConstRawData(
		[&CachedFunctionStatus, &CachedFunctionDisplayName, TargetParamType, WeakAsset = CurrentRigVMAsset](const void* RawData, const int32, const int32) -> bool
		{
			const FBindableValueBase* Base = static_cast<const FBindableValueBase*>(RawData);
			if (const FUAFPropertyBinding* Binding = Base->GetBinding())
			{
				if (Binding->SourceType == EUAFBindingSourceType::Function && !Binding->SourceFunction.IsNone())
				{
					CachedFunctionStatus = ValidateFunctionBinding(
						Binding->SourceFunction, TargetParamType, Cast<UUAFRigVMAsset>(WeakAsset.Get()),
						&CachedFunctionDisplayName);
				}
			}
			return false;
		});

	InHeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.f)
	[
		SNew(SHorizontalBox)

		// Constant value editor — greyed out when a binding is active.
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.IsEnabled_Lambda([HasBinding]() -> bool
			{
				return !HasBinding();
			})
			[
				ConstantWidget
			]
		]

		// Binding picker combo (tabbed: Variables | Functions).
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f, 0.f, 0.f, 0.f)
		[
			SNew(SComboButton)
			.ToolTipText_Lambda([Handle = PropertyHandle]() -> FText
			{
				FText Result;
				Handle->EnumerateConstRawData(
					[&Result](const void* RawData, const int32, const int32) -> bool
					{
						const FBindableValueBase* Base = static_cast<const FBindableValueBase*>(RawData);
						if (const FUAFPropertyBinding* PropertyBinding = Base->GetBinding())
						{
							if (PropertyBinding->SourceType == EUAFBindingSourceType::Function && !PropertyBinding->SourceFunction.IsNone())
							{
								Result = FText::FromName(PropertyBinding->SourceFunction.GetEventName());
							}
							else if (!PropertyBinding->SourceVariable.IsNone())
							{
								Result = FText::FromName(PropertyBinding->SourceVariable.GetName());
							}
						}
						return false;
					});
				return Result;
			})
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(0.f, 2.f, 2.f, 2.f)
				[
					SNew(SImage)
					.Image_Lambda([Handle = PropertyHandle, TargetParamType, CachedFunctionStatus]() -> const FSlateBrush*
					{
						bool bIsFunctionBinding = false;
						Handle->EnumerateConstRawData(
							[&bIsFunctionBinding](const void* RawData, const int32, const int32) -> bool
							{
								const FBindableValueBase* Base = static_cast<const FBindableValueBase*>(RawData);
								if (const FUAFPropertyBinding* PropertyBinding = Base->GetBinding())
								{
									bIsFunctionBinding = PropertyBinding->SourceType == EUAFBindingSourceType::Function && !PropertyBinding->SourceFunction.IsNone();
								}
								return false;
							});
						if (bIsFunctionBinding)
						{
							if (CachedFunctionStatus != EFunctionBindingStatus::Valid)
							{
								return FAppStyle::GetBrush(TEXT("Icons.WarningWithColor"));
							}
							return FAppStyle::GetBrush(TEXT("GraphEditor.Function_16x"));
						}
						FEdGraphPinType PinType = UE::UAF::UncookedOnly::FUtils::GetPinTypeFromParamType(TargetParamType);
						return FBlueprintEditorUtils::GetIconFromPin(PinType, true);
					})
					.ColorAndOpacity_Lambda([Handle = PropertyHandle, TargetParamType, CachedFunctionStatus]() -> FSlateColor
					{
						bool bIsFunctionBinding = false;
						Handle->EnumerateConstRawData(
							[&bIsFunctionBinding](const void* RawData, const int32, const int32) -> bool
							{
								const FBindableValueBase* Base = static_cast<const FBindableValueBase*>(RawData);
								if (const FUAFPropertyBinding* PropertyBinding = Base->GetBinding())
								{
									bIsFunctionBinding = PropertyBinding->SourceType == EUAFBindingSourceType::Function && !PropertyBinding->SourceFunction.IsNone();
								}
								return false;
							});
						if (bIsFunctionBinding && CachedFunctionStatus != EFunctionBindingStatus::Valid)
						{
							return FLinearColor::Red;
						}
						FEdGraphPinType PinType = UE::UAF::UncookedOnly::FUtils::GetPinTypeFromParamType(TargetParamType);
						return GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);
					})
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(0.f, 2.f, 0.f, 2.f)
				[
					SNew(STextBlock)
					.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
					.Text_Lambda([Handle = PropertyHandle, CachedFunctionStatus, CachedFunctionDisplayName]() -> FText
					{
						FText Result;
						Handle->EnumerateConstRawData(
							[&Result, CachedFunctionStatus, CachedFunctionDisplayName](const void* RawData, const int32, const int32) -> bool
							{
								const FBindableValueBase* Base = static_cast<const FBindableValueBase*>(RawData);
								if (const FUAFPropertyBinding* Binding = Base->GetBinding())
								{
									if (Binding->SourceType == EUAFBindingSourceType::Function && !Binding->SourceFunction.IsNone())
									{
										// Use the resolved name from GUID lookup (handles renames).
										// Fall back to parsing EventName for legacy data without GUID.
										FString Name;
										if (!CachedFunctionDisplayName.IsNone())
										{
											Name = CachedFunctionDisplayName.ToString();
										}
										else
										{
											Name = Binding->SourceFunction.GetEventName().ToString();
											Name.RemoveFromStart(TEXT("__InternalCall_"));
										}

										if (CachedFunctionStatus == EFunctionBindingStatus::FunctionNotFound)
										{
											Name += TEXT(" (Missing)");
										}
										else if (CachedFunctionStatus == EFunctionBindingStatus::ReturnTypeIncompatible)
										{
											Name += TEXT(" (Type Mismatch)");
										}
										Result = FText::FromString(Name);
									}
									else if (!Binding->SourceVariable.IsNone())
									{
										FString DisplayName = Binding->SourceVariable.GetName().ToString();
										if (Binding->SourceType == EUAFBindingSourceType::SubProperty
											&& Binding->SubPropertyPath.NumSegments() > 0)
										{
											DisplayName += TEXT(".") + Binding->SubPropertyPath.ToString();
										}
										Result = FText::FromString(DisplayName);
									}
								}
								return false;
							});
						return Result;
					})
				]
			]
			.OnGetMenuContent_Lambda(
				[this, PickerArgs, SetBindingDirect, TargetParamType]() -> TSharedRef<SWidget>
			{
				TSharedRef<SWidgetSwitcher> Switcher = SNew(SWidgetSwitcher);

				// Tab 0: Variables (existing variable picker)
				Switcher->AddSlot()
				[
					SNew(SVariablePicker).Args(PickerArgs)
				];

				// Tab 1: Functions (parameterless function picker, content-only = no nested combo button)
				Switcher->AddSlot()
				[
					SNew(SRigVMFunctionPicker)
					.CurrentAsset(FAssetData(const_cast<UUAFRigVMAsset*>(CurrentRigVMAsset.Get())))
					.AllowNew(false)
					.AllowClear(false)
					.ContentOnly(true)
					.FilterAssets(FilterAssets)
					.IsContextSensitive(bIsContextSensitive)
					.OnFilterFunction_Lambda([TargetParamType](const FRigVMGraphFunctionHeader& InHeader) -> bool
					{
						return IsFunctionCompatibleWithBindableType(InHeader, TargetParamType);
					})
					.OnRigVMFunctionPicked_Lambda(
						[Handle = PropertyHandle, SetBindingDirect, WeakAsset = CurrentRigVMAsset]
						(const FRigVMGraphFunctionHeader& InHeader)
					{
						if (!InHeader.IsValid())
						{
							return;
						}

						const UUAFRigVMAsset* Asset = WeakAsset.Get();
						FUAFPropertyBinding NewBinding;
						NewBinding.SourceType = EUAFBindingSourceType::Function;
						NewBinding.SourceFunction = FAnimNextFunctionReference::FromHeader(InHeader, Asset);
						SetBindingDirect(Handle, NewBinding);
					})
				];

				return SNew(SBox)
					.WidthOverride(400.f)
					.HeightOverride(400.f)
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(4.f)
						[
							SNew(SSegmentedControl<int32>)
							.Value(0)
							.OnValueChanged_Lambda([Switcher](int32 Value)
							{
								Switcher->SetActiveWidgetIndex(Value);
							})
							+ SSegmentedControl<int32>::Slot(0)
							.Text(LOCTEXT("VariablesTab", "Variables"))
							+ SSegmentedControl<int32>::Slot(1)
							.Text(LOCTEXT("FunctionsTab", "Functions"))
						]
						+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						[
							Switcher
						]
					];
			})
		]

		// Clear button — only visible when a binding is active.
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("ClearBindingTooltip", "Clear Binding"))
			.Visibility_Lambda([HasBinding]() -> EVisibility
			{
				return HasBinding() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.OnClicked_Lambda([Handle = PropertyHandle, ClearBindingDirect]() -> FReply
			{
				ClearBindingDirect(Handle);
				return FReply::Handled();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.X"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];

	// Right-click "Clear Binding" context menu action on the detail row.
	InHeaderRow.AddCustomContextMenuAction(
		FUIAction(
			FExecuteAction::CreateLambda([Handle = PropertyHandle, ClearBindingDirect]() { ClearBindingDirect(Handle); }),
			FCanExecuteAction::CreateLambda([HasBinding]() { return HasBinding(); })),
		LOCTEXT("ClearBindingAction",        "Clear Binding"),
		LOCTEXT("ClearBindingActionTooltip", "Remove the variable binding from this property"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.X"));
}

void FBindableValueCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle>      InPropertyHandle,
	IDetailChildrenBuilder&          ChildBuilder,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// For FBindableStruct with a known StructClass, expose the inner struct fields directly as children.
	if (bIsBindableStruct && ResolvedStructClass && ConstantValueHandle.IsValid())
	{
		// Lambda to check if binding is active (greyed out when bound).
		auto HasBinding = [Handle = PropertyHandle]() -> bool
		{
			bool bResult = false;
			Handle->EnumerateConstRawData([&bResult](const void* RawData, const int32, const int32) -> bool
			{
				bResult = static_cast<const FBindableValueBase*>(RawData)->HasBinding();
				return false;
			});
			return bResult;
		};

		TSharedRef<FInstancedStructProvider> StructProvider =
			MakeShared<FInstancedStructProvider>(ConstantValueHandle);
		ConstantValueHandle->RemoveChildren();
		TArray<TSharedPtr<IPropertyHandle>> ChildProperties = ConstantValueHandle->AddChildStructure(StructProvider);
		for (const TSharedPtr<IPropertyHandle>& ChildHandle : ChildProperties)
		{
			if (ChildHandle.IsValid())
			{
				ChildBuilder.AddProperty(ChildHandle.ToSharedRef())
					.IsEnabled(TAttribute<bool>::CreateLambda([HasBinding]() { return !HasBinding(); }));
			}
		}
	}
	else if (bIsBindableTransform && ConstantValueHandle.IsValid())
	{
		auto HasBinding = [Handle = PropertyHandle]() -> bool
		{
			bool bResult = false;
			Handle->EnumerateConstRawData([&bResult](const void* RawData, const int32, const int32) -> bool
			{
				bResult = static_cast<const FBindableValueBase*>(RawData)->HasBinding();
				return false;
			});
			return bResult;
		};

		// Iterate children generically -- the property system applies the correct
		// customizations for each sub-property (FVector inline editors, etc.).
		uint32 NumChildren = 0;
		ConstantValueHandle->GetNumChildren(NumChildren);
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = ConstantValueHandle->GetChildHandle(ChildIndex);
			if (ChildHandle.IsValid())
			{
				ChildBuilder.AddProperty(ChildHandle.ToSharedRef())
					.IsEnabled(TAttribute<bool>::CreateLambda([HasBinding]() { return !HasBinding(); }));
			}
		}
	}
	// All other types: intentionally empty -- suppresses ConstantValue and Binding as separate child rows.
}

} // namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE
