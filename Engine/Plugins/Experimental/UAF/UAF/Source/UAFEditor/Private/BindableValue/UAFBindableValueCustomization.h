// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Variables/VariablePickerArgs.h"

class UUAFRigVMAsset;

namespace UE::UAF::Editor
{

/**
 * Property type customization for FBindableValueBase subclasses
 * (FBindableBool, FBindableFloat, FBindableDouble, etc.).
 *
 * Places the constant-value widget and the SVariablePickerCombo side-by-side
 * in the value content area. Reads/writes FBindableValueBase::Binding directly
 * via EnumerateRawData — Binding has bare UPROPERTY() (no EditAnywhere) so it
 * never appears in the property tree and cannot be reached via GetChildHandle.
 */
class FBindableValueCustomization : public IPropertyTypeCustomization
{
public:
	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle>      InPropertyHandle,
		FDetailWidgetRow&                HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle>      InPropertyHandle,
		IDetailChildrenBuilder&          ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<IPropertyHandle> ConstantValueHandle;
	const UScriptStruct* ResolvedStructClass = nullptr;
	bool bIsBindableStruct = false;
	bool bIsBindableTransform = false;

	bool bIsContextSensitive = true;
	TArray<TWeakObjectPtr<const UUAFRigVMAsset>> FilterAssets;
	TWeakObjectPtr<const UUAFRigVMAsset> CurrentRigVMAsset;
	FOnIsContextSensitive OnIsContextSensitiveDelegate;
};

} // namespace UE::UAF::Editor
