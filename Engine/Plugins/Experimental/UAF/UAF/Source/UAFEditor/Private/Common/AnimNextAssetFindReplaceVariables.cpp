// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAssetFindReplaceVariables.h"

#include "AnimNextEdGraph.h"
#include "AnimNextExports.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "EditorStyleSet.h"
#include "IWorkspaceEditorModule.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "UncookedOnlyUtils.h"
#include "Components/VerticalBox.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Param/ParamCompatibility.h"
#include "Param/ParamUtils.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "Variables/SVariablePickerCombo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAssetFindReplaceVariables)

#define LOCTEXT_NAMESPACE "AnimNextAssetFindReplaceVariables"

FString UAnimNextAssetFindReplaceVariables::GetFindResultStringFromAssetData(const FAssetData& InAssetData) const
{
	FAnimNextAssetRegistryExports AnimNextExports;
	UE::UAF::UncookedOnly::FUtils::GetExportsOfTypeForAsset<FAnimNextVariableReferenceData>(InAssetData, AnimNextExports);

	TArray<FString> ReferenceNames;
	
	for (const FAnimNextExport& Export : AnimNextExports.Exports)
	{
		if (SearchReference.GetName() == Export.Identifier)
		{
			if (const FAnimNextVariableReferenceData* ReferenceData = Export.Data.GetPtr<FAnimNextVariableReferenceData>())
			{
				ReferenceNames.Add(ReferenceData->PinPath);
			}
		}
	}
	
	TStringBuilder<512> Builder;
	if(ReferenceNames.Num() > 0)
	{
		for(int32 NameIndex = 0; NameIndex < ReferenceNames.Num(); ++NameIndex)
		{
			Builder.Append(ReferenceNames[NameIndex]);
			if(NameIndex != ReferenceNames.Num() - 1)
			{
				Builder.Append(TEXT(", "));
			}
		}
	}
	return FString(Builder.ToString());
}

TConstArrayView<UClass*> UAnimNextAssetFindReplaceVariables::GetSupportedAssetTypes() const
{
	static UClass* Types[] = { UUAFRigVMAsset::StaticClass() };
	return Types;
}

bool UAnimNextAssetFindReplaceVariables::ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const
{
	TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin();
	if (SharedWorkspaceEditor.IsValid())
	{
		if (CurrentSearchScope == ESearchScope::Asset)
		{
			if (const UObject* FocusedObject = SharedWorkspaceEditor->GetFocusedWorkspaceDocument().Object)
			{
				if (InAssetData.GetSoftObjectPath() != FSoftObjectPath(FocusedObject->GetTypedOuter<UUAFRigVMAsset>()))
				{
					return true;
				}
			}	
		}
		else if (CurrentSearchScope == ESearchScope::Workspace)
		{
			TArray<FAssetData> Assets;
			constexpr bool bIncludeAssetReferences = true;
			SharedWorkspaceEditor->GetAssets(Assets, bIncludeAssetReferences);

			if (!Assets.Contains(InAssetData))
			{
				return true;
			}
		}
	}
	
	FAnimNextAssetRegistryExports AnimNextExports;
	UE::UAF::UncookedOnly::FUtils::GetExportsOfTypeForAsset<FAnimNextVariableReferenceData>(InAssetData, AnimNextExports);
	
	for (const FAnimNextExport& Export : AnimNextExports.Exports)
	{
		if (SearchReference.GetName() == Export.Identifier)
		{
			return false;
		}
	}

	return true;
}

void UAnimNextAssetFindReplaceVariables::ReplaceInAsset(const FAssetData& InAssetData) const
{
	if (UUAFRigVMAsset* RigVMAsset = Cast<UUAFRigVMAsset>(InAssetData.GetAsset()))
	{
		if (UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(RigVMAsset))
		{
			UE::UAF::UncookedOnly::FUtils::ReplaceVariableReferences(EditorData, SearchReference, ReplaceReference);
		}		
	}	
}

void UAnimNextAssetFindReplaceVariables::RemoveInAsset(const FAssetData& InAssetData) const
{	
	if (UUAFRigVMAsset* RigVMAsset = Cast<UUAFRigVMAsset>(InAssetData.GetAsset()))
	{
		if (UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(RigVMAsset))
		{
			// Replace with None reference (removes nodes)
			UE::UAF::UncookedOnly::FUtils::ReplaceVariableReferences(EditorData, SearchReference, FAnimNextSoftVariableReference());
		}		
	}
}

bool UAnimNextAssetFindReplaceVariables::CanCurrentlyReplace() const
{
	if (!SearchReference.IsNone() && !ReplaceReference.IsNone())
	{
		return true;
	}
	
	return false;
}

void UAnimNextAssetFindReplaceVariables::ExtendToolbar(FToolMenuSection& InSection)
{
	InSection.AddSubMenu
	(
		"SearchScopeSubMenu",
		LOCTEXT("SearchScopeLabel", "Search Scope"),
		LOCTEXT("SearchScopeTooltip", "Select scope to perform the search"),
		FNewToolMenuDelegate::CreateLambda(
			[this](UToolMenu* InSubmenu)
			{

				if (UEnum* ScopeEnum = FindObject<UEnum>(nullptr, TEXT("/Script/UAFEditor.ESearchScope")))
				{
					for (int32 Index = 0; Index < ScopeEnum->NumEnums(); ++Index)
					{
						const uint64 Value = ScopeEnum->GetValueByIndex(Index);
						ESearchScope TypedValue = static_cast<ESearchScope>(Value);
						if (Value != ScopeEnum->GetMaxEnumValue())
						{
							InSubmenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntry
							(
								ScopeEnum->GetNameByIndex(Index),
								ScopeEnum->GetDisplayNameTextByIndex(Index),
								ScopeEnum->GetToolTipTextByIndex(Index),
								GetIconFromSearchScope(TypedValue),
								FUIAction
								(
									FExecuteAction::CreateUObject(this, &UAnimNextAssetFindReplaceVariables::SetSearchScope, TypedValue),
									FCanExecuteAction::CreateUObject(this, &UAnimNextAssetFindReplaceVariables::IsSearchScopeEnable, TypedValue),
									FGetActionCheckState::CreateUObject(this, &UAnimNextAssetFindReplaceVariables::IsCurrentSearchScope, TypedValue)
								),									
								EUserInterfaceActionType::RadioButton
							));
						}
					
					}
				}
			}),
			true,
			TAttribute<FSlateIcon>::CreateLambda(
				[this]() -> FSlateIcon
				{
					return GetIconFromSearchScope(CurrentSearchScope);
				}
			),
			true
	);	
}

void UAnimNextAssetFindReplaceVariables::SetWorkspaceEditor(TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor)
{
	InWorkspaceEditor->OnFocusedDocumentChanged().AddUObject(this, &UAnimNextAssetFindReplaceVariables::OnWorkspaceDocumentChanged);
	WeakWorkspaceEditor = InWorkspaceEditor;
	
}

void UAnimNextAssetFindReplaceVariables::SetSearchScope(ESearchScope InSearchScope)
{
	CurrentSearchScope = InSearchScope;
	RequestRefreshSearchResults();
}

bool UAnimNextAssetFindReplaceVariables::IsSearchScopeEnable(ESearchScope InSearchScope) const
{
	if (InSearchScope == ESearchScope::Workspace || InSearchScope == ESearchScope::Asset)
	{
		return WeakWorkspaceEditor.IsValid();
	}
	
	return true;
}

ECheckBoxState UAnimNextAssetFindReplaceVariables::IsCurrentSearchScope(ESearchScope InSearchScope) const
{
	return CurrentSearchScope == InSearchScope ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void UAnimNextAssetFindReplaceVariables::OnWorkspaceDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument)
{
	if (CurrentSearchScope == ESearchScope::Asset)
	{
		RequestRefreshSearchResults();
	}
}

FSlateIcon UAnimNextAssetFindReplaceVariables::GetIconFromSearchScope(ESearchScope InSearchScope)
{
	if (InSearchScope == ESearchScope::Global)
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.RelativeCoordinateSystem_World");
	}
	else if (InSearchScope == ESearchScope::Workspace)
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner");
	}
	else if (InSearchScope == ESearchScope::Asset)
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Default");
	}

	return FSlateIcon();
}

TSharedRef<SWidget> UAnimNextAssetFindReplaceVariables::MakeFindReplaceWidget()
{	
	using namespace UE::UAF::Editor;


	auto VariableScopeFilter = FOnFilterVariable::CreateLambda([this](const FAnimNextSoftVariableReference& InVariableReference)
	{
      	// Filter variable picker to correct scope
      	bool bShouldFilterAsset = false;
      	TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin();
      	if (SharedWorkspaceEditor.IsValid())
      	{
      		// Include everything within the export chain of the currently opened asset
      		if (CurrentSearchScope == ESearchScope::Asset)
      		{
      			const UE::Workspace::FWorkspaceDocument& Document = SharedWorkspaceEditor->GetFocusedWorkspaceDocument();
  
      			TArray<const UUAFRigVMAsset*> AssetsInExportChain;
      			Document.Export.GetAssetsOfType<const UUAFRigVMAsset>(AssetsInExportChain, true);
  
      			auto ProcessAsset = [&AssetsInExportChain](const UUAFRigVMAsset* InAsset)
      			{
      				if (InAsset)
      				{
      					const UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<const UUAFRigVMAssetEditorData,const UUAFRigVMAsset>(InAsset);
      					EditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>([&AssetsInExportChain](const UUAFSharedVariablesEntry* VariablesEntry)
      					{
      						// [TODO] struct, currently doesn't have any exports so cannot filter against them
      						if (VariablesEntry->GetAsset())
      						{
      							AssetsInExportChain.AddUnique(VariablesEntry->GetAsset());							
      						}
  
      						return true;
      					});
      				}
      			};
      	
      			for (int32 AssetIndex = 0; AssetIndex < AssetsInExportChain.Num(); ++AssetIndex)
      			{
      				const UUAFRigVMAsset* Asset = AssetsInExportChain[AssetIndex];
      				// This can append to AssetsInExportChain array so it handles recursive shared variable references
      				ProcessAsset(Asset);
      			}
      			
      			// GetAssetsOfType force loads the assets, so if the variable owner is part of the export chain it would have been loaded		
      			const UObject* VariableOwner = InVariableReference.GetSoftObjectPath().ResolveObject();
      			bShouldFilterAsset = !VariableOwner || !AssetsInExportChain.Contains(VariableOwner);
      		}
      		else if (CurrentSearchScope == ESearchScope::Workspace)
      		{
      			TArray<FAssetData> Assets;
      			SharedWorkspaceEditor->GetAssets(Assets);
  
      			if (!Assets.ContainsByPredicate([ObjectPath = InVariableReference.GetSoftObjectPath()](const FAssetData& AssetData)
      			{
      				return AssetData.GetSoftObjectPath() == ObjectPath;
      			}))
      			{
      				bShouldFilterAsset = true;
      			}
      		}
      	}			
      	return (SearchReference != InVariableReference && !bShouldFilterAsset) ? EFilterVariableResult::Include : EFilterVariableResult::Exclude;		
    });

	{
		FVariablePickerArgs SearchArgs;
		SearchArgs.OnVariablePicked = FOnVariablePicked::CreateLambda([this](const FAnimNextSoftVariableReference& InVariableReference, const FAnimNextParamType& InType)
		{
			SearchReference = InVariableReference;
			SearchType = InType;
			
			RequestRefreshSearchResults();
		});

		SearchArgs.OnFilterVariable = VariableScopeFilter;

		// Filter to all declared variables (public and private)
		SearchArgs.FlagInclusionFilter = EAnimNextExportedVariableFlags::Declared;
	
		SearchVariableComboBox = SNew(SVariablePickerCombo)
			.PickerArgs(SearchArgs)
			.OnGetVariableReference_Lambda([this]()
			{
				return SearchReference;
			})
			.OnGetVariableType_Lambda([this]()
			{
				return SearchType;
			});
	}
	
	{
		FVariablePickerArgs ReplaceArgs;
		ReplaceArgs.FlagInclusionFilter = EAnimNextExportedVariableFlags::Declared;
		ReplaceArgs.OnVariablePicked = FOnVariablePicked::CreateLambda([this](const FAnimNextSoftVariableReference& InVariableReference, const FAnimNextParamType& InType)
		{
			ReplaceReference = InVariableReference;
		});
		ReplaceArgs.OnFilterVariable = VariableScopeFilter;
		ReplaceArgs.OnFilterVariableType  = FOnFilterVariableType::CreateLambda([this](const FAnimNextParamType& InParamType)-> EFilterVariableResult
		{
			if(SearchType.IsValid())
			{
				// [TODO] IsCompatible or IsCompatibleWithDataLoss (e.g. UScriptStruct float entry to double RigVM entry fails IsCompatible)
				if(!UE::UAF::FParamUtils::GetCompatibility(SearchType, InParamType).IsCompatibleWithDataLoss())
				{
					return EFilterVariableResult::Exclude;
				}
			}

			if(InParamType.IsValid())
			{
				const FRigVMTemplateArgumentType RigVMType = InParamType.ToRigVMTemplateArgument();
				if(!RigVMType.IsValid() || FRigVMRegistry::Get().GetTypeIndex(RigVMType) == INDEX_NONE)
				{
					return EFilterVariableResult::Exclude;
				}
			}

			return EFilterVariableResult::Include;
		});	
		ReplaceVariableComboBox = SNew(SVariablePickerCombo)
			.PickerArgs(ReplaceArgs)
			.OnGetVariableReference_Lambda([this]()
			{
				return ReplaceReference;
			})
			.OnGetVariableType_Lambda([this]()
			{
				return SearchType;
			});
	}

	return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(6.0f, 0.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(0.15f)
			.VAlign(VAlign_Center)
			[				
				SNew(STextBlock)
				.Text(LOCTEXT("SearchVariableLabel", "Search Variable:"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(0.85f)
			.Padding(6.0f, 0.0f)
			[				
				SearchVariableComboBox.ToSharedRef()
			]
		]
		+SVerticalBox::Slot()
		.Padding(6.0f, 2.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(0.15f)
			.VAlign(VAlign_Center)
			[				
				SNew(STextBlock).Text(LOCTEXT("ReplaceVariableLabel", "Replace Variable:"))
			]
			+SHorizontalBox::Slot()
			.FillWidth(0.85f)
			.Padding(6.0f, 0.0f)
            [	
				ReplaceVariableComboBox.ToSharedRef()
	        ]			
		];
}

void UAnimNextAssetFindReplaceVariables::SetFindReferenceFromEntry(const UAnimNextVariableEntry* InVariableEntry)
{
	SearchReference = FAnimNextSoftVariableReference::FromName(InVariableEntry->GetVariableName(), InVariableEntry->GetTypedOuter<UUAFRigVMAsset>());;
	SearchType = InVariableEntry->GetType();

	RequestRefreshSearchResults();
}

void UAnimNextAssetFindReplaceVariables::SetFindReference(const FAnimNextSoftVariableReference& InVariableReference, const FAnimNextParamType& InVariableType)
{
	SearchReference = InVariableReference;
	SearchType = InVariableType;

	RequestRefreshSearchResults();
}

#undef LOCTEXT_NAMESPACE
