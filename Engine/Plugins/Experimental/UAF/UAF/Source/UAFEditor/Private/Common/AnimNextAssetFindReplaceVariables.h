// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimAssetFindReplace.h"
#include "IWorkspaceEditor.h"
#include "Param/ParamType.h"
#include "Variables/AnimNextSoftVariableReference.h"

#include "AnimNextAssetFindReplaceVariables.generated.h"

class UAnimNextVariableEntry;

namespace UE::UAF::Editor
{
class SVariablePickerCombo;
}

namespace UE::Workspace
{
struct FWorkspaceDocument;
}

UENUM()
enum class ESearchScope : uint8
{
	Global, // Filter against assets with AnimNext variable exports
	Workspace, // Filter against assets within workspace and any of their asset references (declaration or reference)
	Asset // Filter against currently open asset and its recursive reference chain
};

/** Find, replace and remove AnimNext variable references across assets */
UCLASS(MinimalAPI, DisplayName="UAF Variables")
class UAnimNextAssetFindReplaceVariables : public UAnimAssetFindReplaceProcessor
{
	GENERATED_BODY()

public:
	UAnimNextAssetFindReplaceVariables() = default;

	static FSlateIcon GetIconFromSearchScope(ESearchScope InSearchScope);
	
	// Begin UAnimAssetFindReplaceProcessor interface
	virtual FString GetFindResultStringFromAssetData(const FAssetData& InAssetData) const override;
	virtual TConstArrayView<UClass*> GetSupportedAssetTypes() const override;
	virtual bool ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const override;
	virtual void ReplaceInAsset(const FAssetData& InAssetData) const override;
	virtual void RemoveInAsset(const FAssetData& InAssetData) const override;
	virtual bool CanCurrentlyReplace() const override;
	virtual bool CanCurrentlyRemove() const override { return true; }
	virtual void ExtendToolbar(FToolMenuSection& InSection) override;	
	virtual TSharedRef<SWidget> MakeFindReplaceWidget() override;
	virtual bool SupportsMode(EAnimAssetFindReplaceMode InMode) const override { return true; }
	// End UAnimAssetFindReplaceProcessor interface

	void SetFindReferenceFromEntry(const UAnimNextVariableEntry* InVariableEntry);
	void SetFindReference(const FAnimNextSoftVariableReference& InVariableReference, const FAnimNextParamType& InVariableType);
	void SetSearchScope(ESearchScope InSearchScope);
	void SetWorkspaceEditor(TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor);	
protected:	
	bool IsSearchScopeEnable(ESearchScope InSearchScope) const;
	ECheckBoxState IsCurrentSearchScope(ESearchScope InSearchScope) const;
	void OnWorkspaceDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument);
protected:
	TSharedPtr<UE::UAF::Editor::SVariablePickerCombo> SearchVariableComboBox;	
	TSharedPtr<UE::UAF::Editor::SVariablePickerCombo> ReplaceVariableComboBox;

	FAnimNextSoftVariableReference SearchReference;
	FAnimNextParamType SearchType;
	FAnimNextSoftVariableReference ReplaceReference;
	ESearchScope CurrentSearchScope = ESearchScope::Global;
	
	TWeakPtr<UE::Workspace::IWorkspaceEditor> WeakWorkspaceEditor = nullptr;
};
