// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextCollapseNodeItemDetails.h"

#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "StructUtils/InstancedStruct.h"
#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "RigVMModel/RigVMGraph.h"
#include "WorkspaceItemMenuContext.h"
#include "IWorkspaceEditor.h"
#include "ScopedTransaction.h"
#include "RigVMModel/RigVMClient.h"
#include "EdGraph/RigVMEdGraph.h"
#include "ToolMenus.h"
#include "Framework/Commands/GenericCommands.h"
#include "Module/AnimNextModule_EditorData.h"

#define LOCTEXT_NAMESPACE "FAnimNextCollapseNodeItemDetails"

namespace UE::UAF::Editor
{

bool FAnimNextCollapseNodeItemDetails::HandleDoubleClick(const FToolMenuContext& ToolMenuContext) const
{
	const UWorkspaceItemMenuContext* WorkspaceItemContext = ToolMenuContext.FindContext<UWorkspaceItemMenuContext>();
	const UAssetEditorToolkitMenuContext* AssetEditorContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>();
	if (WorkspaceItemContext && AssetEditorContext)
	{
		if (const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorContext->Toolkit.Pin()))
		{
			if (WorkspaceItemContext->SelectedExports.Num() > 0)
			{
				WorkspaceEditor->OpenExports({ WorkspaceItemContext->SelectedExports[0].GetResolvedExport() });
				return true;
			}
		}
	}
	
	return false;
}

bool FAnimNextCollapseNodeItemDetails::CanDelete(const FWorkspaceOutlinerItemExport& Export) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextCollapseGraphOutlinerData::StaticStruct())
	{
		const FAnimNextCollapseGraphOutlinerData& CollapseGraphData = Export.GetData().Get<FAnimNextCollapseGraphOutlinerData>();
		if (CollapseGraphData.SoftEditorObject.IsValid())
		{
			URigVMEdGraph* EdGraph = CollapseGraphData.SoftEditorObject.Get();
			if (EdGraph->bAllowDeletion)
			{
				return true;
			}
		}
	}
	return false;
}

void FAnimNextCollapseNodeItemDetails::Delete(TConstArrayView<FWorkspaceOutlinerItemExport> Exports) const
{
	TMap<UUAFRigVMAssetEditorData*, TArray<UUAFRigVMAssetEntry*>> EntriesToDelete;
	for (const FWorkspaceOutlinerItemExport& Export : Exports)
	{
		const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
		if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextCollapseGraphOutlinerData::StaticStruct())
		{
			const FAnimNextCollapseGraphOutlinerData& CollapseGraphData = Export.GetData().Get<FAnimNextCollapseGraphOutlinerData>();
			if (CollapseGraphData.SoftEditorObject.IsValid())
			{
				URigVMEdGraph* EdGraph = CollapseGraphData.SoftEditorObject.Get();
				if (EdGraph->bAllowDeletion)
				{
					if (URigVMGraph* Model = EdGraph->GetModel())
					{
						if (URigVMCollapseNode* CollapseNode = CastChecked<URigVMCollapseNode>(EdGraph->GetModel()->GetOuter()))
						{
							URigVMGraph* ContainerGraph = CollapseNode->GetGraph();
							if (IRigVMClientHost* ClientHost = ContainerGraph->GetImplementingOuter<IRigVMClientHost>())
							{
								if (URigVMEdGraph* ContainerEdGraph = Cast<URigVMEdGraph>(ClientHost->GetEditorObjectForRigVMGraph(ContainerGraph)))
								{
									FScopedTransaction Transaction(LOCTEXT("DeleteCollapseNodeInOutliner", "Delete Collapse Node"));
									ContainerEdGraph->GetController()->RemoveNode(CollapseNode, true, true);
								}
							}
						}
					}
				}
			}
		}
	}
}

bool FAnimNextCollapseNodeItemDetails::CanRename(const FWorkspaceOutlinerItemExport& Export) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextCollapseGraphOutlinerData::StaticStruct())
	{
		const FAnimNextCollapseGraphOutlinerData& CollapseGraphData = Export.GetData().Get<FAnimNextCollapseGraphOutlinerData>();
		if (CollapseGraphData.SoftEditorObject.IsValid())
		{
			URigVMEdGraph* EdGraph = CollapseGraphData.SoftEditorObject.Get();
			if (EdGraph->bAllowRenaming)
			{
				return true;
			}
		}
	}
	return false;
}

void FAnimNextCollapseNodeItemDetails::Rename(const FWorkspaceOutlinerItemExport& Export, const FText& InName) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextCollapseGraphOutlinerData::StaticStruct())
	{
		const FAnimNextCollapseGraphOutlinerData& CollapseGraphData = Export.GetData().Get<FAnimNextCollapseGraphOutlinerData>();
		if (CollapseGraphData.SoftEditorObject.IsValid())
		{
			URigVMEdGraph* EdGraph = CollapseGraphData.SoftEditorObject.Get();
			if (EdGraph->bAllowRenaming)
			{
				if (const UEdGraphSchema* GraphSchema = EdGraph->GetSchema())
				{
					FGraphDisplayInfo DisplayInfo;
					GraphSchema->GetGraphDisplayInformation(*EdGraph, DisplayInfo);

					// Check if the name is unchanged
					if (InName.EqualTo(DisplayInfo.PlainName))
					{
						return;
					}

					FScopedTransaction Transaction(LOCTEXT("RenameCollapseNodeInOutliner", "Rename Collapse Node"));
					FRigVMControllerCompileBracketScope CompileScope(EdGraph->GetController());
					if (GraphSchema->TryRenameGraph(EdGraph, *InName.ToString()))
					{
						return;
					}
				}
			}
		}
	}
}

bool FAnimNextCollapseNodeItemDetails::ValidateName(const FWorkspaceOutlinerItemExport& Export, const FText& InName, FText& OutErrorMessage) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextCollapseGraphOutlinerData::StaticStruct())
	{
		return true;
	}

	OutErrorMessage = LOCTEXT("UnsupportedTypeCollapseNodeRenameError", "Element type is not supported for rename");
	return false;
}

UPackage* FAnimNextCollapseNodeItemDetails::GetPackage(const FWorkspaceOutlinerItemExport& Export) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextCollapseGraphOutlinerData::StaticStruct())
	{
		const FAnimNextCollapseGraphOutlinerData& CollapseGraphData = Data.Get<FAnimNextCollapseGraphOutlinerData>();
		if (CollapseGraphData.SoftEditorObject.IsValid())
		{
			return CollapseGraphData.SoftEditorObject->GetPackage();
		}
	}
	return nullptr;
}

const FSlateBrush* FAnimNextCollapseNodeItemDetails::GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const
{
	return FAppStyle::GetBrush(TEXT("GraphEditor.SubGraph_24x"));
}

void FAnimNextCollapseNodeItemDetails::RegisterToolMenuExtensions()
{
}

void FAnimNextCollapseNodeItemDetails::UnregisterToolMenuExtensions()
{
}

FString FAnimNextCollapseNodeItemDetails::GetDisplayString(const FWorkspaceOutlinerItemExport& Export) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextCollapseGraphOutlinerData::StaticStruct())
	{
		const FAnimNextCollapseGraphOutlinerData& CollapseGraphData = Data.Get<FAnimNextCollapseGraphOutlinerData>();
		if (CollapseGraphData.SoftEditorObject.IsValid())
		{
			const URigVMEdGraph* EdGraph = CollapseGraphData.SoftEditorObject.Get();
			if (const UEdGraphSchema* GraphSchema = EdGraph->GetSchema())
			{
				FGraphDisplayInfo DisplayInfo;
				GraphSchema->GetGraphDisplayInformation(*EdGraph, DisplayInfo);

				return DisplayInfo.PlainName.ToString();
			}
		}
	}
	return Export.GetIdentifier().ToString();
}

} // UE::UAF::Editor

#undef LOCTEXT_NAMESPACE // "FAnimNextCollapseNodeItemDetails"
