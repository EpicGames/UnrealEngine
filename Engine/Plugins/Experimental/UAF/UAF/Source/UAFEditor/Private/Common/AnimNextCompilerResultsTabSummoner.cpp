// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/AnimNextCompilerResultsTabSummoner.h"

#include "AnimNextEdGraphNode.h"
#include "AnimNextRigVMAsset.h"
#include "EdGraphToken.h"
#include "IAnimNextEditorModule.h"
#include "IWorkspaceEditor.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "WorkspaceTabSummoner"

namespace UE::UAF::Editor
{

// ***************************************************************************

void SAnimNextCompilerResultsWidget::Construct(const FArguments& InArgs, const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditorWeak)
{
	WorkspaceEditorWeak = InWorkspaceEditorWeak;
	CreateMessageLog(InWorkspaceEditorWeak);

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.FillHeight(1.0)
		.Padding(10.f, 10.f, 10.f, 10.f)
		[
			CompilerResults.ToSharedRef()
		]
	];
}

void SAnimNextCompilerResultsWidget::CreateMessageLog(const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditorWeak)
{
	if (const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditorShared = InWorkspaceEditorWeak.Pin())
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		check(MessageLogModule.IsRegisteredLogListing(LogListingName));
		CompilerResultsListing = MessageLogModule.GetLogListing(LogListingName);
		CompilerResults = MessageLogModule.CreateLogListingWidget(CompilerResultsListing.ToSharedRef());
		CompilerResultsListing->OnMessageTokenClicked().AddSP(this, &SAnimNextCompilerResultsWidget::HandleMessageTokenClicked);
	}
}

void SAnimNextCompilerResultsWidget::HandleMessageTokenClicked(const TSharedRef<IMessageToken>& InToken)
{
	switch(InToken->GetType())
	{
	case EMessageToken::EdGraph:
		{
			TSharedRef<FEdGraphToken> EdGraphToken = StaticCastSharedRef<FEdGraphToken>(InToken);
			if(const UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(EdGraphToken->GetGraphObject()))
			{
				UUAFRigVMAsset* Asset = EdGraphNode->GetTypedOuter<UUAFRigVMAsset>();
				if(Asset == nullptr)
				{
					return;
				}

				TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = WorkspaceEditorWeak.Pin();
				if(!WorkspaceEditor.IsValid())
				{
					return;
				}

				WorkspaceEditor->OpenObjects({const_cast<UAnimNextEdGraphNode*>(EdGraphNode)});
			}
			break;
		}
	default:
		break;
	}
}

// ***************************************************************************

FAnimNextCompilerResultsTabSummoner::FAnimNextCompilerResultsTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp)
	: FWorkflowTabFactory(UE::UAF::Editor::CompilerResultsTabName, StaticCastSharedPtr<FAssetEditorToolkit>(InHostingApp))
{
	TabLabel = LOCTEXT("AnimNExtCompilerResultsTabLabel", "Compiler Results");
	TabIcon = FSlateIcon("EditorStyle", "LevelEditor.Tabs.Outliner");
	ViewMenuDescription = LOCTEXT("AnimNExtCompilerResultsTabMenuDescription", "Compiler Results");
	ViewMenuTooltip = LOCTEXT("AnimNExtCompilerResultsTabToolTip", "Shows the Compiler Results tab.");
	bIsSingleton = true;

	AnimNextCompilerResultsWidget = SNew(SAnimNextCompilerResultsWidget, InHostingApp.ToWeakPtr());
}

TSharedRef<SWidget> FAnimNextCompilerResultsTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return AnimNextCompilerResultsWidget.ToSharedRef();
}

FText FAnimNextCompilerResultsTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return ViewMenuTooltip;
}

} // end namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE
