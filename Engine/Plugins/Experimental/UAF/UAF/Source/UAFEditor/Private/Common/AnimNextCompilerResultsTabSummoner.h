// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "IMessageLogListing.h"

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{
class SAnimNextCompilerResultsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimNextCompilerResultsWidget) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditorWeak);

	inline TSharedPtr<class IMessageLogListing> GetCompilerResultsListing() const
	{
		return CompilerResultsListing;
	}

private:
	void CreateMessageLog(const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditorWeak);

	void HandleMessageTokenClicked(const TSharedRef<IMessageToken>& InToken);

	TSharedPtr<class IMessageLogListing> CompilerResultsListing;
	TSharedPtr<class SWidget> CompilerResults;
	TWeakPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditorWeak;
};

struct FAnimNextCompilerResultsTabSummoner : public FWorkflowTabFactory
{
public:
	FAnimNextCompilerResultsTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp);

private:
	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

	// The widget this tab spawner wraps
	TSharedPtr<class SAnimNextCompilerResultsWidget> AnimNextCompilerResultsWidget;
};

} // end namespace UE::UAF::Editor
