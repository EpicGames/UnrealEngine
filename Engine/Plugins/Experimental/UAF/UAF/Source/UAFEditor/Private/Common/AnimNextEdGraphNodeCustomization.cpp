// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/AnimNextEdGraphNodeCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "AnimNextEdGraphNode.h"
#include "Editor/RigVMGraphDetailCustomization.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "ITraitStackEditor.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "IWorkspaceEditor.h"
#include "RigVMDeveloperTypeUtils.h"
#include "Features/IModularFeatures.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/SRigVMNodeLayoutWidget.h"

#define LOCTEXT_NAMESPACE "EdGraphNodeCustomization"

namespace UE::UAF::Editor
{

FAnimNextEdGraphNodeCustomization::FAnimNextEdGraphNodeCustomization(const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditorWeak)
	: WorkspaceEditorWeak(InWorkspaceEditorWeak)
{
}

void FAnimNextEdGraphNodeCustomization::PendingDelete()
{
	CategoryDetailsData.Reset();
	RigVMGraphDetailCustomizationImpl.Reset();

	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = WorkspaceEditorWeak.Pin())
	{
		if(IModularFeatures::Get().IsModularFeatureAvailable(ITraitStackEditor::ModularFeatureName))
		{
			ITraitStackEditor& TraitStackEditor = IModularFeatures::Get().GetModularFeature<ITraitStackEditor>(ITraitStackEditor::ModularFeatureName);
			TraitStackEditor.SetTraitData(WorkspaceEditor.ToSharedRef(), FTraitStackData());
		}
	}
}

void FAnimNextEdGraphNodeCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> WeakObjects;
	DetailBuilder.GetObjectsBeingCustomized(WeakObjects);
	TArray<UObject*> Objects;
	Algo::TransformIf(WeakObjects, Objects, [](TWeakObjectPtr<UObject> InObject){ return InObject.Get() != nullptr; }, [](TWeakObjectPtr<UObject> InObject){ return InObject.Get(); });

	RigVMGraphDetailCustomizationImpl.Reset();

	if (Objects.IsEmpty())
	{
		return;
	}

	CustomizeObjects(DetailBuilder, Objects);
}

void FAnimNextEdGraphNodeCustomization::CustomizeObjects(IDetailLayoutBuilder& DetailBuilder, const TArray<UObject*>& Objects)
{
	const int32 NumNodes = Objects.Num();
	for (int32 NodeIndex = 0; NodeIndex < NumNodes; ++NodeIndex)
	{
		UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(Objects[NodeIndex]);
		if(EdGraphNode == nullptr)
		{
			continue;
		}

		URigVMNode* ModelNode = EdGraphNode->GetModelNode();
		if(ModelNode == nullptr)
		{
			continue;
		}

		if (ModelNode->GetTraitNames().Num() > 0)
		{
			RigVMGraphDetailCustomizationImpl.Reset();
			GenerateTraitData(EdGraphNode, CategoryDetailsData);
		}
		else 
		{
			bool bIsFunction = false;
			bool bIsCollapseNode = false;
			bool bIsAggregate = false;

			if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(ModelNode))
			{
				if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(ModelNode))
				{
					if (URigVMLibraryNode* ReferencedNode = Cast<URigVMLibraryNode>(FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer.GetNodeSoftPath().ResolveObject()))
					{
						LibraryNode = ReferencedNode;
						bIsFunction = true;
					}
				}
				else if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(ModelNode))
				{
					bIsCollapseNode = true;
				}
				bIsAggregate = LibraryNode->IsA<URigVMAggregateNode>();

				if (bIsFunction || bIsCollapseNode || bIsAggregate)
				{
					if (NumNodes > 1) // no multiselection of function / collapse nodes
					{
						continue;
					}

					TWeakPtr<FRigVMEditorBase> WeakEditor; // passing an empty RigVMEditor to FRigVMCollapseGraphLayout
					IRigVMClientHost* RigVMClientHost = EdGraphNode->GetController()->GetClientHost();
					check(RigVMClientHost);

					RigVMGraphDetailCustomizationImpl = MakeShared<FRigVMGraphDetailCustomizationImpl>();
					RigVMGraphDetailCustomizationImpl->CustomizeDetails(DetailBuilder
						, LibraryNode->GetContainedGraph()
						, RigVMClientHost->GetController(LibraryNode->GetContainedGraph())
						, RigVMClientHost
						, WeakEditor);
				}
			}

			GenerateRigVMData(EdGraphNode, CategoryDetailsData);
		}
	}

	// Create a header widget to edit the trait stack
	bool bAreAllNodesTraitStacks = true;
	URigVMController* Controller = nullptr;
	for (TWeakObjectPtr<UObject> WeakObject : Objects)
	{
		if (UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(WeakObject.Get()))
		{
			if (EdGraphNode->GetModelNode())
			{
				bAreAllNodesTraitStacks &= EdGraphNode->GetModelNode()->GetClass()->GetClassPathName() == FTopLevelAssetPath(TEXT("/Script/UAFAnimGraphUncookedOnly"), TEXT("AnimNextTraitStackUnitNode"));
				Controller = EdGraphNode->GetController();
			}
		}
	}

	if(bAreAllNodesTraitStacks && Controller && IModularFeatures::Get().IsModularFeatureAvailable(ITraitStackEditor::ModularFeatureName))
	{
		IDetailCategoryBuilder& HeaderCategory = DetailBuilder.EditCategory(TEXT("TraitStack"), FText::GetEmpty(), ECategoryPriority::Important);
		HeaderCategory.AddCustomRow(FText::GetEmpty());
		HeaderCategory.SetIsEmpty(true);

		ITraitStackEditor& TraitStackEditor = IModularFeatures::Get().GetModularFeature<ITraitStackEditor>(ITraitStackEditor::ModularFeatureName);
		HeaderCategory.HeaderContent(
			TraitStackEditor.CreateTraitStackHeaderWidget(HeaderCategory, Controller),
			true);
	}
	
	for (TSharedPtr<FCategoryDetailsData>& DetailsData : CategoryDetailsData)
	{
		PopulateCategory(DetailBuilder, DetailsData);
	}

	// Pass the TraitStack EdGraphNode to the TraitEditor (or nullptr if we select other node type, like RigVM)
	if (Objects.Num() == 1 && bAreAllNodesTraitStacks)
	{
		if (UAnimNextEdGraphNode* EdGraphNode = Cast<UAnimNextEdGraphNode>(Objects[0]))
		{
			WeakModelNode = EdGraphNode->GetModelNode();
			if (WeakModelNode.Get())
			{
				WeakRigVMClientHost = WeakModelNode->GetImplementingOuter<IRigVMClientHost>();

				if (WeakRigVMClientHost.Get() && WeakModelNode.Get())
				{
					// Add layout customization
					IDetailCategoryBuilder& CustomizationCategory = DetailBuilder.EditCategory("Customization", LOCTEXT("CustomizationCategory", "Customization"), ECategoryPriority::Default);
					CustomizationCategory
					.InitiallyCollapsed(true);

					CustomizationCategory.AddExternalObjectProperty({ WeakModelNode.Get() }, TEXT("Template")); 
					
					EdGraphNode->NodeColor = EdGraphNode->GetModelNode()->GetNodeColor();

					CustomizationCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UAnimNextEdGraphNode, NodeColor))
					.OverrideResetToDefault(FResetToDefaultOverride::Hide())
					.GetPropertyHandle()->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda([this, WeakEdGraphNode = TWeakObjectPtr<UAnimNextEdGraphNode>(EdGraphNode)](const FPropertyChangedEvent& InEvent)
					{
						if (InEvent.ChangeType != EPropertyChangeType::Interactive)
						{
							URigVMNode* ModelNode = WeakModelNode.Get();
							IRigVMClientHost* RigVMClientHost = WeakRigVMClientHost.Get();
							UAnimNextEdGraphNode* EdGraphNode = WeakEdGraphNode.Get();
							if (ModelNode && RigVMClientHost && EdGraphNode)
							{
								URigVMController* Controller = RigVMClientHost->GetController(ModelNode->GetGraph());
								Controller->SetNodeColor(ModelNode, EdGraphNode->NodeColor, true, true, true);
							}
						}
					}));

					// Widget that prevents RigVM compilation as long as it is alive (used to prevent popup from closing for each layout operation)
					class SRigVMCompileLock : public SBox
					{
						SLATE_BEGIN_ARGS(SRigVMCompileLock) {}
						SLATE_DEFAULT_SLOT(FArguments, Content)
						SLATE_END_ARGS()

						void Construct(const FArguments& InArgs, URigVMController* InController)
						{
							Scope = MakeShared<FRigVMControllerCompileBracketScope>(InController);
							SBox::Construct(SBox::FArguments()
								.WidthOverride(300.0f)
								.HeightOverride(300.0f)
								.Content()[InArgs._Content.Widget]);
						}

						TSharedPtr<FRigVMControllerCompileBracketScope> Scope;
					};
					
					FDetailWidgetRow& Row = CustomizationCategory.AddCustomRow(LOCTEXT("LayoutSearchString", "Node Layout"));
					Row
					.NameContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("LayoutSearchString", "Node Layout"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					.ValueContent()
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SComboButton)
							.ButtonContent()
							[
								SNew(STextBlock)
								.Text(LOCTEXT("EditLayout", "Edit Layout"))
								.Font(IDetailLayoutBuilder::GetDetailFont())
							]
							.HasDownArrow(true)
							.OnGetMenuContent_Lambda([this, WeakEdGraphNode = TWeakObjectPtr<UAnimNextEdGraphNode>(EdGraphNode)]() -> TSharedRef<SWidget>
							{
								UAnimNextEdGraphNode* EdGraphNode = WeakEdGraphNode.Get();
								if (EdGraphNode == nullptr)
								{
									return SNullWidget::NullWidget;
								}
								
								return
									SNew(SRigVMCompileLock, EdGraphNode->GetController())
									[
										SNew(SRigVMNodeLayoutWidget)
										.OnGetUncategorizedPins(this, &FAnimNextEdGraphNodeCustomization::GetUncategorizedPins)
										.OnGetCategories(this, &FAnimNextEdGraphNodeCustomization::GetPinCategories)
										.OnGetElementCategory(this, &FAnimNextEdGraphNodeCustomization::GetPinCategory)
										.OnGetElementIndexInCategory(this, &FAnimNextEdGraphNodeCustomization::GetPinIndexInCategory)
										.OnGetElementLabel(this, &FAnimNextEdGraphNodeCustomization::GetPinLabel)
										.OnGetElementColor(this, &FAnimNextEdGraphNodeCustomization::GetPinColor)
										.OnGetElementIcon(this, &FAnimNextEdGraphNodeCustomization::GetPinIcon)
										.OnCategoryAdded(this, &FAnimNextEdGraphNodeCustomization::HandleCategoryAdded)
										.OnCategoryRemoved(this, &FAnimNextEdGraphNodeCustomization::HandleCategoryRemoved)
										.OnCategoryRenamed(this, &FAnimNextEdGraphNodeCustomization::HandleCategoryRenamed)
										.OnElementCategoryChanged(this, &FAnimNextEdGraphNodeCustomization::HandlePinCategoryChanged)
										.OnElementLabelChanged(this, &FAnimNextEdGraphNodeCustomization::HandlePinLabelChanged)
										.OnElementIndexInCategoryChanged(this, &FAnimNextEdGraphNodeCustomization::HandlePinIndexInCategoryChanged)
										.OnValidateCategoryName(this, &FAnimNextEdGraphNodeCustomization::HandleValidateCategoryName)
										.OnValidateElementName(this, &FAnimNextEdGraphNodeCustomization::HandleValidatePinDisplayName)
										.OnGetStructuralHash(this, &FAnimNextEdGraphNodeCustomization::GetNodeLayoutHash)
									];
							})
						]
					];
				}

				if (TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = WorkspaceEditorWeak.Pin())
				{
					if(IModularFeatures::Get().IsModularFeatureAvailable(ITraitStackEditor::ModularFeatureName))
					{
						ITraitStackEditor& TraitStackEditor = IModularFeatures::Get().GetModularFeature<ITraitStackEditor>(ITraitStackEditor::ModularFeatureName);
						TraitStackEditor.SetTraitData(WorkspaceEditor.ToSharedRef(), FTraitStackData(EdGraphNode));
					}
				}
			}
		}
	}

	// hide color editor fo non-trait stacks
	if (!bAreAllNodesTraitStacks)
	{
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimNextEdGraphNode, NodeColor))
		->MarkHiddenByCustomization();
	}
}

void FAnimNextEdGraphNodeCustomization::GenerateTraitData(UAnimNextEdGraphNode* EdGraphNode, TArray<TSharedPtr<FCategoryDetailsData>>& CategoryDetailsData)
{
	if (URigVMNode* ModelNode = EdGraphNode->GetModelNode())
	{
		// Obtain the pins from the stack
		const TArray<URigVMPin*> TraitPins = ModelNode->GetTraitPins();
		if (TraitPins.Num() > 0)
		{
			TMap<const UScriptStruct*, int32> TypeCounts; 

			// For each Trait (represented as a pin in the node)
			for (const URigVMPin* TraitPin : TraitPins)
			{
				if (TraitPin->IsExecuteContext())
				{
					continue;
				}

				// Create a temporary trait instance, in order to get the correct TraitSharedDataStruct
				if (TSharedPtr<FStructOnScope> ScopedTrait = ModelNode->GetTraitInstance(TraitPin->GetFName()))
				{
					// Create a scoped struct with the Trait Shared Instance Data and store it for later use
					// The idea is creating one template per Trait type, but pass all the selected instances
					// So the multiselection works
					const FRigVMTrait* Trait = (FRigVMTrait*)ScopedTrait->GetStructMemory();

					// Import the default pin values into the Trait Shared Instance Data
					FTraitStackDetailsData* TraitData = nullptr;
					if (UScriptStruct* TraitSharedInstanceData = Trait->GetTraitSharedDataStruct())
					{
						const TSharedPtr<FCategoryDetailsData>* TraitDataPtr = CategoryDetailsData.FindByPredicate([TraitSharedInstanceData, TraitPin](const TSharedPtr<FCategoryDetailsData>& InItem)
							{
								return InItem->Type == FCategoryDetailsData::EType::TraitStack && InItem->Name == TraitPin->GetFName();
							});

						if (TraitDataPtr == nullptr)
						{
							TraitDataPtr = &CategoryDetailsData.Add_GetRef(MakeShared<FTraitStackDetailsData>(TraitPin->GetFName(), TypeCounts.FindRef(TraitSharedInstanceData)));
						}

						TraitData = static_cast<FTraitStackDetailsData*>(TraitDataPtr->Get());

						TypeCounts.FindOrAdd(TraitSharedInstanceData)++;

						// Store EdGraphNode and ScopedTraitData, as we will need that later to transfer data if the user makes modifications to the Traits in the details panel
						TraitData->EdGraphNodes.Add(EdGraphNode);
						const int32 ScopedDataIndex = TraitData->ScopedSharedDataInstances.Add(MakeShared<FStructOnScope>(TraitSharedInstanceData));

						// Fill the scoped data with the pin data
						FRigVMPinDefaultValueImportErrorContext ErrorPipe(ELogVerbosity::Verbose);
						LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ErrorPipe.GetMaxVerbosity());
						const FString DefaultValue = TraitPin->GetDefaultValue();
						TraitSharedInstanceData->ImportText(*DefaultValue, TraitData->ScopedSharedDataInstances[ScopedDataIndex]->GetStructMemory(), nullptr, PPF_SerializedAsImportText, &ErrorPipe, TraitSharedInstanceData->GetName());
					}
				}
			}
		}
	}
}

void FAnimNextEdGraphNodeCustomization::GenerateRigVMData(UAnimNextEdGraphNode* EdGraphNode, TArray<TSharedPtr<FCategoryDetailsData>>& CategoryDetailsData)
{
	if (URigVMNode* ModelNode = EdGraphNode->GetModelNode())
	{
		// For nodes that aren't Traits, we display the pins as properties
		const TArray<URigVMPin*> ModelPins = ModelNode->GetPins();
		TArray<TWeakObjectPtr<URigVMPin>> PinsToDisplay;
		PinsToDisplay.Reserve(ModelPins.Num());

		TArray<FString> ModelPinPaths;

		// First, obtain the pins to display
		for (URigVMPin* Pin : ModelPins)
		{
			if (Pin->IsExecuteContext())
			{
				continue;
			}

			const ERigVMPinDirection PinDirection = Pin->GetDirection();
			if (PinDirection != ERigVMPinDirection::Hidden && (PinDirection == ERigVMPinDirection::IO || PinDirection == ERigVMPinDirection::Input))
			{
				PinsToDisplay.Add(Pin);
				ModelPinPaths.Add(Pin->GetPinPath());
			}
		}

		if (PinsToDisplay.Num() > 0)
		{
			const FName NodeName = *EdGraphNode->GetNodeTitle(ENodeTitleType::ListView).ToString(); // TODO : check if this is a good name to use, can't sue GetFName as it comes with instance postfix

			TSharedPtr<FCategoryDetailsData>* RigVMDataPtr = CategoryDetailsData.FindByPredicate([&NodeName](const TSharedPtr<FCategoryDetailsData>& InItem)
				{
					return InItem->Type == FCategoryDetailsData::EType::RigVMNode && InItem->Name == NodeName;
				});

			FRigVMNodeDetailsData* RigVMData = RigVMDataPtr  ? static_cast<FRigVMNodeDetailsData*>(RigVMDataPtr->Get()) : nullptr;
			if (RigVMData == nullptr)
			{
				RigVMDataPtr = &CategoryDetailsData.Add_GetRef(MakeShared<FRigVMNodeDetailsData>(NodeName));
				RigVMData = static_cast<FRigVMNodeDetailsData*>(RigVMDataPtr->Get());

				// Store the model Pin Names that will be shown (only when we create the Data, in case of multiselection)
				for (const TWeakObjectPtr<URigVMPin>& Pin : PinsToDisplay)
				{
					RigVMData->ModelPinsNamesToDisplay.Add(Pin->GetFName());
				}
			}

			// Store EdGraphNode and generated Memory, as we will need that later to transfer data if the user makes modifications in the details panel
			RigVMData->EdGraphNodes.Add(EdGraphNode);
			const int32 MemoryStorageIndex = RigVMData->MemoryStorages.Add(MakeShared<FRigVMMemoryStorageStruct>());

			// Store the Model Pin Paths, needed later to update the value of the correct model pin
			TArray<FString>& PinPaths = RigVMData->ModelPinPaths.Emplace_GetRef();
			Swap(PinPaths, ModelPinPaths);

			// Then, create a custom property bag to store the data, initializing the properties in the struct with the pin default values
			GenerateMemoryStorage(PinsToDisplay, *RigVMData->MemoryStorages[MemoryStorageIndex].Get());
		}
	}
}

void FAnimNextEdGraphNodeCustomization::PopulateCategory(IDetailLayoutBuilder& DetailBuilder, const TSharedPtr<FCategoryDetailsData> &CategoryDetailsData)
{
	switch (CategoryDetailsData->Type)
	{
		case FRigVMNodeDetailsData::EType::TraitStack:
		{
			PopulateCategory(DetailBuilder, StaticCastSharedPtr<FTraitStackDetailsData>(CategoryDetailsData));
			break;
		}
		case FRigVMNodeDetailsData::EType::RigVMNode:
		{
			PopulateCategory(DetailBuilder, StaticCastSharedPtr<FRigVMNodeDetailsData>(CategoryDetailsData));
			break;
		}
		default:
			break;
	}
}

void FAnimNextEdGraphNodeCustomization::PopulateCategory(IDetailLayoutBuilder& DetailBuilder, const TSharedPtr<FTraitStackDetailsData>& TraitData)
{
	if (TraitData == nullptr)
	{
		return;
	}

	check(TraitData->ScopedSharedDataInstances.Num() > 0);
	check(TraitData->ScopedSharedDataInstances.Num() == TraitData->EdGraphNodes.Num());

	// Create a category with the DisplayName of the Trait Shared Data
	const FString TraitDisplayName = *TraitData->ScopedSharedDataInstances[0]->GetStruct()->GetDisplayNameText().ToString();
	const FName CategoryName = (TraitData->InstanceIndex == 0)
		? *TraitDisplayName
		: *FString::Printf(TEXT("%s %d"), *TraitDisplayName, TraitData->InstanceIndex);

	IDetailCategoryBuilder& ParameterCategory = DetailBuilder.EditCategory(CategoryName, FText::GetEmpty(), ECategoryPriority::Important);
	bool bHasProperties = false;

	FAddPropertyParams AddPropertyParams;
	AddPropertyParams.CreateCategoryNodes(true);
	AddPropertyParams.HideRootObjectNode(true);

	IDetailPropertyRow* DetailPropertyRow = ParameterCategory.AddExternalStructureProperty(MakeShared<FStructOnScopeStructureDataProvider>(TraitData->ScopedSharedDataInstances), NAME_None, EPropertyLocation::Default, AddPropertyParams);
	TSharedPtr<IPropertyHandle> PropertyHandle = DetailPropertyRow->GetPropertyHandle();
	if (!PropertyHandle.IsValid())
	{
		return;
	}

	if(IModularFeatures::Get().IsModularFeatureAvailable(ITraitStackEditor::ModularFeatureName))
	{
		URigVMController* Controller = TraitData->EdGraphNodes[0]->GetController();
		
		ITraitStackEditor& TraitStackEditor = IModularFeatures::Get().GetModularFeature<ITraitStackEditor>(ITraitStackEditor::ModularFeatureName);
		ParameterCategory.HeaderContent(
			TraitStackEditor.CreateTraitHeaderWidget(ParameterCategory, *DetailPropertyRow, Controller),
			true);
	}

	TraitData->RootPropertyHandle = PropertyHandle;

	auto VisitChildren = [&bHasProperties](const TSharedRef<IPropertyHandle>& InPropertyHandle, TFunctionRef<void(const TSharedRef<IPropertyHandle>&)> InPredicate)
	{
		auto VisitChildrenRecursive = [&bHasProperties](const TSharedRef<IPropertyHandle>& InPropertyHandle, auto& InVisitChildren, TFunctionRef<void(const TSharedRef<IPropertyHandle>&)> InPredicate)
		{
			uint32 NumChildHandles = 0;
			if(InPropertyHandle->GetNumChildren(NumChildHandles) != FPropertyAccess::Success)
			{
				return;
			}

			if (NumChildHandles > 0)
			{
				bHasProperties = true;

				for(uint32 ChildHandleIndex = 0; ChildHandleIndex < NumChildHandles; ++ChildHandleIndex)
				{
					TSharedPtr<IPropertyHandle> Handle = InPropertyHandle->GetChildHandle(ChildHandleIndex);
					if(Handle.IsValid())
					{
						InPredicate(Handle.ToSharedRef());
						InVisitChildren(Handle.ToSharedRef(), InVisitChildren, InPredicate);
					}
				}
			}
		};

		VisitChildrenRecursive(InPropertyHandle, VisitChildrenRecursive, InPredicate);
	};

	VisitChildren(PropertyHandle.ToSharedRef(), [&TraitData](const TSharedRef<IPropertyHandle>& InPropertyHandle)
	{
		TraitData->PropertyHandles.Add(InPropertyHandle);

		const TWeakPtr<FTraitStackDetailsData> WeakTraitData = TraitData.ToWeakPtr();
		const TWeakPtr<IPropertyHandle> WeakPropertyHandle = InPropertyHandle.ToWeakPtr();

		InPropertyHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda([WeakTraitData, WeakPropertyHandle](const FPropertyChangedEvent& InPropertyChangedEvent)
		{
			TSharedPtr<IPropertyHandle> PinnedPropertyHandle = WeakPropertyHandle.Pin(); 
			if(!PinnedPropertyHandle.IsValid())
			{
				return;
			}

			TSharedPtr<FTraitStackDetailsData> PinnedTraitData = WeakTraitData.Pin(); 
			if(!PinnedTraitData.IsValid())
			{
				return;
			}
				
			// Early out if the change type is interactive e.g. from dragging of a slier, to avoid recompilation spam
			// This will be followed by a ValueSet when finished
			if ((InPropertyChangedEvent.ChangeType & EPropertyChangeType::Interactive) != 0)
			{
				return;
			}	
				
			// Avoid VM recompilation for each Set Default Value
			URigVMController* Controller = PinnedTraitData->EdGraphNodes[0]->GetController();
			FRigVMControllerCompileBracketScope CompileScope(Controller);

			PinnedPropertyHandle->EnumerateConstRawData([&PinnedTraitData, &PinnedPropertyHandle, Controller, &InPropertyChangedEvent](const void* InRawData, const int32 InDataIndex, const int32 InNumDatas)
			{
				check(InNumDatas == PinnedTraitData->ScopedSharedDataInstances.Num());
				check(InNumDatas == PinnedTraitData->EdGraphNodes.Num());
				check(PinnedTraitData->ScopedSharedDataInstances.IsValidIndex(InDataIndex));

				TWeakObjectPtr<UAnimNextEdGraphNode>& EdGraphNode = PinnedTraitData->EdGraphNodes[InDataIndex];
				if (!EdGraphNode.IsValid())
				{
					return true;
				}

				// Transform property path into pin path
				auto PropertyPathToPinPath = [](const TSharedPtr<IPropertyHandle>& InPropertyHandle, const TSharedPtr<FTraitStackDetailsData>& InTraitData, const UAnimNextEdGraphNode* InEdGraphNode)
				{
					// Transform property path into pin path
					FString PropertyPath(InPropertyHandle->GetPropertyPath());

					// Replace delimiter
					FString PinPath = PropertyPath.Replace(TEXT("->"), TEXT("."));
					// Replace array element delimiter
					PinPath = PinPath.Replace(TEXT("["), TEXT("."));
					PinPath = PinPath.Replace(TEXT("]."), TEXT("."));
					PinPath = PinPath.Replace(TEXT("]"), TEXT(""));
					// Replace struct root with trait root
					const FString RootPropertyPath(InTraitData->RootPropertyHandle->GetPropertyPath());
					PinPath = PinPath.Replace(*RootPropertyPath, *InTraitData->Name.ToString());
					// Prefix node path
					const FString NodePath = InEdGraphNode->GetModelNodePath();
					const bool bIsTrait = InTraitData->Type == FCategoryDetailsData::EType::TraitStack;
					return bIsTrait ? (NodePath + TEXT(".") + InTraitData->Name.ToString() + TEXT(".") + PinPath) : (NodePath + TEXT(".") + PinPath);
				};

				const FString PinPath = PropertyPathToPinPath(PinnedPropertyHandle, PinnedTraitData, EdGraphNode.Get());
				if ((InPropertyChangedEvent.ChangeType & EPropertyChangeType::ResetToDefault) != 0)
				{
					Controller->ResetPinDefaultValue(PinPath);
				}
				else
				{
					const FProperty* Property = PinnedPropertyHandle->GetProperty();
					const FString ValueStr = FRigVMStruct::ExportToFullyQualifiedText(Property, (uint8*)InRawData, false);
					Controller->SetPinDefaultValue(PinPath, ValueStr);
				}

				return true;
			});
		}));
	});

	// No properties shown, but we still want to see if the trait is present
	if (!bHasProperties)
	{
		ParameterCategory.AddProperty(PropertyHandle);
		ParameterCategory.SetIsEmpty(true);
	}
}

void FAnimNextEdGraphNodeCustomization::PopulateCategory(IDetailLayoutBuilder& DetailBuilder, const TSharedPtr<FRigVMNodeDetailsData>& RigVMTypeData)
{
	if (RigVMTypeData == nullptr)
	{
		return;
	}

	check(RigVMTypeData->MemoryStorages.Num() > 0);
	check(RigVMTypeData->MemoryStorages.Num() == RigVMTypeData->EdGraphNodes.Num());

	// Create a category with the NodeTitle
	const FName CategoryName = (RigVMTypeData->EdGraphNodes.Num() == 1)
		? RigVMTypeData->Name
		: *FString::Printf(TEXT("%s (% d)"), *RigVMTypeData->Name.ToString(), RigVMTypeData->EdGraphNodes.Num());

	IDetailCategoryBuilder& ParameterCategory = DetailBuilder.EditCategory(CategoryName, FText::GetEmpty(), ECategoryPriority::Default);

	for (const FName& TemplateModelPinName : RigVMTypeData->ModelPinsNamesToDisplay)
	{
		FAddPropertyParams AddPropertyParams;
		IDetailPropertyRow* DetailPropertyRow = ParameterCategory.AddExternalStructureProperty(MakeShared<TInstancedPropertyBagStructureDataProvider<FRigVMMemoryStorageStruct>>(RigVMTypeData->MemoryStorages), TemplateModelPinName, EPropertyLocation::Default, AddPropertyParams);

		if (!DetailPropertyRow)
		{
			continue;
		}

		if (TSharedPtr<IPropertyHandle> Handle = DetailPropertyRow->GetPropertyHandle(); Handle.IsValid())
		{
			const TWeakPtr<FRigVMNodeDetailsData> RigVMTypeDataWeak = RigVMTypeData.ToWeakPtr();

			const auto UpdatePinDefaultValue = [RigVMTypeDataWeak, TemplateModelPinName](const FPropertyChangedEvent& InEvent)
				{
					if (const TSharedPtr<FRigVMNodeDetailsData> RigVMTypeData = RigVMTypeDataWeak.Pin())
					{
						// Avoid VM recompilation for each Set Default Value
						FRigVMControllerCompileBracketScope CompileScope(RigVMTypeData->EdGraphNodes[0]->GetController());

						const int32 NumInstances = RigVMTypeData->MemoryStorages.Num();
						for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; InstanceIndex++)
						{
							TWeakObjectPtr<UAnimNextEdGraphNode> EdGraphNode = RigVMTypeData->EdGraphNodes[InstanceIndex];
							if (EdGraphNode.IsValid())
							{
								const FString ValueStr = RigVMTypeData->MemoryStorages[InstanceIndex].Get()->GetDataAsStringByName(TemplateModelPinName);

								const TArray<FString>& ModelPinPaths = RigVMTypeData->ModelPinPaths[InstanceIndex];
								for (const FString& PinPath : ModelPinPaths)
								{
									const FString ModelPinName = FString::Printf(TEXT(".%s"), *TemplateModelPinName.ToString());
									if (PinPath.EndsWith(ModelPinName, ESearchCase::CaseSensitive))
									{
										EdGraphNode->GetController()->SetPinDefaultValue(PinPath, ValueStr);
										break;
									}
								}

								// Editing some properties can cause pin-regeneration (e.g. via traits GetProgrammaticPins) so repopulate the node
								// TODO: We could probably constrain this with metadata in case this is too disruptive
								EdGraphNode->GetController()->RepopulatePinsOnNode(EdGraphNode->GetModelNode());
							}
						}
					}
				};

			Handle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda(UpdatePinDefaultValue));
			Handle->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda(UpdatePinDefaultValue));
		}
	}
}

void FAnimNextEdGraphNodeCustomization::GenerateMemoryStorage(const TArray<TWeakObjectPtr<URigVMPin>> & ModelPinsToDisplay, FRigVMMemoryStorageStruct& MemoryStorage)
{
	TArray<FRigVMPropertyDescription> PropertyDescriptions;
	PropertyDescriptions.Reserve(ModelPinsToDisplay.Num());

	for (const TWeakObjectPtr<URigVMPin>& ModelPin : ModelPinsToDisplay)
	{
		if (ModelPin.IsValid())
		{
			FRigVMPropertyDescription& PropertyDesc = PropertyDescriptions.AddDefaulted_GetRef();

			PropertyDesc.Name = ModelPin->GetFName();
			PropertyDesc.Property = nullptr;
			PropertyDesc.CPPType = ModelPin->GetCPPType();
			PropertyDesc.CPPTypeObject = ModelPin->GetCPPTypeObject();
			if (ModelPin->IsArray())
			{
				PropertyDesc.Containers.Add(EPinContainerType::Array);
			}
			PropertyDesc.DefaultValue = ModelPin->GetDefaultValue();
		}
	}

	MemoryStorage.AddProperties(PropertyDescriptions);
}


URigVMNode* FAnimNextEdGraphNodeCustomization::GetNodeForLayout() const
{
	return WeakModelNode.Get();
}

const FRigVMNodeLayout* FAnimNextEdGraphNodeCustomization::GetNodeLayout() const
{
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		CachedNodeLayout = Node->GetNodeLayout(true);
		return &CachedNodeLayout.GetValue();
	}
	return nullptr;
}

TArray<FString> FAnimNextEdGraphNodeCustomization::GetUncategorizedPins() const
{
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		struct Local
		{
			static void VisitPinRecursively(URigVMPin* InPin, TArray<FString>& OutPinPaths)
			{
				if(InPin->IsExecuteContext())
				{
					return;
				}
				if(InPin->GetDirection() != ERigVMPinDirection::Input &&
					InPin->GetDirection() != ERigVMPinDirection::Visible)
				{
					return;
				}
				if(!InPin->GetCategory().IsEmpty())
				{
					return;
				}

				OutPinPaths.Add(InPin->GetSegmentPath(true));
				for (URigVMPin* SubPin : InPin->GetSubPins())
				{
				    VisitPinRecursively(SubPin, OutPinPaths);
				}
			}
		};

		TArray<FString> PinPaths;
		for (URigVMPin* Pin : Node->GetPins())
		{
			Local::VisitPinRecursively(Pin, PinPaths);
		}

		return PinPaths;
	}
	return TArray<FString>();
}

TArray<FRigVMPinCategory> FAnimNextEdGraphNodeCustomization::GetPinCategories() const
{
	if (const FRigVMNodeLayout* NodeLayout = GetNodeLayout())
	{
		return NodeLayout->Categories;
	}
	return TArray<FRigVMPinCategory>();
}

FString FAnimNextEdGraphNodeCustomization::GetPinCategory(FString InPinPath) const
{
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		if(const URigVMPin* Pin = Node->FindPin(InPinPath))
		{
			return Pin->GetCategory();
		}
	}

	return FString();
}

int32 FAnimNextEdGraphNodeCustomization::GetPinIndexInCategory(FString InPinPath) const
{
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		if(const URigVMPin* Pin = Node->FindPin(InPinPath))
		{
			return Pin->GetIndexInCategory();
		}
	}
	return INDEX_NONE;
}

FString FAnimNextEdGraphNodeCustomization::GetPinLabel(FString InPinPath) const
{
	if (const FRigVMNodeLayout* NodeLayout = GetNodeLayout())
	{
		if(const FString* DisplayName = NodeLayout->FindDisplayName(InPinPath))
		{
			return *DisplayName;
		}
	}
	return FString();
}

FLinearColor FAnimNextEdGraphNodeCustomization::GetPinColor(FString InPinPath) const
{
	if(WeakRigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(const URigVMPin* Pin = Node->FindPin(InPinPath))
			{
				if(const URigVMEdGraphSchema* Schema = Cast<URigVMEdGraphSchema>(WeakRigVMClientHost->GetRigVMEdGraphSchemaClass()->GetDefaultObject()))
				{
					const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(*Pin->GetCPPType(), Pin->GetCPPTypeObject());
					return Schema->GetPinTypeColor(PinType);
				}
			}
		}
	}
	return FLinearColor::White;
}

const FSlateBrush* FAnimNextEdGraphNodeCustomization::GetPinIcon(FString InPinPath) const
{
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		if(const URigVMPin* Pin = Node->FindPin(InPinPath))
		{
			const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(*Pin->GetCPPType(), Pin->GetCPPTypeObject());
			return FBlueprintEditorUtils::GetIconFromPin(PinType, /* bIsLarge = */ false);
		}
	}
	return nullptr;
}

void FAnimNextEdGraphNodeCustomization::HandleCategoryAdded(FString InCategory)
{
	if(WeakRigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(URigVMController* Controller = WeakRigVMClientHost->GetController(Node->GetGraph()))
			{
				Controller->AddEmptyPinCategory(Node->GetFName(), InCategory);
				CachedNodeLayout.Reset();
			}
		}
	}
}

void FAnimNextEdGraphNodeCustomization::HandleCategoryRemoved(FString InCategory)
{
	if(WeakRigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(URigVMController* Controller = WeakRigVMClientHost->GetController(Node->GetGraph()))
			{
				Controller->RemovePinCategory(Node->GetFName(), InCategory);
				CachedNodeLayout.Reset();
			}
		}
	}
}

void FAnimNextEdGraphNodeCustomization::HandleCategoryRenamed(FString InOldCategory, FString InNewCategory)
{
	if(WeakRigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(URigVMController* Controller = WeakRigVMClientHost->GetController(Node->GetGraph()))
			{
				Controller->RenamePinCategory(Node->GetFName(), InOldCategory, InNewCategory);
				CachedNodeLayout.Reset();
			}
		}
	}
}

void FAnimNextEdGraphNodeCustomization::HandlePinCategoryChanged(FString InPinPath, FString InCategory)
{
	if(WeakRigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(const URigVMPin* Pin = Node->FindPin(InPinPath))
			{
				if(URigVMController* Controller = WeakRigVMClientHost->GetController(Node->GetGraph()))
				{
					Controller->SetPinCategory(Pin->GetPinPath(), InCategory);
					CachedNodeLayout.Reset();
				}
			}
		}
	}
}

void FAnimNextEdGraphNodeCustomization::HandlePinLabelChanged(FString InPinPath, FString InNewLabel)
{
	if(WeakRigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(const URigVMPin* Pin = Node->FindPin(InPinPath))
			{
				if(URigVMController* Controller = WeakRigVMClientHost->GetController(Node->GetGraph()))
				{
					Controller->SetPinDisplayName(Pin->GetPinPath(), InNewLabel);
					CachedNodeLayout.Reset();
				}
			}
		}
	}
}

void FAnimNextEdGraphNodeCustomization::HandlePinIndexInCategoryChanged(FString InPinPath, int32 InIndexInCategory)
{
	if(WeakRigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(const URigVMPin* Pin = Node->FindPin(InPinPath))
			{
				if(URigVMController* Controller = WeakRigVMClientHost->GetController(Node->GetGraph()))
				{
					Controller->SetPinIndexInCategory(Pin->GetPinPath(), InIndexInCategory);
					CachedNodeLayout.Reset();
				}
			}
		}
	}
}


bool FAnimNextEdGraphNodeCustomization::ValidateName(FString InNewName, FText& OutErrorMessage)
{
	if(InNewName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyNamesAreNotAllowed", "Empty names are not allowed.");
		return false;
	}

	if(FChar::IsDigit(InNewName[0]))
	{
		OutErrorMessage = LOCTEXT("NamesCannotStartWithADigit", "Names cannot start with a digit.");
		return false;
	}

	for (int32 i = 0; i < InNewName.Len(); ++i)
	{
		TCHAR& C = InNewName[i];

		const bool bGoodChar = FChar::IsAlpha(C) ||					 // Any letter
			(C == '_') || (C == '-') || (C == ' ') ||				 // _  - space anytime
			FChar::IsDigit(C);										 // 0-9 anytime

		if (!bGoodChar)
		{
			const FText Character = FText::FromString(InNewName.Mid(i, 1));
			OutErrorMessage = FText::Format(LOCTEXT("CharacterNotAllowedFormat", "'{0}' not allowed."), Character);
			return false;
		}
	}

	if (InNewName.Len() > 100)
	{
		OutErrorMessage = LOCTEXT("NameIsTooLong", "Name is too long.");
		return false;
	}

	return true;
}

bool FAnimNextEdGraphNodeCustomization::HandleValidateCategoryName(FString InCategoryPath, FString InNewName, FText& OutErrorMessage)
{
	if(!ValidateName(InNewName, OutErrorMessage))
	{
		return false;
	}
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		const FString ParentCategory = Node->GetParentPinCategory(InCategoryPath);
		if(!ParentCategory.IsEmpty())
		{
			const TArray<FString> SiblingCategories = Node->GetSubPinCategories(ParentCategory);
			const FString NewNameSuffix = TEXT("|") + InNewName;
			if(SiblingCategories.ContainsByPredicate([InNewName, NewNameSuffix](const FString& Category)
			{
				return Category.Equals(InNewName, ESearchCase::IgnoreCase) || Category.EndsWith(NewNameSuffix, ESearchCase::IgnoreCase); 
			}))
			{
				OutErrorMessage = LOCTEXT("NameIsAlreadyUsed", "Duplicate name.");
				return false;
			}
		} 
	}
	return true;
}

bool FAnimNextEdGraphNodeCustomization::HandleValidatePinDisplayName(FString InPinPath, FString InNewName, FText& OutErrorMessage)
{
	if(!ValidateName(InNewName, OutErrorMessage))
	{
		return false;
	}
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		if(const URigVMPin* Pin = Node->FindPin(InPinPath))
		{
			const FString Category = Pin->GetCategory();
			if(!Category.IsEmpty())
			{
				const TArray<URigVMPin*> PinsInCategory = Node->GetPinsForCategory(Category);
				if(PinsInCategory.ContainsByPredicate([InNewName](const URigVMPin* PinInCategory)
				{
					return PinInCategory->GetDisplayName().ToString().Equals(InNewName, ESearchCase::IgnoreCase);
				}))
				{
					OutErrorMessage = LOCTEXT("NameIsAlreadyUsedInCategory", "Duplicate name (category).");
					return false;
				}
			}

			if(const URigVMPin* ParentPin = Pin->GetParentPin())
			{
				const TArray<URigVMPin*> SubPins = ParentPin->GetSubPins();
				if(SubPins.ContainsByPredicate([InNewName](const URigVMPin* SubPin)
				{
					return SubPin->GetDisplayName().ToString().Equals(InNewName, ESearchCase::IgnoreCase);
				}))
				{
					OutErrorMessage = LOCTEXT("NameIsAlreadyUsedWithinPin", "Duplicate name (parent pin).");
					return false;
				}
			}
		}
	}
	return true;
}

uint32 FAnimNextEdGraphNodeCustomization::GetNodeLayoutHash() const
{
	uint32 Hash = 0;
	if(const FRigVMNodeLayout* Layout = GetNodeLayout())
	{
		Hash = HashCombine(Hash, GetTypeHash(*Layout));
	}
	const TArray<FString> UncategorizedPins = GetUncategorizedPins();
	for(const FString& UncategorizedPin : UncategorizedPins)
	{
		Hash = HashCombine(Hash, GetTypeHash(UncategorizedPin));
	}
	return Hash;
}

} // end namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE
