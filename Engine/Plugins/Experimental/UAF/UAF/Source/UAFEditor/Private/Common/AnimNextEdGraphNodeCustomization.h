// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "RigVMCore/RigVMNodeLayout.h"
#include "UObject/WeakInterfacePtr.h"

class IRigVMClientHost;
class URigVMPin;
class URigVMNode;
class IDetailCategoryBuilder;
class UAnimNextEdGraphNode;
class FRigVMGraphDetailCustomizationImpl;
struct FRigVMPinCategory;

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{

class FAnimNextEdGraphNodeCustomization : public IDetailCustomization
{
public:
	FAnimNextEdGraphNodeCustomization() = default;
	explicit FAnimNextEdGraphNodeCustomization(const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditorWeak);

protected:

	// --- IDetailCustomization Begin ---
	/** Called when details should be customized */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** Called when no longer used and will be deleted */
	virtual void PendingDelete() override;

	// --- IDetailCustomization End ---

	struct FCategoryDetailsData
	{
		enum class EType : uint8
		{
			TraitStack = 0,
			RigVMNode,
			// --- ---
			Invalid,
			Num = Invalid
		};

		FCategoryDetailsData() = default;
		explicit FCategoryDetailsData(EType InType)
			: Type(InType)
		{
		}
		FCategoryDetailsData(EType InType, const FName& InName)
			: Type(InType)
			, Name(InName)
		{
		}

		EType Type = EType::Invalid;
		FName Name;
		TArray<TWeakObjectPtr<UAnimNextEdGraphNode>> EdGraphNodes;
	};

	struct FTraitStackDetailsData : FCategoryDetailsData
	{
		FTraitStackDetailsData()
			: FCategoryDetailsData(FCategoryDetailsData::EType::TraitStack)
		{
		}
		explicit FTraitStackDetailsData(const FName InName, int32 InInstanceIndex)
			: FCategoryDetailsData(FCategoryDetailsData::EType::TraitStack, InName)
			, InstanceIndex(InInstanceIndex)
		{
		}

		TArray<TSharedPtr<FStructOnScope>> ScopedSharedDataInstances;
		TSharedPtr<IPropertyHandle> RootPropertyHandle;
		TArray<TSharedRef<IPropertyHandle>> PropertyHandles;
		// For traits that appear more than once, this allows us to disambiguate
		int32 InstanceIndex;
	};

	struct FRigVMNodeDetailsData : FCategoryDetailsData
	{
		FRigVMNodeDetailsData()
			: FCategoryDetailsData(FCategoryDetailsData::EType::RigVMNode)
		{
		}
		explicit FRigVMNodeDetailsData(const FName InName)
			: FCategoryDetailsData(FCategoryDetailsData::EType::RigVMNode, InName)
		{
		}

		TArray<FName> ModelPinsNamesToDisplay;
		TArray <TArray<FString>> ModelPinPaths;
		TArray <TSharedPtr<FRigVMMemoryStorageStruct>> MemoryStorages;
	};

	void CustomizeObjects(IDetailLayoutBuilder& DetailBuilder, const TArray<UObject*>& InObjects);

	static void GenerateTraitData(UAnimNextEdGraphNode* EdGraphNode, TArray<TSharedPtr<FCategoryDetailsData>>& CategoryDetailsData);
	static void GenerateRigVMData(UAnimNextEdGraphNode* EdGraphNode, TArray<TSharedPtr<FCategoryDetailsData>>& CategoryDetailsData);

	static void PopulateCategory(IDetailLayoutBuilder& DetailBuilder, const TSharedPtr<FCategoryDetailsData>& CategoryDetailsData);
	static void PopulateCategory(IDetailLayoutBuilder& DetailBuilder, const TSharedPtr<FTraitStackDetailsData>& TraitData);
	static void PopulateCategory(IDetailLayoutBuilder& DetailBuilder, const TSharedPtr<FRigVMNodeDetailsData>& RigVMTypeData);

	static void GenerateMemoryStorage(const TArray<TWeakObjectPtr<URigVMPin>> & ModelPinsToDisplay, FRigVMMemoryStorageStruct& MemoryStorage);

	URigVMNode* GetNodeForLayout() const;
	const FRigVMNodeLayout* GetNodeLayout() const;
	TArray<FString> GetUncategorizedPins() const;
	TArray<FRigVMPinCategory> GetPinCategories() const;
	FString GetPinCategory(FString InPinPath) const;
	int32 GetPinIndexInCategory(FString InPinPath) const;
	FString GetPinLabel(FString InPinPath) const;
	FLinearColor GetPinColor(FString InPinPath) const;
	const FSlateBrush* GetPinIcon(FString InPinPath) const;
	void HandleCategoryAdded(FString InCategory);
	void HandleCategoryRemoved(FString InCategory);
	void HandleCategoryRenamed(FString InOldCategory, FString InNewCategory);
	void HandlePinCategoryChanged(FString InPinPath, FString InCategory);
	void HandlePinLabelChanged(FString InPinPath, FString InNewLabel);
	void HandlePinIndexInCategoryChanged(FString InPinPath, int32 InIndexInCategory);
	static bool ValidateName(FString InNewName, FText& OutErrorMessage);
	bool HandleValidateCategoryName(FString InCategoryPath, FString InNewName, FText& OutErrorMessage);
	bool HandleValidatePinDisplayName(FString InPinPath, FString InNewName, FText& OutErrorMessage);
	uint32 GetNodeLayoutHash() const;

	/** The asset host we are editing */
	TWeakInterfacePtr<IRigVMClientHost> WeakRigVMClientHost;
	TWeakObjectPtr<URigVMNode> WeakModelNode;
	mutable TOptional<FRigVMNodeLayout> CachedNodeLayout;

	TWeakPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditorWeak;
	TArray<TSharedPtr<FCategoryDetailsData>> CategoryDetailsData;

	TSharedPtr<FRigVMGraphDetailCustomizationImpl> RigVMGraphDetailCustomizationImpl;
};

}
