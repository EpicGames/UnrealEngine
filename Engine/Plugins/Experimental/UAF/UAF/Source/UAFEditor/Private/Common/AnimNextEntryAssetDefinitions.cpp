// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextEntryAssetDefinitions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextEntryAssetDefinitions)

#define LOCTEXT_NAMESPACE "AnimNextAssetDefinitions"

FText UAssetDefinition_AnimNextVariableEntry::GetObjectDisplayNameText(UObject* Object) const
{
	UUAFRigVMAssetEntry* Parameter = CastChecked<UUAFRigVMAssetEntry>(Object);
	return Parameter->GetDisplayName();
}

FText UAssetDefinition_AnimNextEventGraphEntry::GetObjectDisplayNameText(UObject* Object) const
{
	UUAFRigVMAssetEntry* Variable = CastChecked<UUAFRigVMAssetEntry>(Object);
	return Variable->GetDisplayName();
}

FText UAssetDefinition_AnimNextSharedVariablesEntry::GetObjectDisplayNameText(UObject* Object) const
{
	UUAFSharedVariablesEntry* SharedVariables = CastChecked<UUAFSharedVariablesEntry>(Object);
	return SharedVariables->GetDisplayName();
}

#undef LOCTEXT_NAMESPACE
