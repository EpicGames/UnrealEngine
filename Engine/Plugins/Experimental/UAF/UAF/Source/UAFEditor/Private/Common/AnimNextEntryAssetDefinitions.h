// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "Entries/AnimNextEventGraphEntry.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "AnimNextEntryAssetDefinitions.generated.h"

#define LOCTEXT_NAMESPACE "AnimNextAssetDefinitions"

UCLASS()
class UAssetDefinition_AnimNextVariableEntry : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override { return LOCTEXT("UAFVariable", "Variable"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,64,64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimNextVariableEntry::StaticClass(); }
	virtual FText GetObjectDisplayNameText(UObject* Object) const override;
};

UCLASS()
class UAssetDefinition_AnimNextEventGraphEntry : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override { return LOCTEXT("UAFEventGraphEntry", "Event Graph"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,64,64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAnimNextEventGraphEntry::StaticClass(); }
	virtual FText GetObjectDisplayNameText(UObject* Object) const override;
};

UCLASS()
class UAssetDefinition_AnimNextSharedVariablesEntry : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override { return LOCTEXT("UAFSharedVariablesEntry", "Shared Variables"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,64,64)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UUAFSharedVariablesEntry::StaticClass(); }
	virtual FText GetObjectDisplayNameText(UObject* Object) const override;
};

#undef LOCTEXT_NAMESPACE