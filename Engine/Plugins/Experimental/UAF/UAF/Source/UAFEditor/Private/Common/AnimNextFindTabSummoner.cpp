// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/AnimNextFindTabSummoner.h"

#include "AnimNextRigVMAsset.h"
#include "FindInAnimNextRigVMAsset.h"
#include "IAnimNextEditorModule.h"
#include "IWorkspaceEditor.h"
#include "Modules/ModuleManager.h"
#include "AnimAssetFindReplace.h"
#include "AnimNextAssetFindReplaceVariables.h"
#include "PersonaModule.h"

#define LOCTEXT_NAMESPACE "WorkspaceTabSummoner"

namespace UE::UAF::Editor
{

FAnimNextFindTabSummoner::FAnimNextFindTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp)
	: FWorkflowTabFactory(UE::UAF::Editor::FindTabName, StaticCastSharedPtr<FAssetEditorToolkit>(InHostingApp))
{
	TabLabel = LOCTEXT("UAFFindTabLabel", "Find");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults");
	ViewMenuDescription = LOCTEXT("UAFFindTabMenuDescription", "Find");
	ViewMenuTooltip = LOCTEXT("UAFFindTabToolTip", "Search contents of currently selected UAF Asset.");
	bIsSingleton = true;

	AnimNextFindResultsWidget = SNew(SFindInAnimNextRigVMAsset, InHostingApp);
}

TSharedRef<SWidget> FAnimNextFindTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return AnimNextFindResultsWidget.ToSharedRef();
}

FText FAnimNextFindTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return ViewMenuTooltip;
}

FAnimNextFindAndReplaceTabSummoner::FAnimNextFindAndReplaceTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp) 
	: FWorkflowTabFactory(UE::UAF::Editor::FindAndReplaceTabName, StaticCastSharedPtr<FAssetEditorToolkit>(InHostingApp))
{
	TabLabel = LOCTEXT("AnimNextFindAndReplaceTabLabel", "Find and Replace");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults");
	ViewMenuDescription = LOCTEXT("AnimNextFindTabMenuDescription", "Find and Replace");
	ViewMenuTooltip = LOCTEXT("AnimNextFindAndReplaceTabToolTip", "Search and replace contents of Animation(Next) Assets.");
	bIsSingleton = true;


	FPersonaModule& PersonaModule = FModuleManager::Get().LoadModuleChecked<FPersonaModule>("Persona");

	FAnimAssetFindReplaceConfig Config;
	Config.InitialProcessorClass = UAnimNextAssetFindReplaceVariables::StaticClass();
	FindAndReplaceWidget = StaticCastSharedRef<IAnimAssetFindReplace>(PersonaModule.CreateFindReplaceWidget(Config));

	if (UAnimNextAssetFindReplaceVariables* Processor = FindAndReplaceWidget->GetProcessor<UAnimNextAssetFindReplaceVariables>())
	{
		Processor->SetWorkspaceEditor(InHostingApp.ToSharedRef());
	}	
}

TSharedRef<SWidget> FAnimNextFindAndReplaceTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return FindAndReplaceWidget.ToSharedRef();
}

FText FAnimNextFindAndReplaceTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return ViewMenuTooltip;
}
} // end namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE
