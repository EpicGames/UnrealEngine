// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "IMessageLogListing.h"

namespace UE::Workspace
{
	class IWorkspaceEditor;
}	// end namespace UE::Workspace

namespace UE::UAF::Editor
{
	class SFindInAnimNextRigVMAsset;
}	// end namespace UE::UAF::Editor

class IAnimAssetFindReplace;

namespace UE::UAF::Editor
{
struct FAnimNextFindTabSummoner : public FWorkflowTabFactory
{
public:
	FAnimNextFindTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp);

private:
	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

	// The widget this tab spawner wraps
	TSharedPtr<SFindInAnimNextRigVMAsset> AnimNextFindResultsWidget;
};

struct FAnimNextFindAndReplaceTabSummoner : public FWorkflowTabFactory
{
public:
	FAnimNextFindAndReplaceTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp);

private:
	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

	// The widget this tab spawner wraps
	TSharedPtr<IAnimAssetFindReplace> FindAndReplaceWidget;
};

} // end namespace UE::UAF::Editor
