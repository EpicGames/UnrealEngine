// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextGraphItemDetails.h"

#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "StructUtils/InstancedStruct.h"
#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "RigVMModel/RigVMGraph.h"
#include "WorkspaceItemMenuContext.h"
#include "IWorkspaceEditor.h"
#include "ScopedTransaction.h"
#include "RigVMModel/RigVMClient.h"
#include "EdGraph/RigVMEdGraph.h"
#include "ToolMenus.h"
#include "Framework/Commands/GenericCommands.h"
#include "Module/AnimNextModule_EditorData.h"

#define LOCTEXT_NAMESPACE "FAnimNextGraphItemDetails"

namespace UE::UAF::Editor
{

bool FAnimNextGraphItemDetails::HandleDoubleClick(const FToolMenuContext& ToolMenuContext) const
{
	const UWorkspaceItemMenuContext* WorkspaceItemContext = ToolMenuContext.FindContext<UWorkspaceItemMenuContext>();
	const UAssetEditorToolkitMenuContext* AssetEditorContext = ToolMenuContext.FindContext<UAssetEditorToolkitMenuContext>(); 
	if (WorkspaceItemContext && AssetEditorContext)
	{
		if(const TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorContext->Toolkit.Pin()))
		{
			const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = WorkspaceItemContext->SelectedExports[0].GetResolvedExport().GetData();
			if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextGraphOutlinerData::StaticStruct())
			{
				const FAnimNextGraphOutlinerData& GraphData = Data.Get<FAnimNextGraphOutlinerData>();							
				if (GraphData.GetGraphInterface())
				{
					if (URigVMGraph* RigVMGraph = GraphData.GetGraphInterface()->GetRigVMGraph())
					{
						if(const IRigVMClientHost* RigVMClientHost = RigVMGraph->GetImplementingOuter<IRigVMClientHost>())
						{
							if(RigVMClientHost->GetEditorObjectForRigVMGraph(RigVMGraph))
							{
								WorkspaceEditor->OpenExports({WorkspaceItemContext->SelectedExports[0].GetResolvedExport()});
								return true;
							}
						}
					}
				}
			}
		}
	}

	return false;
}

bool FAnimNextGraphItemDetails::CanDelete(const FWorkspaceOutlinerItemExport& Export) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextGraphOutlinerData::StaticStruct())
	{
		return true;
	}
	return false;
}

void FAnimNextGraphItemDetails::Delete(TConstArrayView<FWorkspaceOutlinerItemExport> Exports) const
{
	TMap<UUAFRigVMAssetEditorData*, TArray<UUAFRigVMAssetEntry*>> EntriesToDelete;
	for(const FWorkspaceOutlinerItemExport& Export : Exports)
	{
		const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
		if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextGraphOutlinerData::StaticStruct())
		{
			const FAnimNextGraphOutlinerData& GraphData = Export.GetData().Get<FAnimNextGraphOutlinerData>();

			UUAFRigVMAssetEditorData* EditorData = GraphData.GetEntry()->GetTypedOuter<UUAFRigVMAssetEditorData>();
			if(EditorData == nullptr)
			{
				continue;
			}
			UUAFRigVMAssetEntry* Entry = GraphData.GetEntry();
			if(Entry == nullptr)
			{
				continue;
			}
			TArray<UUAFRigVMAssetEntry*>& Entries = EntriesToDelete.FindOrAdd(EditorData);
			Entries.Add(Entry);
		}
	}

	if(EntriesToDelete.Num() > 0)
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteEntries", "Delete Entries"));
		for(TPair<UUAFRigVMAssetEditorData*, TArray<UUAFRigVMAssetEntry*>> EntryToDeletePair : EntriesToDelete)
		{
			EntryToDeletePair.Key->RemoveEntries(EntryToDeletePair.Value);
		}
	}
}

bool FAnimNextGraphItemDetails::CanRename(const FWorkspaceOutlinerItemExport& Export) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextGraphOutlinerData::StaticStruct())
	{
		return true;
	}
	return false;
}

void FAnimNextGraphItemDetails::Rename(const FWorkspaceOutlinerItemExport& Export, const FText& InName) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextGraphOutlinerData::StaticStruct())
	{
		const FAnimNextGraphOutlinerData& GraphData = Export.GetData().Get<FAnimNextGraphOutlinerData>();
		UUAFRigVMAssetEditorData* EditorData = GraphData.GetEntry()->GetTypedOuter<UUAFRigVMAssetEditorData>();
		FName NewName = FName(*InName.ToString());
		UUAFRigVMAssetEntry* ExistingEntry = EditorData->FindEntry(NewName);
		if (ExistingEntry == nullptr && GraphData.GetEntry()->GetEntryName() != NewName)
		{
			FScopedTransaction Transaction(LOCTEXT("SetName", "Set Name"));
			GraphData.GetEntry()->SetEntryName(NewName);
		}
	}
}

bool FAnimNextGraphItemDetails::ValidateName(const FWorkspaceOutlinerItemExport& Export, const FText& InName, FText& OutErrorMessage) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextGraphOutlinerData::StaticStruct())
	{
		const FAnimNextGraphOutlinerData& GraphData = Export.GetData().Get<FAnimNextGraphOutlinerData>();
		UUAFRigVMAssetEditorData* EditorData = GraphData.GetEntry()->GetTypedOuter<UUAFRigVMAssetEditorData>();
		UUAFRigVMAssetEntry* ExistingEntry = EditorData->FindEntry(FName(*InName.ToString()));
		if(ExistingEntry && GraphData.GetEntry() != ExistingEntry)
		{
			OutErrorMessage = LOCTEXT("NameAlreadyExistsError", "Name already exists in this system");
			return false;
		}

		return true;
	}

	OutErrorMessage = LOCTEXT("UnsupportedTypeRenameError", "Element type is not supported for rename");
	return false;
}

UPackage* FAnimNextGraphItemDetails::GetPackage(const FWorkspaceOutlinerItemExport& Export) const
{
	const TInstancedStruct<FWorkspaceOutlinerItemData>& Data = Export.GetData();
	if (Data.IsValid() && Data.GetScriptStruct() == FAnimNextGraphOutlinerData::StaticStruct())
	{
		const FAnimNextGraphOutlinerData& GraphData = Data.Get<FAnimNextGraphOutlinerData>();
		if (GraphData.SoftEntryPtr.IsValid())
		{
			return GraphData.GetEntry()->GetExternalPackage();
		}
	}
	return nullptr;
}

const FSlateBrush* FAnimNextGraphItemDetails::GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const
{
	return FAppStyle::GetBrush(TEXT("GraphEditor.EventGraph_24x"));
}

void FAnimNextGraphItemDetails::RegisterToolMenuExtensions()
{
}

void FAnimNextGraphItemDetails::UnregisterToolMenuExtensions()
{
}

} // UE::UAF::Editor

#undef LOCTEXT_NAMESPACE // "FAnimNextGraphItemDetails"
