// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextModuleItemDetails.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FAnimNextModuleItemDetails"

namespace UE::UAF::Editor
{

const FSlateBrush* FAnimNextModuleItemDetails::GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const
{
	return FAppStyle::GetBrush(TEXT("GraphEditor.Function_24x"));
}

} // UE::UAF::Editor

#undef LOCTEXT_NAMESPACE // "FAnimNextFunctionItemDetails"
