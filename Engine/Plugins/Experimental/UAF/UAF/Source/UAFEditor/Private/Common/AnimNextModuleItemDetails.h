// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWorkspaceOutlinerItemDetails.h"

namespace UE::UAF::Editor
{

class FAnimNextModuleItemDetails : public UE::Workspace::IWorkspaceOutlinerItemDetails
{
public:
	FAnimNextModuleItemDetails() = default;
	virtual ~FAnimNextModuleItemDetails() override = default;
	virtual const FSlateBrush* GetItemIcon(const FWorkspaceOutlinerItemExport& Export) const override;
};

}
