// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/AnimNextRigVMAssetCommands.h"

#define LOCTEXT_NAMESPACE "AnimNextRigVMAssetCommands"

namespace UE::UAF
{

FAnimNextRigVMAssetCommands::FAnimNextRigVMAssetCommands()
	: TCommands<FAnimNextRigVMAssetCommands>("UAFRigVMAssetCommand", LOCTEXT("UAFRigVMAssetCommands", "UAF Asset Commands"), NAME_None, "AnimNextStyle")
{
}

void FAnimNextRigVMAssetCommands::RegisterCommands()
{
	// On mac command and ctrl are automatically swapped. Command + Space is spotlight search so we use ctrl+space on mac to avoid the conflict
	EModifierKey::Type PlatformSafeCtrl = EModifierKey::Control;
#if PLATFORM_MAC
	PlatformSafeCtrl = EModifierKey::Command;
#endif // PLATFORM_MAC

	UI_COMMAND(FindInAnimNextRigVMAsset, "FindInUAFRigVMAsset", "Search the current UAF Asset.", EUserInterfaceActionType::Button, FInputChord(PlatformSafeCtrl, EKeys::F));
	UI_COMMAND(FindAndReplaceInAnimNextRigVMAsset, "FindAndReplaceInUAFRigVMAsset", "Find and Replace across UAF Assets.", EUserInterfaceActionType::Button, FInputChord(PlatformSafeCtrl | EModifierKey::Shift, EKeys::F));
	UI_COMMAND(OpenUAFBrowser, "OpenUAFBrowser", "Opens the UAF Browser.", EUserInterfaceActionType::Button, FInputChord(PlatformSafeCtrl | EModifierKey::Shift, EKeys::SpaceBar));
}

}

#undef LOCTEXT_NAMESPACE
