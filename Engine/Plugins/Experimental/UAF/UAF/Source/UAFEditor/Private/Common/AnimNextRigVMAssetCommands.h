// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::UAF
{

// Shared commands valid for any UAF RigVM Asset.
class FAnimNextRigVMAssetCommands : public TCommands<FAnimNextRigVMAssetCommands>
{
public:
	FAnimNextRigVMAssetCommands();

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> FindInAnimNextRigVMAsset;

	TSharedPtr<FUICommandInfo> FindAndReplaceInAnimNextRigVMAsset;

	TSharedPtr<FUICommandInfo> OpenUAFBrowser;
};

}