// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceAssetPreview.h"

#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSequence.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"

#define LOCTEXT_NAMESPACE "AnimSequenceAssetPreview"

namespace UE::UAF::Editor
{


//////////////////////////////////////////////////////////////////////////
// FAnimSequenceAssetPreviewFactory

TSharedRef<IUAFAssetPreview> FAnimSequenceAssetPreviewFactory::CreateAssetPreviewWidget(TSharedPtr<FAssetEditorToolkit> InAssetEditorToolkit, const FAssetData& InAssetData) const
{
	return SNew(SAnimSequenceAssetPreview, InAssetEditorToolkit, InAssetData);
}

const UStruct* FAnimSequenceAssetPreviewFactory::GetPreviewType() const
{
	return UAnimSequence::StaticClass();
}


//////////////////////////////////////////////////////////////////////////
// SAnimSequenceAssetPreview

const UStruct* SAnimSequenceAssetPreview::GetAssetPreviewType() const
{
	return UAnimSequence::StaticClass();
}

bool SAnimSequenceAssetPreview::OnSameTypeAssetSelected(const FAssetData& InAssetData)
{
	bool bAssetValidForPreview = false;

	USkeletalMesh* MeshToUse = nullptr;
	UClass* AssetClass = FindObject<UClass>(CachedCurrentAssetData.AssetClassPath);
	if(AssetClass->IsChildOf(UAnimationAsset::StaticClass()) && CachedCurrentAssetData.IsAssetLoaded() && CachedCurrentAssetData.GetAsset())
	{
		// Set up the viewport to show the asset.
		UAnimationAsset* Asset = StaticCast<UAnimationAsset*>(CachedCurrentAssetData.GetAsset());
		USkeleton* Skeleton = Asset->GetSkeleton();
		if(Skeleton)
		{
			MeshToUse = Skeleton->GetAssetPreviewMesh(Asset);
		}
		
		if(MeshToUse)
		{
			if(PreviewComponent->GetSkeletalMeshAsset() != MeshToUse)
			{
				PreviewComponent->SetSkeletalMesh(MeshToUse);
			}

			PreviewComponent->EnablePreview(true, Asset);
			PreviewComponent->PreviewInstance->PlayAnim(true);

			bAssetValidForPreview = true;
		}
	}

	ViewportWidget->SetVisibility(bAssetValidForPreview ? EVisibility::Visible : EVisibility::Hidden);
	return bAssetValidForPreview;
}

void SAnimSequenceAssetPreview::OnCustomizePreviewScene(FAdvancedPreviewScene& InPreviewScene, FEditorViewportClient& InEditorViewportClient)
{
	// @TODO: Note for review, most impl from SAnimationSequenceBrowser.h (Currently private, see IUAFAssetPreview.h for considerations).

	// Setup the preview component to ensure an animation will update when requested.
	// Note: Preview Component lifetime is managed by the preview scene
	PreviewComponent = NewObject<UDebugSkelMeshComponent>();
	PreviewComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	InPreviewScene.AddComponent(PreviewComponent, FTransform::Identity);

	// @TODO: Use these settings?
	//const USkeletalMeshEditorSettings* Options = GetDefault<USkeletalMeshEditorSettings>();
	//PreviewScene.SetLightDirection(Options->AnimPreviewLightingDirection);
	//PreviewScene.SetLightColor(Options->AnimPreviewDirectionalColor);
	//PreviewScene.SetLightBrightness(Options->AnimPreviewLightBrightness);
}

//////////////////////////////////////////////////////////////////////////

} // namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE

