// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/AssetPreview/IUAFAssetViewportPreview.h"

class FSceneViewport;
class FUAFAssetPreviewViewportClient;
class SAssetEditorViewport;
class UDebugSkelMeshComponent;

namespace UE::UAF::Editor
{


/**
 * Experimental. May be removed / relocated whenever we switch to the future generic content browser IAssetPreviewCustomization API.
 *
 * Factory for Anim Sequence Asset Preview widgets
 */
struct FAnimSequenceAssetPreviewFactory : public FUAFAssetPreviewFactory
{
public:

	//~ Begin FUAFAssetPreviewFactory Interface
	virtual TSharedRef<IUAFAssetPreview> CreateAssetPreviewWidget(TSharedPtr<FAssetEditorToolkit> InAssetEditorToolkit, const FAssetData& InAssetData) const override;
	virtual const UStruct* GetPreviewType() const override;
	//~ End FUAFAssetPreviewFactory Interface
};

/** 
 * Experimental. May be removed / relocated whenever we switch to the future generic content browser IAssetPreviewCustomization API. 
 * 
 * Preview widget for anim sequences. Includes an interactable viewport.
 */
class SAnimSequenceAssetPreview : public IUAFAssetViewportPreview
{
public:

	//~ Begin IUAFAssetPreview Interface
	virtual const UStruct* GetAssetPreviewType() const override;
	//~ End IUAFAssetPreview Interface

protected:

	//~ Begin IUAFAssetPreview Interface
	virtual bool OnSameTypeAssetSelected(const FAssetData& InAssetData) override;
	//~ End IUAFAssetPreview Interface

	//~ Begin IUAFAssetViewportPreview Interface
	virtual void OnCustomizePreviewScene(FAdvancedPreviewScene& InPreviewScene, FEditorViewportClient& InEditorViewportClient) override;
	//~ End IUAFAssetViewportPreview Interface

protected:

	/** Skeletal component to preview the animation asset on. Lifetime maintained by PreviewScene in IUAFAssetViewportPreview */
	UDebugSkelMeshComponent* PreviewComponent = nullptr;
};


} // namespace UE::UAF::Editor