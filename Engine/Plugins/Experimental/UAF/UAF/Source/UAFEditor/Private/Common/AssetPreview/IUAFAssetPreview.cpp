// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/AssetPreview/IUAFAssetPreview.h"

#include "Engine/AssetManager.h"

#define LOCTEXT_NAMESPACE "IUAFAssetPreview"

namespace UE::UAF::Editor
{


//////////////////////////////////////////////////////////////////////////
// IUAFAssetPreview

void IUAFAssetPreview::Construct(const FArguments& InArgs, TSharedPtr<FAssetEditorToolkit> InAssetEditorToolkit, const FAssetData& InAssetData)
{
	AssetEditorToolkitPtr = InAssetEditorToolkit;
	CachedCurrentAssetData = InAssetData;
	OnConstructAsyncPreviewWidget(CachedCurrentAssetData);
}

void IUAFAssetPreview::OnSameTypeAssetSelectedAsync(const FAssetData& InAssetData)
{
	CachedCurrentAssetData = InAssetData;

	FAssetData CachedCurrentAssetDataCopy = CachedCurrentAssetData;
	TWeakPtr<SWidget> ThisWeak = AsShared();
	auto SameAssetTypeSelectedCallback = [ThisWeak, CachedCurrentAssetDataCopy]()
	{
		if (TSharedPtr<IUAFAssetPreview> ThisPinned = StaticCastSharedPtr<IUAFAssetPreview>(ThisWeak.Pin()))
		{
			ThisPinned->OnSameTypeAssetSelected(CachedCurrentAssetDataCopy);

			// Also update the widget from loading indicator
			ThisPinned->ChildSlot
			[
				ThisPinned->OnConstructAssetPreviewWidget()
			];
		}
	};

	if (InAssetData.IsAssetLoaded())
	{
		SameAssetTypeSelectedCallback();
	}
	else
	{
		ChildSlot
		[
			OnConstructLoadingAssetPreviewWidget()
		];

		// Not done loading, kick off an async load
		RequestAsyncLoad(InAssetData.GetSoftObjectPath(), FStreamableDelegate::CreateLambda(MoveTemp(SameAssetTypeSelectedCallback)));
	}
}

void IUAFAssetPreview::RequestAsyncLoad(const FSoftObjectPath& SoftObjectPath, TFunction<void()>&& Callback)
{
	RequestAsyncLoad(SoftObjectPath, FStreamableDelegate::CreateLambda(MoveTemp(Callback)));
}

void IUAFAssetPreview::RequestAsyncLoad(const FSoftObjectPath& SoftObjectPath, FStreamableDelegate DelegateToCall)
{
	CancelAsyncLoad();
	
	if (UObject* LoadedObject = SoftObjectPath.ResolveObject())
	{
		// Already loaded, nothing to do
		DelegateToCall.ExecuteIfBound();
		return; 
	}

	TWeakPtr<SWidget> ThisWeak = AsShared();
	StreamingObjectPath = SoftObjectPath;

	auto AsyncLoadCompleteCallback = [ThisWeak, DelegateToCall, SoftObjectPath]()
	{
		if (TSharedPtr<IUAFAssetPreview> ThisPinned = StaticCastSharedPtr<IUAFAssetPreview>(ThisWeak.Pin()))
		{
			// If the object paths don't match, then this delegate was interrupted, so ignore everything and abort.
			if (ThisPinned->StreamingObjectPath != SoftObjectPath)
			{
				return;
			}

			DelegateToCall.ExecuteIfBound();
		}
	};

	StreamingHandle = UAssetManager::GetStreamableManager().RequestAsyncLoad(StreamingObjectPath
		, AsyncLoadCompleteCallback
		, FStreamableManager::AsyncLoadHighPriority);
}

void IUAFAssetPreview::CancelAsyncLoad()
{
	if (StreamingHandle.IsValid())
	{
		StreamingHandle->CancelHandle();
		StreamingHandle.Reset();
	}

	StreamingObjectPath.Reset();
}

void IUAFAssetPreview::OnConstructAsyncPreviewWidget(const FAssetData& InAssetData)
{
	TWeakPtr<SWidget> ThisWeak = AsShared();
	auto UpdateChildSlotCallback = [ThisWeak]()
	{
		if (TSharedPtr<IUAFAssetPreview> ThisPinned = StaticCastSharedPtr<IUAFAssetPreview>(ThisWeak.Pin()))
		{
			ThisPinned->ChildSlot
			[
				ThisPinned->OnConstructAssetPreviewWidget()
			];
		}
	};

	if (InAssetData.IsAssetLoaded())
	{
		UpdateChildSlotCallback();
	}
	else
	{
		ChildSlot
		[
			OnConstructLoadingAssetPreviewWidget()
		];

		// Not done loading, kick off an async load
		RequestAsyncLoad(InAssetData.GetSoftObjectPath(), FStreamableDelegate::CreateLambda(MoveTemp(UpdateChildSlotCallback)));
	}
}

TSharedRef<SWidget> IUAFAssetPreview::OnConstructLoadingAssetPreviewWidget()
{
	if (CachedLoadingAssetPreviewWidget)
	{
		return CachedLoadingAssetPreviewWidget.ToSharedRef();
	}

	// @TODO Fancy spinning circle? Hourglass Icon?
	return SAssignNew(CachedLoadingAssetPreviewWidget, SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.FillHeight(1.0f)
		[
			SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.AutoWrapText(true)
				.Text(LOCTEXT("AssetPreviewLoading", "Loading Asset Preview..."))
		];
}

TSharedRef<SWidget> IUAFAssetPreview::OnConstructAssetPreviewWidget()
{
	ensure(CachedCurrentAssetData.IsAssetLoaded());
	return SNullWidget::NullWidget;
}


//////////////////////////////////////////////////////////////////////////

} // namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE

