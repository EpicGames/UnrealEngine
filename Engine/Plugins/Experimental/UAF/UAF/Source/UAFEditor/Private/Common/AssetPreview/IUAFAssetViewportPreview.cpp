// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/AssetPreview/IUAFAssetViewportPreview.h"

#include "AssetEditorViewportLayout.h"
#include "Engine/AssetManager.h"
#include "SceneInterface.h"
#include "Slate/SceneViewport.h"
#include "Viewports.h"

#define LOCTEXT_NAMESPACE "IUAFAssetViewportPreview"

namespace UE::UAF::Editor
{


//////////////////////////////////////////////////////////////////////////
// IUAFAssetViewportPreview

TSharedRef<SWidget> IUAFAssetViewportPreview::OnConstructAssetPreviewWidget()
{
	using namespace UE::Workspace;

	ensure(CachedCurrentAssetData.IsAssetLoaded());
	if (ViewportWidget)
	{
		return ViewportWidget.ToSharedRef();
	}

	PreviewScene = MakeUnique<FAdvancedPreviewScene>(FPreviewScene::ConstructionValues());

	TSharedPtr<FEditorViewportClient> EditorViewportClient = MakeShared<FEditorViewportClient>(nullptr, PreviewScene.Get());
	EditorViewportClient->ViewportType = LVT_Perspective;
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->SetViewLocation(FVector(0.0f, 400.0f, 200.0f));
	EditorViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);
	EditorViewportClient->SetRealtime(true);
	EditorViewportClient->SetViewMode(VMI_Lit);
	EditorViewportClient->ToggleOrbitCamera(true);

	SAssignNew(ViewportWidget, SWorkspaceViewport)
		.AssetEditorToolkit(AssetEditorToolkitPtr.Pin())
		.ViewportClient(MoveTemp(EditorViewportClient))
		.SceneDescription(nullptr);

	OnCustomizePreviewScene(*PreviewScene.Get(), *ViewportWidget->GetViewportClient().Get());
	
	// Resolve the asset
	bool bAssetValidForPreview = OnSameTypeAssetSelected(CachedCurrentAssetData);
	if (bAssetValidForPreview)
	{
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &IUAFAssetViewportPreview::UpdatePreviewWorld));
	}

	return ViewportWidget.ToSharedRef();
}

EActiveTimerReturnType IUAFAssetViewportPreview::UpdatePreviewWorld(double InCurrentTime, float InDeltaTime)
{
	if (PreviewScene.IsValid()
		&& ViewportWidget->IsParentValid()
		&& ViewportWidget->IsVisible()
		&& ViewportWidget->GetVisibility() == EVisibility::Visible)
	{
		// Tick the world to update preview viewport for tooltips
		PreviewScene->GetWorld()->Tick(LEVELTICK_All, InDeltaTime);
	}

	// We never stop this timer as long as the widget exists. It will no-op if we are not visible
	// @TODO: Possibly add / remove timer as visibility changes? Consider work after a profile phase.
	return EActiveTimerReturnType::Continue;
}


//////////////////////////////////////////////////////////////////////////

} // namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE

