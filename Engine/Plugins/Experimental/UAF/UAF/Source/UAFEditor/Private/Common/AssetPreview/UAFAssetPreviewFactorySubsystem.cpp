// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/AssetPreview/UAFAssetPreviewFactorySubsystem.h"

#include "Common/AssetPreview/IUAFAssetPreview.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFAssetPreviewFactorySubsystem)

#define LOCTEXT_NAMESPACE "UAFAssetPreviewFactorySubsystem"

bool UUAFAssetPreviewFactorySubsystem::AddAssetPreviewFactory(TSharedPtr<UE::UAF::Editor::FUAFAssetPreviewFactory> InFactory)
{
	if (InFactory)
	{
		if (!AssetTypeFactoryMap.Contains(InFactory->GetPreviewType()))
		{
			AssetTypeFactoryMap.Add(InFactory->GetPreviewType(), InFactory);

			return true;
		}
	}

	return false;
}

const TSharedPtr<UE::UAF::Editor::FUAFAssetPreviewFactory> UUAFAssetPreviewFactorySubsystem::GetAssetPreviewFactory(const UStruct* InPreviewType) const
{
	if (InPreviewType)
	{
		if (const TSharedPtr<UE::UAF::Editor::FUAFAssetPreviewFactory>* FactoryPtr = AssetTypeFactoryMap.Find(InPreviewType))
		{
			return *FactoryPtr;
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE

