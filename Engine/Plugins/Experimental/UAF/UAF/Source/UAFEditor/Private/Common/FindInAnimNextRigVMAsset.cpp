// Copyright Epic Games, Inc. All Rights Reserved.

#include "FindInAnimNextRigVMAsset.h"
#include "IWorkspaceEditor.h"
#include "AnimNextEdGraphNode.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "EdGraph/RigVMEdGraph.h"
#include "GraphEditor.h"
#include "IWorkspaceEditorModule.h"
#include "RigVMModel/RigVMGraph.h"
#include "UncookedOnlyUtils.h"

#define LOCTEXT_NAMESPACE "FindInAnimNextRigVMAsset"

namespace UE::UAF::Editor
{

//////////////////////////////////////////////////////////////////////////
// FFindInAnimNextRigVMAssetResult

FFindInAnimNextRigVMAssetResult::FFindInAnimNextRigVMAssetResult(const FFindInGraphResult::FCreateParams& InCreateParams)
	: FFindInGraphResult(InCreateParams)
{
}

void FFindInAnimNextRigVMAssetResult::JumpToNode(TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit, const UEdGraphNode* InNode) const
{
	if (!InNode)
	{
		return;
	}

	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorToolkit.Pin()))
	{		
		// See if there already is an open graph editor, if not open it. Don't just always open as that can change focus contexts from parent modules
		if (TSharedPtr<SGraphEditor> GraphEditor = SGraphEditor::FindGraphEditorForGraph(InNode->GetGraph()))
		{
			GraphEditor->JumpToNode(InNode, false);
		}
		else
		{
			WorkspaceEditor->OpenObjects({ const_cast<UEdGraph*>(InNode->GetGraph()) });
			if (TSharedPtr<SGraphEditor> NewlyOpenedGraphEditor = SGraphEditor::FindGraphEditorForGraph(InNode->GetGraph()))
			{
				NewlyOpenedGraphEditor->JumpToNode(InNode, false);
			}
		}
	}
}


//////////////////////////////////////////////////////////////////////////
// SFindInAnimNextRigVMAsset

TSharedPtr<FFindInGraphResult> SFindInAnimNextRigVMAsset::MakeSearchResult(const FFindInGraphResult::FCreateParams& InParams)
{
	return MakeShared<FFindInAnimNextRigVMAssetResult>(InParams);
}

void SFindInAnimNextRigVMAsset::MatchTokens(const TArray<FString>& Tokens)
{
	RootSearchResult.Reset();

	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(AssetEditorToolkitPtr.Pin()))
	{
		const Workspace::FWorkspaceDocument& FocusedDocument = WorkspaceEditor->GetFocusedWorkspaceDocument();
		if (UEdGraph* Graph = FocusedDocument.GetTypedObject<UEdGraph>())
		{
			// If we just selected a plain graph, rely on default graph search
			MatchTokensInGraph(Graph, Tokens);
		}
		else if (UUAFRigVMAsset* AnimNextRigVMAsset = FocusedDocument.GetTypedObject<UUAFRigVMAsset>())
		{
			// Note: This is where we search subgraphs in say a module
			if (UUAFRigVMAssetEditorData* AnimNextRigVMAssetEditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(AnimNextRigVMAsset))
			{
				TArray<UEdGraph*> ContainedGraphs = AnimNextRigVMAssetEditorData->GetAllEdGraphs();
				for (const UEdGraph* ContainedGraph : ContainedGraphs)
				{
					MatchTokensInGraph(ContainedGraph, Tokens);
				}
			}
		}
	}
}

bool SFindInAnimNextRigVMAsset::MatchTokensInNode(const UEdGraphNode* Node, const TArray<FString>& Tokens)
{	
	// Search all animnext node pins to see if any graphs are referenced. If so add them to the search list
	if (const UAnimNextEdGraphNode* AnimNextEdGraphNode = Cast<UAnimNextEdGraphNode>(Node))
	{
		for (const UEdGraphPin* Pin : AnimNextEdGraphNode->Pins)
		{
			if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
			{
				if (URigVMPin* RigVMPin = AnimNextEdGraphNode->FindModelPinFromGraphPin(Pin))
				{
					if (UClass* PinClass = Cast<UClass>(RigVMPin->GetCPPTypeObject()))
					{
						if (PinClass->IsChildOf<UUAFRigVMAsset>() || RigVMPin->GetMetaData("GetAllowedClasses") == "/Script/UAFAnimGraph.AnimNextAnimGraphSettings:GetAllowedAssetClasses")
						{
							// Search referenced asset if already loaded, don't change return value as this node still technically failed the search
							const FSoftObjectPath ObjectPath(RigVMPin->GetDefaultValue());
							if (UUAFRigVMAsset* AnimNextRigVMAsset = Cast<UUAFRigVMAsset>(ObjectPath.ResolveObject()))
							{
								if (UUAFRigVMAssetEditorData* AnimNextRigVMAssetEditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(AnimNextRigVMAsset))
								{
									TArray<UEdGraph*> ContainedGraphs = AnimNextRigVMAssetEditorData->GetAllEdGraphs();
									for (const UEdGraph* ContainedGraph : ContainedGraphs)
									{
										MatchTokensInGraph(ContainedGraph, Tokens);
									}
								}
							}
						}
					}
				}
			}
		}
	}

	return false;
}


/////////////////////////////////////////////////////

} // namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE

