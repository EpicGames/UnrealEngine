// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/GraphEditorSchemaActions.h"

#include "AnimNextControllerBase.h"
#include "EditorUtils.h"
#include "AnimNextEdGraphNode.h"
#include "EdGraphNode_Comment.h"
#include "Editor.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "EdGraphSchema_K2.h"
#include "GraphEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "AnimNextSharedVariableNode.h"
#include "Editor/RigVMEditorStyle.h"
#include "Entries/AnimNextVariableEntry.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "UAFCompilationScope.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "Common/UAFVariableDragDropContext.h"
#include "Variables/AnimNextSharedVariables.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GraphEditorSchemaActions)

#define LOCTEXT_NAMESPACE "AnimNextSchemaActions"

// *** Base Schema Action ***

const FSlateBrush* FAnimNextSchemaAction::GetIconBrush() const
{
	return FRigVMEditorStyle::Get().GetBrush("RigVM.Unit");
}

const FLinearColor& FAnimNextSchemaAction::GetIconColor() const
{
	return FLinearColor::White;
}

// *** Rig Unit ***

UEdGraphNode* FAnimNextSchemaAction_RigUnit::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode)
{
	IRigVMClientHost* Host = ParentGraph->GetImplementingOuter<IRigVMClientHost>();
	URigVMEdGraphNode* NewNode = nullptr;
	URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(ParentGraph);

	UEdGraphPin* FromPin = nullptr;
	if (FromPins.Num() > 0)
	{
		FromPin = FromPins[0];
	}
	
	if (Host != nullptr && EdGraph != nullptr)
	{
		FName Name = UE::UAF::Editor::FUtils::ValidateName(Cast<UObject>(Host), StructTemplate->GetFName().ToString());
		UAnimNextControllerBase* Controller = CastChecked<UAnimNextControllerBase>(Host->GetRigVMClient()->GetController(ParentGraph));

		Controller->OpenUndoBracket(FString::Printf(TEXT("Add '%s' Node"), *Name.ToString()));

		FRigVMUnitNodeCreatedContext& UnitNodeCreatedContext = Controller->GetUnitNodeCreatedContext();
		FRigVMUnitNodeCreatedContext::FScope ReasonScope(UnitNodeCreatedContext, ERigVMNodeCreatedReason::NodeSpawner, Host);

		if (URigVMUnitNode* ModelNode = Controller->AddUnitNodeOfClass(StructTemplate->GetPathName(), NodeClass.Get()->GetPathName(), FRigVMStruct::ExecuteName, FDeprecateSlateVector2D(Location), Name.ToString(), true, true))
		{
			NewNode = Cast<URigVMEdGraphNode>(EdGraph->FindNodeForModelNodeName(ModelNode->GetFName()));
			check(NewNode);

			if (NewNode)
			{
				if (FromPin)
				{
					NewNode->AutowireNewNode(FromPin);
				}

				Controller->ClearNodeSelection(true, true);
				Controller->SelectNode(ModelNode, true, true, true);
			}
		}
		Controller->CloseUndoBracket();
	}

	return NewNode;
}

// *** Dispatch Factory ***

const FSlateBrush* FAnimNextSchemaAction_DispatchFactory::GetIconBrush() const
{
	return FRigVMEditorStyle::Get().GetBrush("RigVM.Template");
}

UEdGraphNode* FAnimNextSchemaAction_DispatchFactory::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode)
{
	IRigVMClientHost* Host = ParentGraph->GetImplementingOuter<IRigVMClientHost>();
	if(Host == nullptr)
	{
		return nullptr;
	}

	URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(ParentGraph);
	if(EdGraph == nullptr)
	{
		return nullptr;
	}

	const FRigVMTemplate* Template = FRigVMRegistry::Get().FindTemplate(Notation);
	if (Template == nullptr)
	{
		return nullptr;
	}
	
	URigVMEdGraphNode* NewNode = nullptr;
	UEdGraphPin* FromPin = nullptr;
	if (FromPins.Num() > 0)
	{
		FromPin = FromPins[0];
	}

	const int32 NotationHash = (int32)GetTypeHash(Notation);
	const FString TemplateName = TEXT("RigVMTemplate_") + FString::FromInt(NotationHash);

	FName Name = UE::UAF::Editor::FUtils::ValidateName(Cast<UObject>(Host), Template->GetName().ToString());
	URigVMController* Controller = Host->GetRigVMClient()->GetController(ParentGraph);

	Controller->OpenUndoBracket(FString::Printf(TEXT("Add '%s' Node"), *Name.ToString()));

	if (URigVMTemplateNode* ModelNode = Controller->AddTemplateNode(Notation, FDeprecateSlateVector2D(Location), Name.ToString(), true, true))
	{
		NewNode = Cast<URigVMEdGraphNode>(EdGraph->FindNodeForModelNodeName(ModelNode->GetFName()));

		if (NewNode)
		{
			if(FromPin)
			{
				NewNode->AutowireNewNode(FromPin);
			}

			Controller->ClearNodeSelection(true, true);
			Controller->SelectNode(ModelNode, true, true, true);
		}

		Controller->CloseUndoBracket();
	}
	else
	{
		Controller->CancelUndoBracket();
	}

	return NewNode;
}

// *** Variable ***

FAnimNextSchemaAction_Variable::FAnimNextSchemaAction_Variable(FName InName, const UObject* InSourceObject, const FAnimNextParamType& InType, const EVariableAccessorChoice InVariableAccessorChoice)
	: Name(InName)
{
	SetVariableAccessorChoice(InVariableAccessorChoice, true);
	
	check(InSourceObject->IsA<UUAFSharedVariables>() || InSourceObject->IsA<UScriptStruct>() || InSourceObject->Implements<URigVMRuntimeAssetInterface>());
	SourceObjectPath = InSourceObject->GetPathName();

	if(InType.IsObjectType())
	{
		TypeObjectPath = InType.GetValueTypeObject()->GetPathName();
	}

	TypeName = InType.ToRigVMTemplateArgument().GetBaseCPPType();
	
	FEdGraphPinType PinType = UE::UAF::UncookedOnly::FUtils::GetPinTypeFromParamType(InType);
	VariableColor = GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);
}

const FSlateBrush* FAnimNextSchemaAction_Variable::GetIconBrush() const
{
	return  FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));
}

const FLinearColor& FAnimNextSchemaAction_Variable::GetIconColor() const
{
	return VariableColor;
}

void FAnimNextSchemaAction_Variable::SetVariableAccessorChoice(EVariableAccessorChoice InVariableAccessorChoice, bool bUpdateSearchData)
{
	VariableAccessorChoice = InVariableAccessorChoice;
	
	if (bUpdateSearchData)
	{
		static const FText VariablesCategory = LOCTEXT("Variables", "Variables");
		static const FTextFormat GetVariableFormat = LOCTEXT("GetVariableFormat", "Get {0}");
		static const FTextFormat SetVariableFormat = LOCTEXT("SetVariableFormat", "Set {0}");
	
		FText MenuDesc;
		FText ToolTip;
	
		if(VariableAccessorChoice == EVariableAccessorChoice::Get)
		{
			MenuDesc = FText::Format(GetVariableFormat, FText::FromName(Name));
			ToolTip = FText::FromString(FString::Printf(TEXT("Get the value of variable %s"), *Name.ToString()));
		}
		else if (VariableAccessorChoice == EVariableAccessorChoice::Set)
		{
			MenuDesc = FText::Format(SetVariableFormat, FText::FromName(Name));
			ToolTip = FText::FromString(FString::Printf(TEXT("Set the value of variable %s"), *Name.ToString()));
		}
		else
		{
			MenuDesc = FText::FromName(Name);
		}
	
		UpdateSearchData(MenuDesc, ToolTip, VariablesCategory, FText::GetEmpty());
	}
}

UEdGraphNode* FAnimNextSchemaAction_Variable::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode)
{
	const auto AddNodeLambda = [](UEdGraph* ParentGraph, FName InName, const FString& InSourceObjectPath, const FString& InTypeName, const FString& InTypeObjectPath, const FVector2f& Location, const bool bIsGetter) -> URigVMEdGraphNode*
	{
		IRigVMClientHost* Host = ParentGraph->GetImplementingOuter<IRigVMClientHost>();
		if(Host == nullptr)
		{
			return nullptr;
		}

		URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(ParentGraph);
		if(EdGraph == nullptr)
		{
			return nullptr;
		}

		URigVMEdGraphNode* NewNode = nullptr;

		FString NodeName;

		FScopedTransaction Transaction(LOCTEXT("AddVariableNode", "Add Variable Node"));

		UAnimNextControllerBase* Controller = CastChecked<UAnimNextControllerBase>(Host->GetRigVMClient()->GetController(ParentGraph));
		Controller->OpenUndoBracket(TEXT("Add Variable"));

		UObject* CPPTypeObject = nullptr;
		if (!InTypeObjectPath.IsEmpty())
		{
			CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InTypeObjectPath);

			if (CPPTypeObject == nullptr)
			{
				Controller->CancelUndoBracket();
				return nullptr;
			}
		}

		UObject* SourceObject = nullptr;
		if (!InSourceObjectPath.IsEmpty())
		{
			SourceObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InSourceObjectPath);
		}

		if (SourceObject == nullptr)
		{
			Controller->CancelUndoBracket();
			return nullptr;
		}

		if (UUAFRigVMAsset* VariableOwningRigVMAsset = Cast<UUAFRigVMAsset>(SourceObject))
		{
			// Satisfy host dependency
			if (UUAFRigVMAssetEditorData* OwnerEditorData = ParentGraph->GetTypedOuter<UUAFRigVMAssetEditorData>())
			{
				if (UUAFRigVMAssetEditorData* VariableOwnerEditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(VariableOwningRigVMAsset))
				{
					if (OwnerEditorData != VariableOwnerEditorData)
					{
						if (IUAFRigVMExportInterface* VariableExport = Cast<IUAFRigVMExportInterface>(VariableOwnerEditorData->FindEntry(InName)))
						{
							if (VariableExport->GetExportAccessSpecifier() != EAnimNextExportAccessSpecifier::Public)
							{
								UE::UAF::UncookedOnly::FCompilationScope CompileScope(LOCTEXT("ForceVariablePublicJobName", "Force Variable Public Access"), VariableOwningRigVMAsset);							
								VariableExport->SetExportAccessSpecifier(EAnimNextExportAccessSpecifier::Public);
							}
						}
						
						if (!OwnerEditorData->FindEntry(VariableOwningRigVMAsset->GetFName()))
						{
							UE::UAF::UncookedOnly::FCompilationScope CompileScope(LOCTEXT("AddSharedAssetJobName", "Add Shared Asset dependency"), ParentGraph->GetTypedOuter<UUAFRigVMAsset>());
							
							UUAFSharedVariables* VariableOwningSharedVariables = UE::UAF::UncookedOnly::FUtils::GetAsset<UUAFSharedVariables>(VariableOwnerEditorData);
							OwnerEditorData->AddSharedVariables(VariableOwningSharedVariables, true, false);
						}
					}
				}
			}
		}

		URigVMNode* ModelNode = nullptr;

		// If the variable is from 'this' asset, we just need a standard variable node
		if (ParentGraph->GetTypedOuter<UUAFSharedVariables>() == SourceObject)
		{
			ModelNode = Controller->AddVariableNode(InName, InTypeName, CPPTypeObject, bIsGetter, FString(), FDeprecateSlateVector2D(Location), NodeName, true, true);
		}
		else
		{
			ModelNode = Controller->AddSharedVariableNode(InSourceObjectPath, InName, InTypeName, InTypeObjectPath, bIsGetter, FString(), FDeprecateSlateVector2D(Location), NodeName, true, true);
		}

		if (ModelNode)
		{
			for (UEdGraphNode* Node : ParentGraph->Nodes)
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Node))
				{
					if (RigNode->GetModelNodeName() == ModelNode->GetFName())
					{
						NewNode = RigNode;
						break;
					}
				}
			}

			if (NewNode)
			{
				Controller->ClearNodeSelection(true, true);
				Controller->SelectNode(ModelNode, true, true, true);
			}
			Controller->CloseUndoBracket();
		}
		else
		{
			Controller->CancelUndoBracket();
		}

		return NewNode;
	};

	if (VariableAccessorChoice == EVariableAccessorChoice::Deferred)
	{
		static const FName MenuName = "UAFVariableDragDropMenu";
		if(!UToolMenus::Get()->IsMenuRegistered(MenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
			Menu->AddDynamicSection("VariableGetterSetter", FNewToolMenuDelegate::CreateLambda([AddNodeLambda](UToolMenu* InMenu)
			{
				const UUAFVariableDragDropContext* Context = InMenu->FindContext<UUAFVariableDragDropContext>();
				if (Context == nullptr)
				{
					return;
				}

				FToolMenuSection& Section = InMenu->AddSection("VariableGetterSetter", FText::Format(LOCTEXT("SectionFormat", "Variable {0}"), FText::FromName(Context->Name)));

				Section.AddMenuEntry(
					"GetVariable",
					FText::Format(LOCTEXT("GetterFormat", "Get {0}"), FText::FromName(Context->Name)),
					FText::Format(LOCTEXT("GetterTooltipFormat", "Adds a getter node for variable {0}"), FText::FromName(Context->Name)),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([AddNodeLambda, Context]
						{
							AddNodeLambda(Context->ParentGraph, Context->Name, Context->SourceObjectPath, Context->TypeName, Context->TypeObjectPath, Context->Location, true);
						}),
						FCanExecuteAction()
					));

				Section.AddMenuEntry(
					"SetVariable",
					FText::Format(LOCTEXT("SetterFormat", "Set {0}"), FText::FromName(Context->Name)),
					FText::Format(LOCTEXT("SetterTooltipFormat", "Adds a setter node for variable {0}"), FText::FromName(Context->Name)),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([AddNodeLambda, Context]
						{
							AddNodeLambda(Context->ParentGraph, Context->Name, Context->SourceObjectPath, Context->TypeName, Context->TypeObjectPath, Context->Location, false);
						}),
						FCanExecuteAction()
					));
			}));
		}

		UUAFVariableDragDropContext* Context = NewObject<UUAFVariableDragDropContext>();
		Context->ParentGraph = ParentGraph;
		Context->Name = Name;
		Context->Location = Location;
		Context->SourceObjectPath = SourceObjectPath;
		Context->TypeName = TypeName;
		Context->TypeObjectPath = TypeObjectPath;

		FToolMenuContext ToolMenuContext;
		ToolMenuContext.AddObject(Context);

		FSlateApplication::Get().PushMenu(
			FSlateApplication::Get().GetInteractiveTopLevelWindows()[0],
			FWidgetPath(),
			UToolMenus::Get()->GenerateWidget(MenuName, ToolMenuContext),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);

		return nullptr;
	}
	else
	{
		const bool bIsGetter = VariableAccessorChoice == EVariableAccessorChoice::Get;
		return AddNodeLambda(ParentGraph, Name, SourceObjectPath, TypeName, TypeObjectPath, Location, bIsGetter);
	}
}

// *** Add Comment ***

FAnimNextSchemaAction_AddComment::FAnimNextSchemaAction_AddComment()
	: FAnimNextSchemaAction(FText(), LOCTEXT("AddComment", "Add Comment..."), LOCTEXT("AddCommentTooltip", "Create a resizable comment box."))
{
}

UEdGraphNode* FAnimNextSchemaAction_AddComment::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	UEdGraphNode_Comment* const CommentTemplate = NewObject<UEdGraphNode_Comment>();

	FVector2f SpawnLocation = Location;
	FSlateRect Bounds;

	TSharedPtr<SGraphEditor> GraphEditorPtr = SGraphEditor::FindGraphEditorForGraph(ParentGraph);
	if (GraphEditorPtr.IsValid() && GraphEditorPtr->GetBoundsForSelectedNodes(/*out*/ Bounds, 50.0f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}

	return FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(ParentGraph, CommentTemplate, SpawnLocation, bSelectNewNode);
}

const FSlateBrush* FAnimNextSchemaAction_AddComment::GetIconBrush() const
{
	return FAppStyle::Get().GetBrush("Icons.Comment");
}

// *** Graph Function ***

FAnimNextSchemaAction_Function::FAnimNextSchemaAction_Function(const FRigVMGraphFunctionHeader& InReferencedPublicFunctionHeader, const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, const FText& InKeywords)
	: FAnimNextSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InKeywords)
{
	ReferencedPublicFunctionHeader = InReferencedPublicFunctionHeader;
	NodeClass = UAnimNextEdGraphNode::StaticClass();
	bIsLocalFunction = true;
}

FAnimNextSchemaAction_Function::FAnimNextSchemaAction_Function(const URigVMLibraryNode* InFunctionLibraryNode, const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, const FText& InKeywords)
	: FAnimNextSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InKeywords)
{
	ReferencedPublicFunctionHeader = InFunctionLibraryNode->GetFunctionHeader();
	NodeClass = UAnimNextEdGraphNode::StaticClass();
	bIsLocalFunction = true;
}

FAnimNextSchemaAction_Function::FAnimNextSchemaAction_Function(const URigVMLibraryNode* InFunctionLibraryNode)
	: FAnimNextSchemaAction(InFunctionLibraryNode ? FText::FromString(InFunctionLibraryNode->GetNodeCategory()) : FText::GetEmpty(), InFunctionLibraryNode ? FText::FromString(InFunctionLibraryNode->GetNodeDescription()) : FText::GetEmpty(), InFunctionLibraryNode ? InFunctionLibraryNode->GetToolTipText() : FText::GetEmpty(), InFunctionLibraryNode ? FText::FromString(InFunctionLibraryNode->GetNodeKeywords()) : FText::GetEmpty())
{
	if (InFunctionLibraryNode)
	{
		ReferencedPublicFunctionHeader = InFunctionLibraryNode->GetFunctionHeader();
	}
	NodeClass = UAnimNextEdGraphNode::StaticClass();
	bIsLocalFunction = true;
}

const FSlateBrush* FAnimNextSchemaAction_Function::GetIconBrush() const
{
	return FAppStyle::GetBrush("GraphEditor.Function_16x");
}

UEdGraphNode* FAnimNextSchemaAction_Function::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode)
{
	IRigVMClientHost* Host = ParentGraph->GetImplementingOuter<IRigVMClientHost>();
	URigVMEdGraphNode* NewNode = nullptr;
	URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(ParentGraph);

	if (Host != nullptr && EdGraph != nullptr)
	{
		FName Name = UE::UAF::Editor::FUtils::ValidateName(Cast<UObject>(Host), ReferencedPublicFunctionHeader.Name.ToString());
		URigVMController* Controller = EdGraph->GetController();

		FScopedTransaction Transaction(FText::Format(LOCTEXT("AddFunctionReferenceTransaction", "Add {0} Function Reference Node"), FText::FromName(Name)));
		Controller->OpenUndoBracket(FString::Printf(TEXT("Add '%s' Node"), *Name.ToString()));

		URigVMController* FunctionGraphController = nullptr;
		// Make function public automatically when attempting to place a reference to it within a different outer
		TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = ReferencedPublicFunctionHeader.GetFunctionHostObject();
		IRigVMGraphFunctionHost* ReferenceHost = EdGraph->GetImplementingOuter<IRigVMGraphFunctionHost>();
		if (FunctionHost && FunctionHost != ReferenceHost)
		{
			if (!FunctionHost->GetRigVMGraphFunctionStore()->IsFunctionPublic(ReferencedPublicFunctionHeader.LibraryPointer))
			{
				if (const UUAFRigVMAssetEditorData* FunctionEditorData = Cast<UUAFRigVMAssetEditorData>(ReferencedPublicFunctionHeader.LibraryPointer.HostObject.TryLoad()))
				{
					FunctionGraphController = FunctionEditorData->GetController(FunctionEditorData->GetLocalFunctionLibrary()->GetRootGraph());
					FunctionGraphController->OpenUndoBracket(FString::Printf(TEXT("Mark '%s' as Public"), *Name.ToString()));
					FunctionGraphController->MarkFunctionAsPublic(ReferencedPublicFunctionHeader.Name, true, true, true);
				}
			}
		}

		if (URigVMFunctionReferenceNode* ModelNode = Controller->AddFunctionReferenceNodeFromDescription(ReferencedPublicFunctionHeader, FDeprecateSlateVector2D(Location), Name.ToString(), true, true))
		{
			NewNode = Cast<URigVMEdGraphNode>(EdGraph->FindNodeForModelNodeName(ModelNode->GetFName()));
			check(NewNode);

			if (NewNode)
			{
				Controller->ClearNodeSelection(true, true);
				Controller->SelectNode(ModelNode, true, true, true);
			}

			for (const URigVMPin* Pin : ModelNode->GetPins())
			{
				if (!Pin->IsDefinedAsInputVariable())
				{
					continue;
				}
				Controller->BindPinToVariable(Pin->GetPinPath(), Pin->GetName());
			}

			Controller->CloseUndoBracket();
			if (FunctionGraphController)
			{
				FunctionGraphController->CloseUndoBracket();
			}
		}
		else
		{
			Controller->CancelUndoBracket();
			
			// Also cancel/undo marking referenced function as public 
			if (FunctionGraphController)
			{
				FunctionGraphController->CancelUndoBracket();
			}
			Transaction.Cancel();
		}
	}

	return NewNode;
}

FAnimNextSchemaAction_PromoteToVariable::FAnimNextSchemaAction_PromoteToVariable()
: FAnimNextSchemaAction(FText(), LOCTEXT("PromoteToVariable", "Promote to Variable..."), LOCTEXT("AddVariableTooltip", "Adds a variable of the pin type to the currently open asset."))
{
}

UEdGraphNode* FAnimNextSchemaAction_PromoteToVariable::PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode)
{
	
	UEdGraphNode* Result = nullptr;
	if (ParentGraph)
	{
		IRigVMClientHost* Host = ParentGraph->GetImplementingOuter<IRigVMClientHost>();
		if(Host == nullptr)
		{
			return nullptr;
		}
		if (UUAFRigVMAssetEditorData* EditorData = Cast<UUAFRigVMAssetEditorData>(Host))
		{
			if (FromPins.Num() == 1)
			{
				if (UEdGraphPin* FromPin = FromPins[0])
				{
					UUAFRigVMAsset* RigVMAsset = UE::UAF::UncookedOnly::FUtils::GetAsset(EditorData);
					const FAnimNextParamType ParamType = UE::UAF::UncookedOnly::FUtils::GetParamTypeFromPinType(FromPin->PinType);
					if (ParamType.IsValid() && EditorData)
					{
						FScopedTransaction Transaction(LOCTEXT("PromoteToVariableTransaction", "Promote to Variable"));
						
						FString BaseName = FromPin->GetDisplayName().ToString();
						BaseName.ReplaceCharInline(' ', '_');

						const FName BaseFName = FName(*BaseName);
						const FName NewVariableName = UE::UAF::UncookedOnly::FUtils::GetValidVariableName(EditorData, BaseFName);
						if (UAnimNextVariableEntry* VariableEntry = EditorData->AddVariable(NewVariableName, ParamType, FromPin->GetDefaultAsString()))
						{
							UAnimNextControllerBase* Controller = CastChecked<UAnimNextControllerBase>(Host->GetRigVMClient()->GetController(ParentGraph));
							Controller->OpenUndoBracket(TEXT("Promote to Variable"));
							
							const FAnimNextSchemaAction_Variable::EVariableAccessorChoice AccessorType = FromPin->Direction == EEdGraphPinDirection::EGPD_Input ? FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Get : FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Set;						
							FAnimNextSchemaAction_Variable Action(VariableEntry->GetVariableName(), RigVMAsset, VariableEntry->GetType(), AccessorType);

							if (URigVMEdGraphNode* RigVMEdNode = Cast<URigVMEdGraphNode>(Action.PerformAction(ParentGraph, FromPins, Location)))
							{
								if (URigVMVariableNode* SpawnedVariableNode = Cast<URigVMVariableNode>(RigVMEdNode->GetModelNode()))
								{
									const URigVMEdGraphNode* RigVMNode = Cast<URigVMEdGraphNode>(FromPin->GetOwningNode());								
									const URigVMPin* FromRigPin = RigVMNode->FindModelPinFromGraphPin(FromPin);

									const FString FromPinPath = FromPin->Direction == EEdGraphPinDirection::EGPD_Input ? SpawnedVariableNode->GetValuePin()->GetPinPath() : FromRigPin->GetPinPath();
									const FString ToPinPath = FromPin->Direction == EEdGraphPinDirection::EGPD_Output ? SpawnedVariableNode->GetValuePin()->GetPinPath() : FromRigPin->GetPinPath();
									
									RigVMNode->GetController()->AddLink(FromPinPath, ToPinPath, true, true);
								}

								Result = RigVMEdNode;
							}

							if (Result != nullptr)
							{
								Controller->CloseUndoBracket();
							}
							else
							{
								Controller->CancelUndoBracket();
							}
						}
					}
				}
			}

			
		}
	}
	
	return Result;
}

const FSlateBrush* FAnimNextSchemaAction_PromoteToVariable::GetIconBrush() const
{
	return FAnimNextSchemaAction::GetIconBrush();
}

#undef LOCTEXT_NAMESPACE
