// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommonOutlinerMode.h"

#include "IWorkspaceEditor.h"
#include "OutlinerAssetItem.h"
#include "OutlinerEntryItem.h"
#include "ToolMenus.h"
#include "UncookedOnlyUtils.h"
#include "Framework/Commands/GenericCommands.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Common/Outliner/OutlinerItemMenuContexts.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "FileHelpers.h"
#include "Variables/Outliner/VariablesOutlinerCommands.h"

#include "AnimNextRigVMAsset.h"
#include "UAFCompilationScope.h"
#include "IWorkspaceEditorModule.h"
#include "OutlinerCategoryItem.h"
#include "OutlinerCommands.h"
#include "ScopedTransaction.h"
#include "Common/AnimNextAssetFindReplaceVariables.h"
#include "OutlinerDragDropOps.h"
#include "SCommonOutliner.h"
#include "Variables/Outliner/VariablesOutlinerCategoryItem.h"

#define LOCTEXT_NAMESPACE "FCommonOutlinerMode"

namespace UE::UAF::Editor
{
FCommonOutlinerMode::FCommonOutlinerMode(SSceneOutliner* InOutliner, const TSharedRef<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor) : ISceneOutlinerMode(InOutliner), WeakWorkspaceEditor(InWorkspaceEditor)
{
	CommandList = MakeShared<FUICommandList>();	

	RegisterDragDropOperationConstructor<FOutlinerCategoryItem>([](const FSceneOutlinerTreeItemPtr& InItem)-> TSharedPtr<FDragDropOperation>
	{
		FOutlinerCategoryItem* CategoryItem = InItem->CastTo<FOutlinerCategoryItem>();
		check(CategoryItem);

		TSharedPtr<FCategoryDragDropOp> Op = FCategoryDragDropOp::New(StaticCastSharedRef<FOutlinerCategoryItem>(CategoryItem->AsShared()));

		return Op;
	});

	RegisterDragDropOperationParser<FCategoryDragDropOp>([](FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& InDragDropOperation)
	{
		check(InDragDropOperation.IsOfType<FCategoryDragDropOp>());
		const FCategoryDragDropOp& CategoryDragDropOp = static_cast<const FCategoryDragDropOp&>(InDragDropOperation);

		OutPayload.DraggedItems.Add(CategoryDragDropOp.WeakItem.Pin());
	});

	RegisterDragDropOperator<FOutlinerCategoryItem, FOutlinerEntryItem, FCategoryDragDropOp>([](const ISceneOutlinerTreeItem& DragOverTreeItem, TSharedRef<FDragDropOperation> DragDropOperation, FSceneOutlinerDragValidationInfo& OutValidationInfo) -> FReply
	{
		TSharedRef<FCategoryDragDropOp> CategoryDropOp = StaticCastSharedRef<FCategoryDragDropOp>(DragDropOperation);
		const FOutlinerEntryItem* DragOverItem = DragOverTreeItem.CastTo<FOutlinerEntryItem>();
		TSharedPtr<FOutlinerCategoryItem> DropCategoryItem = CategoryDropOp->WeakItem.Pin();
		if (DropCategoryItem.IsValid())
		{
			const FStringView ItemCategoryPath = DragOverItem->GetCategoryPath();
			if (!ItemCategoryPath.IsEmpty() )
			{
				const UUAFRigVMAsset* ThisAsset = DragOverItem->WeakOwner.Get();
				const UUAFRigVMAsset* OtherAsset = DropCategoryItem->WeakOwner.Get();

				const FText EntryCategoryText = FText::FromStringView(ItemCategoryPath);
				const FText DragCategoryText = FText::FromString(DropCategoryItem->CategoryName);

				if (ThisAsset == OtherAsset && ItemCategoryPath != DropCategoryItem->CategoryPath)
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("MakeSubCategory", "Make {0} a sub-category of {1}"), DragCategoryText, EntryCategoryText);
					OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
					return FReply::Handled();
				}
				else if (ThisAsset && ThisAsset != OtherAsset)
				{
					const FText ThisAssetText = FText::FromString(ThisAsset->GetName());
					const FText FormattedMessage = FText::Format(LOCTEXT("MoveAndSubCategory", "Move {0} to {1} and make a sub-category of {2}"), DragCategoryText, ThisAssetText, EntryCategoryText);
					OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);

					return FReply::Handled();
				}
			}
		}

		return FReply::Unhandled();
	},
	[this](ISceneOutlinerTreeItem& DroppedOnTreeItem, TSharedRef<const FDragDropOperation> DragDropOperation)
	{
		TSharedRef<const FCategoryDragDropOp> CategoryDropOp = StaticCastSharedRef<const FCategoryDragDropOp>(DragDropOperation);
		FOutlinerEntryItem* DroppedOnItem = DroppedOnTreeItem.CastTo<FOutlinerEntryItem>();
		TSharedPtr<FOutlinerCategoryItem> DropCategoryItem = CategoryDropOp->WeakItem.Pin();
		if (DropCategoryItem.IsValid())
		{
			const FStringView ItemCategoryPath = DroppedOnItem->GetCategoryPath();
			if (!ItemCategoryPath.IsEmpty())
			{
				const FText EntryCategoryText = FText::FromStringView(ItemCategoryPath);
				const FText DropCategoryText = FText::FromString(DropCategoryItem->ParentCategoryName);
				const FString NewCategoryName = FString(ItemCategoryPath) + TEXT("|") + DropCategoryItem->CategoryName;

				UUAFRigVMAsset* ThisAsset = DroppedOnItem->WeakOwner.Get();
				UUAFRigVMAsset* OtherAsset = DropCategoryItem->WeakOwner.Get();

				if (ThisAsset && ThisAsset == OtherAsset && ItemCategoryPath != DropCategoryItem->CategoryPath)
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("MakeSubCategory", "Make {0} a sub-category of {1}"), DropCategoryText, EntryCategoryText);
					FScopedTransaction Transaction(FormattedMessage);
					DropCategoryItem->Rename( FText::FromString(NewCategoryName));

					return true;
				}
				else if (ThisAsset && OtherAsset && ThisAsset != OtherAsset)
				{
					const FText ThisAssetText = FText::FromString(ThisAsset->GetName());
					const FText FormattedMessage = FText::Format(LOCTEXT("MoveAndSubCategory", "Move {0} to {1} and make a sub-category of {2}"), DropCategoryText, ThisAssetText, EntryCategoryText);

					FScopedTransaction Transaction(FormattedMessage);
					DropCategoryItem->Rename(FText::FromString(NewCategoryName));
					MoveCategoryToAsset(NewCategoryName, OtherAsset, ThisAsset);

					return true;
				}
			}
		}
		return false;
	});

	RegisterDragDropOperator<FOutlinerCategoryItem, FOutlinerAssetItem, FCategoryDragDropOp>([](const ISceneOutlinerTreeItem& DragOverTreeItem, TSharedRef<FDragDropOperation> DragDropOperation, FSceneOutlinerDragValidationInfo& OutValidationInfo) -> FReply
	{
		TSharedRef<FCategoryDragDropOp> CategoryDropOp = StaticCastSharedRef<FCategoryDragDropOp>(DragDropOperation);
		const FOutlinerAssetItem* DragOverItem = DragOverTreeItem.CastTo<FOutlinerAssetItem>();		
		TSharedPtr<FOutlinerCategoryItem> DropCategoryItem = CategoryDropOp->WeakItem.Pin();
		if (DropCategoryItem.IsValid())
		{
			const UUAFRigVMAsset* ThisAsset = DragOverItem->SoftAsset.Get();
			const UUAFRigVMAsset* OtherOwner = DropCategoryItem->WeakOwner.Get();

			if (ThisAsset)
			{
				const FText OtherCategoryText = FText::FromString(DropCategoryItem->GetDisplayString());
				const FText ThisAssetText = FText::FromString(DragOverItem->GetDisplayString());

				if (ThisAsset != OtherOwner)
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("MoveCategory", "Move Category {0} to {1}"), OtherCategoryText, ThisAssetText);
					OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);			
				}
				else
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("CannotMoveCategoryToExitingAsset", "Category {0} is already part of {1}"), OtherCategoryText, ThisAssetText);
					OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FormattedMessage);
				}

				return FReply::Handled();
			}
		}

		return FReply::Unhandled();
	},
	[this](ISceneOutlinerTreeItem& DroppedOnTreeItem, TSharedRef<const FDragDropOperation> DragDropOperation)
	{
			TSharedRef<const FCategoryDragDropOp> CategoryDropOp = StaticCastSharedRef<const FCategoryDragDropOp>(DragDropOperation);
			const FOutlinerAssetItem* DroppedOnItem = DroppedOnTreeItem.CastTo<FOutlinerAssetItem>();		
			TSharedPtr<FOutlinerCategoryItem> DropCategoryItem = CategoryDropOp->WeakItem.Pin();
			if (DropCategoryItem.IsValid())
			{
				UUAFRigVMAsset* ThisAsset = DroppedOnItem->SoftAsset.Get();
				UUAFRigVMAsset* OtherAsset = DropCategoryItem->WeakOwner.Get();

				if (ThisAsset)
				{
					const FText OtherCategoryText = FText::FromString(DropCategoryItem->GetDisplayString());
					const FText ThisAssetText = FText::FromString(DroppedOnItem->GetDisplayString());

					if (ThisAsset != OtherAsset)
					{
						const FText FormattedMessage = FText::Format(LOCTEXT("MoveCategory", "Move Category {0} to {1}"), OtherCategoryText, ThisAssetText);

						FScopedTransaction Transaction(FormattedMessage);
						MoveCategoryToAsset(DropCategoryItem->CategoryPath, OtherAsset, ThisAsset);
					}

					return true;
				}
			}

			return false;
	});

	RegisterDragDropOperator<FOutlinerCategoryItem, FOutlinerCategoryItem, FCategoryDragDropOp>([](const ISceneOutlinerTreeItem& DragOverTreeItem, TSharedRef<FDragDropOperation> DragDropOperation, FSceneOutlinerDragValidationInfo& OutValidationInfo) -> FReply
	{
		TSharedRef<FCategoryDragDropOp> CategoryDropOp = StaticCastSharedRef<FCategoryDragDropOp>(DragDropOperation);			
		const FOutlinerCategoryItem* DragOverItem = DragOverTreeItem.CastTo<FOutlinerCategoryItem>();

		TSharedPtr<FOutlinerCategoryItem> DropCategoryItem = CategoryDropOp->WeakItem.Pin();
		if (DropCategoryItem.IsValid() && (DropCategoryItem.Get() != DragOverItem))
		{
			const UUAFRigVMAsset* ThisOwner = DragOverItem->WeakOwner.Get();
			const UUAFRigVMAsset* OtherOwner = DropCategoryItem->WeakOwner.Get();

			const FText OtherCategoryText = FText::FromString(DropCategoryItem->GetDisplayString());
			const FText ThisParentCategoryText = FText::FromString(DragOverItem->ParentCategoryName);

			if (ThisOwner && ThisOwner == OtherOwner)
			{
				FString OtherCategoryParentPath = DropCategoryItem->CategoryPath;
				OtherCategoryParentPath.RemoveFromEnd(DropCategoryItem->CategoryName);

				FString ThisCategoryParentPath = DragOverItem->CategoryPath;
				ThisCategoryParentPath.RemoveFromEnd(DragOverItem->CategoryName);

				if (!DragOverItem->ParentCategoryName.IsEmpty() && OtherCategoryParentPath != ThisCategoryParentPath)
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("MakeSubCategory", "Make {0} a sub-category of {1}"), OtherCategoryText, ThisParentCategoryText);
					OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
				}
				else 
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("ReorderCategory", "Reorder {0} before {1}"), OtherCategoryText, ThisParentCategoryText);
					OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
				}

				return FReply::Handled();
			}
			else if (ThisOwner)
			{
				const FText ThisAssetText = FText::FromString(DragOverItem->WeakOwner.Get()->GetName());
				const FText FormattedMessage = FText::Format(LOCTEXT("MoveReorderCategory", "Move {0} to {1} and reorder before of {2}"), OtherCategoryText, ThisAssetText, ThisParentCategoryText);

				OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);

				return FReply::Handled();
			}
		}

		return FReply::Unhandled();
	},
	[this](ISceneOutlinerTreeItem& DroppedOnTreeItem, TSharedRef<const FDragDropOperation> DragDropOperation)
	{
		TSharedRef<const FCategoryDragDropOp> CategoryDropOp = StaticCastSharedRef<const FCategoryDragDropOp>(DragDropOperation);
		FOutlinerCategoryItem* DroppedOnItem = DroppedOnTreeItem.CastTo<FOutlinerCategoryItem>();

		TSharedPtr<FOutlinerCategoryItem> DropCategoryItem = CategoryDropOp->WeakItem.Pin();
		if (DropCategoryItem.IsValid())
		{
			const FString ThisCategoryName = DroppedOnItem->CategoryName;
			const FString OtherCategoryName = DropCategoryItem->CategoryName;

			UUAFRigVMAsset* ThisAsset = DroppedOnItem->WeakOwner.Get();
			UUAFRigVMAsset* OtherAsset = DropCategoryItem->WeakOwner.Get();

			const FText OtherCategoryText = FText::FromString(DropCategoryItem->GetDisplayString());
			const FText ThisCategoryText = FText::FromString(DroppedOnItem->GetDisplayString());	

			if (ThisAsset && ThisAsset == OtherAsset)
			{ 
				FString OtherCategoryParentPath = DropCategoryItem->CategoryPath;
				OtherCategoryParentPath.RemoveFromEnd(DropCategoryItem->CategoryName);

				FString ThisCategoryParentPath = DroppedOnItem->CategoryPath;
				ThisCategoryParentPath.RemoveFromEnd(DroppedOnItem->CategoryName);

				const FString NewCategoryName = ThisCategoryParentPath + DropCategoryItem->CategoryName;

				if (!DroppedOnItem->ParentCategoryName.IsEmpty() && OtherCategoryParentPath != ThisCategoryParentPath)
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("MakeSubCategory", "Make {0} a sub-category of {1}"), OtherCategoryText, ThisCategoryText);
					FScopedTransaction Transaction(FormattedMessage);

					ReorderCategoryItem(DropCategoryItem, StaticCastSharedRef<FOutlinerCategoryItem>(DroppedOnItem->AsShared()));

					DropCategoryItem->Rename(FText::FromString(NewCategoryName));
				}
				else 
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("ReorderCategory", "Reorder {0} before {1}"), OtherCategoryText, ThisCategoryText);
					FScopedTransaction Transaction(FormattedMessage);
					ReorderCategoryItem(DropCategoryItem, StaticCastSharedRef<FOutlinerCategoryItem>(DroppedOnItem->AsShared()));
				}

				return true;
			}
			else if (ThisAsset && OtherAsset)
			{
				const FText ThisAssetText = FText::FromString(ThisAsset->GetName());
				const FText FormattedMessage = FText::Format(LOCTEXT("MoveReorderCategory", "Move {0} to {1} and reorder before of {2}"), OtherCategoryText, ThisAssetText, ThisCategoryText);

				FScopedTransaction Transaction(FormattedMessage);

				MoveCategoryToAsset(DropCategoryItem->CategoryPath, OtherAsset, ThisAsset);
				ReorderCategory(DropCategoryItem->CategoryPath, DroppedOnItem->CategoryPath, ThisAsset);

				return true;
			}
		}

		return false;
	});
}

FName FCommonOutlinerMode::ContextToolMenuName("UAF.CommonOutliner.ItemContextMenu");
FName FCommonOutlinerMode::ContextToolAssetSectionName("Asset");
FName FCommonOutlinerMode::ContextToolEntrySectionName("Entry");

FName FCommonOutlinerMode::AddEntryContextMenuName("UAF.Outliner.AddEntryContextMenu");

TSharedPtr<SWidget> FCommonOutlinerMode::CreateContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	{
		UCommonOutlinerItemMenuContext* MenuContext = NewObject<UCommonOutlinerItemMenuContext>();
		MenuContext->WeakWorkspaceEditor = WeakWorkspaceEditor;
		MenuContext->WeakOutliner = StaticCastSharedRef<SCommonOutliner>(SceneOutliner->AsShared());
		MenuContext->WeakCommandList = CommandList;
		TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();
		for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
		{
			if (const FOutlinerAssetItem* AssetItem = Item->CastTo<FOutlinerAssetItem>())
			{
				UUAFRigVMAsset* Asset = AssetItem->SoftAsset.Get();
				if(Asset == nullptr)
				{
					continue;
				}

				UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
				if(EditorData == nullptr)
				{
					continue;
				}

				MenuContext->WeakEditorDatas.Add(EditorData);
			}
		}

		FToolMenuContext Context;
		Context.AddObject(MenuContext);
		if (WeakWorkspaceEditor.IsValid())
		{
			WeakWorkspaceEditor.Pin()->InitToolMenuContext(Context);
		}
		InitToolMenuContext(Context);
		return ToolMenus->GenerateWidget(ContextToolMenuName, Context);
	}
}

void FCommonOutlinerMode::BindCommands(const TSharedRef<FUICommandList>& OutCommandList)
{
	CommandList->MapAction( 
		FOutlinerCommands::Get().SaveAsset, 
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::SaveAsset),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanSaveAsset),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanSaveAsset)
	);

	CommandList->MapAction( 
		FGenericCommands::Get().Rename, 
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::Rename),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanRename),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanRename)
	);

	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::Delete),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanDelete),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanDelete)	
		);

	CommandList->MapAction(
		FGenericCommands::Get().Copy, 
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::Copy),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanCopy),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanCopy)
	);

	CommandList->MapAction( 
		FGenericCommands::Get().Cut, 
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::Cut),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanCut),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanCut)
	);

	CommandList->MapAction( 
		FGenericCommands::Get().Paste, 
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::Paste),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanPaste),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanPaste)
	);

	CommandList->MapAction(
		FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::Duplicate),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanDuplicate),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanDuplicate)
	);

	CommandList->MapAction(
		FOutlinerCommands::Get().FindReferences,
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::FindReferences, ESearchScope::Global),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanFindReferences),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::IsFindReferencesVisible, ESearchScope::Global)
	);

	CommandList->MapAction(
		FOutlinerCommands::Get().FindReferencesInWorkspace,
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::FindReferences, ESearchScope::Workspace),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanFindReferences),
		FIsActionChecked(),
	FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::IsFindReferencesVisible, ESearchScope::Workspace)
	);

	CommandList->MapAction(
		FOutlinerCommands::Get().FindReferencesInAsset,
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::FindReferences, ESearchScope::Asset),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanFindReferences),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::IsFindReferencesVisible, ESearchScope::Asset)
	);

	CommandList->MapAction(
		FOutlinerCommands::Get().MakeEntryPublic,
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::SetEntryExportAccessSpecifiersOnSelection, EAnimNextExportAccessSpecifier::Public),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanModifyEntryExport),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanModifyEntryExport)
	);

	CommandList->MapAction(
		FOutlinerCommands::Get().MakeEntryPrivate,
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::SetEntryExportAccessSpecifiersOnSelection, EAnimNextExportAccessSpecifier::Private),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanModifyEntryExport),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanModifyEntryExport)
	);
	
	CommandList->MapAction(
		FOutlinerCommands::Get().ExpandEntries,
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::SetExpansionOnSelectedEntries, true),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanExpandEntries),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanExpandEntries)
	);
	
	CommandList->MapAction(
		FOutlinerCommands::Get().CollapseEntries,
		FExecuteAction::CreateRaw(this, &FCommonOutlinerMode::SetExpansionOnSelectedEntries, false),
		FCanExecuteAction::CreateRaw(this, &FCommonOutlinerMode::CanCollapseEntries),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FCommonOutlinerMode::CanCollapseEntries)
	);

	RegisterToolMenus();
}

void FCommonOutlinerMode::OnItemClicked(FSceneOutlinerTreeItemPtr Item)
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	HandleItemSelection(Selection);
}

void FCommonOutlinerMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType,
	const FSceneOutlinerItemSelection& Selection)
{
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
	{
		SharedWorkspaceEditor->SetGlobalSelection(SceneOutliner->AsShared(), UE::Workspace::FOnClearGlobalSelection::CreateRaw(this, &FCommonOutlinerMode::ResetOutlinerSelection));
	}

	HandleItemSelection(Selection);
}

FReply FCommonOutlinerMode::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedPtr<FDragDropOperation> FCommonOutlinerMode::CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const
{
	// TODO support multi-item drag-drop
	if (InTreeItems.Num() == 1)
	{
		const FSceneOutlinerTreeItemPtr& Item = InTreeItems[0];
		const FDragDropOpConstructor* FuncPtr = ItemTypeToDragDropOpMap.Find(Item->GetType());

		// Check for derived item type entries
		if (!FuncPtr)
		{
			for (const auto& Pair : ItemTypeToDragDropOpMap)
			{
				if (Item->GetType().IsA(Pair.Key))
				{
					FuncPtr = &Pair.Value;
				}
			}
		}

		if (FuncPtr)
		{
			return (*FuncPtr)(Item->AsShared());
		}

	}

	return nullptr;
}

bool FCommonOutlinerMode::ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const
{
	TArray<FString> KeyArray;
	DragDropOperationTypeToParseFunc.GenerateKeyArray(KeyArray);

	for (const auto& Pair : DragDropOperationTypeToParseFunc)
	{
		if (Operation.IsOfTypeImpl(Pair.Key))
		{
			Pair.Value(OutPayload, Operation);
			return true;
		}
	}

	return ISceneOutlinerMode::ParseDragDrop(OutPayload, Operation);
}

FSceneOutlinerDragValidationInfo FCommonOutlinerMode::ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const
{
	FSceneOutlinerDragValidationInfo OutInfo = FSceneOutlinerDragValidationInfo::Invalid();

	if (Payload.DraggedItems.Num() == 1 && Payload.DraggedItems[0].IsValid() && DropTarget.IsValid())
	{
		const FString OpTypeString = Payload.SourceOperation.GetTypeIdString().GetData();
		FDragDropOperation* DragDropOp = const_cast<FDragDropOperation*>(&Payload.SourceOperation);

		// Target/source item types + drag-drop operation type
		const FDragDropOperator Id({ DropTarget.GetType(), Payload.DraggedItems[0].Pin()->GetType(), OpTypeString});
		if (const FDragDropOperator* Operator = DragDropOperators.FindByKey(Id))
		{
			if (Operator->DragOverFunc(DropTarget, DragDropOp->AsShared(), OutInfo).IsEventHandled())
			{
				return OutInfo;
			}
		}
	}

	return ISceneOutlinerMode::ValidateDrop(DropTarget, Payload);
}

void FCommonOutlinerMode::OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const
{
	if (ValidationInfo.IsValid())
	{
		if (Payload.DraggedItems.Num() == 1)
		{
			const FString OpTypeString = Payload.SourceOperation.GetTypeIdString().GetData();

			const FDragDropOperator Id({ DropTarget.GetType(), Payload.DraggedItems[0].Pin()->GetType(), OpTypeString});
			if (const FDragDropOperator* Operator = DragDropOperators.FindByKey(Id))
			{
				Operator->DragDropFunc(DropTarget, Payload.SourceOperation.AsShared());
			}
		}
	}
}

FReply FCommonOutlinerMode::OnDragOverItem(const FDragDropEvent& Event, const ISceneOutlinerTreeItem& Item) const
{
	TSharedPtr<FDragDropOperation> DragDropOp = Event.GetOperation();
	if (DragDropOp.IsValid())
	{
		FSceneOutlinerDragDropPayload Payload;
		ParseDragDrop(Payload, *DragDropOp.Get());

		if (Payload.DraggedItems.Num() == 1 && Payload.DraggedItems[0].Pin() != Item.AsShared())
		{
			const FString OpTypeString = Payload.SourceOperation.GetTypeIdString().GetData();
			FSceneOutlinerDragValidationInfo OutInfo = FSceneOutlinerDragValidationInfo::Invalid();

			const FDragDropOperator Id({ Item.GetType(), Payload.DraggedItems[0].Pin()->GetType(), OpTypeString});
			if (const FDragDropOperator* Operator = DragDropOperators.FindByKey(Id))
			{
				const FReply Reply = Operator->DragOverFunc(Item, DragDropOp->AsShared(), OutInfo);
				if (Reply.IsEventHandled())
				{
					return Reply;
				}
			}
		}
	}

	return ISceneOutlinerMode::OnDragOverItem(Event, Item);
}

void FCommonOutlinerMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	if (Item.IsValid())
	{
		if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin())
		{
			if(FOutlinerAssetItem* AssetItem = Item->CastTo<FOutlinerAssetItem>())
			{
				if (UUAFRigVMAsset* Asset = AssetItem->WeakOwner.Get())
				{
					const Workspace::FWorkspaceDocument& Document = WorkspaceEditor->GetFocusedWorkspaceDocument();	
					const FWorkspaceOutlinerItemExport RelativeAssetExport = Document.Export.GetRelativeAssetExport(Asset);
					const FWorkspaceOutlinerItemExport AssetExport = UncookedOnly::FUtils::MakeAssetReferenceExport(Asset, RelativeAssetExport);

					if (AssetExport.GetFirstAssetPath().IsValid() && RelativeAssetExport != AssetExport)
					{
						WorkspaceEditor->OpenExports({AssetExport});
						return;
					}
				}
			}
		}
	}

	ISceneOutlinerMode::OnItemDoubleClick(Item);
}

void FCommonOutlinerMode::SetEntryExportAccessSpecifiersOnSelection(EAnimNextExportAccessSpecifier AnimNextExportAccessSpecifier) const
{
	if (!SceneOutliner)
	{
		return;
	}

	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() > 0)
	{
		TArray<UUAFRigVMAsset*> ObjectsToCompile;
		TArray<IUAFRigVMExportInterface*> ExportsToChange;
		Selection.ForEachItem<FOutlinerEntryItem>([&ObjectsToCompile](const FOutlinerEntryItem& SelectedItem)
		{
			if (UUAFRigVMAsset* OuterAsset = SelectedItem.WeakOwner.Get())
			{
				if (OuterAsset->GetClass() && OuterAsset->GetClass() != UUAFSharedVariables::StaticClass())
				{
					ObjectsToCompile.AddUnique(OuterAsset);
				}
			}
		});

		UncookedOnly::FCompilationScope CompileScope(LOCTEXT("ModifiedAssets_SetAccessSpecifierOnSelection", "Set Access Specifier on Selection"), ObjectsToCompile);

		FScopedTransaction Transaction(LOCTEXT("SetAccessSpecifierSelectionTransaction", "Set Access Specifier on Selection"));

		Selection.ForEachItem<FOutlinerEntryItem>([AnimNextExportAccessSpecifier](const FOutlinerEntryItem& SelectedItem)
		{
			SelectedItem.SetAccessSpecifier(AnimNextExportAccessSpecifier);
		});
	}
}

bool FCommonOutlinerMode::CanModifyEntryExport() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num())
	{
		bool bCanModify = false;
		Selection.ForEachItem<FOutlinerEntryItem>([&bCanModify](const FOutlinerEntryItem& EntryItem)
		{
			bCanModify |= EntryItem.CanSetAccessSpecifier();
		});

		return bCanModify;
	}

	return false;
}
	
	
void FCommonOutlinerMode::SetExpansionOnSelectedEntries(bool bExpand) const
{
	if (SceneOutliner)
	{
		const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item :  Selection.SelectedItems)
		{
			if (const TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
			{
				if (ItemPtr->IsA<FOutlinerCategoryItem>() || ItemPtr->IsA<FOutlinerAssetItem>())
				{
					SceneOutliner->SetItemExpansion(ItemPtr, bExpand);
				}
			}
		}
	}
}
	
bool FCommonOutlinerMode::CanExpandEntries() const
{
	if (SceneOutliner)
	{
		const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : Selection.SelectedItems)
		{
			if (const TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
			{
				if ((ItemPtr->IsA<FOutlinerCategoryItem>() || ItemPtr->IsA<FOutlinerAssetItem>()) 
					&& !SceneOutliner->IsItemExpanded(ItemPtr))
				{
					return true;
				}
			}
		}
	}
	
	return false;
}
	
bool FCommonOutlinerMode::CanCollapseEntries() const
{
	if (SceneOutliner)
	{
		const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Item : Selection.SelectedItems)
		{
			if (const TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
			{
				if ((ItemPtr->IsA<FOutlinerCategoryItem>() || ItemPtr->IsA<FOutlinerAssetItem>()) 
					&& SceneOutliner->IsItemExpanded(ItemPtr))
				{
					return true;
				}
			}
		}
	}
	
	return false;
}
	
SCommonOutliner* FCommonOutlinerMode::GetOutliner() const
{
	return static_cast<SCommonOutliner*>(SceneOutliner);
}

void FCommonOutlinerMode::SetHighlightedItem(FSceneOutlinerTreeItemID InID) const
{
	if (SCommonOutliner* Outliner = static_cast<SCommonOutliner*>(SceneOutliner))
	{
		if (FSceneOutlinerTreeItemPtr ItemPtr = Outliner->GetTreeItem(InID))
		{
			Outliner->FrameItem(InID);
			Outliner->SetHighlightedItem(ItemPtr);
		}
	}
}

void FCommonOutlinerMode::ClearHighlightedItem(FSceneOutlinerTreeItemID InID) const
{
	if (SCommonOutliner* Outliner = static_cast<SCommonOutliner*>(SceneOutliner))
	{
		if (FSceneOutlinerTreeItemPtr ItemPtr = Outliner->GetTreeItem(InID))
		{
			Outliner->ClearHighlightedItem(ItemPtr);
		}
	}
}

void FCommonOutlinerMode::InitToolMenuContext(FToolMenuContext& InOutContext) const
{
	UCommonOutlinerItemMenuContext* MenuContext = NewObject<UCommonOutlinerItemMenuContext>();
	MenuContext->WeakWorkspaceEditor = WeakWorkspaceEditor;
	MenuContext->WeakOutliner = StaticCastSharedRef<SCommonOutliner>(SceneOutliner->AsShared());
	MenuContext->WeakCommandList = CommandList;
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();
	for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
	{
		if (const FOutlinerAssetItem* AssetItem = Item->CastTo<FOutlinerAssetItem>())
		{
			UUAFRigVMAsset* Asset = AssetItem->SoftAsset.Get();
			if(Asset == nullptr)
			{
				continue;
			}

			UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
			if(EditorData == nullptr)
			{
				continue;
			}

			MenuContext->WeakEditorDatas.Add(EditorData);
		}
	}

	InOutContext.AddObject(MenuContext);
}

void FCommonOutlinerMode::RegisterToolMenus()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if(!ToolMenus->IsMenuRegistered(ContextToolMenuName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);
		if (UToolMenu* Menu = ToolMenus->RegisterMenu(ContextToolMenuName))
		{
			Menu->AddDynamicSection("AssetDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				const UAssetEditorToolkitMenuContext* EditorContext = InMenu->FindContext<UAssetEditorToolkitMenuContext>();
				const UCommonOutlinerItemMenuContext* MenuContext = InMenu->FindContext<UCommonOutlinerItemMenuContext>();
				if(EditorContext == nullptr || MenuContext == nullptr)
				{
					return;
				}

				auto AddCommandlistMenuEntry = [MenuContext](auto& Section, auto& Command, auto... Parameters)
				{
					if (TSharedPtr<FUICommandList> SharedCommandList = MenuContext->WeakCommandList.Pin())
					{
						if (SharedCommandList->GetVisibility(Command.ToSharedRef()) == EVisibility::Visible)
						{
							Section.AddMenuEntryWithCommandList(Command, SharedCommandList, Parameters...);
						}
					}
				};

				FToolMenuSection& AssetSection = InMenu->AddSection(ContextToolAssetSectionName, LOCTEXT("AssetSectionLabel", "Asset"));
				{
					AddCommandlistMenuEntry(AssetSection, FOutlinerCommands::Get().SaveAsset, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AssetEditor.SaveAsset"));
					AddCommandlistMenuEntry(AssetSection, FGenericCommands::Get().Paste);
				}

				/** Entry specific actions */
				FToolMenuSection& EntriesSection = InMenu->AddSection(ContextToolEntrySectionName, LOCTEXT("EntriesSectionLabel", "Entries"));
				{
					EntriesSection.AddMenuEntryWithCommandList(FGenericCommands::Get().Rename, MenuContext->WeakCommandList.Pin());
					EntriesSection.AddMenuEntryWithCommandList(FGenericCommands::Get().Delete, MenuContext->WeakCommandList.Pin());
					EntriesSection.AddMenuEntryWithCommandList(FGenericCommands::Get().Duplicate, MenuContext->WeakCommandList.Pin());

					EntriesSection.AddMenuEntryWithCommandList(FGenericCommands::Get().Copy, MenuContext->WeakCommandList.Pin());
					EntriesSection.AddMenuEntryWithCommandList(FGenericCommands::Get().Cut, MenuContext->WeakCommandList.Pin());

					EntriesSection.AddMenuEntryWithCommandList(FOutlinerCommands::Get().ExpandEntries, MenuContext->WeakCommandList.Pin());
					EntriesSection.AddMenuEntryWithCommandList(FOutlinerCommands::Get().CollapseEntries, MenuContext->WeakCommandList.Pin());
					
					EntriesSection.AddSubMenu("FindReferencesMenu", LOCTEXT("FindReferencesMenuLabel", "Find References..."), LOCTEXT("FindReferencesMenuToolTip", "Find references to selected variables by different means."),
						FNewToolMenuDelegate::CreateLambda(
							[MenuContext](UToolMenu* InSubmenu)
								{
									InSubmenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntryWithCommandList(FOutlinerCommands::Get().FindReferences, MenuContext->WeakCommandList.Pin(), TAttribute<FText>(), TAttribute<FText>(), UAnimNextAssetFindReplaceVariables::GetIconFromSearchScope(ESearchScope::Global)));

									InSubmenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntryWithCommandList(FOutlinerCommands::Get().FindReferencesInWorkspace, MenuContext->WeakCommandList.Pin(), TAttribute<FText>(), TAttribute<FText>(),
										UAnimNextAssetFindReplaceVariables::GetIconFromSearchScope(ESearchScope::Workspace)));

									InSubmenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitMenuEntryWithCommandList(FOutlinerCommands::Get().FindReferencesInAsset, MenuContext->WeakCommandList.Pin(), TAttribute<FText>(), TAttribute<FText>(),
										UAnimNextAssetFindReplaceVariables::GetIconFromSearchScope(ESearchScope::Asset)));
								}),
								FUIAction(FExecuteAction(), FCanExecuteAction(), FGetActionCheckState(), FIsActionButtonVisible::CreateLambda([MenuContext]() -> bool
								{
									if (TSharedPtr<SSceneOutliner> SharedOutliner = MenuContext->WeakOutliner.Pin())
									{
										if (const FCommonOutlinerMode* Mode = static_cast<const FCommonOutlinerMode*>(SharedOutliner->GetMode()))
										{
											return Mode->CanFindReferences();
										}
									}

									return false;
								})),
								EUserInterfaceActionType::Button,
								false,
								FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Search"))
							);

					if (TSharedPtr<SSceneOutliner> SharedOutliner = MenuContext->WeakOutliner.Pin())
					{
						TArray<FOutlinerEntryItem*> EntryItems;
						SharedOutliner->GetSelection().Get(EntryItems);

						// Check which export access related context menu entries we need to populate
						if (EntryItems.Num() > 0)
						{
							bool bHasPublicVariables = false;
							bool bHasPrivateVariables = false;
							for (const FOutlinerEntryItem* EntryItem : EntryItems)
							{
								if (EntryItem->GetAccessSpecifier() == EAnimNextExportAccessSpecifier::Public)
								{
									bHasPublicVariables = true;
								}
								else if (EntryItem->GetAccessSpecifier() == EAnimNextExportAccessSpecifier::Private)
								{
									bHasPrivateVariables = true;
								}

								if (bHasPrivateVariables && bHasPublicVariables)
								{
									break;
								}
							}

							if (bHasPrivateVariables)
							{
								EntriesSection.AddMenuEntryWithCommandList(FOutlinerCommands::Get().MakeEntryPublic, MenuContext->WeakCommandList.Pin(), LOCTEXT("MakePublicMenuLabel", "Expose as Public"), TAttribute<FText>(), FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Level.VisibleIcon16x"));
							}

							if (bHasPublicVariables)
							{
								EntriesSection.AddMenuEntryWithCommandList(FOutlinerCommands::Get().MakeEntryPrivate, MenuContext->WeakCommandList.Pin(), LOCTEXT("MakePrivateMenuLabel", "Make Private"), TAttribute<FText>(), FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Level.NotVisibleHighlightIcon16x"));
							}
						}
					}
				}

				if (TSharedPtr<SSceneOutliner> Outliner = MenuContext->WeakOutliner.Pin())
				{
					Outliner->AddSourceControlMenuOptions(InMenu);
				}

			}));
		}

		if (UToolMenu* Menu = ToolMenus->RegisterMenu(AddEntryContextMenuName))
		{
			Menu->AddDynamicSection("AssetDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				const UCommonOutlinerItemMenuContext* MenuContext = InMenu->FindContext<UCommonOutlinerItemMenuContext>();
				if(MenuContext == nullptr)
				{
					return;
				}
			}));
		}
	}
}

void FCommonOutlinerMode::SaveAsset() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	Selection.ForEachItem<FOutlinerAssetItem>([](const FOutlinerAssetItem& AssetItem)
	{
		if (const UUAFRigVMAsset* Asset = AssetItem.SoftAsset.Get())
		{
			FEditorFileUtils::PromptForCheckoutAndSave({Asset->GetPackage()}, false, /*bPromptToSave=*/ false);
		}
	});	
}

bool FCommonOutlinerMode::CanSaveAsset() const
{ 
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	return Selection.SelectedItems.ContainsByPredicate([](const TWeakPtr<ISceneOutlinerTreeItem>& WeakItem)
	{
		if (WeakItem.IsValid())
		{
			if (const FOutlinerAssetItem* AssetItem = WeakItem.Pin()->CastTo<FOutlinerAssetItem>())
			{
				if (const UUAFRigVMAsset* Asset = AssetItem->SoftAsset.Get())
				{
					return Asset->GetPackage()->IsDirty();
				}
			}
		}

		return false;
	});
}

void FCommonOutlinerMode::GetAssets(TArray<TSoftObjectPtr<UUAFRigVMAsset>>& OutAssets) const
{
	OutAssets.Append(static_cast<SCommonOutliner*>(SceneOutliner)->Assets);
}

void FCommonOutlinerMode::ResetOutlinerSelection() const
{
	SceneOutliner->ClearSelection();
}

int32 FCommonOutlinerMode::GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const
{
	const FOutlinerItem* OutlinerItem = static_cast<const FOutlinerItem*>(&Item);
	return OutlinerItem->SortValue;
}

}

#undef LOCTEXT_NAMESPACE // "FCommonOutlinerMode"
