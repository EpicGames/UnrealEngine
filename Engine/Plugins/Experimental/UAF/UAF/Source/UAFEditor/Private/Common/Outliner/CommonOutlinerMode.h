// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAnimNextRigVMExportInterface.h"
#include "ISceneOutlinerMode.h"
#include "Common/AnimNextAssetFindReplaceVariables.h"

class UUAFRigVMAssetEditorData;
class UUAFRigVMAsset;

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{
class SCommonOutliner;

struct FOutlinerEntryItem;
struct FOutlinerCategoryItem;

class FCommonOutlinerMode : public ISceneOutlinerMode
{
public:
	FCommonOutlinerMode(SSceneOutliner* InOutliner, const TSharedRef<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor);

	// Begin ISceneOutlinerMode overrides
	virtual TSharedPtr<SWidget> CreateContextMenu() override;
	virtual void BindCommands(const TSharedRef<FUICommandList>& OutCommandList) override;
	virtual bool CanRename() const override { return false; }
	virtual bool CanDelete() const override { return false; }
	virtual bool CanCopy() const override { return false; }
	virtual bool CanCut() const override { return false; }
	virtual bool CanPaste() const override { return false; }
	virtual void OnItemClicked(FSceneOutlinerTreeItemPtr Item) override;
	virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
	virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) override;
	virtual bool CanCustomizeToolbar() const override { return true; }
	virtual ESelectionMode::Type GetSelectionMode() const override { return ESelectionMode::Multi; }
	virtual bool CanSupportDragAndDrop() const override { return true; }
	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const override;
	virtual bool ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const override;
	virtual FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const override;
	virtual void OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const override;
	virtual FReply OnDragOverItem(const FDragDropEvent& Event, const ISceneOutlinerTreeItem& Item) const override;
	virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override; 
	virtual int32 GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const override;
	// End ISceneOutlinerMode overrides
	
	virtual void InitToolMenuContext(FToolMenuContext& InOutContext) const;
		
	void SetHighlightedItem(FSceneOutlinerTreeItemID InID) const;
	void ClearHighlightedItem(FSceneOutlinerTreeItemID InID) const;

	void SaveAsset() const;
	bool CanSaveAsset() const;
	void GetAssets(TArray<TSoftObjectPtr<UUAFRigVMAsset>>& OutAssets) const;
	SCommonOutliner* GetOutliner() const;
protected:
	virtual void HandleItemSelection(const FSceneOutlinerItemSelection& Selection) const {};
	virtual void Rename() const {}
	virtual void Delete() const {}
	virtual void Copy() const {}
	virtual void Cut() const {}
	virtual void Paste() const {}
	virtual bool CanDuplicate() const { return false; }
	virtual void Duplicate() const {}
	virtual void RegisterToolMenus();
	virtual void AddCategory(TWeakObjectPtr<UUAFRigVMAssetEditorData> InWeakEditorData) const {}
	virtual bool CanAddCategory(TWeakObjectPtr<UUAFRigVMAssetEditorData> InWeakEditorData) const { return true; }
	virtual void ReorderEntryItem(TSharedPtr<FOutlinerEntryItem> EntryItem, TSharedPtr<FOutlinerEntryItem> BeforeItem) {}
	virtual void ReorderCategoryItem(TSharedPtr<FOutlinerCategoryItem> EntryItem, TSharedPtr<FOutlinerCategoryItem> BeforeItem) const {}
	virtual void ReorderCategory(const FString& CategoryPath, const FString& BeforeCategoryPath, UUAFRigVMAsset* Asset) const {}
	virtual void MoveCategoryToAsset(const FString& CategoryPath, UUAFRigVMAsset* FromAsset, UUAFRigVMAsset* ToAsset) const {}
	
	virtual void FindReferences(ESearchScope InSearchScope) const {}
	virtual bool CanFindReferences() const { return false; }
	virtual bool IsFindReferencesVisible(ESearchScope InSearchScope) const { return false; }

	void ResetOutlinerSelection() const;

	void SetEntryExportAccessSpecifiersOnSelection(EAnimNextExportAccessSpecifier AnimNextExportAccessSpecifier) const;
	bool CanModifyEntryExport() const;
	
	void SetExpansionOnSelectedEntries(bool bExpand) const;
	bool CanExpandEntries() const;
	bool CanCollapseEntries() const;
	
protected:
	typedef TFunction<TSharedPtr<FDragDropOperation>(const FSceneOutlinerTreeItemPtr&)> FDragDropOpConstructor;
	TMap<FSceneOutlinerTreeItemType, FDragDropOpConstructor> ItemTypeToDragDropOpMap;

	template<typename TreeItem>
	void RegisterDragDropOperationConstructor(FDragDropOpConstructor&& InFunc)
	{
		ItemTypeToDragDropOpMap.Add(TreeItem::Type, InFunc);
	}

	typedef TFunction<void(FSceneOutlinerDragDropPayload&, const FDragDropOperation&)> FDragDropOpParser;
	TMap<FString, FDragDropOpParser> DragDropOperationTypeToParseFunc;

	template<typename DragDropOpType>
	void RegisterDragDropOperationParser(FDragDropOpParser&& InFunc)
	{
		DragDropOperationTypeToParseFunc.Add(DragDropOpType::GetTypeId(), InFunc);
	}

	typedef TFunction<bool(ISceneOutlinerTreeItem&, TSharedRef<const FDragDropOperation>)> FHandleDragDropFunc;
	typedef TFunction<FReply(const ISceneOutlinerTreeItem&, TSharedRef<FDragDropOperation>, FSceneOutlinerDragValidationInfo& OutInfo)> FHandleDragOverFunc;
	struct FDragDropOperator 
	{
		bool operator==(const FDragDropOperator& Other) const
		{
			return (SourceType.IsA(Other.SourceType) || Other.SourceType.IsA(SourceType))
				&& (TargetType.IsA(Other.TargetType) || Other.TargetType.IsA(TargetType)) 
				&& OpType == Other.OpType;
		}

		FSceneOutlinerTreeItemType TargetType;
		FSceneOutlinerTreeItemType SourceType;
		FString OpType;
		FHandleDragOverFunc DragOverFunc;
		FHandleDragDropFunc DragDropFunc;
	};

	TArray<FDragDropOperator> DragDropOperators;

	template<typename SourceTreeItemType, typename TargetTreeItemType, typename DragDropOperationType>
	void RegisterDragDropOperator(FHandleDragOverFunc&& InDragOverFunc, FHandleDragDropFunc&& InDropFunc)
	{
		const FDragDropOperator Operator (TargetTreeItemType::Type, SourceTreeItemType::Type, DragDropOperationType::GetTypeId(), InDragOverFunc, InDropFunc);

		check(!DragDropOperators.Contains(Operator));
		DragDropOperators.Add(Operator);
	}
protected:
	TWeakPtr<UE::Workspace::IWorkspaceEditor> WeakWorkspaceEditor;
	TSharedPtr<FUICommandList> CommandList;

public:
	static FName ContextToolMenuName;
	static FName ContextToolAssetSectionName;
	static FName ContextToolEntrySectionName;
	static FName AddEntryContextMenuName;
};

}
