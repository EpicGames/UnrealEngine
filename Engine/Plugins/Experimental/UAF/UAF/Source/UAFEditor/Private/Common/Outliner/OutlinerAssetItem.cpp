// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerAssetItem.h"

#include "ISceneOutliner.h"
#include "Styling/SlateColor.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "UAFStyle.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "IAssetTools.h"
#include "ScopedTransaction.h"
#include "UncookedOnlyUtils.h"
#include "WorkspaceDragDropOperation.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StarshipCoreStyle.h"
#include "UObject/Package.h"
#include "Common/Outliner/OutlinerItemMenuContexts.h"
#include "Variables/SVariablesView.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Layout/SSpacer.h"

#include "CommonOutlinerMode.h"

#define LOCTEXT_NAMESPACE "OutlinerAssetItem"

namespace UE::UAF::Editor
{
class FCommonOutlinerMode;

const FSceneOutlinerTreeItemType FOutlinerAssetItem::Type(&FOutlinerItem::Type);

class SOutlinerAssetLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SOutlinerAssetLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FOutlinerAssetItem& InTreeItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
		TreeItem = StaticCastSharedRef<FOutlinerAssetItem>(InTreeItem.AsShared());

		TWeakPtr<SOutlinerAssetLabel> WeakLabel = StaticCastSharedRef<SOutlinerAssetLabel>(AsShared());
		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 2.0f)
			[
				SNew(SImage)
				.Image(this, &SOutlinerAssetLabel::GetAssetIcon)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 2.0f)
			[
				SAssignNew(TextBlock, SInlineEditableTextBlock)
				.Font(FStyleFonts::Get().NormalBold)
				.Text(this, &SOutlinerAssetLabel::GetDisplayText)
				.HighlightText(SceneOutliner.GetFilterHighlightText())
				.ColorAndOpacity(this, &SOutlinerAssetLabel::GetForegroundColor)
				.OnTextCommitted(this, &SOutlinerAssetLabel::OnTextCommited)
				.OnVerifyTextChanged(this, &SOutlinerAssetLabel::OnVerifyTextChanged)
				.IsReadOnly(this, &SOutlinerAssetLabel::IsReadOnly)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f, 2.0f, 3.0f)
			.AutoWidth()
			[
				SNew(SImage)
				.Visibility(this, &SOutlinerAssetLabel::GetDirtyImageVisibility)
				.ToolTipText(this, &SOutlinerAssetLabel::GetDirtyTooltipText)
				.Image(this, &SOutlinerAssetLabel::GetDirtyImageBrush)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SBox)
				.Visibility(this, &SOutlinerAssetLabel::GetAddVariableButtonVisibility)
				[
					SNew(SComboButton)
					.ContentPadding(FMargin(0, 0))
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.HasDownArrow(false)
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					.OnGetMenuContent_Lambda([WeakOutliner = SceneOutliner.AsShared().ToWeakPtr(), WeakItem = StaticCastSharedRef<FOutlinerAssetItem>(InTreeItem.AsShared()).ToWeakPtr()]() -> TSharedRef<SWidget>
					{
						TSharedPtr<SSceneOutliner> SharedOutliner = StaticCastSharedPtr<SSceneOutliner>(WeakOutliner.Pin());
						if (SharedOutliner.IsValid())
						{
							if (const FCommonOutlinerMode* OutlinerMode = static_cast<const FCommonOutlinerMode*>(SharedOutliner->GetMode()))
							{
								FToolMenuContext Context;
								OutlinerMode->InitToolMenuContext(Context);

								if (WeakItem.IsValid())
								{
									if (UCommonOutlinerItemMenuContext* MenuContext = Context.FindContext<UCommonOutlinerItemMenuContext>())
									{
										MenuContext->WeakEditorDatas.AddUnique(UncookedOnly::FUtils::GetEditorData(WeakItem.Pin()->SoftAsset.Get()));
									}
								}
								

								return UToolMenus::Get()->GenerateWidget(FCommonOutlinerMode::AddEntryContextMenuName, Context);
							}
						}

						return SNullWidget::NullWidget;
					})
				]
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f)
			.AutoWidth()
			[
				SAssignNew(ImplementedSharedVariablesHBox, SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand)
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SThrobber)
				.Visibility(this, &SOutlinerAssetLabel::GetLoadingIndicatorVisibility)
				.ToolTipText(LOCTEXT("LoadingTooltip", "Asset is loading..."))
			]
		];

		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			for (const FSoftObjectPath& SharedVariablesPath : Item->SharedVariableSourcePaths)
			{
				const FString SharedVariablesAssetName = SharedVariablesPath.GetAssetName();
				const FLinearColor BackgroundColor = FLinearColor::MakeRandomSeededColor(GetTypeHash(SharedVariablesAssetName)).Desaturate(0.4f);

				const FButtonStyle& CloseButtonStyle = FUAFStyle::Get().GetWidgetStyle<FButtonStyle>("VariablesOutliner.SharedVariablesPill.CloseButton");
				TSharedPtr<SBorder> SharedVariablesBorderWidget;
				ImplementedSharedVariablesHBox->AddSlot()
				.VAlign(VAlign_Center)
				.Padding(2.f, 0.f)
				.AutoWidth()
				[
					SAssignNew(SharedVariablesBorderWidget, SBorder)
					.BorderImage(FUAFStyle::Get().GetBrush("VariablesOutliner.SharedVariablesPill.Border"))
					.BorderBackgroundColor(BackgroundColor)
					.ToolTipText( FText::FromString(SharedVariablesAssetName))
					.VAlign(VAlign_Center)
					.Padding(FMargin(6.0f, 1.0f, 4.0f, 1.0f))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.TextStyle(FUAFStyle::Get(), "VariablesOutliner.SharedVariablesPill.Text")
							.Text(FText::FromString(SharedVariablesAssetName))
							.HighlightText(SceneOutliner.GetFilterHighlightText())
							.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
						]
						+SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(SButton)
							.ButtonStyle(&CloseButtonStyle)
							.OnClicked( Item.ToSharedRef(), &FOutlinerAssetItem::OnRemoveSharedVariable, SharedVariablesPath )
							.ContentPadding(FMargin(0.0))
							.ToolTipText(LOCTEXT("RemoveReference_ToolTip", "Removes the SharedVariables reference from this Asset."))
							[
								SNew(SSpacer)
								.Size(CloseButtonStyle.Normal.ImageSize)
							]
						]
					]
					.OnMouseButtonUp(Item.ToSharedRef(), &FOutlinerAssetItem::OnSharedVariableWidgetMouseUp, SharedVariablesPath)
				];

				SharedVariablesBorderWidget->SetOnMouseMove(FPointerEventHandler::CreateLambda([SharedVariablesPath, WeakOutliner = WeakSceneOutliner](const FGeometry&, const FPointerEvent& Event) -> FReply
				{
					const bool bShouldHighlight = Event.GetModifierKeys().IsControlDown();
					{
						if (TSharedPtr<ISceneOutliner> SharedOutliner = WeakOutliner.Pin())
						{
							if (const FCommonOutlinerMode* Mode = static_cast<const FCommonOutlinerMode*>(SharedOutliner->GetMode()))
							{
								if (bShouldHighlight)
								{
									Mode->SetHighlightedItem(GetTypeHash(SharedVariablesPath));
								}
								else
								{
									Mode->ClearHighlightedItem(GetTypeHash(SharedVariablesPath));
								}
							}
						}
					}

					return FReply::Handled();
				}));

				SharedVariablesBorderWidget->SetOnMouseLeave(FSimpleNoReplyPointerEventHandler::CreateLambda([SharedVariablesPath, WeakOutliner = WeakSceneOutliner](const FPointerEvent&)
				{
					if (TSharedPtr<ISceneOutliner> SharedOutliner = WeakOutliner.Pin())
					{
						if (const FCommonOutlinerMode* Mode = static_cast<const FCommonOutlinerMode*>(SharedOutliner->GetMode()))
						{
							Mode->ClearHighlightedItem(GetTypeHash(SharedVariablesPath));
						}
					}
				}));
			}
		}
	}

	bool IsReadOnly() const
	{
		return !CanRename();
	}

	bool CanRename() const
	{
		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			return Item->SoftAsset.Get() != nullptr;
		}
		return false;
	}

	FText GetDirtyTooltipText() const
	{
		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			FTextBuilder TextBuilder;
			TextBuilder.AppendLine(LOCTEXT("ModifiedTooltip", "Modified"));

			if(UUAFRigVMAsset* Asset = Item->SoftAsset.Get())
			{
				const UPackage* Package = Asset->GetPackage();
				check(Package);
				if(Package->IsDirty())
				{
					TextBuilder.AppendLine(FText::FromName(Package->GetFName()));
				}
			}

			return TextBuilder.ToText();
		}
		return FText::GetEmpty();
	}

	const FSlateBrush* GetDirtyImageBrush() const
	{
		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			if(UUAFRigVMAsset* Asset = Item->SoftAsset.Get())
			{
				bool bIsDirty = false;
				const UPackage* Package = Asset->GetPackage();
				check(Package);
				if(Package->IsDirty())
				{
					bIsDirty = true;
				}

				return bIsDirty ? FAppStyle::GetBrush("Icons.DirtyBadge") : nullptr;
			}
		}
		return nullptr;
	}

	EVisibility GetDirtyImageVisibility() const
	{
		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			if(UUAFRigVMAsset* Asset = Item->SoftAsset.Get())
			{
				bool bIsDirty = false;
				const UPackage* Package = Asset->GetPackage();
				check(Package);
				if(Package->IsDirty())
				{
					bIsDirty = true;
				}

				return bIsDirty ? EVisibility::Visible : EVisibility::Collapsed;
			}
		}
		return EVisibility::Collapsed;
	}

	EVisibility GetLoadingIndicatorVisibility() const
	{
		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			UUAFRigVMAsset* Asset = Item->SoftAsset.Get();
			return Asset != nullptr ? EVisibility::Collapsed : EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	}

	FText GetDisplayText() const
	{
		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			return FText::FromString(Item->GetDisplayString());
		}
		return FText();
	}

	const FSlateBrush* GetAssetIcon() const
	{
		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			UUAFRigVMAsset* Asset = Item->SoftAsset.Get();
			return Asset != nullptr ? FSlateIconFinder::FindIconBrushForClass(Asset->GetClass()) : FAppStyle::Get().GetBrush("ClassIcon.Object");
		}
		return FAppStyle::Get().GetBrush("ClassIcon.Object");
	}

	void OnTextCommited(const FText& InLabel, ETextCommit::Type InCommitInfo) const
	{
		if(InCommitInfo == ETextCommit::OnEnter)
		{
			if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
			{
				Item->Rename(InLabel); 
			}
		}
	}

	bool OnVerifyTextChanged(const FText& InLabel, FText& OutErrorMessage) const
	{
		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			return Item->ValidateName(InLabel, OutErrorMessage); 
		}
		return false;
	}

	virtual FSlateColor GetForegroundColor() const override
	{
		TOptional<FLinearColor> BaseColor;
		if (TreeItem.IsValid())
		{
			BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItem.Pin());
		}
		return BaseColor.IsSet() ? BaseColor.GetValue() : FSlateColor::UseForeground();
	}

	EVisibility GetAddVariableButtonVisibility() const
	{
		if (const TSharedPtr<FOutlinerAssetItem> Item = TreeItem.Pin())
		{
			if(Item->SoftAsset.Get())
			{
				return EVisibility::Visible;
			}
		}
		return EVisibility::Collapsed;
	}

	TWeakPtr<FOutlinerAssetItem> TreeItem;
	TSharedPtr<SInlineEditableTextBlock> TextBlock;
	TSharedPtr<SHorizontalBox> ImplementedSharedVariablesHBox;
};

FOutlinerAssetItem::FOutlinerAssetItem(FSceneOutlinerTreeItemType InType, const FItemData& InItemData)
	: FOutlinerItem(InType, { InItemData.Asset.Get(), InItemData.SortValue})
	, SoftAsset(InItemData.Asset)
	, SharedVariableSourcePaths(InItemData.ImplementedSharedVariablesPaths)
{
}

bool FOutlinerAssetItem::IsValid() const
{
	return !SoftAsset.IsNull();
}

FSceneOutlinerTreeItemID FOutlinerAssetItem::GetID() const
{
	return GetTypeHash(SoftAsset.ToSoftObjectPath());
}

FString FOutlinerAssetItem::GetDisplayString() const
{
	return SoftAsset.GetAssetName();
}

TSharedRef<SWidget> FOutlinerAssetItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	SAssignNew(AssetLabel, SOutlinerAssetLabel, *this, Outliner, InRow);
	RenameRequestEvent.BindSP(AssetLabel->TextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
	return AssetLabel->AsShared();
}

FString FOutlinerAssetItem::GetPackageName() const
{
	return SoftAsset.GetLongPackageName();
}

void FOutlinerAssetItem::Rename(const FText& InNewName) const
{
	UUAFRigVMAsset* Asset = SoftAsset.Get();
	if(Asset == nullptr)
	{
		return;
	}

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	const FString CurrentAssetPath = FPackageName::GetLongPackagePath(Asset->GetPackage()->GetName());
	TArray<FAssetRenameData> AssetsToRename = { FAssetRenameData(Asset, CurrentAssetPath, InNewName.ToString()) };
	AssetToolsModule.Get().RenameAssets(AssetsToRename);
}

bool FOutlinerAssetItem::ValidateName(const FText& InNewName, FText& OutErrorMessage) const
{
	UUAFRigVMAsset* Asset = SoftAsset.Get();
	if(Asset == nullptr)
	{
		OutErrorMessage = LOCTEXT("InvalidAssetError", "Asset is invalid");
		return false;
	}

	FString NewName = InNewName.ToString();
	if (NewName.Len() >= NAME_SIZE)
	{
		OutErrorMessage = LOCTEXT("Error_AssetNameTooLarge", "This asset name is too long. Please choose a shorter name.");
		return false;
	}

	if (Asset->GetFName() != FName(*NewName)) // Deliberately ignore case here to allow case-only renames of existing assets
	{
		const FString PackageName = Asset->GetPackage()->GetPathName() / NewName;
		const FString ObjectPath = FString::Printf(TEXT("%s.%s"), *PackageName, *NewName);

		FText ValidationErrorMsg;
		if (!AssetViewUtils::IsValidObjectPathForCreate(ObjectPath, ValidationErrorMsg))
		{
			OutErrorMessage = ValidationErrorMsg;
			return false;
		}
	}

	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	TWeakPtr<IAssetTypeActions> WeakAssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(UUAFRigVMAsset::StaticClass());
	if (TSharedPtr<IAssetTypeActions> AssetTypeActions = WeakAssetTypeActions.Pin())
	{
		if (!AssetTypeActions->CanRename(FAssetData(Asset), &OutErrorMessage))
		{
			return false;
		}
	}

	return true;
}

FReply FOutlinerAssetItem::OnSharedVariableWidgetMouseUp(const FGeometry& Geometry, const FPointerEvent& PointerEvent, const FSoftObjectPath ClickedSharedVariablesPath) const
{
	if (PointerEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		const FWidgetPath WidgetPath = (PointerEvent.GetEventPath() != nullptr) ? *PointerEvent.GetEventPath() : FWidgetPath();

		const bool bCloseAfterSelection = true;
		FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);
		MenuBuilder.BeginSection("", LOCTEXT("SharedVariablesContextMenu", "Shared Variables") );
		{
			MenuBuilder.AddMenuEntry(
					LOCTEXT("RemoveReference", "Remove"),
					LOCTEXT("RemoveReference_ToolTip", "Removes the SharedVariables reference from this Asset."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([this, ClickedSharedVariablesPath]() { OnRemoveSharedVariable(ClickedSharedVariablesPath); }))
			);
		}
		MenuBuilder.EndSection();

		FSlateApplication::Get().PushMenu(AssetLabel.ToSharedRef(), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply FOutlinerAssetItem::OnRemoveSharedVariable(const FSoftObjectPath ClickedSharedVariablesPath) const
{
	if (UUAFRigVMAsset* Asset = SoftAsset.Get())
	{
		UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);

		if (UncookedOnly::FUtils::RemoveSharedVariablesReference(EditorData, ClickedSharedVariablesPath))
		{
		 	return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}
}

#undef LOCTEXT_NAMESPACE // "OutlinerAssetItem"
