// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"
#include "OutlinerEntryItem.h"
#include "UObject/SoftObjectPtr.h"

class UUAFRigVMAsset;

namespace UE::UAF::Editor
{

class IWorkspaceOutlinerItemDetails;

struct FOutlinerAssetItem : FOutlinerItem
{
	static const FSceneOutlinerTreeItemType Type;

	struct FItemData
	{
		const TSoftObjectPtr<UUAFRigVMAsset>& Asset;
		const int32 SortValue;
		const TArrayView<const FSoftObjectPath> ImplementedSharedVariablesPaths;
	};

	FOutlinerAssetItem(FSceneOutlinerTreeItemType InType, const FItemData& InItemData);
	FOutlinerAssetItem(const FItemData& InItemData) : FOutlinerAssetItem(Type, InItemData) {}
	virtual ~FOutlinerAssetItem() override = default;

	// Begin ISceneOutlinerTreeItem overrides
	virtual bool IsValid() const override;
	virtual FSceneOutlinerTreeItemID GetID() const override;
	virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override { return true; }
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	virtual FString GetPackageName() const override;
	// End ISceneOutlinerTreeItem overrides

	// Renames the item to the specified name
	virtual void Rename(const FText& InNewName) const override;

	// Validates the new item name
	virtual bool ValidateName(const FText& InNewName, FText& OutErrorMessage) const override;

	FReply OnSharedVariableWidgetMouseUp(const FGeometry&, const FPointerEvent& PointerEvent, const FSoftObjectPath ClickedSharedVariablesPath) const;
	FReply OnRemoveSharedVariable(const FSoftObjectPath ClickedSharedVariablesPath) const;

	// Soft ptr to the underlying asset, which may not be loaded yet
	TSoftObjectPtr<UUAFRigVMAsset> SoftAsset;

	TArray<const FSoftObjectPath> SharedVariableSourcePaths;

	TSharedPtr<class SOutlinerAssetLabel> AssetLabel;
};

}
