// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerCategoryItem.h"

#include "ISceneOutliner.h"
#include "OutlinerEntryItem.h"
#include "CommonOutlinerMode.h"
#include "Styling/StarshipCoreStyle.h"
#include "OutlinerDragDropOps.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "AnimNextRigVMAsset.h"

#define LOCTEXT_NAMESPACE "OutlinerCategoryItem"

namespace UE::UAF::Editor
{
class FVariableDragDropOp;

const FSceneOutlinerTreeItemType FOutlinerCategoryItem::Type(&FOutlinerItem::Type);

class SOutlinerCategoryLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SOutlinerCategoryLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FOutlinerCategoryItem& InTreeItem, ISceneOutliner& SceneOutliner)
	{
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
		TreeItem = StaticCastSharedRef<FOutlinerCategoryItem>(InTreeItem.AsShared());

		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 2.0f)
			[
				SAssignNew(TextBlock, SInlineEditableTextBlock)
				.Font(FStyleFonts::Get().NormalBold)
				.Text(this, &SOutlinerCategoryLabel::GetDisplayText)
				.HighlightText(SceneOutliner.GetFilterHighlightText())
				.ColorAndOpacity(this, &SOutlinerCategoryLabel::GetForegroundColor)
				.OnTextCommitted(this, &SOutlinerCategoryLabel::OnTextCommited)
				.OnVerifyTextChanged(this, &SOutlinerCategoryLabel::OnVerifyTextChanged)
			]
		];
	}

	FText GetDisplayText() const
	{
		if (const TSharedPtr<FOutlinerCategoryItem> Item = TreeItem.Pin())
		{
			return FText::FromString(Item->GetDisplayString());
		}
		return FText();
	}

	virtual FSlateColor GetForegroundColor() const override
	{
		TOptional<FLinearColor> BaseColor;
		if (TreeItem.IsValid())
		{
			BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItem.Pin());
		}
		return BaseColor.IsSet() ? BaseColor.GetValue() : FSlateColor::UseForeground();
	}


	void OnTextCommited(const FText& InLabel, ETextCommit::Type InCommitInfo) const	
	{
		if(InCommitInfo == ETextCommit::OnEnter)
		{
			if (const TSharedPtr<FOutlinerCategoryItem> Item = TreeItem.Pin())
			{
				Item->Rename(InLabel); 
			}
		}
	}

	bool OnVerifyTextChanged(const FText& InLabel, FText& OutErrorMessage) const
	{
		if (const TSharedPtr<FOutlinerCategoryItem> Item = TreeItem.Pin())
		{
			return Item->ValidateName(InLabel, OutErrorMessage); 
		}
		return false;
	}

	TWeakPtr<FOutlinerCategoryItem> TreeItem;
	TSharedPtr<SInlineEditableTextBlock> TextBlock;
};

FOutlinerCategoryItem::FOutlinerCategoryItem(FSceneOutlinerTreeItemType InType, const FItemData& InItemData) : FOutlinerItem(InType, { InItemData.InWeakOwner, InItemData.SortValue}),
	CategoryName(InItemData.InCategoryName),
	ParentCategoryName(InItemData.InParentCategoryName),
	CategoryPath(InItemData.InCategoryPath)
{
}

TSharedRef<SWidget> FOutlinerCategoryItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	TSharedRef<SOutlinerCategoryLabel> LabelWidget = SNew(SOutlinerCategoryLabel, *this, Outliner);
	RenameRequestEvent.BindSP(LabelWidget->TextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);

	return LabelWidget;
}

FString FOutlinerCategoryItem::GetDisplayString() const
{
	return CategoryName;
}

FSceneOutlinerTreeItemID FOutlinerCategoryItem::GetID() const
{
	UUAFRigVMAsset* Owner = WeakOwner.Get();
	const FSoftObjectPath OwnerPath = Owner;
	return HashCombine(GetTypeHash(OwnerPath), GetTypeHash(CategoryPath));	
}

}

#undef LOCTEXT_NAMESPACE // "OutlinerCategoryItem"
