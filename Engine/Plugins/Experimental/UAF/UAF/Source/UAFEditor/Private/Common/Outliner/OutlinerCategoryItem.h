// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"
#include "OutlinerEntryItem.h"

class UUAFRigVMAsset;

namespace UE::UAF::Editor
{

struct FOutlinerCategoryItem : FOutlinerItem
{
	static const FSceneOutlinerTreeItemType Type;

	struct FItemData
	{
		TWeakObjectPtr<UUAFRigVMAsset> InWeakOwner;
		int32 SortValue;

		FStringView InCategoryName;
		FStringView InParentCategoryName;
		FStringView InCategoryPath;
	};

	FOutlinerCategoryItem(FSceneOutlinerTreeItemType InType, const FItemData& InItemData);

	// Begin ISceneOutlinerTreeItem overrides
	virtual bool CanInteract() const override { return true; }
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override final;
	virtual FString GetDisplayString() const override final;
	virtual FSceneOutlinerTreeItemID GetID() const override;
	// End ISceneOutlinerTreeItem overrides

	FString CategoryName;
	FString ParentCategoryName;
	FString CategoryPath;
};

}
