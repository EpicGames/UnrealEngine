// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerColumns.h"

#include "IAnimNextRigVMExportInterface.h"
#include "ISceneOutliner.h"
#include "OutlinerEntryItem.h"
#include "CommonOutlinerMode.h"
#include "Misc/NotifyHook.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "OutlinerColumns"

namespace UE::UAF::Editor
{

FLazyName OutlinerAccessSpecifier("OutlinerAccessSpecifier");

FOutlinerAccessSpecifierColumn::FOutlinerAccessSpecifierColumn(ISceneOutliner& SceneOutliner) : WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) {}

FName FOutlinerAccessSpecifierColumn::GetID()
{
	return OutlinerAccessSpecifier;
}

SHeaderRow::FColumn::FArguments FOutlinerAccessSpecifierColumn::ConstructHeaderRowColumn()
{
	return
		SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.0f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Fill)
		[
			SNew(SBox) 
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Level.VisibleIcon16x"))
				.ToolTipText(LOCTEXT("AccessSpecifierAccessLevelTooltip", "Access level of this entry"))
			]
		];
}

class SOutlinerAccessSpecifier : public SCompoundWidget, public FNotifyHook
{
	SLATE_BEGIN_ARGS(SOutlinerAccessSpecifier) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FOutlinerEntryItem& InTreeItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakTreeItem = StaticCastSharedRef<FOutlinerEntryItem>(InTreeItem.AsShared());

		ChildSlot
		[
			SNew(SBox)
			.IsEnabled(!InTreeItem.IsValid())
			.WidthOverride(16.0f)
			.HeightOverride(16.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.IsEnabled(this, &SOutlinerAccessSpecifier::IsButtonEnabled)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &SOutlinerAccessSpecifier::OnClicked)
				.Content()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SOutlinerAccessSpecifier::GetImage)
					.ToolTipText(this, &SOutlinerAccessSpecifier::GetTooltipText)
				]
			]
		];
	}

	FReply OnClicked() const
	{
		TSharedPtr<FOutlinerEntryItem> TreeItem = WeakTreeItem.Pin();
		if(!TreeItem.IsValid() || !TreeItem->IsValid())
		{
			return FReply::Unhandled();
		}

		TreeItem->SetAccessSpecifier(TreeItem->GetAccessSpecifier() == EAnimNextExportAccessSpecifier::Public ? EAnimNextExportAccessSpecifier::Private : EAnimNextExportAccessSpecifier::Public);

		return FReply::Handled();	// Fall through so we dont deselect our item
	}

	const FSlateBrush* GetImage() const
	{
		TSharedPtr<FOutlinerEntryItem> TreeItem = WeakTreeItem.Pin();
		if(!TreeItem.IsValid() || !TreeItem->IsValid())
		{
			return nullptr;
		}

		return TreeItem->GetAccessSpecifier() == EAnimNextExportAccessSpecifier::Public ?
			FAppStyle::GetBrush("Level.VisibleIcon16x") :
			FAppStyle::GetBrush("Level.NotVisibleHighlightIcon16x");
	}

	bool IsButtonEnabled() const
	{
		TSharedPtr<FOutlinerEntryItem> TreeItem = WeakTreeItem.Pin();
		if(!TreeItem.IsValid() || !TreeItem->IsValid())
		{
			return false;
		}

		return TreeItem->CanSetAccessSpecifier();
	}

	FText GetTooltipText() const
	{
		TSharedPtr<FOutlinerEntryItem> TreeItem = WeakTreeItem.Pin();
		if(!TreeItem.IsValid() || !TreeItem->IsValid())
		{
			return FText::GetEmpty();
		}

		const FText AccessSpecifier = TreeItem->GetAccessSpecifier() == EAnimNextExportAccessSpecifier::Public ? LOCTEXT("PublicSpecifier", "public") : LOCTEXT("PrivateSpecifier", "private");
		const FText AccessSpecifierDesc = TreeItem->GetAccessSpecifier()  == EAnimNextExportAccessSpecifier::Public ?
			LOCTEXT("PublicSpecifierDesc", "This means that the entry is usable from gameplay and from other AnimNext assets") :
			LOCTEXT("PrivateSpecifierDesc", "This means that the entry is only usable inside this asset");
		return FText::Format(LOCTEXT("AccessSpecifierEntryTooltip", "This entry is {0}.\n{1}"), AccessSpecifier, AccessSpecifierDesc);
	}

	TWeakPtr<FOutlinerEntryItem> WeakTreeItem;
};

const TSharedRef<SWidget> FOutlinerAccessSpecifierColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef Item, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	FOutlinerEntryItem* TreeItem = Item->CastTo<FOutlinerEntryItem>();
	if(TreeItem == nullptr || !TreeItem->IsValid())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SOutlinerAccessSpecifier, *TreeItem, WeakSceneOutliner.Pin().ToSharedRef().Get(), Row);
}

}

#undef LOCTEXT_NAMESPACE // "OutlinerColumns"
