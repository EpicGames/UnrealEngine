// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerColumn.h"

namespace UE::UAF::Editor
{
class FOutlinerAccessSpecifierColumn : public ISceneOutlinerColumn
{
public:
	static FName GetID();

	FOutlinerAccessSpecifierColumn(ISceneOutliner& SceneOutliner);

	// ISceneOutlinerColumn interface
	virtual FName GetColumnID() override final { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override final;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override final;
	virtual bool SupportsSorting() const override final { return false; }

private:
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;
};
}
