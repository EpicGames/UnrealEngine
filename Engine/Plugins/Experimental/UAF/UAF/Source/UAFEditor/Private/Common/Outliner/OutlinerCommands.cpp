// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerCommands.h"

#define LOCTEXT_NAMESPACE "OutlinerCommands"

namespace UE::UAF::Editor
{
void FOutlinerCommands::RegisterCommands()
{
	UI_COMMAND(SaveAsset, "Save Asset", "Save the selected asset.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(FindReferences, "In Project", "Finds all references to the selected items across assets in the project.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FindReferencesInWorkspace, "In Workspace", "Finds all references to the selected items across assets in the current workspace.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FindReferencesInAsset, "In Asset", "Finds all references to the selected items in the currently opened asset.", EUserInterfaceActionType::Button, FInputChord());	

	UI_COMMAND(MakeEntryPublic, "Sets Entry to Public", "Sets Entry to be Public.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MakeEntryPrivate, "Sets Entry to Private", "Sets Entry to be Private.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(OpenInNewTab, "Open in New Tab", "Opens the selected item in a new tab.", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ExpandEntries, "Expand Entries", "Expands entries in the current selection.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CollapseEntries, "Collapse Entries", "Collapses entries in the current selection.", EUserInterfaceActionType::Button, FInputChord());
}
}

#undef LOCTEXT_NAMESPACE
