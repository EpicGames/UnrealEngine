// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "OutlinerCommands"

namespace UE::UAF::Editor
{
class FOutlinerCommands : public TCommands<FOutlinerCommands>
{
public:

	FOutlinerCommands() : TCommands<FOutlinerCommands>( TEXT("Outliner"), LOCTEXT("OutlinerCommandsLabel", "Outliner Commands"), NAME_None, FAppStyle::GetAppStyleSetName() )
	{
	}

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> SaveAsset;
	TSharedPtr<FUICommandInfo> FindReferences;
	TSharedPtr<FUICommandInfo> FindReferencesInWorkspace;
	TSharedPtr<FUICommandInfo> FindReferencesInAsset;
	TSharedPtr<FUICommandInfo> MakeEntryPublic;
	TSharedPtr<FUICommandInfo> MakeEntryPrivate;
	TSharedPtr<FUICommandInfo> OpenInNewTab;
	TSharedPtr<FUICommandInfo> ExpandEntries;
	TSharedPtr<FUICommandInfo> CollapseEntries;
};
}

#undef LOCTEXT_NAMESPACE
