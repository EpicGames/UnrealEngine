// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerDragDropOps.h"

#include "AnimNextEdGraphSchema.h"
#include "ScopedTransaction.h"
#include "Common/GraphEditorSchemaActions.h"
#include "EdGraph/RigVMEdGraph.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Variables/Outliner/VariablesOutlinerCategoryItem.h"
#include "Variables/Outliner/VariablesOutlinerEntryItem.h"

#define LOCTEXT_NAMESPACE "OutlinerDragDropOps"

namespace UE::UAF::Editor
{

TSharedPtr<FVariableDragDropOp> FVariableDragDropOp::New(TSharedPtr<FAnimNextSchemaAction_Variable> InAction, TSharedPtr<FVariablesOutlinerEntryItem> InEntry)
{
	TSharedPtr<FVariableDragDropOp> NewOp = MakeShared<FVariableDragDropOp>();
	NewOp->WeakItem = InEntry;
	NewOp->SourceAction = InAction;
	NewOp->Construct();
	return NewOp;
}

TSharedPtr<FAnimNextSchemaAction_Variable> FVariableDragDropOp::GetAction() const
{
	if (SourceAction.IsValid())
	{
		return StaticCastSharedPtr<FAnimNextSchemaAction_Variable>(SourceAction);
	}

	return nullptr;
}

TSharedPtr<FFunctionDragDropOp> FFunctionDragDropOp::New(TSharedPtr<FAnimNextSchemaAction_Function> InAction, TSharedPtr<FFunctionsOutlinerEntryItem> InEntry)
{
	TSharedPtr<FFunctionDragDropOp> NewOp = MakeShared<FFunctionDragDropOp>();
	NewOp->WeakItem = InEntry;
	NewOp->SourceAction = InAction;
	NewOp->Construct();
	return NewOp;
}

void FFunctionDragDropOp::GetDefaultStatusSymbol(const FSlateBrush*& PrimaryBrushOut, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut) const
{	
	if (SourceAction.IsValid())
	{
		PrimaryBrushOut = StaticCastSharedPtr<FAnimNextSchemaAction_Function>(SourceAction)->GetIconBrush();
		IconColorOut = StaticCastSharedPtr<FAnimNextSchemaAction_Function>(SourceAction)->GetIconColor();
	}
	SecondaryBrushOut = nullptr;
	SecondaryColorOut = FSlateColor::UseForeground();
}
	
FReply FVariableDragDropOp::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
    TSharedPtr<FAnimNextSchemaAction_Variable> SourceVariableAction = GetAction();
    if (SourceVariableAction.IsValid())
    {
    	UEdGraphPin* Pin = GetHoveredPin();
    	const URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(GetHoveredGraph());
    	if (RigVMEdGraph && Pin)
    	{
    		// Create External Variable from the outliner item to be able to reuse the RigVM function to check if it can be linked to the hovered pin
    		FRigVMExternalVariable CreatedExternalVariable;
    		if (TSharedPtr<FVariablesOutlinerEntryItem> VariablesItem = WeakItem.Pin())
    		{
    			if(const IUAFRigVMVariableInterface* Variable = Cast<IUAFRigVMVariableInterface>(VariablesItem->WeakEntry.Get()))
    			{
    				if(const UPropertyBag* PropertyBag = Variable->GetPropertyBag().GetPropertyBagStruct())
    				{
    					const TConstArrayView<FPropertyBagPropertyDesc> VariableDescs = PropertyBag->GetPropertyDescs();
    					if(VariableDescs.Num() != 0)
    					{
    						uint8* Container = const_cast<uint8*>(Variable->GetPropertyBag().GetValue().GetMemory());
    						for(const FPropertyBagPropertyDesc& Desc : VariableDescs)
    						{
    							FRigVMExternalVariable ExternalVariable = FRigVMExternalVariable::Make(Desc.ID, Desc.CachedProperty, Container);
    							if(!ExternalVariable.IsValid())
    							{
    								continue;
    							}
    							
    							CreatedExternalVariable = MoveTemp(ExternalVariable);
    							break;
    						}
    					}
    				}
    			}
    			else if (const FProperty* Property = VariablesItem->PropertyPath.Get())
    			{
    				CreatedExternalVariable = FRigVMExternalVariable::Make(FGuid(), Property, nullptr);
    			}
    		}
    		
    		if (CreatedExternalVariable.IsValid(true))
    		{
    			if (const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetHoveredGraph()))
    			{
    				const URigVMGraph* ModelGraph = Graph->GetModel();
    				const URigVMPin* ModelPin = ModelGraph ? ModelGraph->FindPin(Pin->GetName()) : nullptr;
    				if (ModelPin && ModelPin->CanBeBoundToVariable(CreatedExternalVariable))
    				{
    					if (URigVMController* Controller = Graph->GetController())
    					{
    						FScopedTransaction Transaction(LOCTEXT("LinkVariableToPin", "Link Variable to Node Pin"));
    						Controller->OpenUndoBracket(TEXT("Link Variable to Pin"));
    					
    						FAnimNextSchemaAction_Variable Action = *SourceVariableAction.Get();
    						Action.SetVariableAccessorChoice(FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Get);
    			
    						TArray<UEdGraphPin*> DummyPins;
    						DummyPins.Add(Pin);
    						if (const URigVMEdGraphNode* RigVMEdNode = Cast<URigVMEdGraphNode>(Action.PerformAction(GetHoveredGraph(), DummyPins, GraphPosition)))
    						{
    							const URigVMNode* ModelNode = RigVMEdNode->GetModelNode();
    							const URigVMPin* VariableNodePin = ModelNode ? ModelNode->FindPin(TEXT("Value")) : nullptr;
    							if (VariableNodePin)
    							{
    								Controller->AddLink(VariableNodePin->GetPinPath(), ModelPin->GetPinPath(), true);
    				
    								Controller->CloseUndoBracket();
    								return FReply::Handled();
    							}
    						}
    			
    						Controller->CancelUndoBracket();
    					}
    				}
    			}
    		}
    	}
    }
    
    return FGraphSchemaActionDragDropAction::DroppedOnPin(ScreenPosition, GraphPosition);
}

TSharedPtr<FAnimNextSchemaAction_Function> FFunctionDragDropOp::GetAction() const
{
	if (SourceAction.IsValid())
	{
		return StaticCastSharedPtr<FAnimNextSchemaAction_Function>(SourceAction);
	}

	return nullptr;
}

void FVariableDragDropOp::GetDefaultStatusSymbol(const FSlateBrush*& PrimaryBrushOut, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut) const
{	
	if (SourceAction.IsValid())
	{
		PrimaryBrushOut = StaticCastSharedPtr<FAnimNextSchemaAction_Variable>(SourceAction)->GetIconBrush();
		IconColorOut = StaticCastSharedPtr<FAnimNextSchemaAction_Variable>(SourceAction)->GetIconColor();
	}
	SecondaryBrushOut = nullptr;
	SecondaryColorOut = FSlateColor::UseForeground();
}

TSharedPtr<FCategoryDragDropOp> FCategoryDragDropOp::New(TSharedPtr<FOutlinerCategoryItem> InEntry)
{
	TSharedPtr<FCategoryDragDropOp> NewOp = MakeShared<FCategoryDragDropOp>();
	NewOp->WeakItem = InEntry;

	NewOp->CurrentHoverText = FText::FromString(InEntry->CategoryName);
	NewOp->SetupDefaults();
	NewOp->Construct();

	return NewOp;
}

}

#undef LOCTEXT_NAMESPACE
