// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphEditorDragDropAction.h"
#include "ISceneOutlinerTreeItem.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "UObject/WeakInterfacePtr.h"

struct FAnimNextSchemaAction_Function;
struct FAnimNextSchemaAction_Variable;

namespace UE::UAF::Editor
{
struct FOutlinerCategoryItem;
struct FFunctionsOutlinerEntryItem;
struct FVariablesOutlinerEntryItem;

class FVariableDragDropOp : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FVariableDragDropOp, FGraphSchemaActionDragDropAction)

	static TSharedPtr<FVariableDragDropOp> New(TSharedPtr<FAnimNextSchemaAction_Variable> InAction, TSharedPtr<FVariablesOutlinerEntryItem> InEntry);

	// FGraphSchemaActionDragDropAction interface
	virtual void GetDefaultStatusSymbol(const FSlateBrush*& PrimaryBrushOut, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut) const override;
	virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	
	TSharedPtr<FAnimNextSchemaAction_Variable> GetAction() const;

	TWeakPtr<FVariablesOutlinerEntryItem> WeakItem;
};

class FFunctionDragDropOp : public FGraphSchemaActionDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FFunctionDragDropOp, FGraphSchemaActionDragDropAction)

	static TSharedPtr<FFunctionDragDropOp> New(TSharedPtr<FAnimNextSchemaAction_Function> InAction, TSharedPtr<FFunctionsOutlinerEntryItem> InEntry);

	// FGraphSchemaActionDragDropAction interface
	virtual void GetDefaultStatusSymbol(const FSlateBrush*& PrimaryBrushOut, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut) const override;

	TSharedPtr<FAnimNextSchemaAction_Function> GetAction() const;

	TWeakPtr<FFunctionsOutlinerEntryItem> WeakItem;
};

class FCategoryDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FCategoryDragDropOp, FDecoratedDragDropOp)

	static TSharedPtr<FCategoryDragDropOp> New(TSharedPtr<FOutlinerCategoryItem> InEntry);

	TWeakPtr<FOutlinerCategoryItem> WeakItem;
};

}
