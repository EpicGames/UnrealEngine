// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerEntryItem.h"
#include "OutlinerEntryItemWidget.h"
#include "Widgets/Views/SListView.h"
#include "ISceneOutliner.h"

namespace UE::UAF::Editor
{

const FSceneOutlinerTreeItemType FOutlinerEntryItem::Type(&FOutlinerItem::Type);
FOutlinerEntryItem::FOutlinerEntryItem(FSceneOutlinerTreeItemType InType, const FItemData& InItemData) : FOutlinerItem(InType, InItemData) {}

TSharedRef<SWidget> FOutlinerEntryItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	TSharedRef<SOutlinerEntryLabel> LabelWidget = SNew(SOutlinerEntryLabel, *this, Outliner, InRow);
	RenameRequestEvent.BindSP(LabelWidget->TextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
	return LabelWidget;
}
}
