// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OutlinerItem.h"

enum class EAnimNextExportAccessSpecifier : int32;

namespace UE::UAF::Editor
{

struct FOutlinerEntryItem : FOutlinerItem
{
	static const FSceneOutlinerTreeItemType Type;

	FOutlinerEntryItem(FSceneOutlinerTreeItemType InType, const FItemData& InItemData);
	virtual ~FOutlinerEntryItem() override {}

	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;

	virtual bool IsReadOnly() const = 0;
	virtual void SetAccessSpecifier(const EAnimNextExportAccessSpecifier& InSpecifier) const = 0;
	virtual EAnimNextExportAccessSpecifier GetAccessSpecifier() const = 0;
	virtual bool CanSetAccessSpecifier() const = 0;
	virtual FStringView GetCategoryPath() const = 0;
	virtual void GetItemIconAndColor(const FSlateBrush*& OutIcon, FSlateColor& OutColor) const {}
};

}


