// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerEntryItemWidget.h"

#include "SPinTypeSelector.h"

#define LOCTEXT_NAMESPACE "SOutlinerEntryLabel"

namespace UE::UAF::Editor
{

void SOutlinerEntryLabel::Construct(const FArguments& InArgs, FOutlinerEntryItem& InTreeItem, ISceneOutliner& SceneOutliner,
	const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
	TreeItem = StaticCastSharedRef<FOutlinerEntryItem>(InTreeItem.AsShared());
	
	TSharedRef<SWidget> IconWidget = SNew(SImage)
	.Image_Lambda([this]() -> const FSlateBrush*
	{
		const FSlateBrush* Brush = nullptr;
		if (TreeItem.IsValid())
		{
			FSlateColor Color;
			TreeItem.Pin()->GetItemIconAndColor(Brush, Color);
		}

		return Brush;
	})
	.ColorAndOpacity_Lambda([this]()
	{
		FSlateColor Color = GetForegroundColor();
		if (TreeItem.IsValid())
		{
			const FSlateBrush* Brush = nullptr;
			TreeItem.Pin()->GetItemIconAndColor(Brush, Color);
		}

		return Color;
	})
	.Visibility_Lambda([this]()
	{
		
		const FSlateBrush* Brush = nullptr;
		if (TreeItem.IsValid())
		{
			FSlateColor Color;
			TreeItem.Pin()->GetItemIconAndColor(Brush, Color);
		}

		return Brush != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
	});
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.0f, 2.0f)
		[
			IconWidget
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.0f, 2.0f)
		[
			SAssignNew(TextBlock, SInlineEditableTextBlock)
			.IsReadOnly(this, &SOutlinerEntryLabel::IsReadOnly)
			.Text(this, &SOutlinerEntryLabel::GetDisplayText)
			.HighlightText(SceneOutliner.GetFilterHighlightText())
			.ColorAndOpacity(this, &SOutlinerEntryLabel::GetForegroundColor)
			.OnTextCommitted(this, &SOutlinerEntryLabel::OnTextCommited)
			.OnVerifyTextChanged(this, &SOutlinerEntryLabel::OnVerifyTextChanged)
		]
	];
}

FText SOutlinerEntryLabel::GetDirtyTooltipText() const
{
	if (const TSharedPtr<FOutlinerEntryItem> Item = TreeItem.Pin())
	{
		FTextBuilder TextBuilder;
		TextBuilder.AppendLine(LOCTEXT("ModifiedTooltip", "Modified"));

		if(const UPackage* ExternalPackage = Item->GetPackage())
		{
			if(ExternalPackage && ExternalPackage->IsDirty())
			{
				TextBuilder.AppendLine(FText::FromName(ExternalPackage->GetFName()));
			}
		}

		return TextBuilder.ToText();
	}
	return FText::GetEmpty();
}

const FSlateBrush* SOutlinerEntryLabel::GetDirtyImageBrush() const
{
	if (const TSharedPtr<FOutlinerEntryItem> Item = TreeItem.Pin())
	{
		if(const UPackage* ExternalPackage = Item->GetPackage())
		{
			bool bIsDirty = false;
			if(ExternalPackage && ExternalPackage->IsDirty())
			{
				bIsDirty = true;
			}

			return bIsDirty ? FAppStyle::GetBrush("Icons.DirtyBadge") : nullptr;
		}
	}
	return nullptr;
}

FText SOutlinerEntryLabel::GetDisplayText() const
{
	if (const TSharedPtr<FOutlinerEntryItem> Item = TreeItem.Pin())
	{
		return FText::FromString(Item->GetDisplayString());
	}
	return FText();
}

void SOutlinerEntryLabel::OnTextCommited(const FText& InLabel, ETextCommit::Type InCommitInfo) const
{
	if(InCommitInfo == ETextCommit::OnEnter)
	{
		if (const TSharedPtr<FOutlinerEntryItem> Item = TreeItem.Pin())
		{
			Item->Rename(InLabel); 
		}
	}
}

bool SOutlinerEntryLabel::OnVerifyTextChanged(const FText& InLabel, FText& OutErrorMessage) const
{
	if (const TSharedPtr<FOutlinerEntryItem> Item = TreeItem.Pin())
	{
		return Item->ValidateName(InLabel, OutErrorMessage); 
	}
	return false;
}

bool SOutlinerEntryLabel::IsReadOnly() const
{
	if (const TSharedPtr<FOutlinerEntryItem> Item = TreeItem.Pin())
	{
		return Item->IsReadOnly();
	}
	return false;
}

FSlateColor SOutlinerEntryLabel::GetForegroundColor() const
{
	TOptional<FLinearColor> BaseColor;
	if (TSharedPtr<FOutlinerEntryItem> SharedItem = TreeItem.Pin())
	{
		BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*SharedItem);
	}
	return BaseColor.IsSet() ? BaseColor.GetValue() : FSlateColor::UseForeground();
}
}

#undef LOCTEXT_NAMESPACE // "SOutlinerEntryLabel"
