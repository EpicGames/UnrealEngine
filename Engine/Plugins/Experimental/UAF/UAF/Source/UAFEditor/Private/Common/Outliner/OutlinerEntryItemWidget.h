// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphEditorDragDropAction.h"
#include "ISceneOutliner.h"
#include "OutlinerEntryItem.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

namespace UE::UAF::Editor
{
class SOutlinerEntryLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SOutlinerEntryLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FOutlinerEntryItem& InTreeItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow);	

	virtual FSlateColor GetForegroundColor() const override;
protected:
	FText GetDirtyTooltipText() const;
	const FSlateBrush* GetDirtyImageBrush() const;
	FText GetDisplayText() const;
	void OnTextCommited(const FText& InLabel, ETextCommit::Type InCommitInfo) const;
	bool OnVerifyTextChanged(const FText& InLabel, FText& OutErrorMessage) const;
	bool IsReadOnly() const;

	TWeakPtr<FOutlinerEntryItem> TreeItem;
	TSharedPtr<SInlineEditableTextBlock> TextBlock;

	friend struct FOutlinerEntryItem;
};
}
