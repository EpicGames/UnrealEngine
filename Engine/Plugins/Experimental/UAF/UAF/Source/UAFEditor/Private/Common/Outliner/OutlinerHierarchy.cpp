// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerHierarchy.h"

#include "AnimNextRigVMAsset.h"
#include "ISceneOutlinerMode.h"
#include "AnimNextRigVMAssetEditorData.h"

#include "OutlinerAssetItem.h"
#include "CommonOutlinerMode.h"

#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Variables/SVariablesView.h"
#include "Variables/AnimNextSharedVariables.h"

#include "AnimNextRigVMAsset.h"

namespace UE::UAF::Editor
{

TAutoConsoleVariable<bool> GOutputHierarchyDebugInformationCvar(TEXT("Outliner.OutputHierarchyDebugInformation"), false, TEXT("If enabled will output debug information about the hierarchy generation to the output log")); 

FOutlinerHierarchy::FOutlinerHierarchy(ISceneOutlinerMode* Mode) : ISceneOutlinerHierarchy(Mode) {}

void FOutlinerHierarchy::CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	TArray<TSoftObjectPtr<UUAFRigVMAsset>> Assets;
	static_cast<FCommonOutlinerMode* const>(Mode)->GetAssets(Assets);

	WeakProcessedAssets.Reset();
	WeakProcessedStructs.Reset();

	if(Assets.Num())
	{
		const int32 NumAssets = Assets.Num();
		for(int32 Index = 0; Index < NumAssets; Index++)
		{
			const TSoftObjectPtr<UUAFRigVMAsset>& SoftAsset = Assets[Index];

			TArray<const FSoftObjectPath> SharedVariableSourcePaths;
			ProcessAsset(SoftAsset, (NumAssets - Index), OutItems, SharedVariableSourcePaths);
		}
	}
}

void FOutlinerHierarchy::ProcessAsset(const TSoftObjectPtr<UUAFRigVMAsset>& InSoftAsset, int32 Depth, TArray<FSceneOutlinerTreeItemPtr>& OutItems, TArray<const FSoftObjectPath>& OutSharedVariableSourcePaths) const
{
	UUAFRigVMAsset* Asset = InSoftAsset.Get();
	if(Asset == nullptr)
	{
		return;
	}

	const UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
	if (EditorData == nullptr)
	{
		return;
	}

	if (WeakProcessedAssets.Contains(Asset))
	{
		return;
	}

	WeakProcessedAssets.Add(Asset);

	UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(),LogAnimation, Warning, "ProcessAsset: %ls %i", *InSoftAsset.GetAssetName(), Depth);

	// Adjust asset depth to ensure SharedVariables are always pinned to 0 depth
	const int32 AssetDepth = Asset->GetClass() && Asset->GetClass() == UUAFSharedVariables::StaticClass() ? 0 : Depth;

	TArray<const FSoftObjectPath> SharedAssetPaths;
	RetrieveSharedAssetPaths(Asset, SharedAssetPaths);
	if (const FSceneOutlinerTreeItemPtr AssetItem = Mode->CreateItemFor<FOutlinerAssetItem>(FOutlinerAssetItem::FItemData{InSoftAsset, AssetDepth, SharedAssetPaths}))
	{
		OutItems.Add(AssetItem);

		UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(), LogAnimation, Warning, "* FOutlinerAssetItem: %ls %i", *InSoftAsset.GetAssetName(), AssetDepth);
	}

	CreateItemsForAsset(EditorData, OutItems);
}

void FOutlinerHierarchy::RetrieveSharedAssetPaths(const TSoftObjectPtr<UUAFRigVMAsset>& InSoftAsset, TArray<const FSoftObjectPath>& OutSharedVariableSourcePaths) const
{
	UUAFRigVMAsset* Asset = InSoftAsset.Get();
	if(Asset == nullptr)
	{
		return;
	}

	const UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
	if (EditorData == nullptr)
	{
		return;
	}

	TArray<FSceneOutlinerTreeItemPtr> ParentItems;
	EditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>([this, &OutSharedVariableSourcePaths](const UUAFSharedVariablesEntry* InSharedVariablesEntry)
	{
		switch (InSharedVariablesEntry->Type)
		{
		case EAnimNextSharedVariablesType::Asset:
			{
				if (const UUAFSharedVariables* SharedVariables = InSharedVariablesEntry->GetAsset())
				{
					OutSharedVariableSourcePaths.Add(SharedVariables);
				}
			}
			break;

		case EAnimNextSharedVariablesType::Struct:
			{
				if (const UScriptStruct* Struct = InSharedVariablesEntry->GetStruct())
				{
					OutSharedVariableSourcePaths.Add(Struct);
				}
			}
			break;

		case EAnimNextSharedVariablesType::RigVMAsset:
			{
				if (const TScriptInterface<const IRigVMRuntimeAssetInterface>& RigVMAsset = InSharedVariablesEntry->GetRigVMAsset())
				{
					OutSharedVariableSourcePaths.Add(RigVMAsset.GetObject());
				}
			}
			break;
		}

		return true;
	});
}
}
