// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerHierarchy.h"

class URigVMRuntimeAssetInterface;
class UUAFRigVMAssetEditorData;
class UUAFRigVMAsset;

namespace UE::UAF::Editor
{

extern TAutoConsoleVariable<bool> GOutputHierarchyDebugInformationCvar;
class FOutlinerHierarchy : public ISceneOutlinerHierarchy
{
public:
	FOutlinerHierarchy(ISceneOutlinerMode* Mode);
	FOutlinerHierarchy(const FOutlinerHierarchy&) = delete;
	FOutlinerHierarchy& operator=(const FOutlinerHierarchy&) = delete;

	// Begin ISceneOutlinerHierarchy overrides
	virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
	virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const override {}
	virtual void CreateItemsForAsset(const UUAFRigVMAssetEditorData* EditorData, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const = 0;	
	// End ISceneOutlinerHierarchy overrides
protected:
	void ProcessAsset(const TSoftObjectPtr<UUAFRigVMAsset>& InSoftAsset, int32 Depth, TArray<FSceneOutlinerTreeItemPtr>& OutItems, TArray<const FSoftObjectPath>& OutSharedVariableSourcePaths) const;
	void RetrieveSharedAssetPaths(const TSoftObjectPtr<UUAFRigVMAsset>& InSoftAsset, TArray<const FSoftObjectPath>& OutSharedVariableSourcePaths) const;

	mutable TSet<TWeakObjectPtr<const UUAFRigVMAsset>> WeakProcessedAssets;
	mutable TSet<TWeakObjectPtr<const UScriptStruct>> WeakProcessedStructs;
	mutable TSet<TWeakObjectPtr<const UObject>> WeakProcessedRigVMAssets;
};

}
