// Copyright Epic Games, Inc. All Rights Reserved.

#include "OutlinerItem.h"

#include "AnimNextRigVMAsset.h"
#include "OutlinerEntryItemWidget.h"

namespace UE::UAF::Editor
{

const FSceneOutlinerTreeItemType FOutlinerItem::Type;

FString FOutlinerItem::GetPackageName() const
{
	if (UUAFRigVMAsset* Owner = WeakOwner.Get())
	{
		return Owner->GetPackage()->GetName();
	}

	return ISceneOutlinerTreeItem::GetPackageName();
}

bool FOutlinerItem::IsValid() const
{
	return WeakOwner.Get() != nullptr;
}

UPackage* FOutlinerItem::GetPackage() const
{
	if (UUAFRigVMAsset* Owner = WeakOwner.Get())
	{
		return Owner->GetPackage();
	}

	return nullptr;
}
}
