// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"

class UUAFRigVMAsset;

namespace UE::UAF::Editor
{
struct FOutlinerItem : ISceneOutlinerTreeItem
{
	static const FSceneOutlinerTreeItemType Type;
	struct FItemData
	{
		const TWeakObjectPtr<UUAFRigVMAsset>& Asset;
		const int32 SortValue;
	};

	FOutlinerItem(FSceneOutlinerTreeItemType InType, const FItemData& InItemData) : ISceneOutlinerTreeItem(InType), WeakOwner(InItemData.Asset), SortValue(InItemData.SortValue) {}

	// Begin ISceneOutlinerTreeItem overrides
	virtual FString GetPackageName() const override;
	virtual bool IsValid() const override;
	// End ISceneOutlinerTreeItem overrides
	
	// Renames the item to the specified name
	virtual void Rename(const FText& InNewName) const {}
	// Validates the new item name
	virtual bool ValidateName(const FText& InNewName, FText& OutErrorMessage) const { return false; }
	virtual UPackage* GetPackage() const;

	TWeakObjectPtr<UUAFRigVMAsset> WeakOwner;
	int32 SortValue = 0;
};
}
