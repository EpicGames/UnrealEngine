// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OutlinerItemMenuContexts.generated.h"

class FUICommandList;
class UUAFRigVMAssetEditorData;
class UUAFRigVMAssetEntry;
class SSceneOutliner;

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{
class SCommonOutliner;

UCLASS()
class UVariablesOutlinerItemMenuContext : public UObject
{
	GENERATED_BODY()

public:
	// Currently selected entries
	UPROPERTY()
	TArray<TWeakObjectPtr<UUAFRigVMAssetEntry>> WeakEntries;
};

UCLASS()
class UFunctionsOutlinerItemMenuContext : public UObject
{
	GENERATED_BODY()
};

UCLASS()
class UCommonOutlinerItemMenuContext : public UObject
{
	GENERATED_BODY()

public:
	// Currently selected asset's editor data
	UPROPERTY()
	TArray<TWeakObjectPtr<UUAFRigVMAssetEditorData>> WeakEditorDatas;

	TWeakPtr<SCommonOutliner> WeakOutliner;
	TWeakPtr<Workspace::IWorkspaceEditor> WeakWorkspaceEditor;
	TWeakPtr<FUICommandList> WeakCommandList;
};

}
