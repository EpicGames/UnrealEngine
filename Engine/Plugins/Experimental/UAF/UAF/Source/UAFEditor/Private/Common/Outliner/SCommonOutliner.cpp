// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCommonOutliner.h"

#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SAssetDropTarget.h"
#include "SSceneOutliner.h"
#include "IWorkspaceEditorModule.h"
#include "UncookedOnlyUtils.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "Entries/AnimNextSharedVariablesEntry.h"

#define LOCTEXT_NAMESPACE "SCommonOutliner"

namespace UE::UAF::Editor
{

void SCommonOutliner::SetAssets(TConstArrayView<TSoftObjectPtr<UUAFRigVMAsset>> InAssets)
{
	for(const TSoftObjectPtr<UUAFRigVMAsset>& CurrentSoftAsset : Assets)
	{
		if(UUAFRigVMAsset* CurrentAsset = CurrentSoftAsset.Get())
		{
			UnregisterAssetDelegates(CurrentAsset);
		}
	}

	Assets = InAssets;

	for(const TSoftObjectPtr<UUAFRigVMAsset>& NewSoftAsset : InAssets)
	{
		if(UUAFRigVMAsset* NewAsset = NewSoftAsset.Get())
		{
			RegisterAssetDelegates(NewAsset);
		}
	}

	FullRefresh();
}

void SCommonOutliner::UpdateAssets()
{
	TArray<FSoftObjectPath> AssetsToAsyncLoad;
	TArray<TSoftObjectPtr<UUAFRigVMAsset>> ExportAssets;

	auto HandleAssetPath = [this, &AssetsToAsyncLoad, &ExportAssets](const FSoftObjectPath& Path)
	{
		ExportAssets.Add(TSoftObjectPtr<UUAFRigVMAsset>(Path));

		if (Path.ResolveObject() == nullptr)
		{
			AssetsToAsyncLoad.Add(Path);
		}
	};

	if (FWorkspaceOutlinerAssetReferenceItemData::IsAssetReference(Export)
		|| FWorkspaceOutlinerGroupItemData::IsGroupItem(Export)
		|| FAnimNextCollapseGraphsOutlinerDataBase::IsCollapsedGraphBase(Export))
	{
		// Reference item
		TArray<FSoftObjectPath> AssetPaths;
		Export.GetAssetPaths(AssetPaths);

		for (const FSoftObjectPath& Path : AssetPaths)
		{
			HandleAssetPath(Path);
		}
	}
	else
	{
		const FSoftObjectPath FirstAssetPath = Export.GetFirstAssetPath();
		if(FirstAssetPath.IsAsset())
		{
			HandleAssetPath(FirstAssetPath);
		}
		else if(Export.HasData() && Export.GetData().GetScriptStruct()->IsChildOf(FAnimNextAssetEntryOutlinerData::StaticStruct()))
		{
			const FAnimNextAssetEntryOutlinerData& EntryData = Export.GetData().Get<FAnimNextAssetEntryOutlinerData>();
			if(EntryData.SoftEntryPtr.IsValid())
			{
				if(UUAFRigVMAsset* Asset = EntryData.GetEntry()->GetTypedOuter<UUAFRigVMAsset>())
				{
					HandleAssetPath(Asset);
				}
			}
		}
	}

	SetAssets(ExportAssets);

	// Try async load any missing assets
	for(const FSoftObjectPath& AssetPath : AssetsToAsyncLoad)
	{
		TWeakPtr<SCommonOutliner> WeakOutliner = StaticCastSharedRef<SCommonOutliner>(AsShared());

		AssetPath.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateLambda([WeakOutliner](const FSoftObjectPath& InSoftObjectPath, UObject* InObject)
		{
			UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(InObject);
			if(Asset == nullptr)
			{
				return;
			}

			TSharedPtr<SCommonOutliner> PinnedVariablesOutliner = WeakOutliner.Pin();
			if(!PinnedVariablesOutliner.IsValid())
			{
				return;
			}

			PinnedVariablesOutliner->HandleAssetLoaded(InSoftObjectPath, Asset);
		}));
	}
}

void SCommonOutliner::Construct(const FArguments& InArgs, const FSceneOutlinerInitializationOptions& InitOptions)
{
	BindAssetDelegate = InArgs._BindAssetDelegate;
	UnbindAssetDelegate = InArgs._UnbindAssetDelegate;
	
	// Either both bounds, or none at all
	ensure((BindAssetDelegate.IsBound() && UnbindAssetDelegate.IsBound()) || (!BindAssetDelegate.IsBound() && !UnbindAssetDelegate.IsBound()));

	SSceneOutliner::Construct(SSceneOutliner::FArguments(), InitOptions);
}

void SCommonOutliner::SetExport(const FWorkspaceOutlinerItemExport& InExport)
{
	Export = InExport;

	UpdateAssets();
}

void SCommonOutliner::HandleAssetLoaded(const FSoftObjectPath& InSoftObjectPath, UUAFRigVMAsset* InAsset)
{
	if(InAsset && Assets.Contains(TSoftObjectPtr<UUAFRigVMAsset>(InAsset)))
	{
		RegisterAssetDelegates(InAsset);

		FullRefresh();
	}
}

void SCommonOutliner::RegisterAssetDelegates(const UUAFRigVMAsset* InAsset)
{
	// Bind for any modification callbacks
	UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(InAsset);
	if(EditorData != nullptr)
	{
		BindAssetDelegate.ExecuteIfBound(EditorData);

		for (const TObjectPtr<UUAFRigVMAssetEntry>& Entry : EditorData->Entries)
		{
			if (UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(Entry.Get()))
			{
				if (TObjectPtr<const UUAFSharedVariables> SharedVariablesAsset = SharedVariablesEntry->GetAsset())
				{
					RegisterAssetDelegates(SharedVariablesAsset);
					Assets.AddUnique(Cast<UUAFRigVMAsset>(const_cast<UUAFSharedVariables*>(SharedVariablesAsset.Get())));
				}
			}
		}
	}
}

void SCommonOutliner::UnregisterAssetDelegates(const UUAFRigVMAsset* InAsset)
{
	UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(InAsset);
	if(EditorData != nullptr)
	{
		UnbindAssetDelegate.ExecuteIfBound(EditorData);

		for (const TObjectPtr<UUAFRigVMAssetEntry>& Entry : EditorData->Entries)
		{
			if (UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(Entry.Get()))
			{
				if (TObjectPtr<const UUAFSharedVariables> SharedVariablesAsset = SharedVariablesEntry->GetAsset())
				{
					UnregisterAssetDelegates(SharedVariablesAsset);
				}
			}
		}
	}
}

void SCommonOutliner::SetHighlightedItem(FSceneOutlinerTreeItemPtr Item) const
{
	GetTreeView()->Private_SetItemHighlighted(Item, true);
}

void SCommonOutliner::ClearHighlightedItem(FSceneOutlinerTreeItemPtr Item) const
{
	GetTreeView()->Private_SetItemHighlighted(Item, false);
}

bool SCommonOutliner::HasAssets() const
{
	return Assets.Num() > 0;
}
}

#undef LOCTEXT_NAMESPACE
