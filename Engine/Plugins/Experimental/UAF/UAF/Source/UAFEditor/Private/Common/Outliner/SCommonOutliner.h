// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "SSceneOutliner.h"
#include "WorkspaceAssetRegistryInfo.h"

enum class EAnimNextEditorDataNotifType : uint8;
class UUAFRigVMAsset;
struct FWorkspaceOutlinerItemExport;
class SPositiveActionButton;
class UUAFRigVMAssetEditorData;

namespace UE::Workspace
{
class IWorkspaceEditor;
struct FWorkspaceDocument;
}

namespace UE::UAF::Editor
{

DECLARE_DELEGATE_OneParam(FAssetEditorDataDelegate, UUAFRigVMAssetEditorData*)

class SCommonOutliner : public SSceneOutliner
{
	friend class FCommonOutlinerMode;
public:

	SLATE_BEGIN_ARGS(SCommonOutliner) {}
		SLATE_EVENT(FAssetEditorDataDelegate, BindAssetDelegate)
		SLATE_EVENT(FAssetEditorDataDelegate, UnbindAssetDelegate)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FSceneOutlinerInitializationOptions& InitOptions);

	void SetExport(const FWorkspaceOutlinerItemExport& InExport);
	TConstArrayView<TSoftObjectPtr<UUAFRigVMAsset>> GetAssetsView() const { return Assets; }	
	void UpdateAssets();
	bool HasAssets() const;

protected:
	void SetAssets(TConstArrayView<TSoftObjectPtr<UUAFRigVMAsset>> InAssets);
	void HandleAssetLoaded(const FSoftObjectPath& InSoftObjectPath, UUAFRigVMAsset* InAsset);
	void SetHighlightedItem(FSceneOutlinerTreeItemPtr Item) const;
	void ClearHighlightedItem(FSceneOutlinerTreeItemPtr Item) const;
	void RegisterAssetDelegates(const UUAFRigVMAsset* InAsset);
	void UnregisterAssetDelegates(const UUAFRigVMAsset* InAsset);
private:
	TArray<TSoftObjectPtr<UUAFRigVMAsset>> Assets;
	FWorkspaceOutlinerItemExport Export;

	FAssetEditorDataDelegate BindAssetDelegate;
	FAssetEditorDataDelegate UnbindAssetDelegate;
};
}
