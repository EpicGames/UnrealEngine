// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMAssetViewMenuContext.generated.h"

namespace UE::UAF::Editor
{
	class SRigVMAssetView;
}

UCLASS()
class URigVMAssetViewMenuContext : public UObject
{
	GENERATED_BODY()

	friend class UE::UAF::Editor::SRigVMAssetView;
	
	// The RigVM asset view that we are editing
	TWeakPtr<UE::UAF::Editor::SRigVMAssetView> RigVMAssetView;
};