// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/SActionMenu.h"

#include "Common/GraphEditorSchemaActions.h"
#include "Framework/Application/SlateApplication.h"
#include "IDocumentation.h"
#include "SSubobjectEditor.h"
#include "RigVMHost.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "SGraphPalette.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMModel/RigVMSchema.h"
#include "UncookedOnlyUtils.h"
#include "AnimNextRigVMAssetEditorData.h"

#define LOCTEXT_NAMESPACE "UAFEditor"

namespace UE::UAF::Editor
{

void SActionMenu::CollectAllAnimNextGraphActions(FGraphContextMenuBuilder& MenuBuilder) const
{
	// Disable reporting as the schema's SupportsX functions will output errors
	ContextData.RigVMController->EnableReporting(false);

	if (OnCollectGraphActionsCallback.IsBound())
	{
		OnCollectGraphActionsCallback.Execute(MenuBuilder, ContextData);
	}

	ContextData.RigVMController->EnableReporting(true);
}

SActionMenu::~SActionMenu()
{
	OnClosedCallback.ExecuteIfBound();
	OnCloseReasonCallback.ExecuteIfBound(bActionExecuted, false, !DraggedFromPins.IsEmpty());
}

void SActionMenu::Construct(const FArguments& InArgs, UEdGraph* InGraph, const FWorkspaceOutlinerItemExport& InExport)
{
	check(InGraph);

	ContextData.Graph = InGraph;
	DraggedFromPins = InArgs._DraggedFromPins;
	NewNodePosition = InArgs._NewNodePosition;
	OnClosedCallback = InArgs._OnClosedCallback;
	OnCollectGraphActionsCallback = InArgs._OnCollectGraphActionsCallback;
	bAutoExpandActionMenu = InArgs._AutoExpandActionMenu;
	OnCloseReasonCallback = InArgs._OnCloseReason;

	ContextData.RigVMClientHost = ContextData.Graph->GetImplementingOuter<IRigVMClientHost>();
	check(ContextData.RigVMClientHost);
	ContextData.RigVMHost = ContextData.Graph->GetTypedOuter<URigVMHost>();
	check(ContextData.RigVMHost);
	ContextData.RigVMController = ContextData.RigVMClientHost->GetRigVMClient()->GetController(ContextData.Graph);
	check(ContextData.RigVMController);
	ContextData.RigVMSchema = ContextData.RigVMController->GetGraph()->GetSchema();
	check(ContextData.RigVMSchema);
	ContextData.EditorData = ContextData.Graph->GetTypedOuter<UUAFRigVMAssetEditorData>();
	check(ContextData.EditorData);

	ContextData.Export = InExport;

	SBorder::Construct(SBorder::FArguments()
		.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
		.Padding(5)
		[
			SNew(SBox)
			.WidthOverride(400)
			.HeightOverride(400)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				[
					SAssignNew(GraphActionMenu, SGraphActionMenu)
						.OnActionSelected(this, &SActionMenu::OnActionSelected)
						.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(this, &SActionMenu::OnCreateWidgetForAction))
						.OnCollectAllActions(this, &SActionMenu::CollectAllActions)
						.OnCreateCustomRowExpander_Lambda([](const FCustomExpanderData& InActionMenuData)
						{
							// Default table row doesnt indent correctly
							return SNew(SExpanderArrow, InActionMenuData.TableRow);
						})
						.DraggedFromPins(DraggedFromPins)
						.GraphObj(ContextData.Graph)
						.AlphaSortItems(true)
						.bAllowPreselectedItemActivation(true)
				]
			]
		]
	);
}

void SActionMenu::CollectAllActions(FGraphActionListBuilderBase& OutAllActions)
{
	if (!ContextData.Graph)
	{
		return;
	}

	FGraphContextMenuBuilder MenuBuilder(ContextData.Graph);
	if (!DraggedFromPins.IsEmpty())
	{
		MenuBuilder.FromPin = DraggedFromPins[0];
	}

	// Cannot call GetGraphContextActions() during serialization and GC due to its use of FindObject()
	if(!UE::IsSavingPackage() && !IsGarbageCollecting())
	{
		CollectAllAnimNextGraphActions(MenuBuilder);
	}

	OutAllActions.Append(MenuBuilder);
}

TSharedRef<SEditableTextBox> SActionMenu::GetFilterTextBox()
{
	return GraphActionMenu->GetFilterTextBox();
}

TSharedRef<SWidget> SActionMenu::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	check(InCreateData);
	InCreateData->bHandleMouseButtonDown = false;

	const FSlateBrush* IconBrush = nullptr;
	FLinearColor IconColor;
	TSharedPtr<FAnimNextSchemaAction> Action = StaticCastSharedPtr<FAnimNextSchemaAction>(InCreateData->Action);
	if (Action.IsValid())
	{
		IconBrush = Action->GetIconBrush();
		IconColor = Action->GetIconColor();
	}

	TSharedPtr<SHorizontalBox> WidgetBox = SNew(SHorizontalBox);
	if (IconBrush)
	{
		WidgetBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 0, 0)
			[
				SNew(SImage)
				.ColorAndOpacity(IconColor)
				.Image(IconBrush)
			];
	}

	WidgetBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(IconBrush ? 4.0f : 0.0f, 0, 0, 0)
		[
			SNew(SGraphPaletteItem, InCreateData)
		];

	return WidgetBox->AsShared();
}

void SActionMenu::OnActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& SelectedAction, ESelectInfo::Type InSelectionType)
{
	if (!ContextData.Graph)
	{
		return;
	}

	if (InSelectionType != ESelectInfo::OnMouseClick  && InSelectionType != ESelectInfo::OnKeyPress && !SelectedAction.IsEmpty())
	{
		return;
	}

	for (const TSharedPtr<FEdGraphSchemaAction>& Action : SelectedAction)
	{
		if (Action.IsValid() && ContextData.Graph)
		{
			if (!bActionExecuted && (Action->GetTypeId() != FEdGraphSchemaAction_Dummy::StaticGetTypeId()))
			{
				FSlateApplication::Get().DismissAllMenus();
				bActionExecuted = true;
			}

			Action->PerformAction(ContextData.Graph, DraggedFromPins, NewNodePosition);
		}
	}
}

} 

#undef LOCTEXT_NAMESPACE
