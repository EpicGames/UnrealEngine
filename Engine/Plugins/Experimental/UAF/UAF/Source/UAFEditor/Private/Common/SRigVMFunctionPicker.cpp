// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/SRigVMFunctionPicker.h"

#include "AnimNextRigVMAsset.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Param/ParamType.h"
#include "DetailLayoutBuilder.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "UncookedOnlyUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STableRow.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SRigVMFunctionPicker"

namespace UE::UAF::Editor
{

class SRigVMFunctionRowWidget : public STableRow<TSharedPtr<SRigVMFunctionPicker::FEntry>>
{
	SLATE_BEGIN_ARGS(SRigVMFunctionRowWidget) {}

	SLATE_ATTRIBUTE(FText, HighlightText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<SRigVMFunctionPicker::FEntry> InEntry)
	{
		Entry = InEntry;
		HighlightText = InArgs._HighlightText;
		check(Entry.IsValid());

		TSharedRef<SHorizontalBox> RowContent = SNew(SHorizontalBox);

		// Return type icon — only for function entries
		if (Entry->Type == SRigVMFunctionPicker::EEntryType::Function)
		{
			TSharedPtr<SRigVMFunctionPicker::FFunctionEntry> FuncEntry = StaticCastSharedPtr<SRigVMFunctionPicker::FFunctionEntry>(Entry);
			if (FuncEntry->ReturnTypeIcon)
			{
				RowContent->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.0f, 0.0f, 2.0f, 0.0f)
				[
					SNew(SImage)
					.Image(FuncEntry->ReturnTypeIcon)
					.ColorAndOpacity(FuncEntry->ReturnTypeColor)
					.ToolTipText(FuncEntry->ReturnTypeName)
				];
			}
		}

		// Entry icon (function, asset, none, etc.)
		RowContent->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SImage)
			.Image(Entry->Icon)
		];

		// Entry name
		RowContent->AddSlot()
		.AutoWidth()
		.Padding(4.0f, 2.0f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(Entry->Name)
			.HighlightText(HighlightText)
		];

		STableRow<TSharedPtr<SRigVMFunctionPicker::FEntry>>::Construct(
			STableRow<TSharedPtr<SRigVMFunctionPicker::FEntry>>::FArguments()
			.ToolTipText(Entry->ToolTip)
			.Content()
			[
				RowContent
			],
			InOwnerTable);
	}

	TSharedPtr<SRigVMFunctionPicker::FEntry> Entry;
	TAttribute<FText> HighlightText;
};

void SRigVMFunctionPicker::Construct(const FArguments& InArgs)
{
	CurrentAsset = InArgs._CurrentAsset;
	WeakCurrentAsset = CastChecked<UUAFRigVMAsset>(InArgs._CurrentAsset.GetAsset(), ECastCheckedType::NullAllowed);
	FunctionName = InArgs._FunctionName;
	FunctionToolTip = InArgs._FunctionToolTip;
	OnRigVMFunctionPicked = InArgs._OnRigVMFunctionPicked;
	OnNewFunction = InArgs._OnNewFunction;
	bAllowNew = InArgs._AllowNew;
	bAllowClear = InArgs._AllowClear;
	OnFilterFunction = InArgs._OnFilterFunction;
	FilterAssets = InArgs._FilterAssets;
	bIsContextSensitive = InArgs._IsContextSensitive;

	TextFilter = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::Complex);

	// Inner content: search box + tree view (shared between combo and content-only modes)
	TSharedRef<SWidget> InnerContent =
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(SearchBox, SSearchBox)
			.OnTextChanged_Lambda([this](const FText& InText)
			{
				FilterText = InText;
				RequestRefreshEntries();
			})
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(TreeView, STreeView<TSharedPtr<FEntry>>)
			.SelectionMode(ESelectionMode::Single)
			.TreeItemsSource(&FilteredEntries)
			.OnGenerateRow_Lambda([this](TSharedPtr<FEntry> InItem, const TSharedRef<STableViewBase>& InOwnerTable) -> TSharedRef<ITableRow>
			{
				return SNew(SRigVMFunctionRowWidget, InOwnerTable, InItem)
					.HighlightText_Lambda([this]()
					{
						return FilterText;
					});
			})
			.OnGetChildren_Lambda([this](TSharedPtr<FEntry> InItem, TArray<TSharedPtr<FEntry>>& OutChildren)
			{
				switch(InItem->Type)
				{
				case EEntryType::Asset:
					{
						TSharedPtr<FAssetEntry> AssetEntry = StaticCastSharedPtr<FAssetEntry>(InItem);
						if(FilterText.IsEmpty())
						{
							OutChildren.Append(AssetEntry->Functions);
						}
						else
						{
							OutChildren.Append(AssetEntry->FilteredFunctions);
						}
						break;
					}
				default:
					break;
				}
			})
			.OnSelectionChanged_Lambda([this](TSharedPtr<FEntry> InItem, ESelectInfo::Type)
			{
				FSlateApplication::Get().DismissAllMenus();

				if(!InItem.IsValid())
				{
					return;
				}

				switch(InItem->Type)
				{
				case EEntryType::None:
					{
						OnRigVMFunctionPicked.ExecuteIfBound(FRigVMGraphFunctionHeader());
						break;
					}
				case EEntryType::Asset:
					break;
				case EEntryType::Function:
					{
						TSharedPtr<FFunctionEntry> FunctionEntry = StaticCastSharedPtr<FFunctionEntry>(InItem);
						OnRigVMFunctionPicked.ExecuteIfBound(FunctionEntry->FunctionHeader);
						break;
					}
				case EEntryType::NewFunction:
					{
						OnNewFunction.ExecuteIfBound();
						break;
					}
				}
			})
		];

	if (InArgs._ContentOnly)
	{
		// Content-only mode: render the search + tree directly (for embedding in tabs/popups)
		RefreshEntries();

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				InnerContent
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("ContextSensitiveTooltip", "Only show functions from the current asset context"))
				.Visibility_Lambda([this]() -> EVisibility
				{
					return FilterAssets.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					return bIsContextSensitive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InState)
				{
					bIsContextSensitive = (InState == ECheckBoxState::Checked);
					RefreshEntries();
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ContextSensitiveLabel", "Context Sensitive"))
				]
			]
		];
	}
	else
	{
		// Combo button mode: wrap content in a dropdown
		ChildSlot
		[
			SNew(SComboButton)
			.ToolTipText(FunctionToolTip)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 2.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("GraphEditor.Function_16x")))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FunctionName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
			.OnMenuOpenChanged_Lambda([this](bool bInOpen)
			{
				if(bInOpen)
				{
					SearchBox->SetText(FText::GetEmpty());
					RefreshEntries();
					RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
					{
						FSlateApplication::Get().SetKeyboardFocus(SearchBox);
						return EActiveTimerReturnType::Stop;
					}));
				}
			})
			.MenuContent()
			[
				SNew(SBox)
				.WidthOverride(300.0f)
				.HeightOverride(400.0f)
				[
					SNew(SBorder)
					[
						InnerContent
					]
				]
			]
		];
	}
}

void SRigVMFunctionPicker::RequestRefreshEntries()
{
	RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
	{
		RefreshEntries();
		return EActiveTimerReturnType::Stop;
	}));
}

void SRigVMFunctionPicker::RefreshEntries()
{
	Entries.Reset();
	FilteredEntries.Reset();
	TextFilter->SetFilterText(FilterText);

	if(bAllowClear)
	{
		TSharedPtr<FNoneEntry> NoneEntry = MakeShared<FNoneEntry>();
		NoneEntry->Name = LOCTEXT("NoneEntryLabel", "None");
		NoneEntry->ToolTip = LOCTEXT("NoneEntryTooltip", "Clear the currently selected function");
		NoneEntry->Icon = FAppStyle::GetBrush(TEXT("Icons.XCircle"));
		Entries.Add(NoneEntry);
		FilteredEntries.Add(NoneEntry);
	}

	if(bAllowNew)
	{
		TSharedPtr<FNewFunctionEntry> NewFunctionEntry = MakeShared<FNewFunctionEntry>();
		NewFunctionEntry->Name = LOCTEXT("NewFunctionEntryLabel", "New Function...");
		NewFunctionEntry->ToolTip = LOCTEXT("NewFunctionEntryTooltip", "Create a new function");
		NewFunctionEntry->Icon = FAppStyle::GetBrush(TEXT("Icons.PlusCircle"));
		Entries.Add(NewFunctionEntry);
		FilteredEntries.Add(NewFunctionEntry);
	}

	// Only show public functions — private functions are not compiled into FunctionData
	// and cannot be called via CallFunction at runtime.
	TMap<FAssetData, FRigVMGraphFunctionHeaderArray> FunctionExports;

	UncookedOnly::FUtils::GetExportedFunctionsFromAssetRegistry(AnimNextPublicGraphFunctionsExportsRegistryTag, FunctionExports);
	// TODO: Ideally we can filter functions by schema or execute context, but right now we dont expose the schema and function execute contexts are
	// all FRigVMExecuteContext, rather than the 'most derived' context in the function.
	//	UncookedOnly::FUtils::GetExportedFunctionsFromAssetRegistry(ControlRigAssetPublicGraphFunctionsExportsRegistryTag, FunctionExports);

	// Context sensitivity: remove assets not in FilterAssets
	if (bIsContextSensitive && FilterAssets.Num() > 0)
	{
		for (auto It = FunctionExports.CreateIterator(); It; ++It)
		{
			bool bKeep = false;
			for (const TWeakObjectPtr<const UUAFRigVMAsset>& WeakAsset : FilterAssets)
			{
				if (const UUAFRigVMAsset* Asset = WeakAsset.Get())
				{
					if (It->Key == FAssetData(const_cast<UUAFRigVMAsset*>(Asset)))
					{
						bKeep = true;
						break;
					}
				}
			}
			if (!bKeep)
			{
				It.RemoveCurrent();
			}
		}
	}

	for(const TPair<FAssetData, FRigVMGraphFunctionHeaderArray>& AssetExport : FunctionExports)
	{
		TSharedPtr<FAssetEntry> AssetEntry = MakeShared<FAssetEntry>();
		AssetEntry->Name = FText::FromName(AssetExport.Key.AssetName);
		AssetEntry->ToolTip = FText::FromString(AssetExport.Key.GetFullName());
		AssetEntry->Asset = AssetExport.Key;
		Entries.Add(AssetEntry);

		for(const FRigVMGraphFunctionHeader& FunctionHeader : AssetExport.Value.Headers)
		{
			if (OnFilterFunction.IsBound() && !OnFilterFunction.Execute(FunctionHeader))
			{
				continue;
			}

			TSharedPtr<FFunctionEntry> FunctionEntry = MakeShared<FFunctionEntry>();
			FunctionEntry->Name = FText::FromName(FunctionHeader.Name);
			FunctionEntry->ToolTip = FunctionHeader.Description.Len() > 0 ? FText::FromString(FunctionHeader.Description) : FText::FromName(FunctionHeader.Name);
			FunctionEntry->Icon = FAppStyle::GetBrush(TEXT("GraphEditor.Function_16x"));
			FunctionEntry->FunctionHeader = FunctionHeader;

			// Extract return type icon from the first output argument
			for (const FRigVMGraphFunctionArgument& Arg : FunctionHeader.Arguments)
			{
				if (!Arg.IsExecuteContext() && Arg.Direction == ERigVMPinDirection::Output)
				{
					FRigVMTemplateArgumentType RigVMArgType(Arg.CPPType, Arg.CPPTypeObject.Get());
					FEdGraphPinType PinType = UncookedOnly::FUtils::GetPinTypeFromParamType(
						FAnimNextParamType::FromRigVMTemplateArgument(RigVMArgType));
					FunctionEntry->ReturnTypeIcon = FBlueprintEditorUtils::GetIconFromPin(PinType, true);
					FunctionEntry->ReturnTypeColor = GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);
					FunctionEntry->ReturnTypeName = UEdGraphSchema_K2::TypeToText(PinType);
					break;
				}
			}

			AssetEntry->Functions.Add(FunctionEntry);
		}
	}

	for(const TSharedPtr<FEntry>& Entry : Entries)
	{
		TreeView->SetItemExpansion(Entry, true);
	}

	if(FilterText.IsEmpty())
	{
		FilteredEntries.Reserve(Entries.Num());
	}

	// Filter if we need to
	for(const TSharedPtr<FEntry>& Entry : Entries)
	{
		switch(Entry->Type)
		{
		case EEntryType::Asset:
			{
				TSharedPtr<FAssetEntry> AssetEntry = StaticCastSharedPtr<FAssetEntry>(Entry);
				AssetEntry->FilteredFunctions.Reset();
				if(FilterText.IsEmpty())
				{
					FilteredEntries.Add(AssetEntry);
					AssetEntry->FilteredFunctions.Reserve(AssetEntry->Functions.Num());
					for(const TSharedPtr<FEntry>& FunctionEntry : AssetEntry->Functions)
					{
						AssetEntry->FilteredFunctions.Add(FunctionEntry);
					}
				}
				else
				{
					bool bAddedAsset = false;
					if(TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(AssetEntry->Name.ToString())))
					{
						FilteredEntries.Add(AssetEntry);
						bAddedAsset = true;
					}

					// Also show asset if one of the functions passes the filter
					for(const TSharedPtr<FEntry>& FunctionEntry : AssetEntry->Functions)
					{
						if(TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(FunctionEntry->Name.ToString())))
						{
							AssetEntry->FilteredFunctions.Add(FunctionEntry);
							if(!bAddedAsset)
							{
								FilteredEntries.Add(AssetEntry);
								bAddedAsset = true;
							}
						}
					}
				}
				break;
			}
		default:
			break;
		}
	}

	TreeView->RequestTreeRefresh();
}

}

#undef LOCTEXT_NAMESPACE