// Copyright Epic Games, Inc. All Rights Reserved.

#include "SUAFBrowser.h"

#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "AssetToolsModule.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Common/AssetPreview/AnimSequenceAssetPreview.h"
#include "Common/AssetPreview/IUAFAssetPreview.h"
#include "Common/AssetPreview/UAFAssetPreviewFactorySubsystem.h"
#include "Common/SUAFBrowserFilterSuggestionStrip.h"
#include "Common/UAFBrowserMenuContext.h"
#include "Common/UAFBrowserTabSummoner.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "Engine/Blueprint.h"
#include "Factories/BlueprintFactory.h"
#include "Filters/CustomClassFilterData.h"
#include "Framework/Application/SlateApplication.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/SClassPickerDialog.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "Interfaces/IPluginManager.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageName.h"
#include "NewAssetContextMenu.h"
#include "SAssetPicker.h"
#include "SAssetView.h"
#include "SFilterList.h"
#include "Sound/SoundWave.h"
#include "Settings/UAFEditorProjectSettings.h"
#include "Settings/UAFEditorUserSettings.h"
#include "StatusBarSubsystem.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "TaggedAssetBrowserFilters/TaggedAssetBrowser_CommonFilters.h"
#include "ToolMenus.h"
#include "UncookedOnlyUtils.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "SActionButton.h"
#include "SPositiveActionButton.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SAssetMenuIcon.h"
#include "Widgets/STaggedAssetBrowser.h"

#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"

#define LOCTEXT_NAMESPACE "SUAFBrowser"

namespace UE::UAF::Editor
{
	
FName SUAFBrowser::AddNewMenuName = FName(TEXT("UAFBrowser.AddNewContextMenu"));

/** Class filter that restricts the class viewer to children of a set of allowed parent classes. */
class FUAFBlueprintParentClassFilter : public IClassViewerFilter
{
public:
	TSet<const UClass*> AllowedParentClasses;
	EClassFlags DisallowedClassFlags = CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Hidden | CLASS_HideDropDown;

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedParentClasses, InClass) != EFilterReturn::Failed;
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedParentClasses, InUnloadedClassData) != EFilterReturn::Failed;
	}
};

SUAFBrowser::~SUAFBrowser()
{
	if (UUAFEditorUserSettings* UAFSettings = GetMutableDefault<UUAFEditorUserSettings>())
	{
		UAFSettings->OnBrowserHiddenColumnsChanged.RemoveAll(this);
	}
}

void SUAFBrowser::Construct(const FArguments& InArgs, const TSharedPtr<UE::Workspace::IWorkspaceEditor>& InHostingApp, ETabState::Type InTabState)
{
	WeakHostingApp = InHostingApp;
	TabState = InTabState;

	// Register the shared add-new ToolMenu once, subsequent browser instances reuse the same registration.
	if (!UToolMenus::Get()->IsMenuRegistered(AddNewMenuName))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(AddNewMenuName, NAME_None, EMultiBoxType::Menu))
		{
			// Flat promoted entries for the active section's filter classes, each triggering a name prompt before creation
			Menu->AddDynamicSection(TEXT("UAFBrowserPrioritized"),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					UUAFBrowserMenuContext* Ctx = InMenu->Context.FindContext<UUAFBrowserMenuContext>();
					if (!Ctx)
					{
						return;
					}

					TSharedPtr<SUAFBrowser> Browser = Ctx ? Ctx->WeakOwningBrowser.Pin() : nullptr;
					if (!Browser)
					{
						return;
					}

					TArray<UClass*> CurrentSectionClasses = Browser->GetCurrentSectionFilterClasses();
					if (CurrentSectionClasses.IsEmpty())
					{
						return;
					}

					IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
					TArray<UFactory*> AllFactories = AssetTools.GetNewAssetFactories();

					FToolMenuSection& PrioritizedSection = InMenu->AddSection(
						TEXT("UAFBrowserPrioritized"), LOCTEXT("SuggestedTypes", "Suggested Types"));

					// Mirror the SFactoryMenuEntry layout from NewAssetContextMenu.cpp.
					// Read the same CVar that ContentBrowserStyle.h uses for IsNewStyleEnabled().
					const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("ContentBrowser.EnableNewStyle"));
					const bool bNewStyle = CVar && CVar->GetBool();
					static constexpr float DefaultMenuIconSize = 14.0f;
					static constexpr float VerticalEntryPadding = 4.0f;
					const FVector2D IconContainerSize = bNewStyle ? FVector2D(24, 24) : FVector2D(32, 32);
					const FVector2D IconSize = bNewStyle ? FVector2D(16, 16) : FVector2D(28, 28);
					const FMargin IconSlotPadding = bNewStyle ? FMargin(2, 0, 3, 0) : FMargin(0, 0, 0, 1);
					const FMargin LabelSlotPadding = bNewStyle ? FMargin(4, 0, 6, 0) : FMargin(4, 0, 4, 0);
					const float VPadAdj = ((IconContainerSize.Y - DefaultMenuIconSize) / 2.0f) - VerticalEntryPadding;
					const FMargin ChildSlotPadding = bNewStyle ? FMargin(0, -VPadAdj, 0, -VPadAdj) : FMargin(0);

					// Weak pointer extracted at section-build time, captured by click-time lambdas below.
					TWeakPtr<SUAFBrowser> WeakBrowser = Ctx->WeakOwningBrowser;

					// Helper that builds and registers one menu entry for the prioritized section.
					auto AddSectionEntry = [&](UClass* EntryIconClass
						, const FName& EntryIconOverride
						, const FText& EntryDisplayName
						, TFunction<void(TSharedPtr<SUAFBrowser>)> EntryOnClicked)
					{
						TSharedRef<SWidget> EntryWidget =
							SNew(SBox)
							.Padding(ChildSlotPadding)
							[
								SNew(SHorizontalBox)
								.ToolTipText(EntryDisplayName)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								.Padding(IconSlotPadding)
								[
									SNew(SAssetMenuIcon, EntryIconClass, EntryIconOverride)
									.IconContainerSize(IconContainerSize)
									.IconSize(IconSize)
								]
								+ SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.Padding(LabelSlotPadding)
								[
									SNew(STextBlock)
									.Font(FAppStyle::GetFontStyle("LevelViewportContextMenu.AssetLabel.Text.Font"))
									.Text(EntryDisplayName)
								]
							];

						PrioritizedSection.AddEntry(
							FToolMenuEntry::InitMenuEntry(
								FName(*EntryDisplayName.ToString()),
								FUIAction(FExecuteAction::CreateLambda([WeakBrowser, EntryOnClicked]()
								{
									TSharedPtr<SUAFBrowser> PinnedBrowser = WeakBrowser.Pin();
									if (PinnedBrowser)
									{
										EntryOnClicked(PinnedBrowser);
									}
								})),
								EntryWidget
							)
						);
					};

					for (UClass* TargetClass : CurrentSectionClasses)
					{
						// Collect ALL factories whose supported class is a child of TargetClass.
						// There can be multiple (e.g. several child-class factories for one base class).
						TArray<UFactory*> MatchedFactories;
						for (UFactory* Factory : AllFactories)
						{
							UClass* Supported = Factory->GetSupportedClass();
							if (Supported && Supported->IsChildOf(TargetClass))
							{
								MatchedFactories.Add(Factory);
							}
						}

						if (!MatchedFactories.IsEmpty())
						{
							// Factory path: one entry per factory.
							for (UFactory* Factory : MatchedFactories)
							{
								UClass* IconClass = Factory->GetSupportedClass();
								const FText DisplayName = Factory->GetDisplayName();
								const FName IconOverride = bNewStyle ? Factory->GetNewAssetIconOverride() : Factory->GetNewAssetThumbnailOverride();
								TWeakObjectPtr<UClass> WeakFactoryClass(Factory->GetClass());

								AddSectionEntry(IconClass, IconOverride, DisplayName,
									[WeakFactoryClass, DisplayName](TSharedPtr<SUAFBrowser> Browser)
									{
										UClass* FactoryClass = WeakFactoryClass.Get();
										if (FactoryClass)
										{
											Browser->CreateAssetWithNameDialog(FactoryClass, DisplayName);
										}
									});
							}
						}
						else if (FKismetEditorUtilities::CanCreateBlueprintOfClass(TargetClass))
						{
							// Blueprint path: show a class picker filtered to children of TargetClass,
							// with TargetClass pre-highlighted as the common/default choice.
							const FText DisplayName = TargetClass->GetDisplayNameText();
							TWeakObjectPtr<UClass> WeakParentClass(TargetClass);

							AddSectionEntry(UBlueprint::StaticClass(), NAME_None, DisplayName,
								[WeakParentClass, DisplayName](TSharedPtr<SUAFBrowser> Browser)
								{
									UClass* DefaultParentClass = WeakParentClass.Get();
									if (!DefaultParentClass)
									{
										return;
									}

									FClassViewerInitializationOptions Options;
									Options.Mode = EClassViewerMode::ClassPicker;
									Options.bIsBlueprintBaseOnly = true;
									Options.bShowNoneOption = false;
									Options.ExtraPickerCommonClasses.Add(DefaultParentClass);

									TSharedPtr<FUAFBlueprintParentClassFilter> Filter = MakeShared<FUAFBlueprintParentClassFilter>();
									Filter->AllowedParentClasses.Add(DefaultParentClass);
									Options.ClassFilters.Add(Filter.ToSharedRef());

									const FText TitleText = FText::Format(
										LOCTEXT("PickBlueprintParentClass", "Pick Parent Class for New {0} Blueprint"),
										DisplayName);

									UClass* ChosenClass = nullptr;
									const bool bPicked = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, DefaultParentClass);
									if (!bPicked)
									{
										return;
									}
									if (!ChosenClass)
									{
										ChosenClass = DefaultParentClass;
									}

									// Creating a temp factory to spawn assets is normal.
									UBlueprintFactory* BPFactory = NewObject<UBlueprintFactory>(GetTransientPackage());
									BPFactory->ParentClass = ChosenClass;
									BPFactory->bSkipClassPicker = true;
									Browser->CreateAssetWithNameDialog(BPFactory, DisplayName);
								});
						}
						// else: import-only class — no Add entry
					}
				}));

			// Fallback: full factory list (default content-browser behavior) shown only when the
			// active section has no class filter — i.e. UAFBrowserPrioritized contributed nothing.
			Menu->AddDynamicSection(TEXT("UAFBrowserAllTypes"),
				FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					UUAFBrowserMenuContext* Ctx = InMenu->Context.FindContext<UUAFBrowserMenuContext>();
					if (!Ctx)
					{
						return;
					}

					TSharedPtr<SUAFBrowser> Browser = Ctx ? Ctx->WeakOwningBrowser.Pin() : nullptr;
					if (!Browser)
					{
						return;
					}

					// Yield to the prioritized section whenever it has items to show
					if (!Browser->GetCurrentSectionFilterClasses().IsEmpty())
					{
						return;
					}

					TWeakPtr<SUAFBrowser> WeakBrowser = Ctx->WeakOwningBrowser;

					FNewAssetContextMenu::FOnNewAssetRequested OnNewAsset =
						FNewAssetContextMenu::FOnNewAssetRequested::CreateLambda(
							[WeakBrowser](const FName& /*Path*/, TWeakObjectPtr<UClass> FactoryClass)
							{
								TSharedPtr<SUAFBrowser> PinnedBrowser = WeakBrowser.Pin();
								UClass* FactoryClassPtr = FactoryClass.Get();
								if (!PinnedBrowser || !FactoryClassPtr)
								{
									return;
								}
								UFactory* TempFactory = NewObject<UFactory>(GetTransientPackage(), FactoryClassPtr);
								PinnedBrowser->CreateAssetWithNameDialog(FactoryClassPtr, TempFactory->GetDisplayName());
							});

					FNewAssetContextMenu::FOnImportAssetRequested OnImport =
						FNewAssetContextMenu::FOnImportAssetRequested::CreateLambda(
							[WeakBrowser](const FName& /*Path*/)
							{
								TSharedPtr<SUAFBrowser> PinnedBrowser = WeakBrowser.Pin();
								if (!PinnedBrowser)
								{
									return;
								}
								const FString NewAssetPath = PinnedBrowser->GetNewAssetPath();
								FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools")
									.Get().ImportAssetsWithDialog(NewAssetPath);
							});

					TArray<FName> Paths = { FName(*Browser->GetNewAssetPath()) };
					FNewAssetContextMenu::MakeContextMenu(InMenu, Paths, OnImport, OnNewAsset);
				}));
		}
	}

	if (UUAFEditorUserSettings* UAFSettings = GetMutableDefault<UUAFEditorUserSettings>())
	{
		UAFSettings->OnBrowserHiddenColumnsChanged.AddSP(this, &SUAFBrowser::OnExternalColumnVisibilityChanged);
	}

	FMargin DrawerPadding = TabState == ETabState::SidebarTab || TabState == ETabState::StatusbarTab
		? 4.0f
		: 0.0f;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(DrawerPadding)
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(DrawerPadding)
			[
				Construct_GetMainBrowserContents()
			]
		]
	];
}

TSharedRef<SWidget> SUAFBrowser::Construct_GetMainBrowserContents()
{
	bool bIsOrientedHorizontally = TabState != ETabState::SidebarTab;
	EOrientation Orientation = bIsOrientedHorizontally ? EOrientation::Orient_Horizontal : EOrientation::Orient_Vertical;

	return SNew(SSplitter)
		.Orientation(Orientation)
		+SSplitter::Slot()
		.Value(bIsOrientedHorizontally ? 0.85f : 0.15f)
		[
			bIsOrientedHorizontally
				? Construct_GetTaggedAssetBrowser()
				: Construct_GetAssetPreviewWidget()
		]
		+SSplitter::Slot()
		.Value(bIsOrientedHorizontally ? 0.15f : 0.85f)
		[
			bIsOrientedHorizontally
				? Construct_GetAssetPreviewWidget()
				: Construct_GetTaggedAssetBrowser()
		];
}

TSharedRef<SWidget> SUAFBrowser::Construct_GetTaggedAssetBrowser()
{
	// Setup tagged asset browser config
	STaggedAssetBrowser::FInterfaceOverrideProfiles OverrideProfiles;
	OverrideProfiles.AssetViewOptionsProfileName = FName("TaggedAssetBrowser");
	OverrideProfiles.DefaultFilterMenuExpansion = EAssetTypeCategories::Animation;
	OverrideProfiles.FilterBarSaveName = FName("TaggedAssetBrowser.UAFBrowser");

	FDefaultDetailsTabConfiguration DefaultDetailsTabConfiguration;
	DefaultDetailsTabConfiguration.bUseDefaultDetailsTab = false;

	FSoftObjectPath CategoryConfigurationPath = GetDefault<UUAFEditorProjectSettings>()->UAFBrowserCategoryConfiguration;
	UObject* CategoryConfigurationObject = CategoryConfigurationPath.TryLoad();

	if (UTaggedAssetBrowserConfiguration* CategoryConfiguration = Cast<UTaggedAssetBrowserConfiguration>(CategoryConfigurationObject))
	{
		SAssignNew(SuggestionStrip, SUAFBrowserFilterSuggestionStrip)
			.OnFilterSuggestionSelected(TDelegate<void()>::CreateSP(this, &SUAFBrowser::OnFilterSuggestionSelected));

		// We intentionally do not restrict AvailableClasses as this will filter types we may not know about from beind displayed
		// Ex: Even in contexts such as filtering by tag
		SAssignNew(TaggedAssetBrowser, STaggedAssetBrowser, *CategoryConfiguration)
			.AvailableClasses({})
			.DefaultDetailsTabConfiguration(DefaultDetailsTabConfiguration)
			.InterfaceOverrideProfiles(OverrideProfiles)
			.OnOverrideAssetPickerConfig(STaggedAssetBrowser::FOnOverrideAssetPickerConfig::CreateSP(this, &SUAFBrowser::OnOverrideAssetPickerConfig))
			.OnActiveSectionChanged(STaggedAssetBrowser::FOnActiveSectionChanged::CreateSP(this, &SUAFBrowser::OnActiveSectionChanged))
			.OnPrimaryFilterSelectionChanged(STaggedAssetBrowser::FOnPrimaryFilterSelectionChanged::CreateSP(this, &SUAFBrowser::OnPrimaryFilterSelectionChanged))
			.AdditionalBottomWidget(SuggestionStrip);

		// @TODO: Implement below, not used at the moment.
		// Initialize current section from the browser's initial state so the Add button is correct before any section switch
		CurrentActiveSection = TaggedAssetBrowser->GetActiveSection();

		return TaggedAssetBrowser.ToSharedRef();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SUAFBrowser::Construct_GetAssetPreviewWidget()
{
	// This is typically customized via the exposed AssetPreviewSlot. See: OnAssetSelected.
	return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Expose(AssetPreviewSlot)
		[
			Construct_GetAssetPreviewNotSupportedWidget()
		];
}

TSharedRef<SWidget> SUAFBrowser::Construct_GetAssetPreviewNotSupportedWidget()
{
	return SNew(STextBlock)
		.Justification(ETextJustify::Center)
		.AutoWrapText(true)
		.Text(LOCTEXT("AssetPreviewNotSetup", "Asset Preview Not Setup for Type"));
}

void SUAFBrowser::OnOverrideAssetPickerConfig(FAssetPickerConfig& Config)
{
	static const FString ContentBrowserIniSection = TEXT("ContentBrowser");
	static const FString UAFBrowserConfigName = TEXT("UAFBrowser");

	Config.OnExtendAssetPickerTopBar.BindSP(this, &SUAFBrowser::OnExtendAssetPickerTopBar);
	Config.OnSearchTextChanged.BindSP(this, &SUAFBrowser::OnContentSearchTextChanged);
	Config.bAddFilterUI = true;
	Config.bAllowDragging = true;
	Config.bShowPathInColumnView = true;
	Config.SaveSettingsName = UAFBrowserConfigName;

	// Apply user-configurable defaults from UUAFEditorUserSettings.
	if (const UUAFEditorUserSettings* UAFSettings = GetDefault<UUAFEditorUserSettings>())
	{
		// If the user hasn't set any view type, default to column view
		int32 ViewType_Discarded = INDEX_NONE;
		if (!GConfig->GetInt(*ContentBrowserIniSection, *(UAFBrowserConfigName + TEXT(".CurrentViewType")), ViewType_Discarded, GEditorPerProjectIni))
		{
			Config.InitialAssetViewType = EAssetViewType::Column;
		}

		Config.HiddenColumnNames = UAFSettings->UAFBrowserHiddenColumns;
	}
	
	Config.OnGetAssetContextMenu.Unbind();
	Config.OnAssetsActivated.Unbind();
	Config.OnGetCustomAssetToolTip.Unbind();

	auto OpenAssetCallback = [](const FAssetData& AssetData)
	{
		if (UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			EditorSubsystem->OpenEditorForAsset(AssetData.ToSoftObjectPath());
		}
	};
	Config.OnAssetDoubleClicked.BindLambda(OpenAssetCallback);

	Config.OnAssetSelected.BindSP(this, &SUAFBrowser::OnAssetSelected);
	Config.OnAssetsDragged.BindSP(this, &SUAFBrowser::OnAssetsDragged);
	Config.OnColumnVisibilityChanged.BindSP(this, &SUAFBrowser::OnBrowserColumnVisibilityChanged);
	Config.SaveAssetViewSettingsDelegates.Add(&SaveAssetViewSettingsDelegate);
	Config.LoadAssetViewSettingsDelegates.Add(&LoadAssetViewSettingsDelegate);

	PopulatePluginFilters(Config);
}

void SUAFBrowser::OnExtendAssetPickerTopBar(TSharedRef<SHorizontalBox> InAssetPickerTopBar)
{
	// Insert add/import button(s) at position 0 (before the search bar and filter UI)
	SAssignNew(AddImportButtonContainer, SHorizontalBox);
	AddImportButtonContainer->AddSlot()
		.AutoWidth()
		[
			BuildAddImportButtons()
		];

	InAssetPickerTopBar->InsertSlot(0)
		.AutoWidth()
		[
			AddImportButtonContainer.ToSharedRef()
		];

	if (TabState == ETabState::SidebarTab || TabState == ETabState::StatusbarTab)
	{
		// Note: Value from `FAssetPickerConfig::OnExtendAssetPickerTopBar`.
		// We could make it expose those indicies, but given expected usage that feels over-engineered for now.

		const int ShowDeveloperContentIndex = 2;

		InAssetPickerTopBar->InsertSlot(ShowDeveloperContentIndex + 2)
		.AutoWidth()
		[
			OnExtendAssetPickerTopBar_GetDrawerDockButton()
		];

		InAssetPickerTopBar->InsertSlot(ShowDeveloperContentIndex + 3)
		.AutoWidth()
		[
			OnExtendAssetPickerTopBar_GetToggleSwapHotkeyButton()
		];
	}
}

TSharedRef<SWidget> SUAFBrowser::OnExtendAssetPickerTopBar_GetDrawerDockButton()
{
	if (TabState == ETabState::SidebarTab || TabState == ETabState::StatusbarTab)
	{
		return
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("DockInLayout_Tooltip", "Docks UAF Browser in Tab."))
			.ContentPadding(FMargin(2.f, 4.0f, 2.0f, 4.0f))
			.OnClicked(this, &SUAFBrowser::OnDockDrawerButtonClicked)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.Layout"))
			];
	}
	
	return SNullWidget::NullWidget; 
}

TSharedRef<SWidget> SUAFBrowser::OnExtendAssetPickerTopBar_GetToggleSwapHotkeyButton()
{
	if (TabState == ETabState::SidebarTab || TabState == ETabState::StatusbarTab)
	{
		auto GetSwapBrowserIcon = []() -> const FSlateBrush*
		{
			if (UUAFEditorUserSettings* UAFEditorUserSettings = GetMutableDefault<UUAFEditorUserSettings>())
			{
				return FAppStyle::Get().GetBrush(UAFEditorUserSettings->bSummonUAFBrowserHotkeyOpensSidebar
					? "EditorViewport.Right"
					: "EditorViewport.Bottom");
			}

			return nullptr;
		};

		return
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("SwapHotkey_Tooltip", "Swaps UAF Browser Hotkey Between Sidebar & StatusBar."))
			.ContentPadding(FMargin(2.f, 4.0f, 2.0f, 4.0f))
			.OnClicked(this, &SUAFBrowser::OnSwapHotkeyButtonClicked)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(TAttribute<const FSlateBrush*>::CreateLambda(GetSwapBrowserIcon))
			];
	}

	return SNullWidget::NullWidget;
}

FReply SUAFBrowser::OnSwapHotkeyButtonClicked()
{
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> HostingApp = WeakHostingApp.Pin())
	{
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ForceDismissDrawer();
		HostingApp->GetTabManager()->TryDismissSidebarTab(FUAFBrowserTabSummoner::SidebarTabID);

		if (UUAFEditorUserSettings* UAFEditorUserSettings = GetMutableDefault<UUAFEditorUserSettings>())
		{
			UAFEditorUserSettings->bSummonUAFBrowserHotkeyOpensSidebar = !UAFEditorUserSettings->bSummonUAFBrowserHotkeyOpensSidebar;
		}
	}
	return FReply::Handled();
}

FReply SUAFBrowser::OnDockDrawerButtonClicked()
{
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> HostingApp = WeakHostingApp.Pin())
	{
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ForceDismissDrawer();
		HostingApp->GetTabManager()->TryDismissSidebarTab(FUAFBrowserTabSummoner::SidebarTabID);

		if (TSharedPtr<SDockTab> ExistingTab = HostingApp->GetTabManager()->TryInvokeTab(FUAFBrowserTabSummoner::DockedTabID))
		{
			ExistingTab->ActivateInParent(ETabActivationCause::SetDirectly);
		}
	}
	return FReply::Handled();
}

void SUAFBrowser::OnAssetSelected(const FAssetData& InAssetData)
{
	if (UClass* AssetClass = InAssetData.GetClass())
	{
		// Handle selection of same types
		if (TSharedPtr<IUAFAssetPreview> AssetPreviewWidget = AssetPreviewWidgetWeak.Pin())
		{
			if (AssetClass == AssetPreviewWidget->GetAssetPreviewType())
			{
				AssetPreviewWidget->OnSameTypeAssetSelectedAsync(InAssetData);
				return;
			}
		}

		// Handle selection of different or unsupported types
		if (TSharedPtr<FUAFAssetPreviewFactory> AssetPreviewFactory = GEditor->GetEditorSubsystem<UUAFAssetPreviewFactorySubsystem>()->GetAssetPreviewFactory(AssetClass))
		{
			TSharedPtr<IUAFAssetPreview> AssetPreview = AssetPreviewFactory->CreateAssetPreviewWidget(WeakHostingApp.Pin(), InAssetData);
			AssetPreviewSlot->AttachWidget(
				AssetPreview.ToSharedRef()
			);
			AssetPreviewSlot->SetHorizontalAlignment(HAlign_Fill);
			AssetPreviewSlot->SetVerticalAlignment(VAlign_Fill);

			// We don't attempt to persist the preview between different asset type clicks. Doing so would save perf if clicking:
			// type -> unsupported preview type -> previous type. But that is an edge case w/ overall low cost.
			AssetPreviewWidgetWeak = AssetPreview;
		}
		else
		{
			AssetPreviewSlot->AttachWidget(
				Construct_GetAssetPreviewNotSupportedWidget()
			);
			AssetPreviewSlot->SetHorizontalAlignment(HAlign_Center);
			AssetPreviewSlot->SetVerticalAlignment(VAlign_Center);

			AssetPreviewWidgetWeak = nullptr;
		}
	}
}

void SUAFBrowser::OnAssetsDragged(const TArray<FAssetData>& InAssetData)
{
	// Dismiss all drawers immediately on drag, our workflow involves sidebar drawers that will cover the details panel
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> HostingApp = WeakHostingApp.Pin())
	{
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ForceDismissDrawer();
		HostingApp->GetTabManager()->TryDismissSidebarTab(FUAFBrowserTabSummoner::SidebarTabID);
	}
}

void SUAFBrowser::OnBrowserColumnVisibilityChanged(const TArray<FString>& HiddenColumnIds)
{
	if (UUAFEditorUserSettings* UAFSettings = GetMutableDefault<UUAFEditorUserSettings>())
	{
		if (UAFSettings->UAFBrowserHiddenColumns != HiddenColumnIds)
		{
			UAFSettings->UAFBrowserHiddenColumns = HiddenColumnIds;
			UAFSettings->SaveConfig();

			// Write to .ini
			SaveAssetViewSettingsDelegate.ExecuteIfBound();

			UAFSettings->OnBrowserHiddenColumnsChanged.Broadcast(HiddenColumnIds);
		}
	}
}

void SUAFBrowser::OnExternalColumnVisibilityChanged(const TArray<FString>& HiddenColumnIds)
{
	// Read from .ini
	LoadAssetViewSettingsDelegate.ExecuteIfBound();
}

void SUAFBrowser::OnActiveSectionChanged(const UTaggedAssetBrowserSection* NewSection)
{
	CurrentActiveSection = NewSection;

	if (AddImportButtonContainer.IsValid())
	{
		AddImportButtonContainer->ClearChildren();
		AddImportButtonContainer->AddSlot()
			.AutoWidth()
			[
				BuildAddImportButtons()
			];
	}
}

void SUAFBrowser::OnPrimaryFilterSelectionChanged()
{
	if (AddImportButtonContainer.IsValid())
	{
		AddImportButtonContainer->ClearChildren();
		AddImportButtonContainer->AddSlot()
			.AutoWidth()
			[
				BuildAddImportButtons()
			];
	}
}

TSharedRef<SWidget> SUAFBrowser::BuildAddImportButtons()
{
	TArray<UClass*> SectionClasses = GetCurrentSectionFilterClasses();
	const EUAFAddButtonMode Mode = GetAddButtonMode(SectionClasses);

	TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);

	if (Mode != EUAFAddButtonMode::ImportOnly)
	{
		Box->AddSlot()
			.Padding(FMargin(2, 0, 6, 0))
			.AutoWidth()
			[
				SNew(SPositiveActionButton)
				.Text(LOCTEXT("Add", "Add"))
				.OnGetMenuContent(this, &SUAFBrowser::BuildAddMenu)
			];
	}

	if (Mode == EUAFAddButtonMode::ImportOnly || Mode == EUAFAddButtonMode::Both)
	{
		Box->AddSlot()
			.Padding(FMargin(2, 0, 6, 0))
			.AutoWidth()
			[
				SNew(SActionButton)
				.Text(LOCTEXT("Import", "Import"))
				.Icon(FAppStyle::Get().GetBrush("LevelEditor.ImportContent"))
				.OnClicked(this, &SUAFBrowser::OnImportButtonClicked)
			];
	}

	return Box;
}

TSharedRef<SWidget> SUAFBrowser::BuildAddMenu()
{
	// Lifetime handled by ToolMenus system
	UUAFBrowserMenuContext* Ctx = NewObject<UUAFBrowserMenuContext>();
	Ctx->WeakOwningBrowser = SharedThis(this).ToWeakPtr();

	FToolMenuContext ToolMenuContext;
	ToolMenuContext.AddObject(Ctx);
	return UToolMenus::Get()->GenerateWidget(AddNewMenuName, ToolMenuContext);
}

FReply SUAFBrowser::OnImportButtonClicked()
{
	const FString NewAssetPath = GetNewAssetPath();
	FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools")
		.Get().ImportAssetsWithDialog(NewAssetPath);
	return FReply::Handled();
}

FString SUAFBrowser::GetNewAssetPath() const
{
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> Editor = WeakHostingApp.Pin())
	{
		if (UObject* FocusedAsset = Editor->GetFocusedAsset())
		{
			return FPackageName::GetLongPackagePath(FocusedAsset->GetOutermost()->GetName());
		}
		// Fallback: workspace asset's folder
		if (UObject* WorkspaceAsset = Editor->GetWorkspaceAsset())
		{
			return FPackageName::GetLongPackagePath(WorkspaceAsset->GetOutermost()->GetName());
		}
	}
	return TEXT("/Game");
}

TArray<UClass*> SUAFBrowser::GetCurrentSectionFilterClasses() const
{
	TArray<UClass*> Result;

	if (!TaggedAssetBrowser)
	{
		return Result;
	}

	// Prefer the currently selected primary filters; fall back to all filters in the section
	// so that GetAddButtonMode() returns the correct mode even before the user picks a filter.
	TArray<const UTaggedAssetBrowserFilterBase*> Filters = TaggedAssetBrowser->GetSelectedPrimaryFilters();
	if (Filters.IsEmpty())
	{
		Filters = TaggedAssetBrowser->GetAllPrimaryFilters();
	}

	for (const UTaggedAssetBrowserFilterBase* Filter : Filters)
	{
		if (const UTaggedAssetBrowserFilter_Class* ClassFilter = Cast<UTaggedAssetBrowserFilter_Class>(Filter))
		{
			for (const TObjectPtr<UClass>& Class : ClassFilter->Classes)
			{
				if (Class)
				{
					Result.AddUnique(Class);
				}
			}
		}
	}
	return Result;
}

SUAFBrowser::EUAFAddButtonMode SUAFBrowser::GetAddButtonMode(const TArray<UClass*>& SectionClasses) const
{
	if (SectionClasses.IsEmpty())
	{
		return EUAFAddButtonMode::Default;
	}

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	TArray<UFactory*> AllFactories = AssetTools.GetNewAssetFactories();

	bool bAnyCreatable = false;
	bool bAnyImportOnly = false;

	for (UClass* Class : SectionClasses)
	{
		// Priority 1: a dedicated creatable factory exists
		const bool bHasFactory = AllFactories.ContainsByPredicate([Class](UFactory* Factory)
		{
			UClass* Supported = Factory->GetSupportedClass();
			return Supported && (Supported == Class || Supported->IsChildOf(Class));
		});

		// Priority 2: no factory but the class is blueprintable (child Blueprint can be created)
		const bool bIsBlueprintable = !bHasFactory && FKismetEditorUtilities::CanCreateBlueprintOfClass(Class);

		if (bHasFactory || bIsBlueprintable)
		{
			bAnyCreatable = true;
		}
		else
		{
			bAnyImportOnly = true;
		}
	}

	if (bAnyCreatable && bAnyImportOnly)
	{
		return EUAFAddButtonMode::Both;
	}
	if (bAnyCreatable)
	{
		return EUAFAddButtonMode::AddOnly;
	}
	return EUAFAddButtonMode::ImportOnly;
}

void SUAFBrowser::CreateAssetWithNameDialog(UClass* InFactoryClass, const FText& AssetTypeName)
{
	// Creating a temp factory to spawn assets is normal.
	UFactory* Factory = NewObject<UFactory>(GetTransientPackage(), InFactoryClass);
	CreateAssetWithNameDialog(Factory, AssetTypeName);
}

void SUAFBrowser::CreateAssetWithNameDialog(UFactory* InFactory, const FText& AssetTypeName)
{
	UFactory* Factory = InFactory;
	const FString NewAssetPath = GetNewAssetPath();

	// Allow the factory to configure its properties (e.g. Blueprint class picker) before we ask for a name.
	// Matches the behavior of IAssetTools::CreateAssetWithDialog.
	FEditorDelegates::OnConfigureNewAssetProperties.Broadcast(Factory);
	if (!Factory->ConfigureProperties())
	{
		return;
	}

	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	FString DefaultName;
	FString PackageName;
	AssetTools.CreateUniqueAssetName(NewAssetPath / Factory->GetDefaultNewAssetName(), TEXT(""), PackageName, DefaultName);

	// Local state written by dialog callbacks and read after AddModalWindow returns
	FString ChosenName = DefaultName;
	bool bConfirmed = false;

	// V1 dialog, just ask for asset name. Later on can add relative folder / etc.
	TSharedRef<SWindow> DialogWindow = SNew(SWindow)
		.Title(LOCTEXT("EnterNewAssetNameTitle", "Enter New Asset Name"))
		.ClientSize(FVector2D(400.f, 110.f))
		.SizingRule(ESizingRule::FixedSize)
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	TSharedPtr<SEditableTextBox> TextBox;

	// Confirm: read text, set flag, close window. AddModalWindow is synchronous so all
	// locals above remain alive until after it returns.
	auto DoConfirm = [&]()
	{
		ChosenName = TextBox->GetText().ToString();
		bConfirmed = true;
		DialogWindow->RequestDestroyWindow();
	};

	DialogWindow->SetContent(
		SNew(SBox)
		.Padding(16.f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 0.f, 0.f, 8.f)
			[
				SAssignNew(TextBox, SEditableTextBox)
				.Text(FText::FromString(DefaultName))
				.SelectAllTextWhenFocused(true)
				.OnTextCommitted(FOnTextCommitted::CreateLambda([&DoConfirm](const FText&, ETextCommit::Type CommitType)
				{
					if (CommitType == ETextCommit::OnEnter)
					{
						DoConfirm();
					}
				}))
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 4.f, 0.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("CreateButton", "Create"))
					.OnClicked(FOnClicked::CreateLambda([&DoConfirm]()
					{
						DoConfirm();
						return FReply::Handled();
					}))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("CancelButton", "Cancel"))
					.OnClicked(FOnClicked::CreateLambda([DialogWindow]()
					{
						DialogWindow->RequestDestroyWindow();
						return FReply::Handled();
					}))
				]
			]
		]
	);

	// While a temp factory is normal, we're about to spawn a non engine blocking modal, so pin the factory till we exit.
	TStrongObjectPtr<UFactory> FactoryPinned = TStrongObjectPtr<UFactory>(Factory);
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
	
	// Does not return until modal is closed.
	FSlateApplication::Get().AddModalWindow(DialogWindow, ParentWindow);

	if (bConfirmed && !ChosenName.IsEmpty())
	{
		AssetTools.CreateAsset(ChosenName, NewAssetPath, Factory->GetSupportedClass(), Factory);
	}
}

void SUAFBrowser::PopulatePluginFilters(FAssetPickerConfig& Config)
{
	const UUAFEditorProjectSettings* Settings = GetDefault<UUAFEditorProjectSettings>();

	TSharedRef<FFrontendFilterCategory> PluginsCategory = MakeShared<FFrontendFilterCategory>(
		LOCTEXT("PluginsFilterCategory", "Plugins"),
		LOCTEXT("PluginsFilterCategoryTooltip", "Filter assets by plugin")
	);

	PluginFilters.Reset();

	TSet<FName> ExcludedPlugins(Settings->ExcludedBrowserPluginFilters);
	TSet<FName> AddedPluginNames;

	// Auto-discover project plugins with content
	for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPluginsWithContent())
	{
		if (Plugin->GetType() != EPluginType::Project)
		{
			continue;
		}

		const FName PluginFName(*Plugin->GetName());
		if (ExcludedPlugins.Contains(PluginFName))
		{
			continue;
		}

		FString ContentPath = Plugin->GetMountedAssetPath();
		TSharedRef<FFrontendFilter_UAFPlugin> Filter = MakeShared<FFrontendFilter_UAFPlugin>(
			PluginsCategory,
			Plugin->GetName(),
			Plugin->GetFriendlyName(),
			MoveTemp(ContentPath)
		);

		Config.ExtraFrontendFilters.Add(Filter);
		PluginFilters.Add(Filter);
		AddedPluginNames.Add(PluginFName);
	}

	// Add explicitly configured additional plugins not already discovered
	for (const FName& PluginName : Settings->AdditionalBrowserPluginFilters)
	{
		if (AddedPluginNames.Contains(PluginName))
		{
			continue;
		}

		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName.ToString());
		if (!Plugin.IsValid() || !Plugin->CanContainContent())
		{
			continue;
		}

		FString ContentPath = FString::Printf(TEXT("/%s/"), *Plugin->GetName());
		TSharedRef<FFrontendFilter_UAFPlugin> Filter = MakeShared<FFrontendFilter_UAFPlugin>(
			PluginsCategory,
			Plugin->GetName(),
			Plugin->GetFriendlyName(),
			MoveTemp(ContentPath)
		);

		Config.ExtraFrontendFilters.Add(Filter);
		PluginFilters.Add(Filter);
	}
}

void SUAFBrowser::OnContentSearchTextChanged(const FText& SearchText)
{
	// Capture the currently focused widget so we can restore focus after a chip click
	if (TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetUserFocusedWidget(0))
	{
		if (FocusedWidget->GetType() != FName(TEXT("SUAFBrowserChipButton")))
		{
			WeakSearchBox = FocusedWidget;
		}
	}

	if (!SuggestionStrip.IsValid())
	{
		return;
	}

	const FString Query = SearchText.ToString();
	TArray<TSharedRef<FFilterSuggestion>> Matches;

	if (!Query.IsEmpty())
	{
		// Plugin filters - matched by plugin name or display name
		for (const TSharedRef<FFrontendFilter_UAFPlugin>& Filter : PluginFilters)
		{
			if (Filter->GetPluginName().Contains(Query, ESearchCase::IgnoreCase) ||
				Filter->GetDisplayName().ToString().Contains(Query, ESearchCase::IgnoreCase))
			{
				TSharedRef<FFilterSuggestion> Suggestion = MakeShared<FFilterSuggestion>();
				Suggestion->DisplayName = Filter->GetDisplayName();
				Suggestion->IconName    = Filter->GetIconName();
				Suggestion->Color       = Filter->GetColor();
				Suggestion->Category    = SUAFBrowserFilterSuggestionStrip::CategoryPlugins;
				Suggestion->OnActivate.BindLambda([Filter]() { Filter->SetActive(true); });
				Matches.Add(Suggestion);
			}
		}

		// Obtain the filter list from the cached widget chain
		TSharedPtr<SFilterList> FilterList;
		if (TaggedAssetBrowser.IsValid())
		{
			if (TSharedPtr<SAssetPicker> AssetPicker = TaggedAssetBrowser->GetAssetPicker())
			{
				FilterList = AssetPicker->GetFilterList();
			}
		}

		// All other frontend filters from the filter bar (excluding plugin filters already handled above)
		if (FilterList.IsValid())
		{
			for (const TSharedRef<FFrontendFilter>& Filter : FilterList->GetAllFrontendFilters())
			{
				const bool bIsPluginFilter = PluginFilters.ContainsByPredicate([&Filter](const TSharedRef<FFrontendFilter_UAFPlugin>& PluginFilter) 
				{ 
					return PluginFilter == Filter; 
				});

				if (bIsPluginFilter)
				{
					continue;
				}

				if (Filter->GetName().Contains(Query, ESearchCase::IgnoreCase) ||
					Filter->GetDisplayName().ToString().Contains(Query, ESearchCase::IgnoreCase))
				{
					TSharedRef<FFilterSuggestion> Suggestion = MakeShared<FFilterSuggestion>();
					Suggestion->DisplayName = Filter->GetDisplayName();
					Suggestion->IconName = Filter->GetIconName();
					Suggestion->Color = Filter->GetColor();
					Suggestion->Category = SUAFBrowserFilterSuggestionStrip::CategoryMisc;
					TWeakPtr<FFrontendFilter> WeakFilter = Filter;
					Suggestion->OnActivate.BindLambda([WeakFilter]()
					{
						if (TSharedPtr<FFrontendFilter> PinnedFilter = WeakFilter.Pin())
						{
							PinnedFilter->SetActive(true);
						}
					});
					Matches.Add(Suggestion);
				}
			}

			// Asset class/type filters (e.g. Animation Sequence, Blend Space)
			TWeakPtr<SFilterList> WeakFilterList = FilterList;
			for (const TSharedRef<FCustomClassFilterData>& ClassFilter : FilterList->GetCustomClassFilters())
			{
				if (ClassFilter->GetName().ToString().Contains(Query, ESearchCase::IgnoreCase) ||
					ClassFilter->GetFilterName().Contains(Query, ESearchCase::IgnoreCase))
				{
					TSharedRef<FFilterSuggestion> Suggestion = MakeShared<FFilterSuggestion>();
					Suggestion->DisplayName = ClassFilter->GetName();
					Suggestion->IconName = FName("ClassIcon.Object");
					Suggestion->Color = ClassFilter->GetColor();
					Suggestion->Category  = SUAFBrowserFilterSuggestionStrip::CategoryTypes;
					FTopLevelAssetPath ClassPath = ClassFilter->GetClassPathName();
					Suggestion->OnActivate.BindLambda([WeakFilterList, ClassPath]()
					{
						if (TSharedPtr<SFilterList> PinnedFilterList = WeakFilterList.Pin())
						{
							PinnedFilterList->ActivateClassFilterByPath(ClassPath);
						}
					});
					Matches.Add(Suggestion);
				}
			}
		}
	}

	SuggestionStrip->SetSuggestions(Matches);
}

void SUAFBrowser::OnFilterSuggestionSelected()
{
	// Clear the search text — this triggers OnContentSearchTextChanged("") which collapses the suggestion strip
	if (TaggedAssetBrowser.IsValid())
	{
		if (TSharedPtr<SAssetPicker> AssetPicker = TaggedAssetBrowser->GetAssetPicker())
		{
			AssetPicker->ClearSearchText();
		}
	}

	// Return keyboard focus to the search box so the user can continue typing
	if (TSharedPtr<SWidget> SearchBox = WeakSearchBox.Pin())
	{
		FSlateApplication::Get().SetKeyboardFocus(SearchBox, EFocusCause::SetDirectly);
	}
}

FReply SUAFBrowser::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// When Tab is pressed while the search box has focus and the suggestion strip has chips,
	// redirect focus to the first chip instead of moving to the next focusable widget.
	if (InKeyEvent.GetKey() == EKeys::Tab && !InKeyEvent.IsShiftDown())
	{
		TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetUserFocusedWidget(0);
		if (SuggestionStrip.IsValid() && SuggestionStrip->HasAnySuggestions()
			&& FocusedWidget.IsValid() && FocusedWidget == WeakSearchBox.Pin())
		{
			SuggestionStrip->FocusFirstChip();
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

} // namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE

