// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Common/UAFBrowserFrontendFilters.h"
#include "ContentBrowserDelegates.h"
#include "IContentBrowserSingleton.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Workspace
{
	class IWorkspaceEditor;
}	// end namespace UE::Workspace

namespace UE::UAF::Editor
{
	class IUAFAssetPreview;
};

class UTaggedAssetBrowserSection;
struct FAssetPickerConfig;
class FFrontendFilter;
class SHorizontalBox;
class STaggedAssetBrowser;
class SUAFBrowserFilterSuggestionStrip;

namespace UE::UAF::Editor
{

/** 
 * Experimental. Widget for searching for UAF related assets
 * 
 * Note: The implementation of this widget may change in the future as we move to use an in-progress capability of the content
 * browser to preview assets.
 */
class SUAFBrowser : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SUAFBrowser)
	{}

	SLATE_END_ARGS()

public:

	virtual ~SUAFBrowser() override;

	void Construct(const FArguments& InArgs, const TSharedPtr<UE::Workspace::IWorkspaceEditor>& InHostingApp, ETabState::Type InTabState);

private:

	/** Builds the top level of the UAF Browser, including preview splitter. Contents need to be ordered dynamically based on orientation. */
	TSharedRef<SWidget> Construct_GetMainBrowserContents();

	/** Builds the tagged asset browser part of the UAF browser. IE: Asset picker */
	TSharedRef<SWidget> Construct_GetTaggedAssetBrowser();

	/** Builds the tagged asset preview part of the UAF browser. Ex: Viewport. In the future this may get replaced by a native content browser API. */
	TSharedRef<SWidget> Construct_GetAssetPreviewWidget();

	/** Builds fallack when preview not supported by type in UAF browser Preview. */
	TSharedRef<SWidget> Construct_GetAssetPreviewNotSupportedWidget();

private:

	/** Used for UToolMenu registration of add menu. */
	static FName AddNewMenuName;

	/** How AddButton should appear based on filter(s) active. */
	enum class EUAFAddButtonMode 
	{ 
		AddOnly, 
		ImportOnly,
		Both, 
		Default 
	};

private:

	/** Callback to override asset picker configs for our tagged asset browser */
	void OnOverrideAssetPickerConfig(FAssetPickerConfig& Config);

	/** Populates Config.ExtraFrontendFilters with plugin-based filters discovered from project settings and IPluginManager. */
	void PopulatePluginFilters(FAssetPickerConfig& Config);

	/** Called when the content area search box text changes; updates the suggestion strip. */
	void OnContentSearchTextChanged(const FText& SearchText);

	/** Called when any suggestion chip is clicked; returns keyboard focus to the search box. */
	void OnFilterSuggestionSelected();

	/** Callback to customize top bar of our asset browser. To add layout control buttons. */
	void OnExtendAssetPickerTopBar(TSharedRef<SHorizontalBox> InAssetPickerTopBar);

	/** Builds button for docking the UAF Browser. This just unhides the in-layout browser tab. */
	TSharedRef<SWidget> OnExtendAssetPickerTopBar_GetDrawerDockButton();

	/** Builds button for swapping hotkey behavior (sidebar to statusbar). */
	TSharedRef<SWidget> OnExtendAssetPickerTopBar_GetToggleSwapHotkeyButton();

	/** Callback for when swap hotkey is pressed. Changes hotkey behavior in project settings & hides all drawers. */
	FReply OnSwapHotkeyButtonClicked();

	/** Callback for when dock to drawer button is pressed. Hides all drawers and invokes docked browser tab. */
	FReply OnDockDrawerButtonClicked();

	/** Callback for when the active tagged asset browser section changes. Rebuilds add/import buttons. */
	void OnActiveSectionChanged(const UTaggedAssetBrowserSection* NewSection);

	/** Callback for when the primary filter tree selection changes. Rebuilds add/import buttons. */
	void OnPrimaryFilterSelectionChanged();

	/** Builds the add/import button(s) appropriate for the current active section. */
	TSharedRef<SWidget> BuildAddImportButtons();

	/** Generates the add menu widget for the per-instance ToolMenu. */
	TSharedRef<SWidget> BuildAddMenu();

	/** Shows a modal dialog to name the new asset, then creates it if confirmed. */
	void CreateAssetWithNameDialog(UClass* InFactoryClass, const FText& AssetTypeName);

	/** Overload that accepts a pre-configured factory (e.g. UBlueprintFactory with ParentClass already set). */
	void CreateAssetWithNameDialog(UFactory* InFactory, const FText& AssetTypeName);

	/** Opens the OS import dialog targeting the current asset folder. */
	FReply OnImportButtonClicked();

	/** Returns the package folder path to use for asset creation/import. */
	FString GetNewAssetPath() const;

	/** Collects UClass pointers from the currently selected primary filter items in the tagged asset browser. */
	TArray<UClass*> GetCurrentSectionFilterClasses() const;

	/** Determines which button(s) to show based on which section classes have creatable factories. */
	EUAFAddButtonMode GetAddButtonMode(const TArray<UClass*>& SectionClasses) const;

private:

	/** Intercepts Tab from the search box to redirect focus to the first suggestion chip. */
	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Callback for when an asset is selected. */
	void OnAssetSelected(const FAssetData& InAssetData);

	/** Callback for when assets are dragged */
	void OnAssetsDragged(const TArray<FAssetData>& InAssetData);

	/** Callback to sync settings changes across UAF Browser instances */
	void OnExternalColumnVisibilityChanged(const TArray<FString>& HiddenColumnIds);

	/** Callback to write column visibility changes to .ini & UAF settings */
	void OnBrowserColumnVisibilityChanged(const TArray<FString>& HiddenColumnIds);

private:

	/** App Hosting this Browser, to query settings against */
	TWeakPtr<UE::Workspace::IWorkspaceEditor> WeakHostingApp = nullptr;

	/** The currently active section, updated via OnActiveSectionChanged. */
	TWeakObjectPtr<const UTaggedAssetBrowserSection> CurrentActiveSection;

	/** Container for the add/import button(s). Rebuilt reactively on section change. */
	TSharedPtr<SHorizontalBox> AddImportButtonContainer;

	/** Current Tab host type, so we can resturucture layout based on how this tab is being hosted (Sidebar / statusbar / etc). */
	ETabState::Type TabState = ETabState::ClosedTab;

	/** 
	 * Experimental. Slot used to swap between different asset preview widgets. 
	 * Note: Long term this functionality will get replaced by the generic IAssetPreviewCustomization API.
	 */
	SVerticalBox::FSlot* AssetPreviewSlot = nullptr;

	/**
	 * Experimental. Widget used to preview an asset. Persisted weakly here so that we can tell it to just
	 * update if swapping betweeen assets of the same type
	 */
	TWeakPtr<IUAFAssetPreview> AssetPreviewWidgetWeak = nullptr;

	/** Delegate write asset browser config .ini  whenever changes occur. */
	FSaveAssetViewSettingsDelegate SaveAssetViewSettingsDelegate;

	/** Delegate fired to refresh asset browser whenever config .ini changes occur. */
	FLoadAssetViewSettingsDelegate LoadAssetViewSettingsDelegate;

	/** Plugin frontend filters added to the content area filter bar. Kept so we can match them against search text. */
	TArray<TSharedRef<FFrontendFilter_UAFPlugin>> PluginFilters;

	/** The tagged asset browser widget; held so we can access the asset picker and filter list directly. */
	TSharedPtr<STaggedAssetBrowser> TaggedAssetBrowser;

	/** Horizontally-scrolling strip of filter suggestion chips, shown when search text matches a known filter. */
	TSharedPtr<SUAFBrowserFilterSuggestionStrip> SuggestionStrip;

	/** Weak reference to the content area search box widget, captured when OnContentSearchTextChanged fires. Used to restore focus after a chip click. */
	TWeakPtr<SWidget> WeakSearchBox;
};

} // namespace UE::UAF::Editor