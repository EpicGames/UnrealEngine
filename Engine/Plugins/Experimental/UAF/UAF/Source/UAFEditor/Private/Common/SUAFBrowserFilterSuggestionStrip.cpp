// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/SUAFBrowserFilterSuggestionStrip.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "UAFBrowserFilterSuggestions"

// ---------------------------------------------------------------------------
// SUAFBrowserChipButton
// ---------------------------------------------------------------------------

/**
 * Minimal SButton subclass that passes a hover attribute during construction so that
 * the button renders with hover visuals whenever it has keyboard focus.
 * SetHover is a protected SWidget method, so it must be called from within Construct.
 */
class SUAFBrowserChipButton : public SButton
{
public:
	SLATE_BEGIN_ARGS(SUAFBrowserChipButton)
		: _ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		, _ContentPadding(FMargin(6.f, 2.f))
	{}
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)
		SLATE_ATTRIBUTE(FMargin, ContentPadding)
		SLATE_ATTRIBUTE(FText, ToolTipText)
		SLATE_EVENT(FOnClicked, OnClicked)
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		bHovered = true;
		SButton::OnMouseEnter(MyGeometry, MouseEvent);
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		bHovered = false;
		SButton::OnMouseLeave(MouseEvent);
	}

	/** Retuns TRUE if chip should appear as hovered. */
	bool ShouldAppearHovered() const
	{
		return bHovered || HasKeyboardFocus();
	}

	void Construct(const FArguments& InArgs)
	{
		SButton::Construct(
			SButton::FArguments()
			.ButtonStyle(InArgs._ButtonStyle)
			.ContentPadding(InArgs._ContentPadding)
			.ToolTipText(InArgs._ToolTipText)
			.OnClicked(InArgs._OnClicked)
			[
				InArgs._Content.Widget
			]
		);

		bHovered = false;

		SetHover(TAttribute<bool>::CreateSP(this, &SUAFBrowserChipButton::ShouldAppearHovered));
	}

	/** Keep an internal IsHovered flag*/
	bool bHovered = false;
};

// ---------------------------------------------------------------------------
// SUAFBrowserFilterSuggestionStrip — static members
// ---------------------------------------------------------------------------

const FName SUAFBrowserFilterSuggestionStrip::CategoryPlugins(TEXT("Plugins"));
const FName SUAFBrowserFilterSuggestionStrip::CategoryTypes(TEXT("Types"));
const FName SUAFBrowserFilterSuggestionStrip::CategoryMisc(TEXT("Misc"));

const TArray<FName> SUAFBrowserFilterSuggestionStrip::DefaultCategoryOrder = 
{
	SUAFBrowserFilterSuggestionStrip::CategoryPlugins,
	SUAFBrowserFilterSuggestionStrip::CategoryTypes,
	SUAFBrowserFilterSuggestionStrip::CategoryMisc,
};

// ---------------------------------------------------------------------------
// SUAFBrowserFilterSuggestionStrip — construction
// ---------------------------------------------------------------------------

void SUAFBrowserFilterSuggestionStrip::Construct(const FArguments& InArgs)
{
	OnFilterSuggestionSelectedDelegate = InArgs._OnFilterSuggestionSelected;

	ChildSlot
	[
		SNew(SVerticalBox)

		// Hint text shown above the rows whenever the strip is visible
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(4.f, 2.f, 4.f, 0.f))
		[
			SAssignNew(HintTextWidget, STextBlock)
			.Text(LOCTEXT("KeyboardHint", "Tab to select filter"))
			.TextStyle(FAppStyle::Get(), "SmallText")
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		]

		// Row container — rows are added/removed dynamically in SetSuggestions()
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
			.Padding(FMargin(4.f, 2.f))
			[
				SAssignNew(RowsContainer, SVerticalBox)
			]
		]
	];

	// Start collapsed — shown only when suggestions exist
	SetVisibility(EVisibility::Collapsed);
}

// ---------------------------------------------------------------------------
// SUAFBrowserFilterSuggestionStrip — public interface
// ---------------------------------------------------------------------------

void SUAFBrowserFilterSuggestionStrip::SetSuggestions(const TArray<TSharedRef<FFilterSuggestion>>& Suggestions)
{
	RowsContainer->ClearChildren();
	ActiveRows.Reset();

	if (HintTextWidget.IsValid())
	{
		HintTextWidget->SetText(LOCTEXT("KeyboardHint", "Tab to select filter"));
	}

	if (Suggestions.IsEmpty())
	{
		SetVisibility(EVisibility::Collapsed);
		return;
	}

	// Build the iteration order: known categories first (in DefaultCategoryOrder), then
	// any unknown category keys encountered in Suggestions (in the order they first appear).
	TArray<FName> OrderedKeys = DefaultCategoryOrder;
	for (const TSharedRef<FFilterSuggestion>& Suggestion : Suggestions)
	{
		OrderedKeys.AddUnique(Suggestion->Category);
	}

	for (const FName& CategoryKey : OrderedKeys)
	{
		// Collect all suggestions for this category
		TArray<TSharedRef<FFilterSuggestion>> CategorySuggestions;
		for (const TSharedRef<FFilterSuggestion>& Suggestion : Suggestions)
		{
			if (Suggestion->Category == CategoryKey)
			{
				CategorySuggestions.Add(Suggestion);
			}
		}
		if (CategorySuggestions.IsEmpty())
		{
			continue;
		}

		// Build chip widgets for this category
		FFilterSuggestionRow& NewRow = ActiveRows.AddDefaulted_GetRef();
		NewRow.CategoryKey = CategoryKey;
		for (const TSharedRef<FFilterSuggestion>& Suggestion : CategorySuggestions)
		{
			NewRow.ChipWidgets.Add(BuildChipWidget(Suggestion));
		}

		// Build the row widget: label + horizontally-scrolling chip area
		TSharedPtr<SHorizontalBox> ChipsBox = SNew(SHorizontalBox);

		RowsContainer->AddSlot()
		.AutoHeight()
		.Padding(FMargin(0.f, 1.f))
		[
			SNew(SHorizontalBox)

			// Row label (FName string doubles as display text)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 6.f, 0.f))
			[
				SNew(STextBlock)
				.Text(FText::FromName(CategoryKey))
				.TextStyle(FAppStyle::Get(), "SmallText")
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
				
			// Horizontally-scrolling chip area
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				.ScrollBarVisibility(EVisibility::Collapsed)
				+ SScrollBox::Slot()
				[
					SAssignNew(ChipsBox, SHorizontalBox)
				]
			]
		];

		// Populate the chip area
		for (const TSharedRef<SWidget>& ChipWidget : NewRow.ChipWidgets)
		{
			ChipsBox->AddSlot()
			.AutoWidth()
			.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
			[
				ChipWidget
			];
		}
	}

	SetVisibility(EVisibility::Visible);
}

bool SUAFBrowserFilterSuggestionStrip::HasAnySuggestions() const
{
	return !ActiveRows.IsEmpty();
}

void SUAFBrowserFilterSuggestionStrip::FocusFirstChip()
{
	if (!ActiveRows.IsEmpty() && !ActiveRows[0].ChipWidgets.IsEmpty())
	{
		if (HintTextWidget.IsValid())
		{
			HintTextWidget->SetText(LOCTEXT("KeyboardNavHint", "[ Enter ] Apply  \u00b7  [\u2190][\u2192] Navigate  \u00b7  [ \u2191 ][ \u2193 ] Row"));
		}
		FSlateApplication::Get().SetKeyboardFocus(ActiveRows[0].ChipWidgets[0]);
	}
}

// ---------------------------------------------------------------------------
// SUAFBrowserFilterSuggestionStrip — keyboard navigation
// ---------------------------------------------------------------------------

FReply SUAFBrowserFilterSuggestionStrip::OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	TSharedPtr<SWidget> FocusedWidget = FSlateApplication::Get().GetUserFocusedWidget(0);

	// Find which row and chip index currently has keyboard focus
	int32 FocusedRowIndex = INDEX_NONE;
	int32 FocusedChipIndex = INDEX_NONE;

	for (int32 RowIndex = 0; RowIndex < ActiveRows.Num() && FocusedRowIndex == INDEX_NONE; ++RowIndex)
	{
		for (int32 ChipIndex = 0; ChipIndex < ActiveRows[RowIndex].ChipWidgets.Num(); ++ChipIndex)
		{
			if (ActiveRows[RowIndex].ChipWidgets[ChipIndex] == FocusedWidget)
			{
				FocusedRowIndex = RowIndex;
				FocusedChipIndex = ChipIndex;
				break;
			}
		}
	}

	if (FocusedRowIndex == INDEX_NONE)
	{
		return FReply::Unhandled();
	}

	const FKey PressedKey = InKeyEvent.GetKey();
	const TArray<TSharedRef<SWidget>>& CurrentRowChips = ActiveRows[FocusedRowIndex].ChipWidgets;

	if (PressedKey == EKeys::Left)
	{
		if (FocusedChipIndex > 0)
		{
			FSlateApplication::Get().SetKeyboardFocus(CurrentRowChips[FocusedChipIndex - 1]);
		}
		return FReply::Handled();
	}
	else if (PressedKey == EKeys::Right)
	{
		if (FocusedChipIndex + 1 < CurrentRowChips.Num())
		{
			FSlateApplication::Get().SetKeyboardFocus(CurrentRowChips[FocusedChipIndex + 1]);
		}
		return FReply::Handled();
	}
	else if (PressedKey == EKeys::Up || PressedKey == EKeys::Down)
	{
		if (ActiveRows.Num() <= 1)
		{
			// No adjacent row to navigate to — consume to prevent focus escaping the strip
			return FReply::Handled();
		}

		const int32 DirectionOffset = (PressedKey == EKeys::Down) ? 1 : -1;
		const int32 TargetRowIndex = (FocusedRowIndex + DirectionOffset + ActiveRows.Num()) % ActiveRows.Num();
		const int32 TargetChipIndex = FMath::Min(FocusedChipIndex, ActiveRows[TargetRowIndex].ChipWidgets.Num() - 1);
		FSlateApplication::Get().SetKeyboardFocus(ActiveRows[TargetRowIndex].ChipWidgets[TargetChipIndex]);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

// ---------------------------------------------------------------------------
// SUAFBrowserFilterSuggestionStrip — private helpers
// ---------------------------------------------------------------------------

TSharedRef<SWidget> SUAFBrowserFilterSuggestionStrip::BuildChipWidget(TSharedRef<FFilterSuggestion> Suggestion)
{
	const FName IconName = Suggestion->IconName.IsNone() ? FName("Icons.Help") : Suggestion->IconName;

	return SNew(SUAFBrowserChipButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
		.ContentPadding(FMargin(6.f, 2.f))
		.ToolTipText(Suggestion->DisplayName)
		.OnClicked(this, &SUAFBrowserFilterSuggestionStrip::OnChipClicked, Suggestion)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(IconName))
				.DesiredSizeOverride(FVector2D(12.f, 12.f))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Suggestion->DisplayName)
				.TextStyle(FAppStyle::Get(), "SmallText")
			]
		];
}

FReply SUAFBrowserFilterSuggestionStrip::OnChipClicked(TSharedRef<FFilterSuggestion> Suggestion)
{
	Suggestion->OnActivate.ExecuteIfBound();
	OnFilterSuggestionSelectedDelegate.ExecuteIfBound();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
