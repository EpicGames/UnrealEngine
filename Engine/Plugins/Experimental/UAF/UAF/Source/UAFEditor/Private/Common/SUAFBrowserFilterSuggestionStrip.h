// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "UObject/NameTypes.h"
#include "Widgets/SCompoundWidget.h"

class SVerticalBox;
class STextBlock;

/**
 * Display-and-activation descriptor for a single suggestion chip in the UAF Browser filter suggestion strip.
 * Decouples display (name, icon, color) from activation logic so the strip doesn't need to know the filter type.
 */
struct FFilterSuggestion
{
	FText DisplayName;
	FName IconName;
	FLinearColor Color = FLinearColor::White;

	/**
	 * Row grouping key. Use one of the built-in SUAFBrowserFilterSuggestionStrip::Category* constants
	 * or any FName to create a custom row. The FName string is used as the row label.
	 */
	FName Category = FName(TEXT("Misc"));

	/** Executed when the chip is clicked (or activated via keyboard) to apply the associated filter. */
	TDelegate<void()> OnActivate;
};

/**
 * A horizontally-scrolling, categorized strip of filter suggestion chips.
 * Appears at the bottom of the UAF Browser when the user types text that matches available filters.
 *
 * Suggestions are grouped into named rows determined by FFilterSuggestion::Category.
 * Each row is only visible when it has at least one matching suggestion.
 *
 * Keyboard navigation:
 *   - Tab (from search box): moves focus to the first chip (intercepted by parent SUAFBrowser)
 *   - Left / Right: move focus within the current row
 *   - Up / Down: move focus to the corresponding chip in the adjacent row (wraps, no-op if single row)
 *   - Enter / Space: activate the focused chip (handled natively by SButton)
 */
class SUAFBrowserFilterSuggestionStrip : public SCompoundWidget
{
public:
	/** Built-in category key constants. The FName string doubles as the row label. */
	static const FName CategoryPlugins;
	static const FName CategoryTypes;
	static const FName CategoryMisc;

	SLATE_BEGIN_ARGS(SUAFBrowserFilterSuggestionStrip) {}
		/** Called after any chip's OnActivate fires. Use to restore search box focus and clear search text. */
		SLATE_EVENT(TDelegate<void()>, OnFilterSuggestionSelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Updates the displayed suggestion chips, grouped into rows by Category FName. Collapses the strip when Suggestions is empty. */
	void SetSuggestions(const TArray<TSharedRef<FFilterSuggestion>>& Suggestions);

	/** Returns true if any category row has at least one chip. */
	bool HasAnySuggestions() const;

	/** Moves keyboard focus to the first chip in the first non-empty row. */
	void FocusFirstChip();

	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	/** Internal per-row data, rebuilt on every SetSuggestions() call. */
	struct FFilterSuggestionRow
	{
		FName CategoryKey;
		TArray<TSharedRef<SWidget>> ChipWidgets;
	};

	TSharedRef<SWidget> BuildChipWidget(TSharedRef<FFilterSuggestion> Suggestion);
	FReply OnChipClicked(TSharedRef<FFilterSuggestion> Suggestion);

	/**
	 * Defines the preferred display order for known category keys.
	 * Any unknown keys encountered in SetSuggestions() are appended in the order they first appear.
	 */
	static const TArray<FName> DefaultCategoryOrder;

	/** Live rows, rebuilt on every SetSuggestions() call. Only contains categories with at least one suggestion. */
	TArray<FFilterSuggestionRow> ActiveRows;

	/** Vertical container into which row widgets are added dynamically in SetSuggestions(). */
	TSharedPtr<SVerticalBox> RowsContainer;

	/** Hint text above the rows; toggled between invite and navigation hints depending on focus state. */
	TSharedPtr<STextBlock> HintTextWidget;

	TDelegate<void()> OnFilterSuggestionSelectedDelegate;
};
