// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/TaggedAssetBrowser_UAFBrowserFilters.h"

#include "Animation/Skeleton.h"
#include "IContentBrowserDataModule.h"
#include "UserAssetTagEditorUtilities.h"
#include "UserAssetTagsEditorModule.h"
#include "Logging/StructuredLog.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/STaggedAssetBrowser.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TaggedAssetBrowser_UAFBrowserFilters)

#define LOCTEXT_NAMESPACE "UAFBrowserFilters"


//////////////////////////////////////////////////////////////////////////
// UTaggedAssetBrowserFilter_NativeClass

void UTaggedAssetBrowserFilter_NativeClass::ModifyARFilterInternal(FARFilter& Filter) const
{
	for(UClass* Class : Classes)
	{
		if(Class)
		{
			if (bSearchNativeParentClasses)
			{
				Filter.TagsAndValues.Add(FBlueprintTags::NativeParentClassPath, FObjectPropertyBase::GetExportPath(Class));
			}
			else
			{
				Filter.ClassPaths.Add(Class->GetClassPathName());
			}
		}
	}
	Filter.bRecursiveClasses = bAllowChildClasses;
}


//////////////////////////////////////////////////////////////////////////
// UTaggedAssetBrowserFilter_Skeleton

FString UTaggedAssetBrowserFilter_Skeleton::ToString() const
{
	if (!FilterName.IsNone())
	{
		return FilterName.ToString();
	}

	if (Skeleton)
	{
		return Skeleton->GetFName().ToString();
	}

	return UTaggedAssetBrowserFilterBase::ToString();
}

FText UTaggedAssetBrowserFilter_Skeleton::GetTooltip() const
{
	if (Skeleton)
	{
		FText BaseTooltip = LOCTEXT("UTaggedAssetBrowserFilter_SkeletonTooltip", "Skeleton Filter: {0}.");
		return FText::FormatOrdered(BaseTooltip, FText::FromString(Skeleton->GetFName().ToString()));
	}

	return UTaggedAssetBrowserFilterBase::GetTooltip();
}

FSlateIcon UTaggedAssetBrowserFilter_Skeleton::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(USkeleton::StaticClass());
}

void UTaggedAssetBrowserFilter_Skeleton::ModifyARFilterInternal(FARFilter& Filter) const
{
	static const FName SkeletonTagName("Skeleton");
	if(Skeleton)
	{
		Filter.TagsAndValues.Add(SkeletonTagName, FAssetData(Skeleton).GetExportTextName());
	}
}


//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
