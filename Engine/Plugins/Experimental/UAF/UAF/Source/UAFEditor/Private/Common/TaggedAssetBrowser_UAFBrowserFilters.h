// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TaggedAssetBrowserFilters/TaggedAssetBrowser_CommonFilters.h"
#include "TaggedAssetBrowser_UAFBrowserFilters.generated.h"

#define UE_API UAFEDITOR_API

class USkeleton;

/** 
 * Experimental. Temp filter to allow us to find BP whose BPGC would match a class filter.
 * 
 * Our solution here isn't as robust as we would like. But it does work for UAF Browser MVP needs.
 */
UCLASS(DisplayName="Native Classes")
class UE_API UTaggedAssetBrowserFilter_NativeClass : public UTaggedAssetBrowserFilter_Class
{
	GENERATED_BODY()

public:

	virtual void ModifyARFilterInternal(FARFilter& Filter) const override;

	/** True to search for a native parent class, useful for finding BPs whose BPGCs would match but BP itself does not match. */
	UPROPERTY(EditAnywhere, Category = "Filter")
	bool bSearchNativeParentClasses = false;
};


/** Filter for a specifc skeleton */
UCLASS(DisplayName="Skeleton")
class UE_API UTaggedAssetBrowserFilter_Skeleton : public UTaggedAssetBrowserFilterBase
{
	GENERATED_BODY()

public:

	virtual FString ToString() const override;
	virtual TOptional<FString> GetInstanceIdentifier() const override { return ToString(); }
	virtual FText GetTooltip() const override;
	virtual FSlateIcon GetIcon() const override;

	virtual void ModifyARFilterInternal(FARFilter& Filter) const override;

	/** Skeleton to search for. */
	UPROPERTY(EditAnywhere, Category = "Filter")
	TObjectPtr<USkeleton> Skeleton;

	/** Name for this class filter, by default will use name of the skeleton */
	UPROPERTY(EditAnywhere, Category = "Filter")
	FName FilterName;
};

#undef UE_API