// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrontendFilterBase.h"

/**
 * A content browser frontend filter that restricts the asset view to content from a specific Unreal Engine plugin.
 * These filters appear under the "Plugins" category in the UAF Browser's "Add Filter" dropdown.
 */
class FFrontendFilter_UAFPlugin : public FFrontendFilter
{
public:
	FFrontendFilter_UAFPlugin(TSharedPtr<FFrontendFilterCategory> InCategory,
	                          const FString& InPluginName,
	                          const FString& InFriendlyName,
	                          const FString& InPluginContentPath)
		: FFrontendFilter(MoveTemp(InCategory))
		, PluginName(InPluginName)
		, FriendlyName(InFriendlyName)
		, PluginContentPath(InPluginContentPath)
	{}

	// FFilterBase interface
	virtual FString GetName() const override { return PluginName; }
	virtual FText GetDisplayName() const override { return FText::FromString(FriendlyName); }
	virtual FText GetToolTipText() const override
	{
		return FText::Format(
			NSLOCTEXT("UAFBrowser", "PluginFilterTooltip", "Show only assets from the '{0}' plugin (path: {1})"),
			FText::FromString(FriendlyName),
			FText::FromString(PluginContentPath));
	}
	virtual FLinearColor GetColor() const override { return FLinearColor(0.4f, 0.7f, 1.0f, 1.0f); }
	virtual FName GetIconName() const override { return FName("Icons.Package"); }

	// FFrontendFilter interface
	virtual bool PassesFilter(FAssetFilterType InItem) const override
	{
		return InItem.GetInternalPath().ToString().StartsWith(PluginContentPath);
	}

	/** Returns the plugin name (without the friendly display name). Used for matching against search text. */
	const FString& GetPluginName() const { return PluginName; }

private:
	FString PluginName;
	FString FriendlyName;
	/** Package path prefix for this plugin's content, e.g. "/Cosmo/". */
	FString PluginContentPath;
};
