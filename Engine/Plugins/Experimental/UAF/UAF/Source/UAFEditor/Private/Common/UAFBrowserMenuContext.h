// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UAFBrowserMenuContext.generated.h"

namespace UE::UAF::Editor { class SUAFBrowser; }

/** Context object passed to per-instance UAF Browser add-new ToolMenu sections. */
UCLASS()
class UUAFBrowserMenuContext : public UObject
{
	GENERATED_BODY()

public:
	/** Browser that owns this context object. */
	TWeakPtr<UE::UAF::Editor::SUAFBrowser> WeakOwningBrowser;
};
