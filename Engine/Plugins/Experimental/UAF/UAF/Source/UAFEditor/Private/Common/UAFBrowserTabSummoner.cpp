// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/UAFBrowserTabSummoner.h"

#include "AnimNextRigVMAsset.h"
#include "Common/SUAFBrowser.h"
#include "IAnimNextEditorModule.h"
#include "IWorkspaceEditor.h"
#include "Modules/ModuleManager.h"
#include "PersonaModule.h"
#include "StatusBarSubsystem.h"

#define LOCTEXT_NAMESPACE "WorkspaceTabSummoner"

namespace UE::UAF::Editor
{

const FName FUAFBrowserTabSummoner::DockedTabID(TEXT("UAFBrowserDockedTab"));
const FName FUAFBrowserTabSummoner::SidebarTabID(TEXT("UAFBrowserSidebarTab"));
const FName FUAFBrowserTabSummoner::DrawerTabID(TEXT("UAFBrowserDrawerTab"));
const FName FUAFBrowserTabSummoner::StatusBarSystemDrawerID(TEXT("UAFBrowserDrawer"));

void FUAFBrowserTabSummoner::ToggleDrawer()
{
	GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->TryToggleDrawer(FUAFBrowserTabSummoner::StatusBarSystemDrawerID);
}

FUAFBrowserTabSummoner::FUAFBrowserTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp, ETabState::Type InTabState, FName InTabName, FText InTabLabel)
	: FWorkflowTabFactory(InTabName, StaticCastSharedPtr<FAssetEditorToolkit>(InHostingApp))
	, TabState(InTabState)
{
	TabLabel = InTabLabel;
	ViewMenuDescription = InTabLabel;

	// @TODO: DarenC - Placeholder Icon
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.Palette");
	ViewMenuTooltip = LOCTEXT("UAFBrowserTabToolTip", "Browse / Create Animation related assets.");
	bIsSingleton = true;

	UAFBrowserWidget = SNew(SUAFBrowser, InHostingApp, TabState);
}

TSharedRef<SWidget> FUAFBrowserTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return UAFBrowserWidget.ToSharedRef();
}

FText FUAFBrowserTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return ViewMenuTooltip;
}

} // end namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE
