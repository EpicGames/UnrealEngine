// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "IMessageLogListing.h"

namespace UE::Workspace
{
	class IWorkspaceEditor;
}	// end namespace UE::Workspace

namespace UE::UAF::Editor
{
	class SUAFBrowser;
}	// end namespace UE::UAF::Editor

namespace UE::UAF::Editor
{

struct FUAFBrowserTabSummoner : public FWorkflowTabFactory
{
public:

	/** Docked Tab ID. We have 3 distinct, non synchronized tabs. The UAF editor will manage hotkey / layouts for the user. */
	static const FName DockedTabID;

	/** Sidebar Tab ID. We have 3 distinct, non synchronized tabs. The UAF editor will manage hotkey / layouts for the user. */
	static const FName SidebarTabID;

	/** Drawer Tab ID. We have 3 distinct, non synchronized tabs. The UAF editor will manage hotkey / layouts for the user. */
	static const FName DrawerTabID;

	/** ID for this tab within the status bar system. Used to communicate with status bar system. */
	static const FName StatusBarSystemDrawerID;

	static void ToggleDrawer();

public:

	FUAFBrowserTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp, ETabState::Type InTabState, FName InTabName, FText InTabLabel);

	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

private:

	/** The widget this tab spawner wraps */
	TSharedPtr<SUAFBrowser> UAFBrowserWidget;

	/** Current state of the tab, needed for the tab to change layout*/
	ETabState::Type TabState = ETabState::ClosedTab;
};

} // end namespace UE::UAF::Editor
