// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorCVars.h"

namespace UE::UAF::Editor::CVars
{

TAutoConsoleVariable<bool> GUseWorkspaceEditor(
	TEXT("a.AnimNext.UseWorkspaceEditor"),
	true,
	TEXT("Use the workspace editor to edit AnimNext assets rather than standalone editors"));

}