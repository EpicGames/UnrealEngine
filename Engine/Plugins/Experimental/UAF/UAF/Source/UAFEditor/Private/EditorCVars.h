// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"

namespace UE::UAF::Editor::CVars
{

extern TAutoConsoleVariable<bool> GUseWorkspaceEditor;

}