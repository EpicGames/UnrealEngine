// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtils.h"

#include "IAnimNextUncookedOnlyModule.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "Module/AnimNextModule_EditorData.h"
#include "Param/ParamType.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Module/AnimNextModule.h"
#include "PropertyBagDetails.h"
#include "UncookedOnlyUtils.h"
#include "Modules/ModuleManager.h"
#include "Param/RigVMDispatch_GetParameter.h"
#include "String/ParseTokens.h"
#include "Variables/IVariableBindingType.h"
#include "Widgets/SNullWidget.h"
#include "Common/GraphEditorSchemaActions.h"

#define LOCTEXT_NAMESPACE "AnimNextEditorUtils"

namespace UE::UAF::Editor
{

void FUtils::GetAllEntryNames(const UUAFRigVMAssetEditorData* InEditorData, TSet<FName>& OutNames)
{
	for(const UUAFRigVMAssetEntry* Entry : InEditorData->GetAllEntries())
	{
		OutNames.Add(Entry->GetEntryName());
	}
}

static const int32 MaxNameLength = 100;

FName FUtils::ValidateName(const UObject* InObject, const FString& InName)
{
	struct FNameValidator : public INameValidatorInterface
	{
		explicit FNameValidator(const UObject* InObject)
			: Object(InObject)
		{
		}
		
		virtual EValidatorResult IsValid (const FName& Name, bool bOriginal = false) override
		{
			EValidatorResult ValidatorResult = EValidatorResult::AlreadyInUse;

			if(Name == NAME_None)
			{
				ValidatorResult = EValidatorResult::EmptyName;
			}
			else if(Name.ToString().Len() > MaxNameLength)
			{
				ValidatorResult = EValidatorResult::TooLong;
			}
			else
			{
				// If it is in the names list then it is already in use.
				if(!Names.Contains(Name))
				{
					ValidatorResult = EValidatorResult::Ok;

					// Check for collision with an existing object.
					if (UObject* ExistingObject = StaticFindObject(/*Class=*/ nullptr, const_cast<UObject*>(Object), *Name.ToString(), EFindObjectFlags::ExactClass))
					{
						ValidatorResult = EValidatorResult::AlreadyInUse;
					}
				}
			}
			
			return ValidatorResult;
		}
		
		virtual EValidatorResult IsValid (const FString& Name, bool bOriginal = false) override
		{
			// Converting a string that is too large for an FName will cause an assert, so verify the length
			if(Name.Len() >= NAME_SIZE)
			{
				return EValidatorResult::TooLong;
			}
			else if (!FName::IsValidXName(Name, UE_BLUEPRINT_INVALID_NAME_CHARACTERS))
			{
				return EValidatorResult::ContainsInvalidCharacters;
			}

			// If not defined in name table, not current graph name
			return IsValid( FName(*Name) );
		}

		/** Name set to validate */
		TSet<FName> Names;
		/** The editor data to check for validity within */
		const UObject* Object;
	};
	
	FString Name = InName;
	if (Name.StartsWith(TEXT("RigUnit_")))
	{
		Name.RightChopInline(8, EAllowShrinking::No);
	}

	FNameValidator NameValidator(InObject);

	// Clean up BaseName to not contain any invalid characters, which will mean we can never find a legal name no matter how many numbers we add
	if (NameValidator.IsValid(Name) == EValidatorResult::ContainsInvalidCharacters)
	{
		for (TCHAR& TestChar : Name)
		{
			for (TCHAR BadChar : UE_BLUEPRINT_INVALID_NAME_CHARACTERS)
			{
				if (TestChar == BadChar)
				{
					TestChar = TEXT('_');
					break;
				}
			}
		}
	}
	
	int32 Count = 0;
	FString BaseName = Name;
	while (NameValidator.IsValid(Name) != EValidatorResult::Ok)
	{
		// Calculate the number of digits in the number, adding 2 (1 extra to correctly count digits, another to account for the '_' that will be added to the name
		int32 CountLength = Count > 0 ? (int32)log((double)Count) + 2 : 2;

		// If the length of the final string will be too long, cut off the end so we can fit the number
		if (CountLength + BaseName.Len() > MaxNameLength)
		{
			BaseName.LeftInline(MaxNameLength - CountLength);
		}
		Name = FString::Printf(TEXT("%s_%d"), *BaseName, Count);
		Count++;
	}

	return *Name;
}

FAnimNextParamType FUtils::GetParameterTypeFromMetaData(const FStringView& InStringView)
{
	if(InStringView.Equals(TEXT("bool")))
	{
		return FAnimNextParamType(FAnimNextParamType::EValueType::Bool);
	}
	else if(InStringView.Equals(TEXT("uint8")) || InStringView.Equals(TEXT("byte")))
	{
		return FAnimNextParamType(FAnimNextParamType::EValueType::Byte);
	}
	else if(InStringView.Equals(TEXT("int32")))
	{
		return FAnimNextParamType(FAnimNextParamType::EValueType::Int32);
	}
	else if(InStringView.Equals(TEXT("int64")))
	{
		return FAnimNextParamType(FAnimNextParamType::EValueType::Int64);
	}
	else if(InStringView.Equals(TEXT("float")))
	{
		return FAnimNextParamType(FAnimNextParamType::EValueType::Float);
	}
	else if(InStringView.Equals(TEXT("double")))
	{
		return FAnimNextParamType(FAnimNextParamType::EValueType::Double);
	}
	else if(InStringView.Equals(TEXT("Name")))
	{
		return FAnimNextParamType(FAnimNextParamType::EValueType::Name);
	}
	else if(InStringView.Equals(TEXT("String")))
	{
		return FAnimNextParamType(FAnimNextParamType::EValueType::String);
	}
	else if(InStringView.Equals(TEXT("Text")))
	{
		return FAnimNextParamType(FAnimNextParamType::EValueType::Text);
	}
	else
	{
		int32 SplitIndex = INDEX_NONE;
		if(InStringView.FindChar(TEXT('\''), SplitIndex))
		{
			// Disambiguated by class type: \Path\To\Class'\Path\To\Object, so no need to resolve the object, just the class 
			const FStringView ClassStringView = InStringView.Left(SplitIndex - 1);
			const FStringView ObjectStringView = InStringView.RightChop(SplitIndex + 1);
			
			const FTopLevelAssetPath ClassPath(ClassStringView);
			const FTopLevelAssetPath ObjectPath(ObjectStringView);

			if(ClassPath.IsValid() && ObjectPath.IsValid())
			{
				const FSoftObjectPath SoftClassPath(ClassPath);
				const FSoftObjectPath SoftObjectPath(ObjectPath);
				if(UClass* ResolvedClass = Cast<UClass>(SoftClassPath.ResolveObject()))
				{
					if(ResolvedClass == UScriptStruct::StaticClass()) 
					{
						return FAnimNextParamType(FAnimNextParamType::EValueType::Struct, FAnimNextParamType::EContainerType::None, ResolvedClass);
					}
					else if(ResolvedClass == UEnum::StaticClass()) 
					{
						return FAnimNextParamType(FAnimNextParamType::EValueType::Enum, FAnimNextParamType::EContainerType::None, ResolvedClass);
					}
					else if(ResolvedClass == UClass::StaticClass()) 
					{
						return FAnimNextParamType(FAnimNextParamType::EValueType::Class, FAnimNextParamType::EContainerType::None, ResolvedClass);
					}
					else
					{
						return FAnimNextParamType(FAnimNextParamType::EValueType::Object, FAnimNextParamType::EContainerType::None, ResolvedClass);
					}
				}
			}
		}
		else
		{
			// Class must be inferred: \Path\To\Object, so we need to resolve the object
			const FTopLevelAssetPath ObjectPath(InStringView);
			if(ObjectPath.IsValid())
			{
				const FSoftObjectPath SoftObjectPath(ObjectPath);
				if(UObject* ResolvedObject = Cast<UObject>(SoftObjectPath.ResolveObject()))
				{
					if(UScriptStruct* ResolvedStruct = Cast<UScriptStruct>(ResolvedObject))
					{
						return FAnimNextParamType(FAnimNextParamType::EValueType::Struct, FAnimNextParamType::EContainerType::None, ResolvedStruct);
					}
					else if(UEnum* ResolvedEnum = Cast<UEnum>(ResolvedObject))
					{
						return FAnimNextParamType(FAnimNextParamType::EValueType::Enum, FAnimNextParamType::EContainerType::None, ResolvedEnum);
					}
					else if(UClass* ResolvedClass = Cast<UClass>(ResolvedObject))
					{
						return FAnimNextParamType(FAnimNextParamType::EValueType::Object, FAnimNextParamType::EContainerType::None, ResolvedClass);
					}
				}
			}
		}
	}

	return FAnimNextParamType(); 
}

namespace Private
{

static bool IsPinTypeAllowed(const FEdGraphPinType& PinType)
{
	if ((PinType.PinCategory == UEdGraphSchema_K2::PC_Struct))
	{
		if (const UObject* TypeObject = PinType.PinSubCategoryObject.Get())
		{
			if (TypeObject->IsA<UUserDefinedStruct>())
			{
				return false;
			}
		}
	}
	else if (
		(PinType.PinCategory == UEdGraphSchema_K2::PC_Exec) ||
		(PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard) ||
		(PinType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate) ||
		(PinType.PinCategory == UEdGraphSchema_K2::PC_Delegate) ||
		(PinType.PinCategory == UEdGraphSchema_K2::PC_Interface) ||

		// RigVM does not support these types, so we disallow them
		(PinType.PinCategory == UEdGraphSchema_K2::PC_Int64) ||
		(PinType.PinCategory == UEdGraphSchema_K2::PC_Text) ||
		(PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass) ||
		(PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject)
		)
	{
		return false;
	}
	
	return true;
}

}

void FUtils::GetFilteredVariableTypeTree(TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& TypeTree, ETypeTreeFilter TypeTreeFilter)
{
	check(GetDefault<UEdGraphSchema_K2>());
	GetDefault<UPropertyBagSchema>()->GetVariableTypeTree(TypeTree, TypeTreeFilter);

	// Filter
	for (int32 Index = 0; Index < TypeTree.Num(); )
	{
		TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& PinType = TypeTree[Index];
		if (!PinType.IsValid())
		{
			return;
		}

		if (!Private::IsPinTypeAllowed(PinType->GetPinType(/*bForceLoadSubCategoryObject*/false)))
		{
			TypeTree.RemoveAt(Index);
			continue;
		}

		for (int32 ChildIndex = 0; ChildIndex < PinType->Children.Num(); )
		{
			TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> Child = PinType->Children[ChildIndex];
			if (Child.IsValid())
			{
				if (!Private::IsPinTypeAllowed(Child->GetPinType(/*bForceLoadSubCategoryObject*/false)))
				{
					PinType->Children.RemoveAt(ChildIndex);
					continue;
				}
			}
			++ChildIndex;
		}

		++Index;
	}
};

bool FUtils::IsValidParameterNameString(FStringView InStringView, FText& OutErrorText)
{
	// See if this can be represented as an FName
	if(!FName::IsValidXName(InStringView, INVALID_NAME_CHARACTERS, &OutErrorText))
	{
		return false;
	}

	return IsValidParameterName(FName(InStringView), OutErrorText);
}

bool FUtils::IsValidParameterName(const FName InName, FText& OutErrorText)
{
	const FString NewString = InName.ToString();

	if(NewString.Len() == 0)
	{
		OutErrorText = LOCTEXT("Error_EmptyName", "Empty names are not allowed");
		return false;
	}

	// Check start
	if (FChar::IsDigit(NewString[0]))
	{
		OutErrorText = LOCTEXT("Error_Start", "Name cannot start with a digit");
		return false;
	}

	bool bAllowed = true;
	for (int32 CharIndex = 0; bAllowed && CharIndex < NewString.Len(); ++CharIndex)
	{
		bAllowed &= FChar::IsAlnum(NewString[CharIndex]) ||
					FChar::IsUnderscore(NewString[CharIndex]);
	}

	// Make sure the new name only contains valid characters
	if (!bAllowed)
	{
		OutErrorText = LOCTEXT("Error_CharacterNotAllowed", "Only alpha-numerical or underscore characters are allowed");
		return false;
	}

	return true;
}

bool FUtils::IsValidParameterNameForEntry(const UAnimNextVariableEntry* Entry, const FText& InNewName, FText& OutErrorText)
{
	if(Entry == nullptr)
	{
		OutErrorText = LOCTEXT("InvalidVariableError", "Variable is invalid");
		return false;
	}

	const UUAFRigVMAssetEditorData* EditorData = Entry->GetTypedOuter<UUAFRigVMAssetEditorData>();
	if(EditorData == nullptr)
	{
		OutErrorText = LOCTEXT("InvalidEditorDataError", "Variable has invalid editor data");
		return false;
	}

	const FString NewString = InNewName.ToString();
	if (!IsValidParameterNameString(NewString, OutErrorText))
	{
		return false;
	}

	const FName Name(*NewString);
		
	// Cannot have two entries with matching names
	UUAFRigVMAssetEntry* ExistingEntry = EditorData->FindEntry(Name);
	if(ExistingEntry && ExistingEntry != Entry)
	{
		OutErrorText = LOCTEXT("NameExistsError", "Variable name already exists in this asset");
		return false;
	}
		
	return true;
}

bool FUtils::AddSchemaRigUnitAction(const TSubclassOf<URigVMUnitNode>& UnitNodeClass, UScriptStruct* Struct, const FRigVMFunction& Function, FGraphContextMenuBuilder& InContexMenuBuilder)
{
	FString CategoryMetadata, DisplayNameMetadata, MenuDescSuffixMetadata;
	UE::UAF::Editor::FUtils::GetRigUnitStructMetadata(Struct, CategoryMetadata, DisplayNameMetadata, MenuDescSuffixMetadata);
	if (DisplayNameMetadata.IsEmpty())
	{
		DisplayNameMetadata = Function.GetMethodName().ToString();
	}
	if (!MenuDescSuffixMetadata.IsEmpty())
	{
		MenuDescSuffixMetadata = TEXT(" ") + MenuDescSuffixMetadata;
	}
	const FText NodeCategory = FText::FromString(CategoryMetadata);
	const FText MenuDesc = FText::FromString(DisplayNameMetadata + MenuDescSuffixMetadata);
	const FText ToolTip = Struct->GetToolTipText();

	if (MenuDesc.IsEmpty())
	{
		return false;
	}

	InContexMenuBuilder.AddAction(MakeShared<FAnimNextSchemaAction_RigUnit>(UnitNodeClass, Struct, NodeCategory, MenuDesc, ToolTip));
	return true;
}

void FUtils::GetRigUnitStructMetadata(const UScriptStruct* Struct, FString& OutCategoryMetadata, FString& OutDisplayNameMetadata, FString& OutMenuDescSuffixMetadata)
{
	check(Struct);
	Struct->GetStringMetaDataHierarchical(FRigVMStruct::CategoryMetaName, &OutCategoryMetadata);
	Struct->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &OutDisplayNameMetadata);
	Struct->GetStringMetaDataHierarchical(FRigVMStruct::MenuDescSuffixMetaName, &OutMenuDescSuffixMetadata);
}

} // end namespace

#undef LOCTEXT_NAMESPACE