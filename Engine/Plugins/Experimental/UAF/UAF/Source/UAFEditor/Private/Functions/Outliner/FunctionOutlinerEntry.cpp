// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionOutlinerEntry.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "GraphEditorSettings.h"
#include "IAnimNextRigVMExportInterface.h"
#include "ISceneOutliner.h"
#include "ScopedTransaction.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"

class SInlineEditableTextBlock;

#define LOCTEXT_NAMESPACE "FFunctionsOutlinerEntryItem"

namespace UE::UAF::Editor
{

const FSceneOutlinerTreeItemType FFunctionsOutlinerEntryItem::Type(&FOutlinerEntryItem::Type);

FFunctionsOutlinerEntryItem::FFunctionsOutlinerEntryItem(const FItemData& ItemData)
	: FOutlinerEntryItem(FFunctionsOutlinerEntryItem::Type, {ItemData.Asset, ItemData.SortValue}),
		WeakLibraryNode(ItemData.LibraryNode)
{
	Flags.bIsExpanded = false;
}

bool FFunctionsOutlinerEntryItem::IsValid() const
{
	return WeakLibraryNode.IsValid();
}

FSceneOutlinerTreeItemID FFunctionsOutlinerEntryItem::GetID() const
{
	if (URigVMLibraryNode* Node = WeakLibraryNode.Get())
	{
		return GetTypeHash(Node);
	}

	return GetTypeHash(WeakLibraryNode);
}

FString FFunctionsOutlinerEntryItem::GetDisplayString() const
{
	if (URigVMLibraryNode* Node = WeakLibraryNode.Get())
	{
		return Node->GetName();
	}

	return FString();
}

FString FFunctionsOutlinerEntryItem::GetPackageName() const
{
	if (URigVMLibraryNode* Node = WeakLibraryNode.Get())
	{
		return Node->GetOutermost()->GetName();
	}

	return ISceneOutlinerTreeItem::GetPackageName();
}

void FFunctionsOutlinerEntryItem::Rename(const FText& InNewName) const
{
	if (URigVMLibraryNode* FunctionNode = WeakLibraryNode.Get())
	{
		if (const URigVMGraph* ContainedGraph = FunctionNode->GetContainedGraph())
		{
			if (IRigVMClientHost* RigVMHost = FunctionNode->GetImplementingOuter<IRigVMClientHost>())
			{
				if (URigVMEdGraph* ContainedEdGraph = Cast<URigVMEdGraph>(RigVMHost->GetEditorObjectForRigVMGraph(ContainedGraph)))
				{
					if (const UEdGraphSchema* GraphSchema = ContainedEdGraph->GetSchema())
					{
						FGraphDisplayInfo DisplayInfo;
						GraphSchema->GetGraphDisplayInformation(*ContainedEdGraph, DisplayInfo);

						// Check if the name is unchanged
						if (InNewName.EqualTo(DisplayInfo.PlainName))
						{
							return;
						}

						FScopedTransaction Transaction(LOCTEXT("RenameFunctionInOutliner", "Rename Function"));
						FRigVMControllerCompileBracketScope CompileScope(ContainedEdGraph->GetController());
						GraphSchema->TryRenameGraph(ContainedEdGraph, *InNewName.ToString());
					}
				}
			}
		}
	}
}

bool FFunctionsOutlinerEntryItem::ValidateName(const FText& InNewName, FText& OutErrorMessage) const
{
	if (InNewName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("NameEmpty", "Function name cannot be empty");
		return false;
	}

	if (URigVMLibraryNode* FunctionNode = WeakLibraryNode.Get())
	{
		if (const URigVMGraph* ContainedGraph = FunctionNode->GetContainedGraph())
		{
			if (IRigVMClientHost* RigVMHost = FunctionNode->GetImplementingOuter<IRigVMClientHost>())
			{
				const URigVMLibraryNode* ExistingFunction = RigVMHost->GetLocalFunctionLibrary()->FindFunction(*InNewName.ToString());
				if (ExistingFunction && ExistingFunction != FunctionNode)
				{
					OutErrorMessage = LOCTEXT("NameExistsError", "Function name already exists in this asset");
					return false;
				}
			}
		}
	}

	return true;
}

FStringView FFunctionsOutlinerEntryItem::GetCategoryPath() const
{
	return FStringView();
}

void FFunctionsOutlinerEntryItem::SetAccessSpecifier(const EAnimNextExportAccessSpecifier& InSpecifier) const
{
	if (URigVMLibraryNode* Node = WeakLibraryNode.Get())
	{
		if (UUAFRigVMAssetEditorData* EditorData = Node->GetTypedOuter<UUAFRigVMAssetEditorData>())
		{
			EditorData->MarkFunctionPublic(Node->GetFName(), InSpecifier == EAnimNextExportAccessSpecifier::Public);
		}
	}
}

EAnimNextExportAccessSpecifier FFunctionsOutlinerEntryItem::GetAccessSpecifier() const
{
	if (URigVMLibraryNode* Node = WeakLibraryNode.Get())
	{
		if (UUAFRigVMAssetEditorData* EditorData = Node->GetTypedOuter<UUAFRigVMAssetEditorData>())
		{
			if(EditorData->GetLocalFunctionLibrary()->IsFunctionPublic(Node->GetFName()))
			{
				return EAnimNextExportAccessSpecifier::Public;
			}

			return EAnimNextExportAccessSpecifier::Private;
		}
	}

	return EAnimNextExportAccessSpecifier::Private;
}

bool FFunctionsOutlinerEntryItem::CanSetAccessSpecifier() const
{
	if (URigVMLibraryNode* Node = WeakLibraryNode.Get())
	{
		if (Node->GetTypedOuter<UUAFRigVMAssetEditorData>())
		{
			return true;
		}
	}

	return false;
}

bool FFunctionsOutlinerEntryItem::IsReadOnly() const
{
	return !WeakLibraryNode.IsValid();
}

void FFunctionsOutlinerEntryItem::GetItemIconAndColor(const FSlateBrush*& OutBrush, FSlateColor& OutColor) const
{
	if (const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>())
    {    
	    if (URigVMLibraryNode* Node = WeakLibraryNode.Get())
	    {
		    if (Node->IsPure())
		    {
			    OutColor = Settings->PureFunctionCallNodeTitleColor;
			    OutBrush = FAppStyle::GetBrush(TEXT("GraphEditor.PureFunction_16x"));

			    return;
		    }
	    }
	    
	    OutColor = Settings->FunctionCallNodeTitleColor;
	    OutBrush = FAppStyle::GetBrush(TEXT("GraphEditor.Function_16x"));
	}
}}

#undef LOCTEXT_NAMESPACE // "FFunctionsOutlinerEntryItem"
