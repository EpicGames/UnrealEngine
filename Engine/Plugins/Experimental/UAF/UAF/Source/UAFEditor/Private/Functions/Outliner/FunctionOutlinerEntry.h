// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"
#include "Common/Outliner/OutlinerEntryItem.h"

class URigVMLibraryNode;
class UUAFSharedVariablesEntry;

namespace UE::UAF::Editor
{

struct FFunctionsOutlinerEntryItem : FOutlinerEntryItem
{
	static const FSceneOutlinerTreeItemType Type;
	
	struct FItemData
	{
		const TWeakObjectPtr<UUAFRigVMAsset>& Asset;
		const int32 SortValue;
		URigVMLibraryNode* LibraryNode;
	};

	FFunctionsOutlinerEntryItem() = delete;
	FFunctionsOutlinerEntryItem(const FItemData& ItemData);

	// Begin ISceneOutlinerTreeItem overrides
	virtual bool IsValid() const override;
	virtual FSceneOutlinerTreeItemID GetID() const override;
	virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override { return true; }
	virtual FString GetPackageName() const override;
	// End ISceneOutlinerTreeItem overrides

	// Renames the item to the specified name
	virtual void Rename(const FText& InNewName) const override;

	// Validates the new item name
	virtual bool ValidateName(const FText& InNewName, FText& OutErrorMessage) const override;

	virtual FStringView GetCategoryPath() const override;

	virtual void SetAccessSpecifier(const EAnimNextExportAccessSpecifier& InSpecifier) const override;
	virtual EAnimNextExportAccessSpecifier GetAccessSpecifier() const override;
	virtual bool CanSetAccessSpecifier() const override;
	virtual bool IsReadOnly() const override;
	virtual void GetItemIconAndColor(const FSlateBrush*& OutBrush, FSlateColor& OutColor) const override;

	// Ptr to the underlying RigVM library node representing this function within the RigVMAsset
	TWeakObjectPtr<URigVMLibraryNode> WeakLibraryNode;

	// The data interface entry this entry is from, if any
	TWeakObjectPtr<const UUAFSharedVariablesEntry> WeakSharedVariablesEntry;
};

}
