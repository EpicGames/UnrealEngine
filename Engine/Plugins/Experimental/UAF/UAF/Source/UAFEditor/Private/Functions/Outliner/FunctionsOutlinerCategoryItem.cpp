// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionsOutlinerCategoryItem.h"

#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FunctionsOutlinerCategoryItem"

namespace UE::UAF::Editor
{

const FSceneOutlinerTreeItemType FFunctionsOutlinerCategoryItem::Type(&FOutlinerCategoryItem::Type);

FFunctionsOutlinerCategoryItem::FFunctionsOutlinerCategoryItem(const FOutlinerCategoryItem::FItemData& InItemData) : FOutlinerCategoryItem(FFunctionsOutlinerCategoryItem::Type, InItemData)
{
}

void FFunctionsOutlinerCategoryItem::Rename(const FText& InNewName) const
{
	if(UUAFRigVMAsset* Owner = WeakOwner.Get())
	{
		if (UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(Owner))
		{
			FScopedTransaction Transaction(LOCTEXT("RenameCategory", "Rename category"));
			const TArray<URigVMLibraryNode*> LibraryNodes = EditorData->GetRigVMClient()->GetFunctionLibrary()->GetFunctions();

			FString NewCategoryPath = ParentCategoryName;
			if (!NewCategoryPath.IsEmpty())
			{
				NewCategoryPath.Append(TEXT("|"));
			}
			NewCategoryPath.Append(InNewName.ToString());

			for (URigVMLibraryNode* LibraryNode : LibraryNodes)
			{
				if (LibraryNode->GetNodeCategory() == CategoryPath)
				{
					if (URigVMGraph* Model = LibraryNode->GetContainedGraph())
					{
						if (URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
						{
							if (URigVMController* Controller = EditorData->GetOrCreateController(OuterNode->GetGraph()))
							{
								Controller->SetNodeCategory(OuterNode, NewCategoryPath, true, false, true);
							}
						}
					}
				}
			}
		}
	}
}

bool FFunctionsOutlinerCategoryItem::ValidateName(const FText& InNewName, FText& OutErrorMessage) const
{
	if (InNewName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("NameEmpty", "Category name cannot be empty");
		return false;
	}

	if(UUAFRigVMAsset* Owner = WeakOwner.Get())
	{
		if (UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(Owner))
		{
			const TArray<URigVMLibraryNode*> LibraryNodes = EditorData->GetRigVMClient()->GetFunctionLibrary()->GetFunctions();

			FString NewCategoryPath = ParentCategoryName;
			if (!NewCategoryPath.IsEmpty())
			{
				NewCategoryPath.Append(TEXT("|"));
			}
			NewCategoryPath.Append(InNewName.ToString());

			URigVMLibraryNode* const* ExistingNodeWithCategoryName = LibraryNodes.FindByPredicate([NewCategoryPath, CurrentCategoryPath = CategoryPath](const URigVMLibraryNode* Node)
				{
					return Node->GetNodeCategory().Equals(NewCategoryPath) && !Node->GetNodeCategory().Equals(CurrentCategoryPath);
				});

			if (ExistingNodeWithCategoryName)
			{
				OutErrorMessage = LOCTEXT("NameExistsError", "Category name already exists in this asset");
				return false;
			}

			return true;
		}
	}

	return false;
}
}
#undef LOCTEXT_NAMESPACE
