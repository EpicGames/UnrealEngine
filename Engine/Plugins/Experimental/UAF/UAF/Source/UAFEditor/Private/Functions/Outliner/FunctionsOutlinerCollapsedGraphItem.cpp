// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionsOutlinerCollapsedGraphItem.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "IAnimNextRigVMExportInterface.h"
#include "ScopedTransaction.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"

#define LOCTEXT_NAMESPACE "FFunctionsOutlinerCollapsedGraphItem"

namespace UE::UAF::Editor
{

const FSceneOutlinerTreeItemType FFunctionsOutlinerCollapsedGraphItem::Type(&FOutlinerEntryItem::Type);

FFunctionsOutlinerCollapsedGraphItem::FFunctionsOutlinerCollapsedGraphItem(const FItemData& ItemData)
	: FOutlinerEntryItem(FFunctionsOutlinerCollapsedGraphItem::Type, {ItemData.Asset, ItemData.SortValue}),
	WeakCollapseNode(ItemData.CollapseNode), WeakOwningNode(ItemData.OwningNode)
{
}

bool FFunctionsOutlinerCollapsedGraphItem::IsValid() const
{
	return WeakCollapseNode.IsValid();
}

FSceneOutlinerTreeItemID FFunctionsOutlinerCollapsedGraphItem::GetID() const
{
	if (URigVMCollapseNode* Node = WeakCollapseNode.Get())
	{
		return GetTypeHash(Node);
	}

	return GetTypeHash(WeakCollapseNode);
}

FString FFunctionsOutlinerCollapsedGraphItem::GetDisplayString() const
{
	if (URigVMCollapseNode* Node = WeakCollapseNode.Get())
	{
		return Node->GetName();
	}

	return FString();
}

FString FFunctionsOutlinerCollapsedGraphItem::GetPackageName() const
{
	if (URigVMCollapseNode* Node = WeakCollapseNode.Get())
	{
		return Node->GetOutermost()->GetName();
	}

	return ISceneOutlinerTreeItem::GetPackageName();
}

bool FFunctionsOutlinerCollapsedGraphItem::IsReadOnly() const
{
	return false;
}
EAnimNextExportAccessSpecifier FFunctionsOutlinerCollapsedGraphItem::GetAccessSpecifier() const
{
	if (URigVMLibraryNode* OwningNode = WeakOwningNode.Get())
	{
		if (UUAFRigVMAssetEditorData* EditorData = OwningNode->GetTypedOuter<UUAFRigVMAssetEditorData>())
		{
			if(EditorData->GetLocalFunctionLibrary()->IsFunctionPublic(OwningNode->GetFName()))
			{
				return EAnimNextExportAccessSpecifier::Public;
			}

			return EAnimNextExportAccessSpecifier::Private;
		}
	}
	
	return EAnimNextExportAccessSpecifier::Private;
}

bool FFunctionsOutlinerCollapsedGraphItem::CanSetAccessSpecifier() const
{
	return false;
}

FStringView FFunctionsOutlinerCollapsedGraphItem::GetCategoryPath() const
{
	return FStringView();
}

void FFunctionsOutlinerCollapsedGraphItem::GetItemIconAndColor(const FSlateBrush*& OutBrush, FSlateColor& OutColor) const
{
	OutBrush = FAppStyle::GetBrush(TEXT("GraphEditor.SubGraph_16x"));
}

void FFunctionsOutlinerCollapsedGraphItem::Rename(const FText& InNewName) const
{
	if (URigVMCollapseNode* CollapseNode = WeakCollapseNode.Get())
	{
		if (const URigVMGraph* ContainedGraph = CollapseNode->GetContainedGraph())
		{
			if (IRigVMClientHost* RigVMHost = CollapseNode->GetImplementingOuter<IRigVMClientHost>())
			{
				if (URigVMEdGraph* ContainedEdGraph = Cast<URigVMEdGraph>(RigVMHost->GetEditorObjectForRigVMGraph(ContainedGraph)))
				{
					if (const UEdGraphSchema* GraphSchema = ContainedEdGraph->GetSchema())
					{
						FGraphDisplayInfo DisplayInfo;
						GraphSchema->GetGraphDisplayInformation(*ContainedEdGraph, DisplayInfo);

						// Check if the name is unchanged
						if (InNewName.EqualTo(DisplayInfo.PlainName))
						{
							return;
						}

						FScopedTransaction Transaction(LOCTEXT("RenameCollapsedGraphInOutliner", "Rename Collapsed Graph"));
						FRigVMControllerCompileBracketScope CompileScope(ContainedEdGraph->GetController());
						GraphSchema->TryRenameGraph(ContainedEdGraph, *InNewName.ToString());
					}
				}
			}
		}
	}
}

bool FFunctionsOutlinerCollapsedGraphItem::ValidateName(const FText& InNewName, FText& OutErrorMessage) const
{
	if (InNewName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("NameEmpty", "Collapsed Graph name cannot be empty");
		return false;
	}

	return true;
}
}

#undef LOCTEXT_NAMESPACE // "FFunctionsOutlinerCollapsedGraphItem"
