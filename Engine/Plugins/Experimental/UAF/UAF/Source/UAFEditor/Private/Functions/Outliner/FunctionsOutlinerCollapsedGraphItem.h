// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Common/Outliner/OutlinerEntryItem.h"

class UUAFRigVMAsset;
class URigVMCollapseNode;
class URigVMLibraryNode;

namespace UE::UAF::Editor
{
struct FFunctionsOutlinerCollapsedGraphItem : FOutlinerEntryItem
{
	static const FSceneOutlinerTreeItemType Type;
	
	struct FItemData
	{
		const TWeakObjectPtr<UUAFRigVMAsset>& Asset;
		const int32 SortValue;
		URigVMCollapseNode* CollapseNode;
		URigVMLibraryNode* OwningNode;
	};
	
	FFunctionsOutlinerCollapsedGraphItem() = delete;
	FFunctionsOutlinerCollapsedGraphItem(const FItemData& ItemData);
	
	// Begin ISceneOutlinerTreeItem overrides
	virtual bool IsValid() const override;
	virtual FSceneOutlinerTreeItemID GetID() const override;
	virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override { return true; }
	virtual FString GetPackageName() const override;
	// End ISceneOutlinerTreeItem overrides
	
	virtual bool IsReadOnly() const override;
	virtual void SetAccessSpecifier(const EAnimNextExportAccessSpecifier& InSpecifier) const override {}
	virtual EAnimNextExportAccessSpecifier GetAccessSpecifier() const override;
	virtual bool CanSetAccessSpecifier() const override;
	virtual FStringView GetCategoryPath() const override;
	virtual void GetItemIconAndColor(const FSlateBrush*& OutBrush, FSlateColor& OutColor) const override;
	virtual void Rename(const FText& InNewName) const override;
	virtual bool ValidateName(const FText& InNewName, FText& OutErrorMessage) const override;

	TWeakObjectPtr<URigVMCollapseNode> WeakCollapseNode;
	TWeakObjectPtr<URigVMLibraryNode> WeakOwningNode;
};
}
