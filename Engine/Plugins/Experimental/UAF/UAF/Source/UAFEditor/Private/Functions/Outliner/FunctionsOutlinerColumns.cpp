// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionsOutlinerColumns.h"

#include "EditorUtils.h"
#include "FunctionOutlinerEntry.h"
#include "ISceneOutliner.h"
#include "PropertyBagDetails.h"
#include "SPinTypeSelector.h"
#include "UncookedOnlyUtils.h"
#include "Common/Outliner/OutlinerAssetItem.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FunctionsOutlinerColumns"

namespace UE::UAF::Editor
{

FFunctionsOutlinerOutputColumn::FFunctionsOutlinerOutputColumn(ISceneOutliner& SceneOutliner): WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) {}

FLazyName FunctionsOutlinerType("Output");
FName FFunctionsOutlinerOutputColumn::GetID()
{
	return FunctionsOutlinerType;
}

SHeaderRow::FColumn::FArguments FFunctionsOutlinerOutputColumn::ConstructHeaderRowColumn()
{
	return
		SHeaderRow::Column(GetColumnID())
		.FillWidth(1.0f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.ShouldGenerateEmptyWidgetForSpacing(false)
		[
			SNew(SBox) 
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OutputLabel", "Output"))
				.ToolTipText(LOCTEXT("OutputTooltip", "Output type(s) of the function"))
			]
		];
}


class SFunctionsOutlinerFunctionOutput : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SFunctionsOutlinerFunctionOutput) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FFunctionsOutlinerEntryItem& InTreeItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakTreeItem = StaticCastSharedRef<FFunctionsOutlinerEntryItem>(InTreeItem.AsShared());

		if (URigVMLibraryNode* LibraryNode = InTreeItem.WeakLibraryNode.Get())
		{
			FunctionHeader = LibraryNode->GetFunctionHeader();
			for (FRigVMGraphFunctionArgument& Argument : FunctionHeader.Arguments)
			{
				if (Argument.Direction == ERigVMPinDirection::Output)
				{
					FunctionOutputs.Add(&Argument);
				}
			}
		}
		
		ChildSlot
		[
			SAssignNew(HBox, SHorizontalBox)
			.ToolTipText_Lambda([this]()-> FText
			{
				const FText TooltipFormat(LOCTEXT("SFunctionsOutlinerFunctionOutput_Format", "{0}\n\nOutputs:{1}"));
				
				const FString OutputString = [this]() -> FString
                {
				    if (FunctionOutputs.Num() == 0)
				    {
					    return TEXT("none");
				    }
				    else
				    {
						FString TempString;
					    for (const FRigVMGraphFunctionArgument* Output : FunctionOutputs)
					    {
						    if (TempString.Len() > 0)
						    {
							    TempString.Append(TEXT(", "));
						    }
						    
						    TempString.Appendf(TEXT("%s(%s)"), *Output->DisplayName.ToString(), *Output->CPPType.ToString());
					    }
					    
					    return TempString;
				    }
                }();
				
				return FText::Format(TooltipFormat, FText::FromName(FunctionHeader.Name), FText::FromString(OutputString));
			})
		];

		if (FunctionOutputs.Num() == 0)
		{
			HBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(2.0f, 3.0f, 2.0f, 3.0f))
			[
				SNew(STextBlock)
				.Text_Lambda([]() -> FText
				{
					return LOCTEXT("NoneLabel", "none");
				})
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			];
		}
		else if (FunctionOutputs.Num() == 1)
		{
			HBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SPinTypeSelector, FGetPinTypeTree::CreateStatic(&Editor::FUtils::GetFilteredVariableTypeTree))
				.TargetPinType_Lambda([Argument = *FunctionOutputs[0]]() -> FEdGraphPinType
				{
					FRigVMTemplateArgumentType RigVMArgumentType(Argument.CPPType, Argument.CPPTypeObject.Get());						
					return UncookedOnly::FUtils::GetPinTypeFromParamType(FAnimNextParamType::FromRigVMTemplateArgument(RigVMArgumentType));
				})
				.ReadOnly(true)
				.Schema(GetDefault<UPropertyBagSchema>())
				.bAllowArrays(true)
				.TypeTreeFilter(ETypeTreeFilter::None)
				.SelectorType(SPinTypeSelector::ESelectorType::None)
			];

			HBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.f, 0.f))
			[
				SNew(STextBlock)
				.Text_Lambda([Argument = *FunctionOutputs[0]]() -> FText
				{
					return FText::FromName(Argument.DisplayName);
				})
			];
		}
		else if (FunctionOutputs.Num() >= 2)
		{
			HBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(FMargin(2.0f, 3.0f, 2.0f, 3.0f))
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush(TEXT("Kismet.VariableList.TypeIcon")))
				.ColorAndOpacity(GetForegroundColor())
			];

			HBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.f, 0.f))
			[
				SNew(STextBlock)
				.Text_Lambda([]() -> FText
				{
					return LOCTEXT("MultipleValuesLabel", "multiple");
				})
				.ColorAndOpacity(FSlateColor::UseForeground())
			];
		}
	}

	TWeakPtr<FFunctionsOutlinerEntryItem> WeakTreeItem;
	FRigVMGraphFunctionHeader FunctionHeader;
	TArray<FRigVMGraphFunctionArgument*> FunctionOutputs;
	TSharedPtr<SHorizontalBox> HBox;
};

const TSharedRef<SWidget> FFunctionsOutlinerOutputColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef Item, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	FOutlinerAssetItem* AssetItem = Item->CastTo<FOutlinerAssetItem>();
	if (AssetItem)
	{
		return SNullWidget::NullWidget;
	}
	
	// Return a dummy box widget, with ShouldGenerateEmptyWidgetForSpacing settings this will prevent us from skipping outside of the header, avoiding alignment issues
	FFunctionsOutlinerEntryItem* TreeItem = Item->CastTo<FFunctionsOutlinerEntryItem>();
	if (TreeItem == nullptr)
	{
		return SNew(SBox);
	}

	return SNew(SFunctionsOutlinerFunctionOutput, *TreeItem, WeakSceneOutliner.Pin().ToSharedRef().Get(), Row);
}


}

#undef LOCTEXT_NAMESPACE
