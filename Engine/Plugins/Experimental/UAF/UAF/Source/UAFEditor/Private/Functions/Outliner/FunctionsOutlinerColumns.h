// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerColumn.h"

namespace UE::UAF::Editor
{

class FFunctionsOutlinerOutputColumn : public ISceneOutlinerColumn
{
public:
	static FName GetID();

	FFunctionsOutlinerOutputColumn(ISceneOutliner& SceneOutliner);

	// ISceneOutlinerColumn interface
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return false; }

private:
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;
};

}
