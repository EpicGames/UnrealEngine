// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionsOutlinerHierarchy.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "FunctionOutlinerEntry.h"
#include "FunctionsOutlinerCategoryItem.h"
#include "FunctionsOutlinerCollapsedGraphItem.h"
#include "ISceneOutlinerMode.h"
#include "Common/Outliner/OutlinerAssetItem.h"

namespace UE::UAF::Editor
{

FSceneOutlinerTreeItemPtr FFunctionsOutlinerHierarchy::FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate)
{
	FSceneOutlinerTreeItemPtr ParentItem;
	
	uint32 ParentHash = INDEX_NONE;
	FSoftObjectPath OwnerPath;
	FString CategoryPath;
	
	if (const FFunctionsOutlinerEntryItem* EntryItem = Item.CastTo<FFunctionsOutlinerEntryItem>())
	{
		if (URigVMLibraryNode* LibraryNode = EntryItem->WeakLibraryNode.Get())
		{
			if (UUAFRigVMAsset* Asset = LibraryNode->GetTypedOuter<UUAFRigVMAsset>())
			{
				OwnerPath = Asset;
			}

			uint32 CategoryHash = INDEX_NONE;
			if (!LibraryNode->GetNodeCategory().IsEmpty())
			{
				CategoryPath = LibraryNode->GetNodeCategory();
				CategoryHash = GetTypeHash(LibraryNode->GetNodeCategory());
			}
			
			check(OwnerPath.IsValid());
			ParentHash = CategoryHash != INDEX_NONE ? HashCombine(GetTypeHash(OwnerPath), CategoryHash) : GetTypeHash(OwnerPath);				
		}
	}
	
	if (const FFunctionsOutlinerCollapsedGraphItem* CollapsedGraphItem = Item.CastTo<FFunctionsOutlinerCollapsedGraphItem>())
	{
		if (URigVMLibraryNode* LibraryNode = CollapsedGraphItem->WeakOwningNode.Get())
		{
			ParentHash = GetTypeHash(LibraryNode);
		}
	}

	if (const FFunctionsOutlinerCategoryItem* CategoryItem = Item.CastTo<FFunctionsOutlinerCategoryItem>())
	{
		if (UUAFRigVMAsset* Asset = CategoryItem->WeakOwner.Get())
		{
			OwnerPath = Asset;
			const uint32 AssetHash = GetTypeHash(OwnerPath);
			const FStringView ParentCategory = CategoryItem->ParentCategoryName;
			CategoryPath = CategoryItem->ParentCategoryName;
			ParentHash = ParentCategory.Len() ? HashCombine(AssetHash, GetTypeHash(ParentCategory)) : AssetHash;
		}
	}

	if (const FSceneOutlinerTreeItemPtr* ParentItemPtr = Items.Find(ParentHash))
	{
		return *ParentItemPtr;
	}
	
	TSoftObjectPtr<UUAFRigVMAsset> SoftAsset = Cast<UUAFRigVMAsset>(OwnerPath.ResolveObject());

	if (!CategoryPath.IsEmpty())
	{
		UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(SoftAsset.Get());
		const int32 CategoryIndex = 0;
		ensure(CategoryIndex != INDEX_NONE);
	
		FStringView ParentCategoryName;
		FStringView SubCategoryName;
	
		int32 Index = INDEX_NONE;
		if (CategoryPath.FindLastChar('|', Index))
		{
			ParentCategoryName = MakeStringView(CategoryPath).Left(Index);
			SubCategoryName = MakeStringView(CategoryPath).RightChop(Index+1);
		}
		else
		{
			SubCategoryName = CategoryPath;
		}
	
		if (FSceneOutlinerTreeItemPtr CategoryItem = Mode->CreateItemFor<FFunctionsOutlinerCategoryItem>(FOutlinerCategoryItem::FItemData{UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(EditorData), CategoryIndex, SubCategoryName, ParentCategoryName, CategoryPath}, bCreate))
		{			
			UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(),LogAnimation, Warning, "+ FFunctionsOutlinerCategoryItem: %ls  %i", *CategoryPath, 0);
			
			return CategoryItem;
		}
	}
	else if (OwnerPath.IsValid())
	{
		TArray<const FSoftObjectPath> SharedVariableSourcePaths;
		if (SoftAsset.IsValid())
		{
			RetrieveSharedAssetPaths(SoftAsset, SharedVariableSourcePaths);
			if (FSceneOutlinerTreeItemPtr AssetItem = Mode->CreateItemFor<FOutlinerAssetItem>(FOutlinerAssetItem::FItemData{SoftAsset, 0, SharedVariableSourcePaths}, bCreate))
			{
				UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(),LogAnimation, Warning, "+ FOutlinerAssetItem: %ls  %i", *SoftAsset.GetAssetName(), 0);				
				return AssetItem;
			}
		}
	}

	return ParentItem;
}

FFunctionsOutlinerHierarchy::FFunctionsOutlinerHierarchy(ISceneOutlinerMode* Mode)
	: FOutlinerHierarchy(Mode)
{
}

void FFunctionsOutlinerHierarchy::RecurseCollapseNodeGraphs(const UUAFRigVMAssetEditorData* EditorData, TArray<FSceneOutlinerTreeItemPtr>& OutItems, URigVMLibraryNode* LibraryNode) const
{
	int32 SubIndex = 0;
	for (URigVMNode* Node : LibraryNode->GetContainedGraph()->GetNodes())
	{
		if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
		{
			if (FSceneOutlinerTreeItemPtr CollapsedItem = Mode->CreateItemFor<FFunctionsOutlinerCollapsedGraphItem>(FFunctionsOutlinerCollapsedGraphItem::FItemData{UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(EditorData), ++SubIndex, CollapseNode, LibraryNode}))
			{
				OutItems.Add(CollapsedItem);
				
				RecurseCollapseNodeGraphs(EditorData, OutItems, CollapseNode);
			}
		}
	}
}

void FFunctionsOutlinerHierarchy::CreateItemsForAsset(const UUAFRigVMAssetEditorData* EditorData, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	TSet<FString> ParsedCategories;
	auto RecursivelyAddCategories = [this, &EditorData, &OutItems, &ParsedCategories](const FString InCategoryPath)
	{	
		const int32 InsertionIndex = OutItems.Num(); 
		FString CategoryPath = InCategoryPath;
		while (!CategoryPath.IsEmpty())
		{
			FString CategoryParent;
			FString Category;
			
			if (!CategoryPath.Split(TEXT("|"), &CategoryParent, &Category, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
			{
				Category = CategoryPath;
			}
			
			if (!ParsedCategories.Contains(CategoryPath))
			{
				if (auto Item = Mode->CreateItemFor<FFunctionsOutlinerCategoryItem>(FOutlinerCategoryItem::FItemData{UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(EditorData), 0, Category, CategoryParent, CategoryPath}))
				{
					OutItems.Insert(Item, InsertionIndex);
					ParsedCategories.Add(CategoryPath);
				
					UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(),LogAnimation, Warning, "+ FFunctionsOutlinerCategoryItem: %ls  %i", *CategoryPath, 0);
				}
			}			
			
			CategoryPath = CategoryParent;
		}
	};
	
	
	if (URigVMFunctionLibrary* FunctionLibrary = EditorData->GetLocalFunctionLibrary())
	{
		TArray<URigVMLibraryNode*> LibraryNodes = FunctionLibrary->GetFunctions();

		int32 Index = 0;
		for (URigVMLibraryNode* LibraryNode : LibraryNodes)
		{
			if (FSceneOutlinerTreeItemPtr Item = Mode->CreateItemFor<FFunctionsOutlinerEntryItem>(FFunctionsOutlinerEntryItem::FItemData{UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(EditorData), Index++, LibraryNode}))
			{
				if (FFunctionsOutlinerEntryItem* EntryItem = Item->CastTo<FFunctionsOutlinerEntryItem>())
				{
					EntryItem->SortValue = Index++;	

					UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(), LogAnimation, Warning, "\t- FFunctionsOutlinerEntryItem: %ls %i %ls", *EntryItem->WeakLibraryNode.Get()->GetName(), Index, *LibraryNode->GetNodeCategory());
				}

				const FString CategoryPath = LibraryNode->GetNodeCategory();
				RecursivelyAddCategories(CategoryPath);
				OutItems.Add(Item);

				if (LibraryNode->GetContainedGraph())
				{
					RecurseCollapseNodeGraphs(EditorData, OutItems, LibraryNode);
				}
			}
		}
	}
}

}
