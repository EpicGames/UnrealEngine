// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/Outliner/OutlinerHierarchy.h"

class URigVMLibraryNode;

namespace UE::UAF::Editor
{
class FFunctionsOutlinerHierarchy : public FOutlinerHierarchy
{
public:
	virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;

	FFunctionsOutlinerHierarchy(ISceneOutlinerMode* Mode);
	virtual void CreateItemsForAsset(const UUAFRigVMAssetEditorData* EditorData, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
private:
	void RecurseCollapseNodeGraphs(const UUAFRigVMAssetEditorData* EditorData, TArray<FSceneOutlinerTreeItemPtr>& OutItems, URigVMLibraryNode* LibraryNode) const;
};
}
