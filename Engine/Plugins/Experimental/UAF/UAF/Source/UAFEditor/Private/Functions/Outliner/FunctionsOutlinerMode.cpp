// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionsOutlinerMode.h"

#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "FunctionOutlinerEntry.h"
#include "FunctionsOutlinerCollapsedGraphItem.h"
#include "FunctionsOutlinerView.h"
#include "FunctionsOutlinerHierarchy.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "Common/GraphEditorSchemaActions.h"
#include "Common/Outliner/OutlinerAssetItem.h"
#include "Common/Outliner/OutlinerCommands.h"
#include "EdGraph/RigVMEdGraph.h"
#include "Functions/Outliner/FunctionsOutlinerCategoryItem.h"

#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "Common/Outliner/OutlinerItemMenuContexts.h"
#include "Common/Outliner/OutlinerDragDropOps.h"
#include "Common/Outliner/SCommonOutliner.h"
#include "Settings/UAFEditorUserSettings.h"

#define LOCTEXT_NAMESPACE "FFunctionsOutlinerMode"

namespace UE::UAF::Editor
{

FFunctionsOutlinerMode::FFunctionsOutlinerMode(SCommonOutliner* InFunctionsOutliner, const TSharedRef<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor)
	: FCommonOutlinerMode(InFunctionsOutliner, InWorkspaceEditor)
{
	RegisterDragDropOperationConstructor<FFunctionsOutlinerEntryItem>([](const FSceneOutlinerTreeItemPtr& InItem)-> TSharedPtr<FDragDropOperation>
	{
		FFunctionsOutlinerEntryItem* FunctionItem = InItem->CastTo<FFunctionsOutlinerEntryItem>();
		check(FunctionItem);
		
		if(URigVMLibraryNode* Node = FunctionItem->WeakLibraryNode.Get())
		{
			TSharedPtr<FAnimNextSchemaAction_Function> Action = MakeShared<FAnimNextSchemaAction_Function>(Node);

			return FFunctionDragDropOp::New(Action,	StaticCastSharedRef<FFunctionsOutlinerEntryItem>(FunctionItem->AsShared()));
		}
		return nullptr;
	});
	
	RegisterDragDropOperationParser<FFunctionDragDropOp>([](FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& InDragDropOperation)
	{
		check(InDragDropOperation.IsOfType<FFunctionDragDropOp>());
		const FFunctionDragDropOp& FunctionDragDrop = static_cast<const FFunctionDragDropOp&>(InDragDropOperation);
				
		OutPayload.DraggedItems.Add(FunctionDragDrop.WeakItem.Pin());
	});
	
	RegisterDragDropOperator<FFunctionsOutlinerEntryItem, FOutlinerCategoryItem, FFunctionDragDropOp>([](const ISceneOutlinerTreeItem& DragOverTreeItem, TSharedRef<FDragDropOperation> DragDropOperation, FSceneOutlinerDragValidationInfo& OutValidationInfo) -> FReply
		{		
			TSharedRef<FFunctionDragDropOp> FuncDragDrop = StaticCastSharedRef<FFunctionDragDropOp>(DragDropOperation);			
			const FOutlinerCategoryItem* DragOverItem = DragOverTreeItem.CastTo<FOutlinerCategoryItem>();			
			TSharedPtr<FFunctionsOutlinerEntryItem> DropFuncItem = FuncDragDrop->WeakItem.Pin();
			
			if (URigVMLibraryNode* LibraryNode = DropFuncItem->WeakLibraryNode.Get())
			{
				const FText OtherFuncName = FText::FromString(DropFuncItem->GetDisplayString());
				const FText ThisFuncCategoryName = FText::FromString(DragOverItem->GetDisplayString());
				
				const UUAFRigVMAssetEditorData* FuncEditorData = LibraryNode->GetTypedOuter<UUAFRigVMAssetEditorData>();
				const UUAFRigVMAssetEditorData* CategoryEditorData = UncookedOnly::FUtils::GetEditorData(DragOverItem->WeakOwner.Get());

				if (FuncEditorData == CategoryEditorData)
				{
					if (LibraryNode->GetNodeCategory() == DragOverItem->CategoryPath)
					{
						const FText FormattedMessage = FText::Format(LOCTEXT("FuncAlreadyPartOfCategoryFormat", "{0} is already part of Category {1}"), OtherFuncName, ThisFuncCategoryName);
					
						OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, FormattedMessage);
					}
					else if (!LibraryNode->GetNodeCategory().IsEmpty())
					{
						const FText FormattedMessage = FText::Format(LOCTEXT("MoveFuncToCategoryFormat", "Moving {0} to Category {1}"), OtherFuncName, ThisFuncCategoryName);
						OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
					}
					else
					{
						const FText FormattedMessage = FText::Format(LOCTEXT("AddFuncToCategoryFormat", "Add {0} to Category {1}"), OtherFuncName, ThisFuncCategoryName);
						OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
					}
					
					return FReply::Handled();
				}
			}
		
			return FReply::Unhandled();
		},
	[this](ISceneOutlinerTreeItem& DroppedOnTreeItem, TSharedRef<const FDragDropOperation> DragDropOperation)
		{
			TSharedRef<const FFunctionDragDropOp> FuncDragDrop = StaticCastSharedRef<const FFunctionDragDropOp>(DragDropOperation);			
			const FOutlinerCategoryItem* DragOverItem = DroppedOnTreeItem.CastTo<FOutlinerCategoryItem>();			
			TSharedPtr<FFunctionsOutlinerEntryItem> DropFuncItem = FuncDragDrop->WeakItem.Pin();
			
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(DropFuncItem->WeakLibraryNode.Get()))
			{
				const FText OtherFuncName = FText::FromString(DropFuncItem->GetDisplayString());
				const FText ThisFuncCategoryText = FText::FromString(DragOverItem->GetDisplayString());
			
				const UUAFRigVMAssetEditorData* VariableEditorData = CollapseNode->GetTypedOuter<UUAFRigVMAssetEditorData>();
				const UUAFRigVMAssetEditorData* CategoryEditorData = UncookedOnly::FUtils::GetEditorData(DragOverItem->WeakOwner.Get());

				if (VariableEditorData && VariableEditorData == CategoryEditorData)
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("MoveFuncToCategoryFormat", "Moving {0} to Category {1}"), OtherFuncName, ThisFuncCategoryText);	
					FScopedTransaction Transaction(FormattedMessage);
					
					if (URigVMGraph* Model = CollapseNode->GetGraph())
					{
						if (IRigVMClientHost* Host = CollapseNode->GetImplementingOuter<IRigVMClientHost>())
						{
							if (URigVMController* Controller = Host->GetOrCreateController(CollapseNode->GetGraph()))
							{
								return Controller->SetNodeCategory(CollapseNode, DragOverItem->CategoryPath, true, false, true);
							}
						}						
					}
				}
				
			}
			return false;
		});
		
	RegisterDragDropOperator<FFunctionsOutlinerEntryItem, FOutlinerAssetItem, FFunctionDragDropOp>([](const ISceneOutlinerTreeItem& DragOverTreeItem, TSharedRef<FDragDropOperation> DragDropOperation, FSceneOutlinerDragValidationInfo& OutValidationInfo) -> FReply
	{		
		TSharedRef<FFunctionDragDropOp> FuncDragDrop = StaticCastSharedRef<FFunctionDragDropOp>(DragDropOperation);			
		const FOutlinerAssetItem* DragOverItem = DragOverTreeItem.CastTo<FOutlinerAssetItem>();			
		TSharedPtr<FFunctionsOutlinerEntryItem> DropFuncItem = FuncDragDrop->WeakItem.Pin();
		
		if (URigVMLibraryNode* LibraryNode = DropFuncItem->WeakLibraryNode.Get())
		{
			const FText OtherFuncName = FText::FromString(DropFuncItem->GetDisplayString());
			const FText ThisFuncCategoryText = FText::FromString(DropFuncItem->WeakLibraryNode.Get()->GetNodeCategory());
			
			const UUAFRigVMAssetEditorData* FuncEditorData = LibraryNode->GetTypedOuter<UUAFRigVMAssetEditorData>();
			const UUAFRigVMAssetEditorData* AssetEditorData = UncookedOnly::FUtils::GetEditorData(DragOverItem->WeakOwner.Get());

			if (FuncEditorData == AssetEditorData)
			{
				if (!LibraryNode->GetNodeCategory().IsEmpty())
				{
					const FText FormattedMessage = FText::Format(LOCTEXT("RemoveFuncFromCategoryFormat", "Remove {0} from Category {1}"), OtherFuncName, ThisFuncCategoryText);
					OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
					
					return FReply::Handled();
				}				
			}
		}
	
		return FReply::Unhandled();
	},
	[this](ISceneOutlinerTreeItem& DroppedOnTreeItem, TSharedRef<const FDragDropOperation> DragDropOperation)
	{
		TSharedRef<const FFunctionDragDropOp> FuncDragDrop = StaticCastSharedRef<const FFunctionDragDropOp>(DragDropOperation);	
		const FOutlinerAssetItem* DragOverItem = DroppedOnTreeItem.CastTo<FOutlinerAssetItem>();
		TSharedPtr<FFunctionsOutlinerEntryItem> DropFuncItem = FuncDragDrop->WeakItem.Pin();
		
		if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(DropFuncItem->WeakLibraryNode.Get()))
		{
			const FText OtherFuncName = FText::FromString(DropFuncItem->GetDisplayString());
			const FText ThisFuncCategoryText = FText::FromString(CollapseNode->GetNodeCategory());
		
			const UUAFRigVMAssetEditorData* FuncEditorData = CollapseNode->GetTypedOuter<UUAFRigVMAssetEditorData>();
			const UUAFRigVMAssetEditorData* AssetEditorData = UncookedOnly::FUtils::GetEditorData(DragOverItem->WeakOwner.Get());

			if (FuncEditorData == AssetEditorData)
			{
				const FText FormattedMessage = FText::Format(LOCTEXT("RemovingFuncFromCategoryFormat", "Removing {0} from Category {1}"), OtherFuncName, ThisFuncCategoryText);	
				FScopedTransaction Transaction(FormattedMessage);
				
				if (URigVMGraph* Model = CollapseNode->GetGraph())
				{
					if (IRigVMClientHost* Host = CollapseNode->GetImplementingOuter<IRigVMClientHost>())
					{
						if (URigVMController* Controller = Host->GetOrCreateController(CollapseNode->GetGraph()))
						{
							return Controller->SetNodeCategory(CollapseNode, TEXT(""), true, false, true);
						}
					}
				}
			}
		}
		return false;
	});
}

bool FFunctionsOutlinerMode::CanRename() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();
		return ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract();
	}
	
	return false;
}

void FFunctionsOutlinerMode::Rename() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();

		if (ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
		{
			SceneOutliner->SetPendingRenameItem(ItemToRename);
			SceneOutliner->ScrollItemIntoView(ItemToRename);
		}
	}
}

bool FFunctionsOutlinerMode::CanDelete() const
{	
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();
	for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
	{
		if (const FFunctionsOutlinerEntryItem* FunctionEntry = Item->CastTo<FFunctionsOutlinerEntryItem>())
		{
			return true;
		}
		else if (const FFunctionsOutlinerCategoryItem* CategoryItem = Item->CastTo<FFunctionsOutlinerCategoryItem>())
		{
			return true;
		}
	}
	
	return false;
}

void FFunctionsOutlinerMode::Delete() const
{
	const FSceneOutlinerItemSelection Selection = SceneOutliner->GetSelection();
	
	FScopedTransaction Transaction(LOCTEXT("DeleteFunction", "Delete Function(s)"));
	
	Selection.ForEachItem<FFunctionsOutlinerEntryItem>([](FFunctionsOutlinerEntryItem& FunctionItem)
	{
		if (URigVMLibraryNode* LibraryNode = FunctionItem.WeakLibraryNode.Get())
		{
			if (IRigVMClientHost* Host = LibraryNode->GetImplementingOuter<IRigVMClientHost>())
			{
				URigVMController* Controller = Host->GetController(LibraryNode->GetGraph());
				Controller->RemoveFunctionFromLibrary(LibraryNode->GetFName(), true, true);
			}
		}
	});
	
	if (Selection.Num<FFunctionsOutlinerCategoryItem>())
	{
	}
	
	Selection.ForEachItem<FFunctionsOutlinerCategoryItem>([](FFunctionsOutlinerCategoryItem& FunctionItem)
	{
		if (UUAFRigVMAsset* Asset = FunctionItem.WeakOwner.Get())
		{
			UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(Asset);
			if (URigVMFunctionLibrary* FunctionLibrary = EditorData->GetLocalFunctionLibrary())
			{
				TArray<URigVMLibraryNode*> LibraryNodes = FunctionLibrary->GetFunctions();
				for (URigVMLibraryNode* LibraryNode : LibraryNodes)
				{
					if (LibraryNode->GetNodeCategory().StartsWith(FunctionItem.CategoryPath))
					{
						URigVMController* Controller = LibraryNode->GetImplementingOuter<IRigVMClientHost>()->GetController(LibraryNode->GetGraph());
						Controller->OpenUndoBracket("Delete Function(s)");
						Controller->RemoveFunctionFromLibrary(LibraryNode->GetFName(), true, true);
						Controller->CloseUndoBracket();
					}
				}
			}
		}
	});
}

bool FFunctionsOutlinerMode::CanCopy() const
{
	return FCommonOutlinerMode::CanCopy();
}

bool FFunctionsOutlinerMode::CanCut() const
{
	return FCommonOutlinerMode::CanCut();
}

bool FFunctionsOutlinerMode::CanPaste() const
{
	return FCommonOutlinerMode::CanPaste();
}

bool FFunctionsOutlinerMode::CanDuplicate() const
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();
	for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
	{
		if (const FFunctionsOutlinerEntryItem* FunctionEntry = Item->CastTo<FFunctionsOutlinerEntryItem>())
		{
			return true;
		}
	}
	
	return false;
}

void FFunctionsOutlinerMode::Duplicate() const
{	
	FScopedTransaction Transaction(LOCTEXT("DuplicateFunction", "Duplicate Function(s)"));

	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();
	for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
	{
		if (const FFunctionsOutlinerEntryItem* FunctionEntry = Item->CastTo<FFunctionsOutlinerEntryItem>())
		{
			if (URigVMLibraryNode* Node = FunctionEntry->WeakLibraryNode.Get())
			{
				if (IRigVMClientHost* Host = Node->GetImplementingOuter<IRigVMClientHost>())
				{				
					if (URigVMGraph* ContainedGraph = Node->GetContainedGraph())
					{
						if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(ContainedGraph->GetOuter()))
						{
							if (URigVMController* Controller = Host->GetOrCreateController(CollapseNode->GetGraph()))
							{
								TArray<FName> NodeNamesToExport;
								NodeNamesToExport.Add(CollapseNode->GetFName());
								const FString FunctionNodeText = Controller->ExportNodesToText(NodeNamesToExport);
								
								URigVMController* LibraryController = Host->GetOrCreateController(Host->GetLocalFunctionLibrary());
								TArray<FName> ImportedNodeNames = LibraryController->ImportNodesFromText(FunctionNodeText, true, true);
								
								if (ImportedNodeNames.Num() != 0)
								{
									if (URigVMCollapseNode* NewCollapseNode = Cast<URigVMCollapseNode>(Host->GetLocalFunctionLibrary()->FindFunction(ImportedNodeNames[0])))
									{								
									}
									else
									{
										LibraryController->Undo();
									}
								}								
							}
						}
					}
				}
			}
		}
	}
	
	FCommonOutlinerMode::Duplicate();
}

void FFunctionsOutlinerMode::InitToolMenuContext(FToolMenuContext& InOutContext) const
{
	FCommonOutlinerMode::InitToolMenuContext(InOutContext);
	
	UFunctionsOutlinerItemMenuContext* FunctionsContext = NewObject<UFunctionsOutlinerItemMenuContext>();
	InOutContext.AddObject(FunctionsContext);
}

void FFunctionsOutlinerMode::Copy() const
{
	FCommonOutlinerMode::Copy();
}

void FFunctionsOutlinerMode::Cut() const
{
	FCommonOutlinerMode::Cut();
}

void FFunctionsOutlinerMode::Paste() const
{
	FCommonOutlinerMode::Paste();
}

void FFunctionsOutlinerMode::HandleItemSelection(const FSceneOutlinerItemSelection& Selection) const
{
	if (TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin())
	{
		TArray<FSceneOutlinerTreeItemPtr> SelectedItems;
		Selection.Get(SelectedItems);
		TArray<UObject*> EntriesToShow;
		EntriesToShow.Reserve(SelectedItems.Num());
		for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
		{
			if (FFunctionsOutlinerEntryItem* FunctionItem = Item->CastTo<FFunctionsOutlinerEntryItem>())
			{
				if (FunctionItem->WeakLibraryNode.IsValid())
				{
					const URigVMLibraryNode* FunctionNode = FunctionItem->WeakLibraryNode.Get();
					if (URigVMGraph* ContainedGraph = FunctionNode->GetContainedGraph())
					{					
						if (IRigVMClientHost* Host = FunctionNode->GetImplementingOuter<IRigVMClientHost>())
						{
							if (URigVMEdGraph* ContainedEdGraph = Cast<URigVMEdGraph>(Host->GetEditorObjectForRigVMGraph(ContainedGraph)))
							{							
								EntriesToShow.Add(ContainedEdGraph);
							}
						}						
					}
				}
			}
		}

		WorkspaceEditor->SetDetailsObjects(EntriesToShow);
	}
}

void FFunctionsOutlinerMode::RegisterToolMenus()
{
	FCommonOutlinerMode::RegisterToolMenus();
	
	UToolMenus* ToolMenus = UToolMenus::Get();
		
	if (UToolMenu* Menu = ToolMenus->ExtendMenu(ContextToolMenuName))
	{	
		FToolMenuSection& FunctionsSection = Menu->AddDynamicSection("Functions", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			PopulateNewFunctionToolMenuEntries(InMenu, false);

			if (const UFunctionsOutlinerItemMenuContext* FunctionOutlinerContext = InMenu->FindContext<UFunctionsOutlinerItemMenuContext>())
			{
				if (const UCommonOutlinerItemMenuContext* OutlinerContext = InMenu->FindContext<UCommonOutlinerItemMenuContext>())
				{
					FToolMenuSection& Section = InMenu->AddSection("Navigation", LOCTEXT("FunctionNavigationLabel", "Navigation"), FToolMenuInsert(FCommonOutlinerMode::ContextToolEntrySectionName, EToolMenuInsertType::Before));
					Section.AddMenuEntryWithCommandList(FOutlinerCommands::Get().OpenInNewTab, OutlinerContext->WeakCommandList.Pin());
				}
			}
		}));
	}
	
	if (UToolMenu* Menu = ToolMenus->ExtendMenu(AddEntryContextMenuName))
	{
		
		Menu->AddDynamicSection("FunctionsDynamic", FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
		{			
			PopulateNewFunctionToolMenuEntries(InMenu, false);			
		}));
	}
}

void FFunctionsOutlinerMode::Rebuild()
{
	Hierarchy = CreateHierarchy();
}

TUniquePtr<ISceneOutlinerHierarchy> FFunctionsOutlinerMode::CreateHierarchy()
{
	return MakeUnique<FFunctionsOutlinerHierarchy>(this);
}

void FFunctionsOutlinerMode::PopulateNewFunctionToolMenuEntries(UToolMenu* InMenu, bool bAddSeparator)
{
	const UCommonOutlinerItemMenuContext* OutlinerContext = InMenu->FindContext<UCommonOutlinerItemMenuContext>();
	const UFunctionsOutlinerItemMenuContext* FunctionsOutliner = InMenu->FindContext<UFunctionsOutlinerItemMenuContext>();
	if(OutlinerContext == nullptr || FunctionsOutliner == nullptr)
	{
		return;
	}
	
	if (OutlinerContext->WeakEditorDatas.Num() == 1)
	{
		FToolMenuSection& Section = InMenu->AddSection("Functions", LOCTEXT("FunctionsSectionLabel", "Functions"), FToolMenuInsert("Assets", EToolMenuInsertType::First));
		Section.AddMenuEntry("AddNewFunction", LOCTEXT("AddFunctionLabel", "Add Function"), LOCTEXT("AddFunctionTooltip", "Adds a new Function"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Function_16x"),
			FUIAction(FExecuteAction::CreateLambda([WeakOutliner = OutlinerContext->WeakOutliner, WeakEditorDatas=OutlinerContext->WeakEditorDatas]()
			{
				if (WeakEditorDatas.Num() == 1)
				{
					if (UUAFRigVMAssetEditorData* EditorData = WeakEditorDatas[0].Get())
					{
						FScopedTransaction Transaction(LOCTEXT("AddFunctionTransaction", "Adding new Function"));
						if (URigVMLibraryNode* NewFunction = EditorData->AddFunction(TEXT("NewFunction"), true))
						{
							// Prompt user to rename added variable in outliner
							TSharedPtr<SCommonOutliner> SharedOutliner = StaticCastSharedPtr<SCommonOutliner>(WeakOutliner.Pin());
							if (SharedOutliner.IsValid())
							{
								// FVariablesOutlinerEntryItem::SharedVariablesEntry is only populated for Struct based SharedVariables
								const UUAFSharedVariablesEntry* SharedVariablesEntry = nullptr;
								FSceneOutlinerTreeItemID EntryID = HashCombine(GetTypeHash(NewFunction));
								SharedOutliner->OnItemAdded(EntryID, SceneOutliner::ENewItemAction::Select | SceneOutliner::ENewItemAction::Rename);
							}
						}
					}
				}			
			}))
		);
	}
}

void FFunctionsOutlinerMode::BindCommands(const TSharedRef<FUICommandList>& OutCommandList)
{
	CommandList->MapAction(
		FOutlinerCommands::Get().OpenInNewTab,
		FExecuteAction::CreateRaw(this, &FFunctionsOutlinerMode::OpenSelectedFunctionsInNewTab),
		FCanExecuteAction::CreateRaw(this, &FFunctionsOutlinerMode::CanOpenSelectedFunctionsInNewTab),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FFunctionsOutlinerMode::CanOpenSelectedFunctionsInNewTab));

	FCommonOutlinerMode::BindCommands(OutCommandList);
}

void FFunctionsOutlinerMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	FDocumentTracker::EOpenDocumentCause OpenCause = FDocumentTracker::NavigatingCurrentDocument;
	if (const UUAFEditorUserSettings* UAFEditorUserSettings = GetMutableDefault<UUAFEditorUserSettings>())
	{
		if (UAFEditorUserSettings->bAutoOpenFunctionsInSeparateTabs)
		{
			OpenCause = FDocumentTracker::OpenNewDocument;
		}
	}
	
	if (OpenFunctionItem(Item, OpenCause))
	{
		return;
	}

	FCommonOutlinerMode::OnItemDoubleClick(Item);
}

bool FFunctionsOutlinerMode::CanRenameItem(const ISceneOutlinerTreeItem& Item) const
{
	return Item.IsA<FFunctionsOutlinerEntryItem>() || Item.IsA<FFunctionsOutlinerCategoryItem>() || Item.IsA<FFunctionsOutlinerCollapsedGraphItem>();
}

bool FFunctionsOutlinerMode::UpdateOperationDecorator(const FDragDropEvent& Event, const FSceneOutlinerDragValidationInfo* ValidationInfo) const
{
	TSharedPtr<FFunctionDragDropOp> GraphDropOp = Event.GetOperationAs<FFunctionDragDropOp>();
	if (GraphDropOp.IsValid())
	{
		if (ValidationInfo)
		{
			const FSlateBrush* ValidHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
			const FSlateBrush* InvalidSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));	
			GraphDropOp->SetSimpleFeedbackMessage(ValidationInfo->IsValid() ? ValidHoverIcon : InvalidSymbol, FLinearColor::White, ValidationInfo->ValidationText);
		}
		else
		{
			GraphDropOp->HoverTargetChanged();
		}
		
		return true;
	}
	
	return false;
}

bool FFunctionsOutlinerMode::OpenFunctionItem(const FSceneOutlinerTreeItemPtr Item, FDocumentTracker::EOpenDocumentCause OpenDocumentCause) const
{
	if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin())
	{
		const Workspace::FWorkspaceDocument& Document = WorkspaceEditor->GetFocusedWorkspaceDocument();
		FWorkspaceOutlinerItemExport DocumentExport = Document.Export;
		if (FFunctionsOutlinerEntryItem* FunctionItem = Item->CastTo<FFunctionsOutlinerEntryItem>())
		{
			if (FunctionItem->WeakLibraryNode.IsValid())
			{
				const URigVMLibraryNode* FunctionNode = FunctionItem->WeakLibraryNode.Get();
				UUAFRigVMAsset* Asset = FunctionNode->GetTypedOuter<UUAFRigVMAsset>();
				UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(Asset);

				const FWorkspaceOutlinerItemExport RelativeAssetExport = DocumentExport.GetRelativeAssetExport(Asset);
				const FWorkspaceOutlinerItemExport AssetExport = UncookedOnly::FUtils::MakeAssetReferenceExport(Asset, RelativeAssetExport);
				if (AssetExport.GetFirstAssetPath().IsValid() && RelativeAssetExport != AssetExport)
				{
					WorkspaceEditor->OpenExports({UncookedOnly::FUtils::MakeFunctionExport(FunctionNode, AssetExport)}, OpenDocumentCause);
					return true;
				}

				if (URigVMGraph* ContainedGraph = FunctionNode->GetContainedGraph())
				{
					if (IRigVMClientHost* Host = FunctionNode->GetImplementingOuter<IRigVMClientHost>())
					{
						if (URigVMEdGraph* ContainedEdGraph = Cast<URigVMEdGraph>(Host->GetEditorObjectForRigVMGraph(ContainedGraph)))
						{
							WorkspaceEditor->OpenObjects({ContainedEdGraph}, OpenDocumentCause);
							return true;
						}
					}
				}
			}
		}
		else if (FFunctionsOutlinerCollapsedGraphItem* CollapsedGraphItem = Item->CastTo<FFunctionsOutlinerCollapsedGraphItem>())
		{
			if (CollapsedGraphItem->WeakOwningNode.IsValid() && CollapsedGraphItem->WeakCollapseNode.IsValid())
			{
				const URigVMLibraryNode* FunctionNode = CollapsedGraphItem->WeakOwningNode.Get();
				UUAFRigVMAsset* Asset = FunctionNode->GetTypedOuter<UUAFRigVMAsset>();

				const FWorkspaceOutlinerItemExport RelativeAssetExport = DocumentExport.GetRelativeAssetExport(Asset);
				const FWorkspaceOutlinerItemExport AssetExport = UncookedOnly::FUtils::MakeAssetReferenceExport(Asset, RelativeAssetExport);
				if (AssetExport.GetFirstAssetPath().IsValid() && RelativeAssetExport != AssetExport)
				{
					const FWorkspaceOutlinerItemExport FunctionsExport = UncookedOnly::FUtils::MakeFunctionExport(FunctionNode, AssetExport);
					const FWorkspaceOutlinerItemExport CollapsedGraphExport = UncookedOnly::FUtils::MakeCollapsedGraphExport(CollapsedGraphItem->WeakCollapseNode.Get(), FunctionsExport);
					WorkspaceEditor->OpenExports({CollapsedGraphExport}, OpenDocumentCause);
					return true;
				}

				if (URigVMGraph* ContainedGraph = CollapsedGraphItem->WeakCollapseNode->GetContainedGraph())
				{
					if (IRigVMClientHost* Host = FunctionNode->GetImplementingOuter<IRigVMClientHost>())
					{
						if (URigVMEdGraph* ContainedEdGraph = Cast<URigVMEdGraph>(Host->GetEditorObjectForRigVMGraph(ContainedGraph)))
						{
							WorkspaceEditor->OpenObjects({ContainedEdGraph}, OpenDocumentCause);
							return true;
						}
					}
				}
			}
		}
	}

	return false;
}

void FFunctionsOutlinerMode::OpenSelectedFunctionsInNewTab() const
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();
	for (const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
	{
		OpenFunctionItem(Item, FDocumentTracker::ForceOpenNewDocument);
	}
}

bool FFunctionsOutlinerMode::CanOpenSelectedFunctionsInNewTab() const
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();
	for (const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
	{
		if (const FFunctionsOutlinerEntryItem* FunctionEntry = Item->CastTo<FFunctionsOutlinerEntryItem>())
		{
			return true;
		}
		else if (const FFunctionsOutlinerCollapsedGraphItem* CategoryItem = Item->CastTo<FFunctionsOutlinerCollapsedGraphItem>())
		{
			return true;
		}
	}

	return false;
}

}

#undef LOCTEXT_NAMESPACE // "FFunctionsOutlinerMode"
