// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerMode.h"
#include "Common/Outliner/CommonOutlinerMode.h"

namespace UE::UAF::Editor
{
class SCommonOutliner;

class FFunctionsOutlinerMode : public FCommonOutlinerMode
{
public:
	FFunctionsOutlinerMode(SCommonOutliner* InFunctionsOutliner, const TSharedRef<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor);

	// Begin FCommonOutlinerMode overrides
	virtual bool CanRename() const override;
	virtual bool CanDelete() const override;
	virtual bool CanCopy() const override;
	virtual bool CanCut() const override;
	virtual bool CanPaste() const override;
	virtual bool CanDuplicate() const override;
	virtual void Rename() const override;
	virtual void Delete() const override;
	virtual void Copy() const override;
	virtual void Cut() const override;
	virtual void Paste() const override;
	virtual void Duplicate() const override;
	virtual void InitToolMenuContext(FToolMenuContext& InOutContext) const override;
	virtual void HandleItemSelection(const FSceneOutlinerItemSelection& Selection) const override;
	virtual void RegisterToolMenus() override;
	// End FCommonOutlinerMode overrides

	// Begin ISceneOutlinerMode overrides
	virtual void Rebuild() override;
	virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;
	virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override;
	virtual bool UpdateOperationDecorator(const FDragDropEvent& Event, const FSceneOutlinerDragValidationInfo* ValidationInfo) const override;
	virtual void BindCommands(const TSharedRef<FUICommandList>& OutCommandList) override;
protected:
	virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;
	// End ISceneOutlinerMode overrides
	
	static void PopulateNewFunctionToolMenuEntries(UToolMenu* InMenu, bool bAddSeparator);	

private: 
	bool OpenFunctionItem(const FSceneOutlinerTreeItemPtr Item, FDocumentTracker::EOpenDocumentCause OpenDocumentCause = FDocumentTracker::NavigatingCurrentDocument) const;
	void OpenSelectedFunctionsInNewTab() const;
	bool CanOpenSelectedFunctionsInNewTab() const;

};
}
