// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionsOutlinerView.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "AssetToolsModule.h"
#include "FunctionsOutlinerColumns.h"
#include "FunctionsOutlinerMode.h"
#include "IAssetTools.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "SceneOutlinerSourceControlColumn.h"
#include "Common/Outliner/OutlinerColumns.h"
#include "Common/Outliner/SCommonOutliner.h"
#include "Variables/Outliner/VariablesOutlinerColumns.h"

#define LOCTEXT_NAMESPACE "FunctionsOutliner"

namespace UE::UAF::Editor
{

void SFunctionsView::Construct(const FArguments& InArgs, TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor)
{
	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.OutlinerIdentifier = TEXT("AnimNextFunctionsOutliner");
	InitOptions.bShowHeaderRow = true;
	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn(), false, 0.5f));
	InitOptions.ColumnMap.Add(FFunctionsOutlinerOutputColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 20, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShared<FFunctionsOutlinerOutputColumn>(InSceneOutliner); }), false, 0.5f));	
	InitOptions.ColumnMap.Add(FSceneOutlinerSourceControlColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 30, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShared<FSceneOutlinerSourceControlColumn>(InSceneOutliner); }), true));	
	InitOptions.ColumnMap.Add(FOutlinerAccessSpecifierColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 40, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShared<FOutlinerAccessSpecifierColumn>(InSceneOutliner); }), false));
	
	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([this, WeakWorkspaceEditor=InWorkspaceEditor.ToWeakPtr()](SSceneOutliner* InOutliner) { check(WeakWorkspaceEditor.IsValid()); return new FFunctionsOutlinerMode(static_cast<SCommonOutliner*>(InOutliner), WeakWorkspaceEditor.Pin().ToSharedRef()); });

	FunctionsOutliner = SNew(UE::UAF::Editor::SCommonOutliner, InitOptions)
	.BindAssetDelegate_Lambda([WeakThis=StaticCastWeakPtr<SFunctionsView>(AsWeak())](UUAFRigVMAssetEditorData* EditorData)
	{
		if (WeakThis.IsValid())
		{
			TSharedPtr<SFunctionsView> SharedView = WeakThis.Pin();
			EditorData->ModifiedDelegate.AddSP(SharedView.Get(), &SFunctionsView::OnEditorDataModified);
			EditorData->RigVMGraphModifiedEvent.AddSP(SharedView.Get(), &SFunctionsView::OnRigVMModified);
		}
	})
	.UnbindAssetDelegate_Lambda([WeakThis=StaticCastWeakPtr<SFunctionsView>(AsWeak())](UUAFRigVMAssetEditorData* EditorData)
	{
		if (WeakThis.IsValid())
		{
			TSharedPtr<SFunctionsView> SharedView = WeakThis.Pin();
			EditorData->ModifiedDelegate.RemoveAll(SharedView.Get());
			EditorData->RigVMGraphModifiedEvent.RemoveAll(SharedView.Get());
		}
	});
	FunctionsOutliner->SetEnabled(MakeAttributeSP(FunctionsOutliner.Get(), &SCommonOutliner::HasAssets));

	InWorkspaceEditor->OnFocusedDocumentChanged().AddSP(this, &SFunctionsView::HandleFocusedDocumentChanged);
	const Workspace::FWorkspaceDocument& Document = InWorkspaceEditor->GetFocusedWorkspaceDocument();
	HandleFocusedDocumentChanged(Document);

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	AssetTools.OnAssetPostRename().AddSPLambda(this, [this](const TArray<FAssetRenameData>&){ FunctionsOutliner->UpdateAssets(); });

	ChildSlot
	[
		FunctionsOutliner.ToSharedRef()
	];
}


void SFunctionsView::OnEditorDataModified(UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType, UObject* InSubject) const
{
	switch(InType)
	{
	case EAnimNextEditorDataNotifType::UndoRedo:
	case EAnimNextEditorDataNotifType::VariableCategoryChanged:
	case EAnimNextEditorDataNotifType::CategoryAdded:
	case EAnimNextEditorDataNotifType::CategoryChanged:
	case EAnimNextEditorDataNotifType::CategoryRemoved:
		FunctionsOutliner->FullRefresh();
		break;
	case EAnimNextEditorDataNotifType::EntryAdded:
	case EAnimNextEditorDataNotifType::EntryRemoved:
		FunctionsOutliner->UpdateAssets();
		break;
	default:
		break;
	}
}

void SFunctionsView::OnRigVMModified(ERigVMGraphNotifType NotifType, URigVMGraph* Graph, UObject* Subject) const
{
	switch(NotifType)
	{
	case ERigVMGraphNotifType::FunctionRenamed:
	case ERigVMGraphNotifType::LibraryTemplateChanged:
	case ERigVMGraphNotifType::FunctionVariantGuidChanged:
		FunctionsOutliner->FullRefresh();
		break;
	case ERigVMGraphNotifType::NodeAdded:
	case ERigVMGraphNotifType::NodeRemoved:
	case ERigVMGraphNotifType::NodeCategoryChanged:
		{
			if (Cast<URigVMFunctionLibrary>(Graph) || Cast<URigVMFunctionInterfaceNode>(Subject))
			{
				FunctionsOutliner->FullRefresh();
			}
			break;
		}
	default:
		break;
	}
}

void SFunctionsView::SetExportDirectly(const FWorkspaceOutlinerItemExport& InExport) const
{
	if (FunctionsOutliner.IsValid())
	{
		FunctionsOutliner->SetExport(InExport);
	}
}

void SFunctionsView::HandleFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument) const
{
	const FWorkspaceOutlinerItemExport& Export = InDocument.Export;
	FunctionsOutliner->SetExport(Export);
}

const FLazyName FunctionsTabName("FunctionsTab");
FAnimNextFunctionsTabSummoner::FAnimNextFunctionsTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp)
	: FWorkflowTabFactory(FunctionsTabName, StaticCastSharedPtr<FAssetEditorToolkit>(InHostingApp))
{
	TabLabel = LOCTEXT("AnimNextFunctionsTabLabel", "Functions");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Function_16x");

	ViewMenuDescription = LOCTEXT("AnimNextFunctionsTabMenuDescription", "Functions");
	ViewMenuTooltip = LOCTEXT("AnimNextFunctionsTabToolTip", "Shows the Functions tab.");
	bIsSingleton = true;

	FunctionsView = SNew(SFunctionsView, InHostingApp.ToSharedRef());

	const Workspace::FWorkspaceDocument& Document = InHostingApp->GetFocusedWorkspaceDocument();
	FunctionsView->FunctionsOutliner->SetExport(Document.Export);
}

TSharedRef<SWidget> FAnimNextFunctionsTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return FunctionsView.ToSharedRef();
}

FText FAnimNextFunctionsTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{	
	return ViewMenuTooltip;
}
}

#undef LOCTEXT_NAMESPACE // "FunctionsOutliner"
