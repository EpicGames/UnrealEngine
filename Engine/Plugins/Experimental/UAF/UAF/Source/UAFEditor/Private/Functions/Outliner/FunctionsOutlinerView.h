// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSceneOutliner.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "Widgets/SCompoundWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class URigVMGraph;
enum class ERigVMGraphNotifType : uint8;

namespace UE::Workspace
{
struct FWorkspaceDocument;
class IWorkspaceEditor;
}

struct FWorkspaceOutlinerItemExport;
enum class EAnimNextEditorDataNotifType : FPlatformTypes::uint8;
class UUAFRigVMAssetEditorData;

namespace UE::UAF::Editor
{
class SCommonOutliner;

extern const FLazyName FunctionsTabName;

class SFunctionsView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SFunctionsView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor);
	void SetExportDirectly(const FWorkspaceOutlinerItemExport& InExport) const;
private:
	void HandleFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument) const;

	void OnEditorDataModified(UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType, UObject* InSubject) const;
	void OnRigVMModified(ERigVMGraphNotifType NotifType, URigVMGraph* Graph, UObject* Subject) const; 
	
	friend struct FAnimNextFunctionsTabSummoner;
	TSharedPtr<SCommonOutliner> FunctionsOutliner;
};

struct FAnimNextFunctionsTabSummoner : public FWorkflowTabFactory
{
public:
	FAnimNextFunctionsTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp);

private:
	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

	// The widget this tab spawner wraps
	TSharedPtr<SFunctionsView> FunctionsView;
};

}
