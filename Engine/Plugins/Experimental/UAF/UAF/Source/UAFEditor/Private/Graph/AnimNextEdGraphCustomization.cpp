// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/AnimNextEdGraphCustomization.h"

#include "AnimNextEdGraph.h"
#include "DetailLayoutBuilder.h"
#include "Editor/RigVMGraphDetailCustomization.h"
#include "RigVMModel/RigVMController.h"
#include "IWorkspaceEditor.h"

#define LOCTEXT_NAMESPACE "AnimNextEdGraphCustomization"

namespace UE::UAF::Editor
{

void FAnimNextEdGraphCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	RigVMGraphDetailCustomizationImpl.Reset();

	if (Objects.Num() != 1)
	{
		return;
	}

	if (UAnimNextEdGraph* EdGraph = Cast<UAnimNextEdGraph>(Objects[0].Get()))
	{
		if(URigVMGraph* Model = EdGraph->GetModel())
		{
			TWeakPtr<FRigVMEditorBase> WeakEditor; // passing an empty RigVMEditor to FRigVMCollapseGraphLayout
			IRigVMClientHost* RigVMClientHost = EdGraph->GetController()->GetClientHost();
			check(RigVMClientHost);

			RigVMGraphDetailCustomizationImpl = MakeShared<FRigVMGraphDetailCustomizationImpl>();
			RigVMGraphDetailCustomizationImpl->CustomizeDetails(DetailBuilder
				, Model
				, RigVMClientHost->GetController(Model)
				, RigVMClientHost
				, WeakEditor);
		}
	}
}

} // end namespace UE::UAF::Editor

#undef LOCTEXT_NAMESPACE
