// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"

class URigVMPin;
class IDetailCategoryBuilder;
class UAnimNextEdGraphNode;
class FRigVMGraphDetailCustomizationImpl;

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{
	class FAnimNextEditorModule;
}

namespace UE::UAF::Editor
{

class FAnimNextEdGraphCustomization : public IDetailCustomization
{
	FAnimNextEdGraphCustomization() = default;

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

	TSharedPtr<FRigVMGraphDetailCustomizationImpl> RigVMGraphDetailCustomizationImpl;
};

}
