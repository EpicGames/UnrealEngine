// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextGraphPanelPinFactory.h"

#include "NodeFactory.h"
#include "SGraphPinModuleEvent.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "Variables/SGraphPinVariableReference.h"
#include "Variables/AnimNextVariableReference.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

TSharedPtr<SGraphPin> FAnimNextGraphPanelPinFactoryEditor::CreatePin(UEdGraphPin* InPin) const
{
	if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(InPin->GetOwningNode()))
	{
		URigVMPin* ModelPin = RigNode->GetModelPinFromPinPath(InPin->GetName());
		if (ModelPin)
		{
			static const FString META_AnimNextModuleEvent("AnimNextModuleEvent");
			static const FName META_CustomWidget("CustomWidget");
			if(InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Name && ModelPin->GetMetaData(META_CustomWidget) == META_AnimNextModuleEvent)
			{
				return SNew(UE::UAF::Editor::SGraphPinModuleEvent, InPin);
			}
		}
	}

	if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		const UScriptStruct* Struct = Cast<UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get());
		if (Struct && (Struct->IsChildOf<FAnimNextVariableReference>()))
		{
			return SNew(UE::UAF::Editor::SGraphPinVariableReference, InPin);
		}
	}

	return nullptr;
}
