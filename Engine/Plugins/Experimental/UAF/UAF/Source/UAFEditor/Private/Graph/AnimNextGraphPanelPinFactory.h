// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraphUtilities.h"

#define UE_API UAFEDITOR_API

class FAnimNextGraphPanelPinFactoryEditor : public FGraphPanelPinFactory
{
public:
	// FGraphPanelPinFactory interface
	UE_API virtual TSharedPtr<SGraphPin> CreatePin(UEdGraphPin* InPin) const override;
};

#undef UE_API
