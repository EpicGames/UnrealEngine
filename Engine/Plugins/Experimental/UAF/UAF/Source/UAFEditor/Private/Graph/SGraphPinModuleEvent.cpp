// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphPinModuleEvent.h"
#include "SModuleEventPicker.h"

namespace UE::UAF::Editor
{

void SGraphPinModuleEvent::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SGraphPinModuleEvent::GetDefaultValueWidget()
{
	TArray<UObject*> ContextObjects;
	ContextObjects.Add(GraphPinObj->GetOwningNode());
	return SNew(SModuleEventPicker)
		.ContextObjects(ContextObjects)
		.InitiallySelectedEvent(FName(GraphPinObj->GetDefaultAsString()))
		.OnEventPicked_Lambda([this](FName InEventName)
		{
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, InEventName.ToString());
		})
		.OnGetSelectedEvent_Lambda([this]()
		{
			return FName(GraphPinObj->GetDefaultAsString());
		});
}

}
