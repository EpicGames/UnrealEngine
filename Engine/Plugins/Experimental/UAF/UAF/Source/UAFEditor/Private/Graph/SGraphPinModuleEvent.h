// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"

namespace UE::UAF::Editor
{

class SGraphPinModuleEvent : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinModuleEvent) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

private:
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
};

}