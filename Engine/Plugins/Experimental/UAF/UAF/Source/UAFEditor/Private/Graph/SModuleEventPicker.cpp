// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModuleEventPicker.h"

#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SComboBox.h"
#include "Module/AnimNextModule.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "UObject/UObjectIterator.h"

namespace UE::UAF::Editor
{

void SModuleEventPicker::Construct(const FArguments& InArgs)
{
	Algo::Transform(InArgs._ContextObjects, ContextObjects, [](UObject* ContextObject){ return TWeakObjectPtr<UObject>(ContextObject); });

	RefreshEntries();

	TSharedPtr<FName> InitialItem;
	for (const TSharedPtr<FName>& Item : EventNamesSource)
	{
		if (InArgs._InitiallySelectedEvent == *Item)
		{
			InitialItem = Item;
			break;
		}
	}

	ChildSlot
	[
		SNew(SComboBox<TSharedPtr<FName>>)
		.ContentPadding(0.0f)
		.OptionsSource(&EventNamesSource)
		.InitiallySelectedItem(InitialItem)
		.OnComboBoxOpening_Lambda([this]()
		{
			RefreshEntries();
		})
		.OnSelectionChanged_Lambda([OnEventPicked = InArgs._OnEventPicked](TSharedPtr<FName> InItem, ESelectInfo::Type InSelectInfo)
		{
			if(InItem.IsValid())
			{
				OnEventPicked.ExecuteIfBound(*InItem);
			}
		})
		.OnGenerateWidget_Lambda([](TSharedPtr<FName> InItem)
		{
			return SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(FText::FromName(InItem.IsValid() ? *InItem : NAME_None));
		})
		.Content()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text_Lambda([OnGetSelectedEvent = InArgs._OnGetSelectedEvent]()
			{
				if (OnGetSelectedEvent.IsBound())
				{
					return FText::FromName(OnGetSelectedEvent.Execute());
				}
				return FText::GetEmpty();
			})
		]
	];
}

void SModuleEventPicker::RefreshEntries()
{
	static const FName META_Hidden("Hidden");

	TArray<FName> EventNames;
	EventNamesSource.Empty();
	
	// Add default event names
	for (TObjectIterator<UScriptStruct> It; It; ++It)
	{
		const UScriptStruct* Struct = *It;
		if (Struct->IsChildOf(FRigUnit_AnimNextModuleEventBase::StaticStruct()) &&
			Struct != FRigUnit_AnimNextModuleEventBase::StaticStruct() &&
			!Struct->HasMetaData(META_Hidden))
		{
			TInstancedStruct<FRigUnit_AnimNextModuleEventBase> StructInstance;
			StructInstance.InitializeAsScriptStruct(Struct);
			const FRigUnit_AnimNextModuleEventBase& Event = StructInstance.Get();
			if (Event.IsTask() && Event.IsUserEvent() && !Event.GetEventName().IsNone())
			{
				EventNames.AddUnique(Event.GetEventName());
			}
		}
	}

	// Parse out implemented events from the bytecode of the outer modules
	for (TWeakObjectPtr<UObject> WeakObject : ContextObjects)
	{
		UObject* Object = WeakObject.Get();
		if (Object == nullptr)
		{
			continue;
		}

		UUAFSystem* Module = Cast<UUAFSystem>(Object);
		if(Module == nullptr)
		{
			Module = Object->GetTypedOuter<UUAFSystem>();
		}

		if (Module == nullptr)
		{
			continue;
		}

		const URigVM* VM = Module->GetRigVM();
		const FRigVMByteCode& ByteCode = VM->GetByteCode();
		const TArray<const FRigVMFunction*>& Functions = VM->GetFunctions();
		FRigVMInstructionArray Instructions = ByteCode.GetInstructions();
		for (int32 EntryIndex = 0; EntryIndex < ByteCode.NumEntries(); EntryIndex++)
		{
			const FRigVMByteCodeEntry& Entry = ByteCode.GetEntry(EntryIndex);
			const FRigVMInstruction& Instruction = Instructions[Entry.InstructionIndex];
			const FRigVMExecuteOp& Op = ByteCode.GetOpAt<FRigVMExecuteOp>(Instruction);
			const FRigVMFunction* Function = Functions[Op.CallableIndex];
			check(Function != nullptr);

			if (Function->Struct->IsChildOf(FRigUnit_AnimNextModuleEventBase::StaticStruct()) &&
				Function->Struct != FRigUnit_AnimNextModuleEventBase::StaticStruct() &&
				!Function->Struct->HasMetaData(META_Hidden))
			{
				TInstancedStruct<FRigUnit_AnimNextModuleEventBase> StructInstance;
				StructInstance.InitializeAsScriptStruct(Function->Struct);
				const FRigUnit_AnimNextModuleEventBase& Event = StructInstance.Get();
				if (Event.IsTask() && Event.IsUserEvent() && !Event.GetEventName().IsNone())
				{
					EventNames.AddUnique(Entry.Name);
				}
			}
		}
	}

	if(EventNames.Num() == 0)
	{
		return;
	}

	EventNamesSource.Reserve(EventNames.Num());
	for (FName EventName : EventNames)
	{
		EventNamesSource.Add(MakeShared<FName>(EventName));
	}
}

}
