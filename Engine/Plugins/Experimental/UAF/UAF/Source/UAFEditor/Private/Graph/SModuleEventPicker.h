// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace UE::UAF::Editor
{

class SModuleEventPicker : public SCompoundWidget
{
public:
	using FOnEventPicked = TDelegate<void(FName InEventName)>;
	using FOnGetSelectedEvent = TDelegate<FName()>;

	SLATE_BEGIN_ARGS(SModuleEventPicker) {}

	SLATE_ARGUMENT(TArray<UObject*>, ContextObjects)

	SLATE_EVENT(FOnEventPicked, OnEventPicked)

	SLATE_EVENT(FOnGetSelectedEvent, OnGetSelectedEvent)

	SLATE_ARGUMENT(FName, InitiallySelectedEvent)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	void RefreshEntries();

	// Names to be displayed in the combo box
	TArray<TSharedPtr<FName>> EventNamesSource;

	TArray<TWeakObjectPtr<UObject>> ContextObjects;
};

}
