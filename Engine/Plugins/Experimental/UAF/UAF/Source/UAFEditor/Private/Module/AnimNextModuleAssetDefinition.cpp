// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextModuleAssetDefinition.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "ContentBrowserMenuContexts.h"
#include "FileHelpers.h"
#include "IWorkspaceEditorModule.h"
#include "IWorkspaceEditor.h"
#include "ToolMenus.h"
#include "UncookedOnlyUtils.h"
#include "Workspace/AnimNextWorkspaceFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextModuleAssetDefinition)

#define LOCTEXT_NAMESPACE "AssetDefinition_AnimNextGraph"

EAssetCommandResult UAssetDefinition_AnimNextModule::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace UE::UAF::Editor;
	using namespace UE::Workspace;

	for (UUAFSystem* Asset : OpenArgs.LoadObjects<UUAFSystem>())
	{
		IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<IWorkspaceEditorModule>("WorkspaceEditor");
		WorkspaceEditorModule.OpenWorkspaceForObject(Asset, EOpenWorkspaceMethod::Default, UAnimNextWorkspaceFactory::StaticClass());
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
