// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Module/AnimNextModule.h"
#include "AssetDefinitionDefault.h"
#include "AnimNextModuleAssetDefinition.generated.h"

#define LOCTEXT_NAMESPACE "AnimNextAssetDefinitions"

UCLASS()
class UAssetDefinition_AnimNextModule : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override { return LOCTEXT("UAFSystem", "UAF System"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,96,48)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UUAFSystem::StaticClass(); }
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")) };
		return Categories;
	}
	virtual bool ShouldSaveExternalPackages() const override { return true; }
};

#undef LOCTEXT_NAMESPACE