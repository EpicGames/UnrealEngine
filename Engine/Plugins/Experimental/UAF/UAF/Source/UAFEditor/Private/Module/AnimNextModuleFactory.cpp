// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/AnimNextModuleFactory.h"

#include "UAFCompilationScope.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModule_EditorData.h"
#include "UncookedOnlyUtils.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Entries/AnimNextEventGraphEntry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextModuleFactory)

UUAFSystemFactory::UUAFSystemFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UUAFSystem::StaticClass();
}

bool UUAFSystemFactory::ConfigureProperties()
{
	return true;
}

UObject* UUAFSystemFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	EObjectFlags FlagsToUse = Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted;
	if(InParent == GetTransientPackage())
	{
		FlagsToUse &= ~RF_Standalone;
	}

	UUAFSystem* NewModule = NewObject<UUAFSystem>(InParent, Class, Name, FlagsToUse);

	// Create internal editor data
	UUAFSystem_EditorData* EditorData = NewObject<UUAFSystem_EditorData>(NewModule, TEXT("EditorData"), RF_Transactional);
	NewModule->EditorData = EditorData;
	EditorData->Initialize(/*bRecompileVM*/false);

	UAnimNextEventGraphEntry* GraphEntry = EditorData->AddEventGraph(UUAFSystem_EditorData::DefaultEventGraphName, FRigUnit_AnimNextInitializeEvent::StaticStruct());

	URigVMController* Controller = EditorData->GetController(GraphEntry->Graph);
	Controller->AddUnitNode(FRigUnit_AnimNextPrePhysicsEvent::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, 200.0f), FString(), false);

	// Compile the initial skeleton
	UE::UAF::UncookedOnly::Compilation::RequestAssetCompilation(NewModule);
	
	check(!EditorData->bErrorsDuringCompilation);

	return NewModule;
}
