// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModuleEventPropertyCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "PropertyHandle.h"
#include "SNameComboBox.h"
#include "Graph/SModuleEventPicker.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModule_EditorData.h"
#include "Module/RigVMTrait_ModuleEventDependency.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "ModuleEventPropertyCustomization"

namespace UE::UAF::Editor
{

bool FModuleEventPropertyTypeIdentifier::IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const
{
	static const FString META_AnimNextModuleEvent("AnimNextModuleEvent");
	static const FName META_CustomWidget("CustomWidget");
	return InPropertyHandle.GetMetaData(META_CustomWidget) == META_AnimNextModuleEvent;
}

void FModuleEventPropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TArray<UObject*> Objects;
	InPropertyHandle->GetOuterObjects(Objects);

	FName SelectedValue;
	InPropertyHandle->GetValue(SelectedValue);
	TWeakPtr<IPropertyHandle> WeakPropertyHandle = InPropertyHandle;

	InHeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(SModuleEventPicker)
		.ContextObjects(Objects)
		.InitiallySelectedEvent(SelectedValue)
		.OnEventPicked_Lambda([WeakPropertyHandle](FName InEventName)
		{
			if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
			{
				PropertyHandle->SetValue(InEventName);
			}
		})
		.OnGetSelectedEvent_Lambda([WeakPropertyHandle]()
		{
			if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
			{
				FName Value;
				if (PropertyHandle->GetValue(Value) == FPropertyAccess::MultipleValues)
				{
					return FName(*LOCTEXT("MultipleValues", "Multiple Values").ToString());
				}
				else
				{
					return Value;
				}
			}

			return FName();
		})
	];
}

void FModuleEventPropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{

}

}

#undef LOCTEXT_NAMESPACE