// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Param/ParamTypePropertyCustomization.h"

namespace UE::UAF::Editor
{

class FModuleEventPropertyTypeIdentifier : public IPropertyTypeIdentifier
{
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override;
};

class FModuleEventPropertyCustomization : public IPropertyTypeCustomization
{
private:
	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;


};

}
