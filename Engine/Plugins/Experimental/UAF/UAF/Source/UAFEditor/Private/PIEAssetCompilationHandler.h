// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOutOfDateAssetCompilation.h"

namespace UE::UAF::Editor
{

class FOutOfDateAssetCompilation : public PIE::IOutOfDateAssetCompilation
{
private:
	virtual bool HandleOutOfDateAssets_Internal(const FRequestPlaySessionParams& InPlaySessionParams, TArray<UObject*>& OutAssetsWithErrors) const override;
};
}
