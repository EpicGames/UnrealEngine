// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextActorLocatorEditor.h"

#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocator.h"
#include "Textures/SlateIcon.h"
#include "GameFramework/Actor.h"
#include "Param/AnimNextActorLocatorFragment.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "AnimNextActorLocatorEditor"

namespace UE::UAF::Editor
{

UE::UniversalObjectLocator::ELocatorFragmentEditorType FActorLocatorEditor::GetLocatorFragmentEditorType() const
{
	return UE::UniversalObjectLocator::ELocatorFragmentEditorType::Absolute;
}

bool FActorLocatorEditor::IsAllowedInContext(FName InContextName) const
{
	return InContextName == "UAFContext";
}

bool FActorLocatorEditor::IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return false;
}

UObject* FActorLocatorEditor::ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return nullptr;
}

TSharedPtr<SWidget> FActorLocatorEditor::MakeEditUI(const FEditUIParameters& InParameters)
{
	return nullptr;
}

FText FActorLocatorEditor::GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const
{
	return LOCTEXT("AnimNextActorLocatorName", "Current Actor");
}

FText FActorLocatorEditor::GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const
{
	return LOCTEXT("AnimNextActorLocatorTooltip", "The current actor");
}

FSlateIcon FActorLocatorEditor::GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Actor");
}

UClass* FActorLocatorEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
{
	return AActor::StaticClass(); 
}

FUniversalObjectLocatorFragment FActorLocatorEditor::MakeDefaultLocatorFragment() const
{
	FUniversalObjectLocatorFragment NewFragment(FAnimNextActorLocatorFragment::FragmentType);
	return NewFragment;
}

}

#undef LOCTEXT_NAMESPACE
