// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UniversalObjectLocatorEditor.h"

namespace UE::UAF::Editor
{

class FActorLocatorEditor : public UE::UniversalObjectLocator::ILocatorFragmentEditor
{
	// ILocatorFragmentEditor interface
	virtual UE::UniversalObjectLocator::ELocatorFragmentEditorType GetLocatorFragmentEditorType() const override;
	virtual bool IsAllowedInContext(FName InContextName) const override;
	virtual bool IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const override;
	virtual UObject* ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const override;
	virtual TSharedPtr<SWidget> MakeEditUI(const FEditUIParameters& InParameters) override;
	virtual FText GetDisplayText(const FUniversalObjectLocatorFragment* InFragment = nullptr) const override;
	virtual FText GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment = nullptr) const override;
	virtual FSlateIcon GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment = nullptr) const override;
	virtual UClass* ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const override; 
	virtual FUniversalObjectLocatorFragment MakeDefaultLocatorFragment() const override;
};

}
