// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextComponentLocatorEditor.h"

#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocator.h"
#include "Textures/SlateIcon.h"
#include "Modules/ModuleManager.h"
#include "Param/AnimNextComponentLocatorFragment.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "IUniversalObjectLocatorCustomization.h"
#include "Components/ActorComponent.h"
#include "Styling/AppStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "EntityComponentLocatorEditor"

namespace UE::UAF::Editor
{

class SComponentLocatorEditor : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SComponentLocatorEditor){}
	SLATE_END_ARGS()

	class FComponentFilter : public IClassViewerFilter
	{
	public:
		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
		{
			return InClass->IsChildOf(UActorComponent::StaticClass());
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
		{
			return false;
		}
	};

	void Construct(const FArguments& InArgs, TSharedPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> InHandle)
	{
		WeakHandle = InHandle;
		CurrentClass = InHandle->GetResolvedClass() ? InHandle->GetResolvedClass() : InHandle->GetContextClass();

		FClassViewerInitializationOptions Options;
		Options.ClassFilters.Add(MakeShared<FComponentFilter>());
		Options.InitiallySelectedClass = const_cast<UClass*>(CurrentClass);
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(400.0f)
			.HeightOverride(400.0f)
			[
				ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateRaw(this, &SComponentLocatorEditor::HandleClassPicked))
			]
		];
	}

	void HandleClassPicked(UClass* InClass)
	{
		if(InClass != CurrentClass)
		{
			if(TSharedPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> Handle = WeakHandle.Pin())
			{
				FUniversalObjectLocatorFragment NewFragment(FAnimNextComponentLocatorFragment::FragmentType);
				FAnimNextComponentLocatorFragment* Payload = NewFragment.GetPayloadAs(FAnimNextComponentLocatorFragment::FragmentType);
				Payload->Path = InClass;
				Handle->SetValue(NewFragment);
			}
			
			CurrentClass = InClass;
		}
	}

private:
	const UClass* CurrentClass = nullptr;
	TWeakPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> WeakHandle;
};


UE::UniversalObjectLocator::ELocatorFragmentEditorType FComponentLocatorEditor::GetLocatorFragmentEditorType() const
{
	return UE::UniversalObjectLocator::ELocatorFragmentEditorType::Relative;
}

bool FComponentLocatorEditor::IsAllowedInContext(FName InContextName) const
{
	return InContextName == "UAFContext";
}

bool FComponentLocatorEditor::IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return false;
}

UObject* FComponentLocatorEditor::ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return nullptr;
}

TSharedPtr<SWidget> FComponentLocatorEditor::MakeEditUI(const FEditUIParameters& InParameters)
{
	return SNew(SComponentLocatorEditor, InParameters.Handle);
}

FText FComponentLocatorEditor::GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimNextComponentLocatorFragment::FragmentType);
		const FAnimNextComponentLocatorFragment* Payload = InFragment->GetPayloadAs(FAnimNextComponentLocatorFragment::FragmentType);
		if(Payload && Payload->Path.IsValid())
		{
			static const FTextFormat Format(LOCTEXT("ComponentLabelFormat", "Get {0}"));
			return FText::Format(Format, FText::FromString(Payload->Path.GetAssetName()));
		}
	}
	
	return LOCTEXT("AnimationEntityComponentLocatorName", "Component");
}

FText FComponentLocatorEditor::GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimNextComponentLocatorFragment::FragmentType);
		const FAnimNextComponentLocatorFragment* Payload = InFragment->GetPayloadAs(FAnimNextComponentLocatorFragment::FragmentType);
		if(Payload && Payload->Path.IsValid())
		{
			static const FTextFormat Format(LOCTEXT("ComponentTooltipFormat", "Get the first component of type '{0}'"));
			return FText::Format(Format, FText::FromString(Payload->Path.GetAssetName()));
		}
	}

	return LOCTEXT("AnimationEntityComponentLocatorTooltip", "An actor component of a selected class");
}

FSlateIcon FComponentLocatorEditor::GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const
{
	static const FName ActorComponentClassIconName("ClassIcon." + UActorComponent::StaticClass()->GetName());
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimNextComponentLocatorFragment::FragmentType);
		const FAnimNextComponentLocatorFragment* Payload = InFragment->GetPayloadAs(FAnimNextComponentLocatorFragment::FragmentType);
		if(Payload && Payload->Path.IsValid())
		{
			if(UClass* Class = Cast<UClass>(Payload->Path.ResolveObject()))
			{
				const FName ComponentClassIconName("ClassIcon." + Class->GetName());
				FSlateIcon(FAppStyle::GetAppStyleSetName(), ComponentClassIconName);
			}
		}
	}
	
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), ActorComponentClassIconName);
}

UClass* FComponentLocatorEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
{
	ensure(InFragment.GetFragmentTypeHandle() == FAnimNextComponentLocatorFragment::FragmentType);
	const FAnimNextComponentLocatorFragment* Payload = InFragment.GetPayloadAs(FAnimNextComponentLocatorFragment::FragmentType);
	if(Payload && Payload->Path.IsValid())
	{
		return Cast<UClass>(Payload->Path.ResolveObject());
	}

	return UActorComponent::StaticClass(); 
}

FUniversalObjectLocatorFragment FComponentLocatorEditor::MakeDefaultLocatorFragment() const
{
	FUniversalObjectLocatorFragment NewFragment(FAnimNextComponentLocatorFragment::FragmentType);
	return NewFragment;
}

}

#undef LOCTEXT_NAMESPACE
