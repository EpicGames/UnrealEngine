// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UniversalObjectLocatorEditorContext.h"
#include "Component/AnimNextComponent.h"
#include "Modules/ModuleManager.h"
#include "AnimNextEditorModule.h"

namespace UE::UAF::Editor
{

class FLocatorContext : public UE::UniversalObjectLocator::ILocatorFragmentEditorContext
{
	// ILocatorFragmentEditorContext interface
	virtual UObject* GetContext(const IPropertyHandle& InPropertyHandle) const override
	{
		// TODO: This needs to defer to project/schedule/workspace defaults similar to SParameterPicker
		return UUAFComponent::StaticClass()->GetDefaultObject();
	}

	virtual bool IsFragmentAllowed(FName InFragmentName) const override
	{
		FAnimNextEditorModule& EditorModule = FModuleManager::GetModuleChecked<FAnimNextEditorModule>("UAFEditor");
		return EditorModule.LocatorFragmentEditorNames.Contains(InFragmentName);
	}
};

}
