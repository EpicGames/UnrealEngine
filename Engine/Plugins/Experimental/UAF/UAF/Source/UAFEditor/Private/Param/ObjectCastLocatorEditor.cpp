// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectCastLocatorEditor.h"

#include "ClassViewerModule.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocator.h"
#include "IUniversalObjectLocatorCustomization.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Param/AnimNextObjectFunctionLocatorFragment.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Input/SComboButton.h"
#include "Modules/ModuleManager.h"
#include "Param/AnimNextObjectCastLocatorFragment.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "ObjectCastLocatorEditor"

namespace UE::UAF::Editor
{

class SObjectCastLocatorEditor : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SObjectCastLocatorEditor){}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TSharedPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> InHandle)
	{
		WeakHandle = InHandle;
		CurrentClass = InHandle->GetResolvedClass() ? InHandle->GetResolvedClass() : InHandle->GetContextClass();

		FClassViewerInitializationOptions Options;
		Options.InitiallySelectedClass = const_cast<UClass*>(CurrentClass);
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(400.0f)
			.HeightOverride(400.0f)
			[
				ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateRaw(this, &SObjectCastLocatorEditor::HandleClassPicked))
			]
		];
	}

	void HandleClassPicked(UClass* InClass)
	{
		if(InClass != CurrentClass)
		{
			if(TSharedPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> Handle = WeakHandle.Pin())
			{
				FUniversalObjectLocatorFragment NewFragment(FAnimNextObjectCastLocatorFragment::FragmentType);
				FAnimNextObjectCastLocatorFragment* Payload = NewFragment.GetPayloadAs(FAnimNextObjectCastLocatorFragment::FragmentType);
				Payload->Path = InClass;
				Handle->SetValue(NewFragment);
			}
			
			CurrentClass = InClass;
		}
	}

private:
	const UClass* CurrentClass = nullptr;
	TWeakPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> WeakHandle;
};

UE::UniversalObjectLocator::ELocatorFragmentEditorType FObjectCastLocatorEditor::GetLocatorFragmentEditorType() const
{
	return UE::UniversalObjectLocator::ELocatorFragmentEditorType::Relative;
}

bool FObjectCastLocatorEditor::IsAllowedInContext(FName InContextName) const
{
	return InContextName == "UAFContext";
}

bool FObjectCastLocatorEditor::IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return false;
}

UObject* FObjectCastLocatorEditor::ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return nullptr;
}

TSharedPtr<SWidget> FObjectCastLocatorEditor::MakeEditUI(const FEditUIParameters& InParameters)
{
	return SNew(SObjectCastLocatorEditor, InParameters.Handle);
}

FText FObjectCastLocatorEditor::GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimNextObjectCastLocatorFragment::FragmentType);
		const FAnimNextObjectCastLocatorFragment* Payload = InFragment->GetPayloadAs(FAnimNextObjectCastLocatorFragment::FragmentType);
		if(Payload && Payload->Path.IsValid())
		{
			static const FTextFormat Format(LOCTEXT("CastLabelFormat", "As {0}"));
			return FText::Format(Format, FText::FromString(Payload->Path.GetAssetName()));
		}
	}

	return LOCTEXT("ObjectCastLocatorName", "Cast");
}

FText FObjectCastLocatorEditor::GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimNextObjectCastLocatorFragment::FragmentType);
		const FAnimNextObjectCastLocatorFragment* Payload = InFragment->GetPayloadAs(FAnimNextObjectCastLocatorFragment::FragmentType);
		if(Payload && Payload->Path.IsValid())
		{
			static const FTextFormat Format(LOCTEXT("CastTooltipFormat", "Reinterprets an object as a {0}"));
			return FText::Format(Format, FText::FromString(Payload->Path.ToString()));
		}
	}

	return LOCTEXT("ObjectCastLocatorTooltip", "A reinterprets an object as a different type");
}

FSlateIcon FObjectCastLocatorEditor::GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.K2Node_DynamicCast");
}

UClass* FObjectCastLocatorEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
{
	ensure(InFragment.GetFragmentTypeHandle() == FAnimNextObjectCastLocatorFragment::FragmentType);
	const FAnimNextObjectCastLocatorFragment* Payload = InFragment.GetPayloadAs(FAnimNextObjectCastLocatorFragment::FragmentType);
	if(Payload && Payload->Path.IsValid())
	{
		return Cast<UClass>(Payload->Path.ResolveObject());
	}

	return nullptr; 
}

FUniversalObjectLocatorFragment FObjectCastLocatorEditor::MakeDefaultLocatorFragment() const
{
	FUniversalObjectLocatorFragment NewFragment(FAnimNextObjectCastLocatorFragment::FragmentType);
	return NewFragment;
}

}

#undef LOCTEXT_NAMESPACE
