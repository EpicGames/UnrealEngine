// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectFunctionLocatorEditor.h"

#include "ClassViewerModule.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "UniversalObjectLocator.h"
#include "IUniversalObjectLocatorCustomization.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Param/AnimNextObjectFunctionLocatorFragment.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Input/SComboButton.h"
#include "Modules/ModuleManager.h"
#include "Framework/PropertyViewer/IFieldIterator.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Widgets/PropertyViewer/SPropertyViewer.h"
#include "Param/ParamUtils.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "ObjectFunctionLocatorEditor"

namespace UE::UAF::Editor
{

class SObjectFunctionLocatorEditor : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SObjectFunctionLocatorEditor){}
	SLATE_END_ARGS()

	class FFieldIterator : public UE::PropertyViewer::IFieldIterator
	{
		virtual TArray<FFieldVariant> GetFields(const UStruct* InStruct, const FName InFieldName, const UStruct* InContainerStruct) const override
		{
			TArray<FFieldVariant> Result;
			for (TFieldIterator<UFunction> FunctionIt(InStruct, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
			{
				UFunction* Function = *FunctionIt;
				if (Function->HasAllFunctionFlags(FUNC_BlueprintCallable | FUNC_BlueprintPure) &&
					Function->NumParms == 1 &&
					Function->GetReturnProperty() != nullptr &&
					Function->GetReturnProperty()->IsA<FObjectProperty>())
				{
					Result.Add(FFieldVariant(Function));
				}
			}
			return Result;
		}
	} FieldIterator;

	void Construct(const FArguments& InArgs, TSharedPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> InHandle)
	{
		WeakHandle = InHandle;
		CurrentClass = InHandle->GetContextClass();

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(400.0f)
			.HeightOverride(400.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(ClassComboButton, SComboButton)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							return CurrentClass != nullptr ? FText::Format(LOCTEXT("CurrentClassNameFormat", "Class: {0}"), CurrentClass->GetDisplayNameText()) : LOCTEXT("ChooseClass", "Choose Class");
						})
					]
					.OnGetMenuContent_Lambda([this]()
					{
						FClassViewerInitializationOptions Options;
						FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
						return SNew(SBox)
							.WidthOverride(400.0f)
							.HeightOverride(400.0f)
							[
								ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateRaw(this, &SObjectFunctionLocatorEditor::HandleClassPicked))
							];
					})
				]
				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SAssignNew(PropertyViewer, UE::PropertyViewer::SPropertyViewer)
					.FieldIterator(&FieldIterator)
					.OnSelectionChanged(this, &SObjectFunctionLocatorEditor::HandleFunctionPicked)
				]
			]
		];

		PropertyViewer->AddContainer(CurrentClass);

		// Add any BP function libraries that can potentially hoist this class
		TArray<UClass*> Classes;
		GetDerivedClasses(UBlueprintFunctionLibrary::StaticClass(), Classes, true);
		for (const UClass* Class : Classes)
		{
			for (TFieldIterator<UFunction> FieldIt(Class, EFieldIteratorFlags::IncludeSuper); FieldIt; ++FieldIt)
			{
				UFunction* Function = *FieldIt;
				if (FParamUtils::CanUseFunction(Function, CurrentClass))
				{
					PropertyViewer->AddContainer(Class);
					break;
				}
			}
		}
	}

	void HandleClassPicked(UClass* InClass)
	{
		ClassComboButton->SetIsOpen(false);

		CurrentClass = InClass;

		PropertyViewer->RemoveAll();
		PropertyViewer->AddContainer(InClass);
	}

	void HandleFunctionPicked(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, TArrayView<const FFieldVariant> InFields, ESelectInfo::Type InSelectionType)
	{
		if(InFields.Num() > 0)
		{
			if(const UFunction* Function = InFields[0].Get<UFunction>())
			{
				if(TSharedPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> Handle = WeakHandle.Pin())
				{
					FUniversalObjectLocatorFragment NewFragment(FAnimNextObjectFunctionLocatorFragment::FragmentType);
					FAnimNextObjectFunctionLocatorFragment* Payload = NewFragment.GetPayloadAs(FAnimNextObjectFunctionLocatorFragment::FragmentType);
					Payload->Path = Function;
					Handle->SetValue(NewFragment);
				}
			}
		}
	}

private:
	TWeakPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> WeakHandle;
	const UClass* CurrentClass = nullptr;
	TSharedPtr<UE::PropertyViewer::SPropertyViewer> PropertyViewer;
	TSharedPtr<SComboButton> ClassComboButton;
};

UE::UniversalObjectLocator::ELocatorFragmentEditorType FObjectFunctionLocatorEditor::GetLocatorFragmentEditorType() const
{
	return UE::UniversalObjectLocator::ELocatorFragmentEditorType::Relative;
}

bool FObjectFunctionLocatorEditor::IsAllowedInContext(FName InContextName) const
{
	return InContextName == "UAFContext";
}

bool FObjectFunctionLocatorEditor::IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return false;
}

UObject* FObjectFunctionLocatorEditor::ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return nullptr;
}

TSharedPtr<SWidget> FObjectFunctionLocatorEditor::MakeEditUI(const FEditUIParameters& InParameters)
{
	return SNew(SObjectFunctionLocatorEditor, InParameters.Handle);
}

FText FObjectFunctionLocatorEditor::GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimNextObjectFunctionLocatorFragment::FragmentType);
		const FAnimNextObjectFunctionLocatorFragment* Payload = InFragment->GetPayloadAs(FAnimNextObjectFunctionLocatorFragment::FragmentType);
		if(Payload && Payload->Path.IsValid())
		{
			return FText::FromString(Payload->Path.GetSubPathString());
		}
	}

	return LOCTEXT("ObjectFunctionLocatorName", "Function");
}

FText FObjectFunctionLocatorEditor::GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimNextObjectFunctionLocatorFragment::FragmentType);
		const FAnimNextObjectFunctionLocatorFragment* Payload = InFragment->GetPayloadAs(FAnimNextObjectFunctionLocatorFragment::FragmentType);
		if(Payload && Payload->Path.IsValid())
		{
			static const FTextFormat TextFormat(LOCTEXT("ObjectFunctionLocatorTooltipFormat", "Call the function {0}"));
			return FText::Format(TextFormat, FText::FromString(Payload->Path.ToString()));
		}
	}

	return LOCTEXT("ObjectFunctionLocatorTooltip", "A function to call to get an object");
}

FSlateIcon FObjectFunctionLocatorEditor::GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
}

UClass* FObjectFunctionLocatorEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
{
	ensure(InFragment.GetFragmentTypeHandle() == FAnimNextObjectFunctionLocatorFragment::FragmentType);
	const FAnimNextObjectFunctionLocatorFragment* Payload = InFragment.GetPayloadAs(FAnimNextObjectFunctionLocatorFragment::FragmentType);
	if(Payload && Payload->Path.IsValid())
	{
		if(UFunction* Function = Cast<UFunction>(Payload->Path.ResolveObject()))
		{
			if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Function->GetReturnProperty()))
			{
				return ObjectProperty->PropertyClass;
			}
		}
	}

	return nullptr;
}

FUniversalObjectLocatorFragment FObjectFunctionLocatorEditor::MakeDefaultLocatorFragment() const
{
	FUniversalObjectLocatorFragment NewFragment(FAnimNextObjectFunctionLocatorFragment::FragmentType);
	return NewFragment;
}

}

#undef LOCTEXT_NAMESPACE
