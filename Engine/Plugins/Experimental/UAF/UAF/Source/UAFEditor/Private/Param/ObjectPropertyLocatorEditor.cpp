// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectPropertyLocatorEditor.h"

#include "ClassViewerModule.h"
#include "IUniversalObjectLocatorCustomization.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorFragmentTypeHandle.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Framework/PropertyViewer/IFieldExpander.h"
#include "Framework/PropertyViewer/IFieldIterator.h"
#include "Modules/ModuleManager.h"
#include "Param/AnimNextObjectPropertyLocatorFragment.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/PropertyViewer/SPropertyViewer.h"

#define LOCTEXT_NAMESPACE "ObjectPropertyLocatorEditor"

namespace UE::UAF::Editor
{

class SObjectPropertyLocatorEditor : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SObjectPropertyLocatorEditor){}
	SLATE_END_ARGS()

	class FFieldIterator : public UE::PropertyViewer::IFieldIterator
	{
		virtual TArray<FFieldVariant> GetFields(const UStruct* InStruct, const FName InFieldName, const UStruct* InContainerStruct) const override
		{
			auto HasNestedObjectProperties = [](const UStruct* InPropertyStruct, auto& InHasNestedObjectProperties)
			{
				for (TFieldIterator<FProperty> PropertyIt(InPropertyStruct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
				{
					FProperty* Property = *PropertyIt;
					if (Property && Property->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_Edit | CPF_EditConst))
					{
						if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
						{
							return true;
						}
						else if(FStructProperty* StructProperty = CastField<FStructProperty>(Property))
						{
							return InHasNestedObjectProperties(StructProperty->Struct, InHasNestedObjectProperties);
						}
					}
				}

				return false;
			};

			TArray<FFieldVariant> Result;
			for (TFieldIterator<FProperty> PropertyIt(InStruct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
			{
				FProperty* Property = *PropertyIt;
				if (Property && Property->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_Edit | CPF_EditConst))
				{
					if(FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
					{
						Result.Add(FFieldVariant(ObjectProperty));
					}
					else if(FStructProperty* StructProperty = CastField<FStructProperty>(Property))
					{
						if(HasNestedObjectProperties(StructProperty->Struct, HasNestedObjectProperties))
						{
							Result.Add(FFieldVariant(StructProperty));
						}
					}
				}
			}
			return Result;
		}
	} FieldIterator;

	struct FFieldExpander : UE::PropertyViewer::IFieldExpander
	{
		virtual TOptional<const UClass*> CanExpandObject(const FObjectPropertyBase* Property, const UObject* Instance) const override
		{
			return TOptional<const UClass*>();
		}

		virtual bool CanExpandScriptStruct(const FStructProperty* StructProperty) const override
		{
			// Expand structs that have object properties
			if (TFieldIterator<FObjectProperty> PropertyIt(StructProperty->Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt)
			{
				return true;
			}
			return false;
		}

		virtual TOptional<const UStruct*> GetExpandedFunction(const UFunction* Function) const override
		{ 
			return TOptional<const UStruct*>();
		}
	} FieldExpander;
	
	void Construct(const FArguments& InArgs, TSharedPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> InHandle)
	{
		WeakHandle = InHandle;
		CurrentClass = InHandle->GetContextClass();

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(400.0f)
			.HeightOverride(400.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(ClassComboButton, SComboButton)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text_Lambda([this]()
						{
							return CurrentClass != nullptr ? FText::Format(LOCTEXT("CurrentClassNameFormat", "Class: {0}"), CurrentClass->GetDisplayNameText()) : LOCTEXT("ChooseClass", "Choose Class");
						})
					]
					.OnGetMenuContent_Lambda([this]()
					{
						FClassViewerInitializationOptions Options;
						FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
						return SNew(SBox)
							.WidthOverride(400.0f)
							.HeightOverride(400.0f)
							[
								ClassViewerModule.CreateClassViewer(Options, FOnClassPicked::CreateRaw(this, &SObjectPropertyLocatorEditor::HandleClassPicked))
							];
					})
				]
				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SAssignNew(PropertyViewer, UE::PropertyViewer::SPropertyViewer)
					.FieldIterator(&FieldIterator)
					.FieldExpander(&FieldExpander)
					.OnSelectionChanged(this, &SObjectPropertyLocatorEditor::HandlePropertyPicked)
				]
			]
		];

		PropertyViewer->AddContainer(CurrentClass);
	}

	void HandleClassPicked(UClass* InClass)
	{
		ClassComboButton->SetIsOpen(false);

		CurrentClass = InClass;

		PropertyViewer->RemoveAll();
		PropertyViewer->AddContainer(InClass);
	}

	void HandlePropertyPicked(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, TArrayView<const FFieldVariant> InFields, ESelectInfo::Type InSelectionType)
	{
		if(InFields.Num() > 0)
		{
			if(FObjectProperty* Property = InFields.Last().Get<FObjectProperty>())
			{
				if(TSharedPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> Handle = WeakHandle.Pin())
				{
					TArray<FProperty*> Properties;
					Properties.Reserve(InFields.Num());
					Algo::Transform(InFields, Properties, [](const FFieldVariant& InField){ return InField.Get<FProperty>(); });

					FUniversalObjectLocatorFragment NewFragment(FAnimNextObjectPropertyLocatorFragment::FragmentType);
					FAnimNextObjectPropertyLocatorFragment* Payload = NewFragment.GetPayloadAs(FAnimNextObjectPropertyLocatorFragment::FragmentType);
					*Payload = FAnimNextObjectPropertyLocatorFragment( Properties );
					Handle->SetValue(NewFragment);
				}
			}
		}
	}

private:
	TWeakPtr<UE::UniversalObjectLocator::IFragmentEditorHandle> WeakHandle;
	const UClass* CurrentClass = nullptr;
	TSharedPtr<UE::PropertyViewer::SPropertyViewer> PropertyViewer;
	TSharedPtr<SComboButton> ClassComboButton;
};

UE::UniversalObjectLocator::ELocatorFragmentEditorType FObjectPropertyLocatorEditor::GetLocatorFragmentEditorType() const
{
	return UE::UniversalObjectLocator::ELocatorFragmentEditorType::Relative;
}

bool FObjectPropertyLocatorEditor::IsAllowedInContext(FName InContextName) const
{
	return InContextName == "UAFContext";
}

bool FObjectPropertyLocatorEditor::IsDragSupported(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return false;
}

UObject* FObjectPropertyLocatorEditor::ResolveDragOperation(TSharedPtr<FDragDropOperation> DragOperation, UObject* Context) const
{
	return nullptr;
}

TSharedPtr<SWidget> FObjectPropertyLocatorEditor::MakeEditUI(const FEditUIParameters& InParameters)
{
	return SNew(SObjectPropertyLocatorEditor, InParameters.Handle);
}

FText FObjectPropertyLocatorEditor::GetDisplayText(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimNextObjectPropertyLocatorFragment::FragmentType);
		const FAnimNextObjectPropertyLocatorFragment* Payload = InFragment->GetPayloadAs(FAnimNextObjectPropertyLocatorFragment::FragmentType);
		if(Payload && !Payload->Path.IsEmpty() && !Payload->Path[0].IsPathToFieldEmpty())
		{
			// Currently have to resolve here as FFieldPath does not allow access to the underlying FName path chain
			TStringBuilder<256> StringBuilder;
			if(FProperty* RootProperty = Payload->Path[0].Get())
			{
				RootProperty->GetFName().AppendString(StringBuilder);

				for(int32 SegmentIndex = 1; SegmentIndex < Payload->Path.Num(); ++SegmentIndex)
				{
					StringBuilder.Append(TEXT("."));
					if(FProperty* SegmentProperty = Payload->Path[SegmentIndex].Get())
					{
						if(UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(SegmentProperty->GetOwnerUObject()))
						{
							StringBuilder.Append(SegmentProperty->GetDisplayNameText().ToString());
						}
						else
						{
							SegmentProperty->GetFName().AppendString(StringBuilder);
						}
					}
					else
					{
						StringBuilder.Append(TEXT("Unknown"));
					}
				}
			}

			return FText::FromStringView(StringBuilder);
		}
	}

	return LOCTEXT("ObjectPropertyLocatorName", "Property");
}

FText FObjectPropertyLocatorEditor::GetDisplayTooltip(const FUniversalObjectLocatorFragment* InFragment) const
{
	if(InFragment != nullptr)
	{
		ensure(InFragment->GetFragmentTypeHandle() == FAnimNextObjectPropertyLocatorFragment::FragmentType);
		const FAnimNextObjectPropertyLocatorFragment* Payload = InFragment->GetPayloadAs(FAnimNextObjectPropertyLocatorFragment::FragmentType);
		if(Payload && !Payload->Path.IsEmpty() && !Payload->Path[0].IsPathToFieldEmpty())
		{
			static const FTextFormat TextFormat(LOCTEXT("ObjectPFopertyLocatorTooltipFormat", "Dereference the property {0}"));
			TStringBuilder<256> StringBuilder;
			StringBuilder.Append(Payload->Path[0].ToString());
			for(int32 SegmentIndex = 1; SegmentIndex < Payload->Path.Num(); ++SegmentIndex)
			{
				StringBuilder.Append(TEXT("."));
				Payload->Path[SegmentIndex].Get()->GetFName().AppendString(StringBuilder);
			}

			return FText::Format(TextFormat, FText::FromStringView(StringBuilder));
		}
	}

	return LOCTEXT("ObjectPropertyLocatorTooltip", "Dereference a property to get an object");
}

FSlateIcon FObjectPropertyLocatorEditor::GetDisplayIcon(const FUniversalObjectLocatorFragment* InFragment) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.VariableIcon");
}

UClass* FObjectPropertyLocatorEditor::ResolveClass(const FUniversalObjectLocatorFragment& InFragment, UObject* InContext) const
{
	ensure(InFragment.GetFragmentTypeHandle() == FAnimNextObjectPropertyLocatorFragment::FragmentType);
	const FAnimNextObjectPropertyLocatorFragment* Payload = InFragment.GetPayloadAs(FAnimNextObjectPropertyLocatorFragment::FragmentType);
	if(Payload && !Payload->Path.IsEmpty() && !Payload->Path[0].IsPathToFieldEmpty())
	{
		if(FObjectProperty* Property = CastField<FObjectProperty>(Payload->Path.Last().Get()))
		{
			return Property->PropertyClass;
		}
	}

	return nullptr;
}

FUniversalObjectLocatorFragment FObjectPropertyLocatorEditor::MakeDefaultLocatorFragment() const
{
	FUniversalObjectLocatorFragment NewFragment(FAnimNextObjectPropertyLocatorFragment::FragmentType);
	return NewFragment;
}

}

#undef LOCTEXT_NAMESPACE
