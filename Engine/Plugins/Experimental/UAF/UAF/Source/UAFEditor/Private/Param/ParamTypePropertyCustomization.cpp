// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParamTypePropertyCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyBagDetails.h"
#include "EdGraphSchema_K2.h"
#include "EditorUtils.h"
#include "UncookedOnlyUtils.h"
#include "Param/ParamType.h"
#include "PropertyHandle.h"
#include "SPinTypeSelector.h"

#define LOCTEXT_NAMESPACE "ParamTypePropertyCustomization"

namespace UE::UAF::Editor
{

void FParamTypePropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	auto GetPinInfo = [WeakPropertyHandle = TWeakPtr<IPropertyHandle>(InPropertyHandle)]()
	{
		if(TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
		{
			TArray<const void*> RawData;
			PropertyHandle->AccessRawData(RawData);

			if(RawData.Num() > 0 && RawData[0] != nullptr)
			{
				const FAnimNextParamType& FirstType = *static_cast<const FAnimNextParamType*>(RawData[0]);
				bool bMultipleValues = false;

				for(int32 ValueIndex = 1; ValueIndex < RawData.Num(); ++ValueIndex)
				{
					const FAnimNextParamType& OtherType = *static_cast<const FAnimNextParamType*>(RawData[ValueIndex]);
					if(OtherType != FirstType)
					{
						bMultipleValues = true;
						break;
					}
				}
				
				if(!bMultipleValues)
				{
					return UncookedOnly::FUtils::GetPinTypeFromParamType(FirstType);
				}
			}
		}

		return FEdGraphPinType();
	};

	auto PinInfoChanged = [WeakPropertyHandle = TWeakPtr<IPropertyHandle>(InPropertyHandle)](const FEdGraphPinType& PinType)
	{
		if(TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
		{
			FAnimNextParamType ParamType = UncookedOnly::FUtils::GetParamTypeFromPinType(PinType);
			if(ParamType.IsValid())
			{
				TArray<void*> RawData;
				PropertyHandle->AccessRawData(RawData);

				if(RawData.Num() > 0)
				{
					PropertyHandle->NotifyPreChange();

					for(void* ValuePtr : RawData)
					{
						FAnimNextParamType& Type = *static_cast<FAnimNextParamType*>(ValuePtr);
						Type = ParamType;
					}

					PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				}
			}
		}
	};

	InHeaderRow
	.NameContent()
	[
		InPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SPinTypeSelector, FGetPinTypeTree::CreateStatic(&Editor::FUtils::GetFilteredVariableTypeTree))
				.TargetPinType_Lambda(GetPinInfo)
				.OnPinTypeChanged_Lambda(PinInfoChanged)
				.Schema(GetDefault<UPropertyBagSchema>())
				.bAllowArrays(true)
				.TypeTreeFilter(ETypeTreeFilter::None)
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

void FParamTypePropertyTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

}

#undef LOCTEXT_NAMESPACE