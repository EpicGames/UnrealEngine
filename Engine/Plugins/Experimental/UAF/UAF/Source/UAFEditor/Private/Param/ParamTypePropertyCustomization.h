// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"

namespace UE::UAF::Editor
{

class FParamTypePropertyTypeCustomization : public IPropertyTypeCustomization
{
private:
	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
};

}