// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAnalyzer.h"

#if UAF_TRACE_ENABLED

#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/ProviderLock.h"
#include "AnimNextProvider.h"
#include "TraceServices/Utils.h"
#include "StructUtils/PropertyBag.h"
#include "Serialization/ObjectReader.h"

FUAFAnalyzer::FUAFAnalyzer(TraceServices::IAnalysisSession& InSession, FUAFProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FUAFAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Module, "UAF", "Instance");
	Builder.RouteEvent(RouteId_InstanceVariables, "UAF", "InstanceVariables");
	Builder.RouteEvent(RouteId_InstanceVariablesStruct, "UAF", "InstanceVariablesStruct");
	Builder.RouteEvent(RouteId_InstanceVariableDescriptions, "UAF", "InstanceVariableDescriptions");
}

bool FUAFAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FUAFAnalyzer"));

	TraceServices::FProviderEditScopeLock ProviderEditScope(Provider);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_Module:
		{
			uint64 InstanceId = EventData.GetValue<uint64>("InstanceId");
			uint64 HostInstanceId = EventData.GetValue<uint64>("HostInstanceId");
			uint64 AssetId = EventData.GetValue<uint64>("AssetId");
			uint64 OuterObjectId = EventData.GetValue<uint64>("OuterObjectId");
			Provider.AppendInstance(InstanceId, HostInstanceId, AssetId, OuterObjectId);
			break;
		}
		case RouteId_InstanceVariables:
		{
			uint64 ModuleInstanceId = EventData.GetValue<uint64>("InstanceId");
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");
			uint32 VariableDescHash = EventData.GetValue<uint32>("VariableDescriptionHash");
			TArrayView<const uint8> VariableData = EventData.GetArrayView<uint8>("VariableData");

			Provider.AppendVariables(Context.EventTime.AsSeconds(Cycle), RecordingTime, ModuleInstanceId, EPropertyVariableDataType::PropertyBag, VariableDescHash, VariableData);
			break;
		}
		case RouteId_InstanceVariablesStruct:
		{
			uint64 ModuleInstanceId = EventData.GetValue<uint64>("InstanceId");
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");
			TArrayView<const uint8> VariableData = EventData.GetArrayView<uint8>("VariableData");

			Provider.AppendVariables(Context.EventTime.AsSeconds(Cycle), RecordingTime, ModuleInstanceId, EPropertyVariableDataType::InstancedStruct, 0, VariableData);
			break;
		}
		case RouteId_InstanceVariableDescriptions:
		{
			uint32 VariableDescHash = EventData.GetValue<uint32>("VariableDescriptionHash");
			TArrayView<const uint8> VariableDescData = EventData.GetArrayView<uint8>("VariableDescriptionData");
			FString Name;
			if (EventData.GetString("Name", Name) == false)
			{
				Name = TEXT("Unknown Name");
			}

			Provider.AppendVariableDescriptions(VariableDescHash, VariableDescData, Name);
			break;
		}
	}

	return true;
}
#endif // UAF_TRACE_ENABLED
