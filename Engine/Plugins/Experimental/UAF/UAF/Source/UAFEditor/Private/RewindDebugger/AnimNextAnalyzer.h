// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RewindDebugger/UAFTrace.h"

#if UAF_TRACE_ENABLED
#include "Trace/Analyzer.h"

class FUAFProvider;
namespace TraceServices { class IAnalysisSession; }

class FUAFAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FUAFAnalyzer(TraceServices::IAnalysisSession& InSession, FUAFProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_Module,
		RouteId_InstanceVariables,
		RouteId_InstanceVariablesStruct,
		RouteId_InstanceVariableDescriptions
	};

	TraceServices::IAnalysisSession& Session;
	FUAFProvider& Provider;
};
#endif // UAF_TRACE_ENABLED
