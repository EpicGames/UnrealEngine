// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextModuleTrack.h"

#include "ObjectTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextModuleTrack)

#if UAF_TRACE_ENABLED
#include "AnimNextProvider.h"
#include "Common/ProviderLock.h"
#include "Editor.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"
#include "Modules/ModuleManager.h"
#include "ObjectAsTraceIdProxyArchiveReader.h"
#include "PropertyEditorModule.h"
#include "UAFAssetInstance.h"
#include "Serialization/MemoryReader.h"
#include "StructUtils/PropertyBag.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "TraceServices/Model/Frames.h"

#define LOCTEXT_NAMESPACE "AnimNextModuleTrack"

namespace UE::UAF::Editor
{

static const FName AnimNextModulesName("AnimNextModules");

FName FAnimNextModuleTrackCreator::GetTargetTypeNameInternal() const
{
	return FUAFAssetInstance::StaticStruct()->GetFName();
}

FText FAnimNextModuleTrack::GetDisplayNameInternal() const
{
	if (DisplayNameCache.IsEmpty())
	{
		const IGameplayProvider* GameplayProvider = IRewindDebugger::Instance()->GetAnalysisSession()->ReadProvider<IGameplayProvider>("GameplayProvider");

		if (InstanceId != 0 && GameplayProvider)
		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
			const FObjectInfo& ModuleInfo = GameplayProvider->GetObjectInfo(InstanceId);
			DisplayNameCache = FText::FromString(ModuleInfo.Name);
			return DisplayNameCache;
		}

		// Don't cache this since it is a placeholder name, will be replaced once the real one is sent
		return NSLOCTEXT("RewindDebugger", "AnimNextSystemTrackName", "System");
	}

	return DisplayNameCache;
}

void FAnimNextModuleTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({AnimNextModulesName, LOCTEXT("UAFSystems", "UAF Systems")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FAnimNextModuleTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& ObjectId) const
{
	return MakeShared<FAnimNextModuleTrack>(ObjectId.GetMainId());
}

FAnimNextModuleTrack::FAnimNextModuleTrack(uint64 InInstanceId) :
	InstanceId(InInstanceId)
{
	Initialize();
}


FAnimNextModuleTrack::~FAnimNextModuleTrack()
{
	if (UPropertyBagDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get())
	{
		DetailsObject->ClearFlags(RF_Standalone);
	}
}

void FAnimNextModuleTrack::Initialize()
{
	ExistenceRange = MakeShared<SEventTimelineView::FTimelineEventData>();
	ExistenceRange->EventWindows.Add({0,0, GetDisplayNameInternal(), GetDisplayNameInternal(), FLinearColor(0.1f,0.15f,0.11f)});

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InitializeDetailsObject();
}

UPropertyBagDetailsObject* FAnimNextModuleTrack::InitializeDetailsObject()
{
	UPropertyBagDetailsObject* DetailsObject = NewObject<UPropertyBagDetailsObject>();
	DetailsObject->SetFlags(RF_Standalone);
	DetailsObjectWeakPtr = MakeWeakObjectPtr(DetailsObject);
	DetailsView->SetObject(DetailsObject);
	return DetailsObject;
}


TSharedPtr<SWidget> FAnimNextModuleTrack::GetTimelineViewInternal()
{
	return SNew(SEventTimelineView)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.EventData_Raw(this, &FAnimNextModuleTrack::GetExistenceRange);
}

bool FAnimNextModuleTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnimNextModuleTrack::UpdateInternal);

	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	const TRange<double> ViewRange = RewindDebugger->GetCurrentViewRange();

	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	bool bChanged = false;

	if (const FUAFProvider* UAFProvider = AnalysisSession->ReadProvider<FUAFProvider>("UAFProvider"))
	{
		double CurrentScrubTime = IRewindDebugger::Instance()->CurrentTraceTime();

		UPropertyBagDetailsObject* DetailsObject = DetailsObjectWeakPtr.Get();
		if (DetailsObject == nullptr)
		{
			// this should not happen unless the object was garbage collected (which should not happen since it's marked as Standalone)
			Initialize();

			DetailsObject = DetailsObjectWeakPtr.Get();
			check(DetailsObject);
		}

		if (InstanceId != 0)
		{
			TraceServices::FProviderReadScopeLock UAFProviderReadScope(*UAFProvider);
			if (const FDataInterfaceData* Data = UAFProvider->GetDataInterfaceData(InstanceId))
			{
				if (PreviousScrubTime != CurrentScrubTime)
				{
					PreviousScrubTime = CurrentScrubTime;

					DetailsObject->Properties.SetNum(0, EAllowShrinking::No);
					DetailsObject->NativeProperties.SetNum(0, EAllowShrinking::No);

					bool bFrameFound;
					TraceServices::FFrame MarkerFrame;
					{
						TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
						const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
						bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, CurrentScrubTime, MarkerFrame);
					}
		
					if (bFrameFound)
					{
						Data->VariablesTimeline.EnumerateEvents(MarkerFrame.StartTime, MarkerFrame.EndTime, [UAFProvider, DetailsObject, AnalysisSession, this](double InStartTime, double InEndTime, uint32 InDepth, const FPropertyVariableData& VariableListData)
						{
							const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
							if (VariableListData.ValueType == EPropertyVariableDataType::PropertyBag)
							{
								// First look up property description
								const FPropertyDescriptionData* DescriptionData = UAFProvider->GetPropertyDescriptionData(VariableListData.DescriptionHash);
								if (DescriptionData == nullptr)
								{
									// Can't do anything without a description
									return TraceServices::EEventEnumerate::Continue;
								}

								// Load the property descriptions
								FMemoryReader DescriptionReader(DescriptionData->PropertyBagData);
								FObjectAsTraceIdProxyArchiveReader DescriptionReaderProxy(DescriptionReader, GameplayProvider);
								DescriptionReaderProxy.UsingCustomVersion(UE::UAF::FUAFTrace::CustomVersionGUID);
								DescriptionReaderProxy << PropertyDescriptions;
								DetailsObject->Properties.SetNum(DetailsObject->Properties.Num() + 1);
								DetailsObject->Properties.Last().Name = DescriptionData->Name;
								FInstancedPropertyBag& PropertyBag = DetailsObject->Properties.Last().PropertyBag;
								PropertyBag.AddProperties(PropertyDescriptions);

								// Load the property values
								FMemoryReader Reader(VariableListData.ValueData);
								FObjectAsTraceIdProxyArchiveReader ReaderProxy(Reader, GameplayProvider);

								UPropertyBag* PropertyBagStruct = const_cast<UPropertyBag*>(DetailsObject->Properties.Last().PropertyBag.GetPropertyBagStruct());
								if (PropertyBagStruct != nullptr)
								{
									PropertyBagStruct->SerializeItem(ReaderProxy, PropertyBag.GetMutableValue().GetMemory(), nullptr);
								}
							}
							else //(VariableListData.ValueType == EPropertyVariableDataType::InstancedStruct)
							{
								FMemoryReader Reader(VariableListData.ValueData);
								FObjectAsTraceIdProxyArchiveReader ReaderProxy(Reader, GameplayProvider);

								DetailsObject->NativeProperties.SetNum(DetailsObject->NativeProperties.Num() + 1);
								FInstancedStruct& InstancedStruct = DetailsObject->NativeProperties.Last();

								FInstancedStruct::StaticStruct()->SerializeItem(ReaderProxy, &InstancedStruct, nullptr);
							}

							return TraceServices::EEventEnumerate::Continue;
						});
					}
				}

				ExistenceRange->EventWindows.SetNum(1, EAllowShrinking::No);
				ExistenceRange->EventWindows[0].TimeStart = Data->StartTime;
				ExistenceRange->EventWindows[0].TimeEnd = Data->EndTime;
			}
		}
	}

	return bChanged;
}

bool FAnimNextModuleTrack::HandleDoubleClickInternal()
{
	if (InstanceId == RewindDebugger::FObjectId::InvalidId)
	{
		return false;
	}

	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		FString AssetPath;
		{
			const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
			const FUAFProvider* AnimNextProvider = IRewindDebugger::Instance()->GetAnalysisSession()->ReadProvider<FUAFProvider>("UAFProvider");

			TraceServices::FProviderReadScopeLock AnimNextProviderReadScope(*AnimNextProvider);
			if (const FDataInterfaceData* ModuleData = AnimNextProvider->GetDataInterfaceData(InstanceId))
			{
				TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
				const FObjectInfo& AssetInfo = GameplayProvider->GetObjectInfo(ModuleData->AssetId);
				if (!EnumHasAnyFlags(AssetInfo.Flags, EObjectInfoFlags::TransientObject))
				{
					AssetPath = AssetInfo.PathName;
				}
			}
		}

		if (!AssetPath.IsEmpty())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetPath);
			return true;
		}
	}
	return false;
}

bool FAnimNextModuleTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& ObjectId) const
{
	return true;
}

} // UE::UAF::Editor

#undef LOCTEXT_NAMESPACE

#endif // UAF_TRACE_ENABLED
