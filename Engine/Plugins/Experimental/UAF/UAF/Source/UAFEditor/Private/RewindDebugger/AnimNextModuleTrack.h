// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RewindDebugger/UAFTrace.h"

#if UAF_TRACE_ENABLED
#include "IRewindDebuggerTrackCreator.h"
#include "RewindDebuggerTrack.h"
#include "Textures/SlateIcon.h"
#include "SEventTimelineView.h"
#include "IDetailsView.h"
#endif // UAF_TRACE_ENABLED
#include "StructUtils/PropertyBag.h"

#include "AnimNextModuleTrack.generated.h"

USTRUCT()
struct FInstancedPropertyBagDetails
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Details)
	FString Name;

	UPROPERTY(EditAnywhere, Category = Details, meta=(ShowOnlyInnerProperties))
	FInstancedPropertyBag PropertyBag;
};

UCLASS()
class UPropertyBagDetailsObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category=Details, meta=(TitleProperty = "Name"))
	TArray<FInstancedPropertyBagDetails> Properties;

	UPROPERTY(EditAnywhere, Category=Details)
	TArray<FInstancedStruct> NativeProperties;
};

#if UAF_TRACE_ENABLED
namespace UE::UAF::Editor
{
	
class FAnimNextModuleTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FAnimNextModuleTrack(uint64 InstanceId);
	virtual ~FAnimNextModuleTrack();
	
	TSharedPtr<SEventTimelineView::FTimelineEventData> GetExistenceRange() const { return ExistenceRange; }
private:
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual bool UpdateInternal() override;
	
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override
	{
		return DetailsView;
	}
	
	virtual FSlateIcon GetIconInternal() override
	{
		return Icon;
	}
	
	virtual FName GetNameInternal() const override
	{
		return "AnimNextModule";
	}
	
	virtual uint64 GetObjectIdInternal() const override
	{
		return InstanceId;
	}
	
	virtual FText GetDisplayNameInternal() const override;
	virtual bool HandleDoubleClickInternal() override;

	void Initialize();
	UPropertyBagDetailsObject* InitializeDetailsObject();

	TSharedPtr<IDetailsView> DetailsView;
	FSlateIcon Icon;
	uint64 InstanceId; 
	double PreviousScrubTime = -1.0;
	TWeakObjectPtr<UPropertyBagDetailsObject> DetailsObjectWeakPtr;
	TSharedPtr<SEventTimelineView::FTimelineEventData> ExistenceRange;
	mutable FText DisplayNameCache;

	TArray<FPropertyBagPropertyDesc> PropertyDescriptions;
};

#if WITH_ENGINE
class FAnimNextModuleTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	
	
	virtual FName GetNameInternal() const override
	{
		return "AnimNextModule";
	}
	
	virtual bool IsCreatingPrimaryChildTrackInternal() const override
	{
		return true;
	}
};
#endif // WITH_ENGINE

}
#endif // UAF_TRACE_ENABLED
