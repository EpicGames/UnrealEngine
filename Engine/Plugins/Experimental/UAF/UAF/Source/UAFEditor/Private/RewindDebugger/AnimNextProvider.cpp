// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextProvider.h"
#if UAF_TRACE_ENABLED
#include "ObjectTrace.h"

namespace TraceServices
{
thread_local FProviderLock::FThreadLocalState GUAFProviderLockState;
}

FName FUAFProvider::ProviderName("UAFProvider");

#define LOCTEXT_NAMESPACE "UAFProvider"

FUAFProvider::FUAFProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
{
}

void FUAFProvider::AppendInstance(uint64 InstanceId, uint64 HostInstanceId, uint64 AssetId, uint64 OuterObjectId)
{
	EditAccessCheck();

	if (HostInstanceId == OuterObjectId)
	{
		ComponentIdToSystemId.Add(OuterObjectId, InstanceId);
	}

	TSharedRef<FDataInterfaceData> Data = MakeShared<FDataInterfaceData>(Session);
	Data->InstanceId = InstanceId;
	Data->HostInstanceId = HostInstanceId;
	Data->OuterObjectId = OuterObjectId;
	Data->AssetId = AssetId;

	DataInterfaceData.Add(InstanceId, Data);
	
	TArray<TSharedRef<FDataInterfaceData>>& ChildList = HostToChildDataMap.FindOrAdd(HostInstanceId);
	ChildList.AddUnique(Data);
}

void FUAFProvider::AppendVariables(double ProfileTime, double RecordingTime, uint64 DataInterfaceId, EPropertyVariableDataType PropertyVariableDataType, uint32 PropertyDescriptionHash, const TArrayView<const uint8>& VariableData)
{
	EditAccessCheck();
	
	if (const TSharedRef<FDataInterfaceData>* Data = DataInterfaceData.Find(DataInterfaceId))
	{
		if (!VariableData.IsEmpty())
		{
			(*Data)->VariablesTimeline.AppendEvent(ProfileTime, FPropertyVariableData(PropertyVariableDataType, PropertyDescriptionHash, VariableData));
		}
		(*Data)->EndTime = RecordingTime;
		if ((*Data)->StartTime < 0)
		{
			(*Data)->StartTime = RecordingTime;
		}
	}
}

void FUAFProvider::AppendVariableDescriptions(uint32 PropertyDescriptionHash, const TArrayView<const uint8>& VariableDescriptionData, const FString& Name)
{
	EditAccessCheck();
	
	if (HashToPropertyDescriptionDataMap.Contains(PropertyDescriptionHash) == false)
	{
		if (!VariableDescriptionData.IsEmpty())
		{
			HashToPropertyDescriptionDataMap.Add(PropertyDescriptionHash, FPropertyDescriptionData(VariableDescriptionData, Name));
		}
	}
}


bool FUAFProvider::GetModuleId(uint64 ComponentId, uint64& OutModuleId) const
{
	ReadAccessCheck();
	
	if (const uint64* Id = ComponentIdToSystemId.Find(ComponentId))
	{
		OutModuleId = *Id;
		return true;
	}
	return false;
}

const FDataInterfaceData* FUAFProvider::GetDataInterfaceData(uint64 DataInterfaceId) const
{
	ReadAccessCheck();

	const TSharedRef<FDataInterfaceData>* Data = DataInterfaceData.Find(DataInterfaceId);
	if (Data)
	{
		return &**Data;
	}
	return nullptr;
}

void FUAFProvider::EnumerateChildInstances(uint64 InstanceId, TFunctionRef<void(const FDataInterfaceData&)> Callback) const
{
	ReadAccessCheck();

	if (const TArray<TSharedRef<FDataInterfaceData>>* Children = HostToChildDataMap.Find(InstanceId))
	{
		for (auto Data : *Children)
		{
			check (Data->HostInstanceId == InstanceId);
			Callback(*Data);
		}
	}

}

void FUAFProvider::EnumerateDataInterfaces(const double InStartTime, TFunctionRef<void(const TSharedRef<const FDataInterfaceData>)> Callback) const
{
	ReadAccessCheck();

	for (const auto& [InstanceId, InterfaceData] : DataInterfaceData)
	{
		if (InterfaceData->StartTime <= InStartTime && InterfaceData->EndTime >= InStartTime)
		{
			Callback(InterfaceData);
		}
	}
}

#undef LOCTEXT_NAMESPACE
#endif // UAF_TRACE_ENABLED
