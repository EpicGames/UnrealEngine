// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RewindDebugger/UAFTrace.h"

#if UAF_TRACE_ENABLED
#include "Engine/EngineBaseTypes.h"
#include "Model/PointTimeline.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Model/IntervalTimeline.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Common/ProviderLock.h"

namespace TraceServices
{
	class IAnalysisSession;
	extern thread_local FProviderLock::FThreadLocalState GUAFProviderLockState;
}

struct FPropertyDescriptionData
{
	FPropertyDescriptionData(TConstArrayView<uint8> DataView, const FString& InName) : PropertyBagData(DataView), Name(InName) {}
	TArray<uint8> PropertyBagData;

	FString Name;
};

enum class EPropertyVariableDataType
{
	PropertyBag,
	InstancedStruct,
};

struct FPropertyVariableData
{
	FPropertyVariableData(EPropertyVariableDataType VariableDataType, uint32 DescriptionHash, TConstArrayView<uint8> Data)
		: ValueType (VariableDataType), DescriptionHash(DescriptionHash) , ValueData(Data) {}

	EPropertyVariableDataType ValueType;
	uint32 DescriptionHash;
	TArray<uint8> ValueData;
};

struct FDataInterfaceData
{
	FDataInterfaceData(TraceServices::IAnalysisSession& Session)
		: VariablesTimeline(Session.GetLinearAllocator( )),
			InstanceId (0),
			HostInstanceId (0),
			OuterObjectId (0),
			AssetId (0),
			StartTime (-1),
			EndTime (-1)
	{
	}

	TraceServices::TPointTimeline<FPropertyVariableData> VariablesTimeline;
	uint64 InstanceId;
	uint64 HostInstanceId;
	uint64 OuterObjectId;
	uint64 AssetId;
	double StartTime;
	double EndTime;
};

class FUAFProvider : public TraceServices::IProvider, public TraceServices::IEditableProvider
{
public:
	static FName ProviderName;

	FUAFProvider(TraceServices::IAnalysisSession& InSession);

	//~ IProvider (read lock)
	virtual void BeginRead() const override
	{
		Lock.BeginRead(TraceServices::GUAFProviderLockState);
	}

	virtual void EndRead() const override
	{
		Lock.EndRead(TraceServices::GUAFProviderLockState);
	}

	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(TraceServices::GUAFProviderLockState);
	}

	//~ IEditableProvider (edit lock)
	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(TraceServices::GUAFProviderLockState);
	}

	virtual void EndEdit() const override
	{
		Lock.EndWrite(TraceServices::GUAFProviderLockState);
	}

	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(TraceServices::GUAFProviderLockState);
	}

	void AppendInstance(uint64 InstanceId, uint64 HostInstanceId,  uint64 AssetId, uint64 OuterObjectId);
	void AppendVariables(double ProfileTime, double RecordingTime, uint64 InModuleInstanceId, EPropertyVariableDataType PropertyVariableDataType, uint32 PropertyDescriptionHash, const TArrayView<const uint8>& VariableData);
	void AppendVariableDescriptions(uint32 PropertyDescriptionHash, const TArrayView<const uint8>& VariableDescriptionData, const FString& Name);

	bool GetModuleId(uint64 ComponentId, uint64& OutModuleId) const;
	const FDataInterfaceData* GetDataInterfaceData(uint64 DataInterfaceId) const;
	const FPropertyDescriptionData* GetPropertyDescriptionData(uint32 PropertyDescriptionHash) const { ReadAccessCheck(); return HashToPropertyDescriptionDataMap.Find(PropertyDescriptionHash);}

	void EnumerateChildInstances(uint64 InstanceId, TFunctionRef<void(const FDataInterfaceData&)> Callback) const;
	void EnumerateDataInterfaces(const double StartTime, TFunctionRef<void(const TSharedRef<const FDataInterfaceData>)> Callback) const;

private:
	TraceServices::IAnalysisSession& Session;
	mutable TraceServices::FProviderLock Lock;

	TMap<uint64, uint64> ComponentIdToSystemId;
	TMap<uint64, TSharedRef<FDataInterfaceData>> DataInterfaceData;
	TMap<uint64, TArray<TSharedRef<FDataInterfaceData>>> HostToChildDataMap;
	TMap<uint32, FPropertyDescriptionData> HashToPropertyDescriptionDataMap;
};
#endif //UAF_TRACE_ENABLED