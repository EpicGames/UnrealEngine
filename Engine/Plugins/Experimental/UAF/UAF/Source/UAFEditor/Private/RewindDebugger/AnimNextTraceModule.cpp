// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextTraceModule.h"

#if UAF_TRACE_ENABLED
#include "AnimNextProvider.h"
#include "AnimNextAnalyzer.h"

FName FUAFTraceModule::ModuleName("UAF");

void FUAFTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("UAF");
}

void FUAFTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FUAFProvider> UAFProvider = MakeShared<FUAFProvider>(InSession);
	InSession.AddProvider(FUAFProvider::ProviderName, UAFProvider);

	InSession.AddAnalyzer(new FUAFAnalyzer(InSession, *UAFProvider));
}

void FUAFTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("UAF"));
}

void FUAFTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}
#endif //UAF_TRACE_ENABLED
