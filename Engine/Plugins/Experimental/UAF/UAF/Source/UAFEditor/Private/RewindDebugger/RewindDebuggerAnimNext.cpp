// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerAnimNext.h"

#if UAF_TRACE_ENABLED
#include "IGameplayProvider.h"

FRewindDebuggerUAF::FRewindDebuggerUAF()
{

}

void FRewindDebuggerUAF::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
	// Future place to sync anim next workspace editor debugging when scrubbing
}

#endif // UAF_TRACE_ENABLED
