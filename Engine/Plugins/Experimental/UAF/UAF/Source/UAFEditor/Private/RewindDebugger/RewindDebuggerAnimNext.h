// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RewindDebugger/UAFTrace.h"

#if UAF_TRACE_ENABLED
#include "CoreMinimal.h"
#include "IRewindDebugger.h"
#include "IRewindDebuggerExtension.h"

// Rewind debugger extension for Chooser support

class FRewindDebuggerUAF : public IRewindDebuggerExtension
{
public:
	FRewindDebuggerUAF();
	virtual ~FRewindDebuggerUAF() {};

	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
};
#endif //UAF_TRACE_ENABLED
