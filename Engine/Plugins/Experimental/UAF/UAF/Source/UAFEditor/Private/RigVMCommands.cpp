// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCommands.h"

#define LOCTEXT_NAMESPACE "AnimNextRigVMCommands"

namespace UE::UAF
{

FRigVMCommands::FRigVMCommands()
	: TCommands<FRigVMCommands>("AnimNextRigVM", LOCTEXT("AnimNextRigVMCommands", "RigVM"), NAME_None, "AnimNextStyle")
{
}

void FRigVMCommands::RegisterCommands()
{
	UI_COMMAND(Compile, "Compile", "Compile all relevant assets", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AutoCompile, "Auto Compile", "Automatically compile on every edit", EUserInterfaceActionType::ToggleButton, FInputChord());
}

}

#undef LOCTEXT_NAMESPACE
