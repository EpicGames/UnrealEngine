// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::UAF
{

// The commands for general AnimNext RVM graph editing
class FRigVMCommands : public TCommands<FRigVMCommands>
{
public:
	FRigVMCommands();

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> Compile;
	TSharedPtr<FUICommandInfo> AutoCompile;
};

}
