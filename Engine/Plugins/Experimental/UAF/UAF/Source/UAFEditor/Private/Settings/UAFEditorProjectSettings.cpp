// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFEditorProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFEditorProjectSettings)

UUAFEditorProjectSettings::UUAFEditorProjectSettings()
{
	UAFBrowserCategoryConfiguration = "/UAFAnimGraph/Internal/TABC_UAFBrowserDefaultConfig.TABC_UAFBrowserDefaultConfig";
}
