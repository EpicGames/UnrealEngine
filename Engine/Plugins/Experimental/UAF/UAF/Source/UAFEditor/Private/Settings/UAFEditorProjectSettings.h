// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "UAFEditorProjectSettings.generated.h"

/**
 * UAF Editor Settings that are shared across a project.
 */
UCLASS(MinimalAPI, Config = Editor, defaultconfig)
class UUAFEditorProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	UUAFEditorProjectSettings();

public:
	/** The configuration used the UAF browser for setting up categories */
	UPROPERTY(config, EditAnywhere, DisplayName = "UAF Browser Configuration", Category = UAF, meta = (AllowedClasses = "/Script/UserAssetTagsEditor.TaggedAssetBrowserConfiguration"))
	FSoftObjectPath UAFBrowserCategoryConfiguration;

	/** Additional plugin names to include in the UAF Browser filter bar beyond the auto-discovered project plugins. */
	UPROPERTY(config, EditAnywhere, DisplayName = "Additional Browser Plugin Filters", Category = "UAF Browser Plugin Filters")
	TArray<FName> AdditionalBrowserPluginFilters;

	/** Plugin names to exclude from the auto-discovered project plugin list in the UAF Browser filter bar. */
	UPROPERTY(config, EditAnywhere, DisplayName = "Excluded Browser Plugin Filters", Category = "UAF Browser Plugin Filters")
	TArray<FName> ExcludedBrowserPluginFilters;
};