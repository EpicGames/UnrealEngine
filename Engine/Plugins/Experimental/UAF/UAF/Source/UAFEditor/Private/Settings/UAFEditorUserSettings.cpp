// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFEditorUserSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UAFEditorUserSettings)

UUAFEditorUserSettings::UUAFEditorUserSettings()
{
	// Initialize hidden columns. Asset browser doesn't support an allow-list  
	// since the columns that asset(s) may have aren't known ahead of time.
	UAFBrowserHiddenColumns =
	{
		"ItemDiskSize",
		"bUseNormalizedRootMotionScale",
		"Compressed Size (KB)",
		"ImportFileFramerate",
		"ImportResampleFramerate",
		"Interpolation",
		"Number of Frames",
		"Number of Keys",
		"NumberOfSampledKeys",
		"ParentAsset",
		"PreviewSkeletalMesh",
		"RetargetSource",
		"RetargetSourceAsset",
		"Skeleton",
		"Target Frame Rate",
		"Class"
	};
}
