// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "UAFEditorUserSettings.generated.h"

/**
 * UAF Editor Settings that are unique per user.
 */
UCLASS(MinimalAPI, Config = EditorPerProjectUserSettings, meta = (DisplayName = "UAF Editor User Settings"))
class UUAFEditorUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	UUAFEditorUserSettings();

public:

	/** True if the UAF Browser hotkey opens the sidebar (Right), false will open status bar (Bottom) */
	UPROPERTY(EditAnywhere, Category="UAF", Config)
	bool bSummonUAFBrowserHotkeyOpensSidebar = true;
	
	/** If true, functions in UAF workspaces will open in a separate tab per default */
	UPROPERTY(EditAnywhere, Category="UAF", Config)
	bool bAutoOpenFunctionsInSeparateTabs = true;
	
	/** If true, when user starts PIE any UAF assets requiring compilation will automatically be compiled */
	UPROPERTY(EditAnywhere, Category="UAF", Config)
	bool bAutoCompileOutOfDateAssets = true;

	/** Default columns to hide in the UAF Browser asset list. Applies to all browser instances (docked, sidebar, drawer). */
	UPROPERTY(EditAnywhere, Category="UAF|Browser", Config)
	TArray<FString> UAFBrowserHiddenColumns;

	/** Broadcasts to synchronize column visibility across all browser instances. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBrowserHiddenColumnsChanged, const TArray<FString>& /*HiddenColumnIds*/);
	FOnBrowserHiddenColumnsChanged OnBrowserHiddenColumnsChanged;
};
