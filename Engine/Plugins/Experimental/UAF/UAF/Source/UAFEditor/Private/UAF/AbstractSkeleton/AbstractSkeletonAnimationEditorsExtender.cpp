// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbstractSkeletonAnimationEditorsExtender.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonLabelBinding.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Styling/AppStyle.h"
#include "AssetRegistry/AssetData.h"

#define LOCTEXT_NAMESPACE "UE::UAF::UAnimationEditorsAssetFamilyExtension_AbstractSkeleton"

TObjectPtr<UClass> UAnimationEditorsAssetFamilyExtension_AbstractSkeleton::GetAssetClass() const
{
	return UAbstractSkeletonSetBinding::StaticClass();
}

FText UAnimationEditorsAssetFamilyExtension_AbstractSkeleton::GetAssetTypeDisplayName() const
{
	return LOCTEXT("DisplayName", "Abstract Skeleton");
}

const FSlateBrush* UAnimationEditorsAssetFamilyExtension_AbstractSkeleton::GetAssetTypeDisplayIcon() const
{
	return FAppStyle::Get().GetBrush("Persona.AssetClass.SkeletalMesh");
}

void UAnimationEditorsAssetFamilyExtension_AbstractSkeleton::FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths.Add(UAbstractSkeletonSetBinding::StaticClass()->GetClassPathName());
	Filter.ClassPaths.Add(UAbstractSkeletonLabelBinding::StaticClass()->GetClassPathName());

	if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
	{
		const FAssetData SkeletonAssetData = FAssetData(SkeletonAsset.Get(), true);
		if (SkeletonAssetData.IsValid())
		{
			Filter.TagsAndValues.Add(
				TEXT("Skeleton"),
				SkeletonAssetData.GetExportTextName());
		}
	}

	AssetRegistryModule.Get().GetAssets(Filter, OutAssets);
}

bool UAnimationEditorsAssetFamilyExtension_AbstractSkeleton::IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	FAssetDataTagMapSharedView::FFindTagResult Result = InAssetData.TagsAndValues.FindTag(TEXT("Skeleton"));
	if (Result.IsSet())
	{
		if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
		{
			return Result.GetValue() == FSoftObjectPath(SkeletonAsset).ToString();
		}
	}

	return false;
}

void UAnimationEditorsAssetFamilyExtension_AbstractSkeleton::FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface)
{
	TObjectPtr<USkeleton> Skeleton;

	if (const UAbstractSkeletonSetBinding* SetBinding = Cast<const UAbstractSkeletonSetBinding>(InAsset))
	{
		Skeleton = SetBinding->GetSkeleton();
	}
	else if (const UAbstractSkeletonLabelBinding* LabelBinding = Cast<const UAbstractSkeletonLabelBinding>(InAsset))
	{
		Skeleton = LabelBinding->GetSkeleton();
	}

	if (AssetFamilyInterface.IsAssetTypeInFamily<USkeleton>())
	{
		AssetFamilyInterface.SetAssetOfType<USkeleton>(Skeleton);
	}

	if (AssetFamilyInterface.IsAssetTypeInFamilyAndUnassigned<USkeletalMesh>() && Skeleton)
	{
		AssetFamilyInterface.SetAssetOfType<USkeletalMesh>(Skeleton->GetPreviewMesh());
	}
};

void UAnimationEditorsAssetFamilyExtension_AbstractSkeleton::GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const
{
	OutAfterClass = USkeleton::StaticClass()->GetFName();
	OutBeforeClass = USkeletalMesh::StaticClass()->GetFName();
}

#undef LOCTEXT_NAMESPACE
