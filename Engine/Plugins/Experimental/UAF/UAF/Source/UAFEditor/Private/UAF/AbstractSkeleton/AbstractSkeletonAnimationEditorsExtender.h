// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationEditorsAssetFamilyExtension.h"

#include "AbstractSkeletonAnimationEditorsExtender.generated.h"

UCLASS()
class UAnimationEditorsAssetFamilyExtension_AbstractSkeleton : public UAnimationEditorsAssetFamilyExtension
{
	GENERATED_BODY()

public:
	// UAnimationEditorsAssetFamilyExtension
	virtual TObjectPtr<UClass> GetAssetClass() const override;
	virtual FText GetAssetTypeDisplayName() const override;
	virtual const FSlateBrush* GetAssetTypeDisplayIcon() const override;
	virtual void FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual bool IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual void FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) override;
	virtual void GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const override;
};