// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbstractSkeletonEditor.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ToolMenus.h"
#include "PersonaModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPersonaToolkit.h"
#include "IPersonaEditorModeManager.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonLabelBinding.h"
#include "UAF/AbstractSkeleton/Labels/SLabelsTab.h"
#include "UAF/AbstractSkeleton/Sets/SSetsTab.h"
#include "Animation/Skeleton.h"

#define LOCTEXT_NAMESPACE "UE::UAF::FAbstractSkeletonEditor"

namespace UE::UAF::Editor
{
	namespace Tabs
	{
		static const FName ViewportId(TEXT("AbstractSkeletonBindingEditor_ViewportTab"));
		static const FName DetailsId(TEXT("AbstractSkeletonBindingEditor_PropertiesTab"));
		static const FName SetsId(TEXT("AbstractSkeletonBindingEditor_SetsTab"));
		static const FName LabelsId(TEXT("AbstractSkeletonBindingEditor_LabelsTab"));
	}

	void FAbstractSkeletonEditor::InitEditor(const TArray<UObject*>& InObjects, const TSharedPtr<IToolkitHost> InToolkitHost)
	{
		TObjectPtr<UObject> InObject = InObjects[0];

		bool bDefaultShowSetsTab = true;
		FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

		if (InObject->IsA<UAbstractSkeletonSetBinding>())
		{
			SetBinding = CastChecked<UAbstractSkeletonSetBinding>(InObject);

			PersonaToolkit = PersonaModule.CreatePersonaToolkit(
				SetBinding.Get(),
				FPersonaToolkitArgs(),
				SetBinding->GetSkeleton());

			// Try find a LabelBinding to open by default
			if (SetBinding->GetSkeleton())
			{
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
				IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

				FARFilter Filter;
				Filter.ClassPaths.Add(UAbstractSkeletonLabelBinding::StaticClass()->GetClassPathName());
				Filter.TagsAndValues.Add(TEXT("Skeleton"), FAssetData(SetBinding->GetSkeleton()).GetExportTextName());

				TArray<FAssetData> AssetList;
				AssetRegistry.GetAssets(Filter, AssetList);

				if (!AssetList.IsEmpty())
				{
					LabelBinding = CastChecked<UAbstractSkeletonLabelBinding>(AssetList[0].GetAsset());
				}
			}
		}
		else if (InObject->IsA<UAbstractSkeletonLabelBinding>())
		{
			LabelBinding = CastChecked<UAbstractSkeletonLabelBinding>(InObject);
			bDefaultShowSetsTab = false;

			PersonaToolkit = PersonaModule.CreatePersonaToolkit(
				LabelBinding.Get(),
				FPersonaToolkitArgs(),
				LabelBinding->GetSkeleton().Get());

			// Try find a SetBinding to open by default
			if (LabelBinding->GetSkeleton())
			{
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
				IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

				FARFilter Filter;
				Filter.ClassPaths.Add(UAbstractSkeletonSetBinding::StaticClass()->GetClassPathName());
				Filter.TagsAndValues.Add(TEXT("Skeleton"), FAssetData(LabelBinding->GetSkeleton()).GetExportTextName());

				TArray<FAssetData> AssetList;
				AssetRegistry.GetAssets(Filter, AssetList);

				if (!AssetList.IsEmpty())
				{
					SetBinding = CastChecked<UAbstractSkeletonSetBinding>(AssetList[0].GetAsset());
				}
			}
		}

		const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("AbstractSkeletonBindingEditorToolkit")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.3f)
					->SetHideTabWell(true)
					->AddTab(Tabs::ViewportId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.4f)
					->AddTab(Tabs::SetsId, ETabState::OpenedTab)
					->AddTab(Tabs::LabelsId, ETabState::OpenedTab)
					->SetForegroundTab(bDefaultShowSetsTab ? Tabs::SetsId : Tabs::LabelsId)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.3f)
					->AddTab(Tabs::DetailsId, ETabState::OpenedTab)
				)
			);


		FAssetEditorToolkit::InitAssetEditor(EToolkitMode::Standalone, InToolkitHost, "AbstractSkeletonEditor", Layout, true, true, InObjects);

		ExtendToolbar();
		RegenerateMenusAndToolbars();

		if (InObject->IsA<UAbstractSkeletonSetBinding>())
		{
			if (SetsTab.IsValid())
			{
				SetsTab->ActivateInParent(ETabActivationCause::SetDirectly);
			}
			else
			{
				GetAssociatedTabManager()->TryInvokeTab(Tabs::SetsId);
			}		
		}
		else
		{
			if (LabelsTab.IsValid())
			{
				LabelsTab->ActivateInParent(ETabActivationCause::SetDirectly);
			}
			else
			{
				GetAssociatedTabManager()->TryInvokeTab(Tabs::LabelsId);
			}
		}
	}

	void FAbstractSkeletonEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		CreateEditorModeManager();

		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("AbstractSkeletonEditorToolkit", "Abstract Skeleton Editor"));

		// Sets tab
		InTabManager->RegisterTabSpawner(Tabs::SetsId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&)
			{
				TSharedRef<SDockTab> Tab = SNew(SDockTab);
				SetsTab = Tab;

				TSharedRef<SWidget> TabWidget = SNew(SSetsTab)
					.ParentTab(Tab)
					.SetBinding(SetBinding);

				Tab->SetContent(TabWidget);

				return Tab;
			}))
			.SetDisplayName(LOCTEXT("SetsTab_DisplayName", "Sets"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), ("ClassIcon.GroupActor")));

		// Labels tab
		InTabManager->RegisterTabSpawner(Tabs::LabelsId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&)
			{
				TSharedRef<SDockTab> Tab = SNew(SDockTab);
				LabelsTab = Tab;

				TSharedRef<SWidget> TabWidget = SNew(SLabelsTab)
					.ParentTab(Tab)
					.LabelBinding(LabelBinding)
					.AbstractSkeletonEditor(SharedThis(this));

				Tab->SetContent(TabWidget);

				return Tab;
			}))
			.SetDisplayName(LOCTEXT("LabelsTab_DisplayName", "Labels"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), ("Icons.Tag")));

		// Viewport tab
		InTabManager->RegisterTabSpawner(Tabs::ViewportId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&)
			{
				FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

				FPersonaViewportArgs PersonaViewportArgs(PersonaToolkit->GetPreviewScene());

				TSharedRef<FWorkflowTabFactory> ViewportTabFactory = PersonaModule.CreatePersonaViewportTabFactory(SharedThis(this), PersonaViewportArgs);

				FWorkflowTabSpawnInfo SpawnInfo;

				return SNew(SDockTab)
				[
					ViewportTabFactory->CreateTabBody(SpawnInfo)
				];
			}))
			.SetDisplayName(LOCTEXT("ViewportTab_DisplayName", "Viewport"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}

	void FAbstractSkeletonEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
		InTabManager->UnregisterTabSpawner(Tabs::SetsId);
		InTabManager->UnregisterTabSpawner(Tabs::LabelsId);
		InTabManager->UnregisterTabSpawner(Tabs::ViewportId);
		InTabManager->UnregisterTabSpawner(Tabs::DetailsId);
	}
	
	void FAbstractSkeletonEditor::ExtendToolbar()
	{
		TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
		AddToolbarExtender(ToolbarExtender);

		ToolbarExtender->AddToolBarExtension(
			"Asset",
			EExtensionHook::After,
			GetToolkitCommands(),
			FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ToolbarBuilder)
				{
					FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

					if (SetBinding.IsValid())
					{
						TSharedRef<class IAssetFamily> AssetFamily = PersonaModule.CreatePersonaAssetFamily(SetBinding->GetSkeleton().Get());
						TSharedRef<SWidget> Widget = PersonaModule.CreateAssetFamilyShortcutWidget(SharedThis(this), AssetFamily);
						AddToolbarWidget(Widget);
					}
					else if (LabelBinding.IsValid())
					{
						TSharedRef<class IAssetFamily> AssetFamily = PersonaModule.CreatePersonaAssetFamily(LabelBinding->GetSkeleton().Get());
						TSharedRef<SWidget> Widget = PersonaModule.CreateAssetFamilyShortcutWidget(SharedThis(this), AssetFamily);
						AddToolbarWidget(Widget);
					}
				}
			));
	}

	void FAbstractSkeletonEditor::CreateEditorModeManager()
	{
		FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
		TSharedPtr<IPersonaEditorModeManager> PersonaModeManager = MakeShareable(PersonaModule.CreatePersonaEditorModeManager());

		EditorModeManager = PersonaModeManager;
	}

	FText FAbstractSkeletonEditor::GetToolkitName() const
	{
		return LOCTEXT("ToolkitName", "Abstract Skeleton Editor");
	}

	const FSlateBrush* FAbstractSkeletonEditor::GetDefaultTabIcon() const
	{
		return FAppStyle::Get().GetBrush("SkeletonTree.Bone");
	}

	FLinearColor FAbstractSkeletonEditor::GetDefaultTabColor() const
	{
		return FLinearColor::White;
	}

	FText FAbstractSkeletonEditor::GetToolkitToolTipText() const
	{
		return LOCTEXT("ToolkitTooltip", "Editor for manipulating a Skeleton's Set Bindings and Label Bindings");
	}

	TSharedPtr<IPersonaToolkit> FAbstractSkeletonEditor::GetPersonaToolkit() const
	{
		return PersonaToolkit;
	}
}

#undef LOCTEXT_NAMESPACE