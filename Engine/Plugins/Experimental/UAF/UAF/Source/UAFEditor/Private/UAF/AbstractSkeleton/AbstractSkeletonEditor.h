// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AbstractSkeleton/IAbstractSkeletonEditor.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

class IPersonaToolkit;
class UAbstractSkeletonLabelBinding;
class UAbstractSkeletonSetBinding;

namespace UE::UAF::Editor
{
	class FAbstractSkeletonEditor : public FWorkflowCentricApplication, public IAbstractSkeletonEditor
	{
	public:
		void InitEditor(const TArray<UObject*>& InObjects, const TSharedPtr<IToolkitHost> InToolkitHost);

		// Begin FWorkflowCentricApplication
		virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
		virtual void CreateEditorModeManager() override;

		virtual FText GetToolkitName() const override;
		virtual const FSlateBrush* GetDefaultTabIcon() const override;
		virtual FLinearColor GetDefaultTabColor() const override;
		virtual FText GetToolkitToolTipText() const override;
		virtual FName GetToolkitFName() const override { return "AbstractSkeletonEditor"; }
		virtual FText GetBaseToolkitName() const override { return INVTEXT("Abstract Skeleton Editor"); }
		virtual FString GetWorldCentricTabPrefix() const override { return "Abstract Skeleton Editor "; }
		virtual FLinearColor GetWorldCentricTabColorScale() const override { return {}; }
		// End FWorkflowCentricApplication

		// Begin IAbstractSkeletonEditor
		virtual TSharedPtr<IPersonaToolkit> GetPersonaToolkit() const override;
		// End IAbstractSkeletonEditor

	private:
		void ExtendToolbar();

		TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding;

		TWeakObjectPtr<UAbstractSkeletonLabelBinding> LabelBinding;

		TSharedPtr<IPersonaToolkit> PersonaToolkit;

		TSharedPtr<SDockTab> SetsTab;

		TSharedPtr<SDockTab> LabelsTab;
	};

}
