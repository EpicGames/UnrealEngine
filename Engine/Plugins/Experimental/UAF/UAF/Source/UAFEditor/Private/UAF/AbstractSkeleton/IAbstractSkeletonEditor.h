// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IPersonaToolkit;

namespace UE::UAF::Editor
{

	class IAbstractSkeletonEditor
	{
	public:
		virtual ~IAbstractSkeletonEditor() = default;
		virtual TSharedPtr<IPersonaToolkit> GetPersonaToolkit() const = 0;
	};

}