// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/Text/SRichTextBlock.h"

class UAbstractSkeletonLabelBinding;

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::FBoneDragDropOp"

namespace UE::UAF::Editor
{
	class FBoneDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FBoneDragDropOp, FDecoratedDragDropOp)

		static TSharedRef<FBoneDragDropOp> New(const TWeakObjectPtr<UAbstractSkeletonLabelBinding> InLabelBinding, const FName InBoneName)
		{
			TSharedRef<FBoneDragDropOp> Operation = MakeShared<FBoneDragDropOp>();
			Operation->LabelBinding = InLabelBinding;
			Operation->BoneName = InBoneName;

			Operation->Construct();

			return Operation;
		}

		TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(FMargin(8.0f, 4.0f))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(2.0f))
					[
						SNew(SImage)
							.Visibility_Lambda([this]()
								{
									return HoveredLabel != NAME_None ? EVisibility::Collapsed : EVisibility::Visible;
								})
							.Image(FAppStyle::Get().GetBrush("SkeletonTree.Bone"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(2.0f))
					[
						SNew(STextBlock)
							.Visibility_Lambda([this]()
								{
									return HoveredLabel != NAME_None ? EVisibility::Collapsed : EVisibility::Visible;
								})
							.Text(FText::FromName(BoneName))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(2.0f))
					[
						SNew(SImage)
							.Visibility_Lambda([this]()
								{
									return HoveredLabel == NAME_None ? EVisibility::Collapsed : EVisibility::Visible;
								})
							.Image(FAppStyle::Get().GetBrush("Icons.Link"))
							.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.AccentBlue"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(2.0f))
					[
						SNew(SRichTextBlock)
							.DecoratorStyleSet(&FAppStyle::Get())
							.Visibility_Lambda([this]()
								{
									return HoveredLabel == NAME_None ? EVisibility::Collapsed : EVisibility::Visible;
								})
							.Text_Lambda([this]()
								{
									return FText::Format(LOCTEXT("BindBoneToLabelTooltip", "Bind <RichTextBlock.Bold>{0}</> to <RichTextBlock.Bold>{1}</>"), FText::FromName(BoneName), FText::FromName(HoveredLabel));
								})
					]
					
				];
		}

		void SetHoveredLabel(const FName InLabel)
		{
			HoveredLabel = InLabel;
		}

		TWeakObjectPtr<UAbstractSkeletonLabelBinding> LabelBinding;
		FName BoneName;
		FName HoveredLabel;
	};
}

#undef LOCTEXT_NAMESPACE