// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UAbstractSkeletonLabelCollection;
class UAbstractSkeletonLabelBinding;

namespace UE::UAF::Editor
{

	class ILabelBindingWidget
	{
	public:
		virtual ~ILabelBindingWidget() = default;

		virtual void ScrollToLabel(const TObjectPtr<const UAbstractSkeletonLabelCollection> InLabelCollection, const FName InLabel) = 0;

		virtual void RepopulateTreeData() = 0;

		virtual TWeakObjectPtr<UAbstractSkeletonLabelBinding> GetLabelBinding() const = 0;
	};

}