// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::UAF::Editor
{

	class ILabelSkeletonTreeWidget
	{
	public:
		virtual ~ILabelSkeletonTreeWidget() = default;
		
		virtual void ScrollToBone(const FName InBoneName) = 0;

		virtual void RepopulateTreeData() = 0;
	};

}