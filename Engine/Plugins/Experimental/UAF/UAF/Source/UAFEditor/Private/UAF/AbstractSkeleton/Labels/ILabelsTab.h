// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE::UAF::Editor
{
	class IAbstractSkeletonEditor;
	class ILabelBindingWidget;
	class ILabelSkeletonTreeWidget;

	class ILabelsTab
	{
	public:
		virtual ~ILabelsTab() = default;
		
		virtual TSharedPtr<ILabelBindingWidget> GetLabelBindingWidget() const = 0;

		virtual TSharedPtr<ILabelSkeletonTreeWidget> GetLabelSkeletonTreeWidget() const = 0;

		virtual void RepopulateLabelData() = 0;

		virtual TWeakPtr<IAbstractSkeletonEditor> GetAbstractSkeletonEditor() const = 0;
	};

}