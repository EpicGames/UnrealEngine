// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Labels/LabelBindingAssetDefinition.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonEditor.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonLabelBinding.h"

#define LOCTEXT_NAMESPACE "UE::UAF::UAssetDefinition_AbstractSkeletonLabelBinding"

FText UAssetDefinition_AbstractSkeletonLabelBinding::GetAssetDisplayName() const
{
	return LOCTEXT("LabelBinding", "UAF Label Binding");
}

FLinearColor UAssetDefinition_AbstractSkeletonLabelBinding::GetAssetColor() const
{
	return FLinearColor(FColor(21, 119, 53));
}

TSoftClassPtr<UObject> UAssetDefinition_AbstractSkeletonLabelBinding::GetAssetClass() const
{
	return UAbstractSkeletonLabelBinding::StaticClass();
}

EAssetCommandResult UAssetDefinition_AbstractSkeletonLabelBinding::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		TArray<UObject*> Assets = OpenArgs.LoadObjects<UObject>();

		MakeShared<UE::UAF::Editor::FAbstractSkeletonEditor>()->InitEditor(Assets, OpenArgs.ToolkitHost);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AbstractSkeletonLabelBinding::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")) };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
