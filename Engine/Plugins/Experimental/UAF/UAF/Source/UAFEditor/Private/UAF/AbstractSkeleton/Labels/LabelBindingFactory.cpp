// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Labels/LabelBindingFactory.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonLabelBinding.h"
#include "UObject/Package.h"

UAbstractSkeletonLabelBindingFactory::UAbstractSkeletonLabelBindingFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAbstractSkeletonLabelBinding::StaticClass();
}

bool UAbstractSkeletonLabelBindingFactory::ConfigureProperties()
{
	return true;
}

UObject* UAbstractSkeletonLabelBindingFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	EObjectFlags FlagsToUse = Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted;
	if (InParent == GetTransientPackage())
	{
		FlagsToUse &= ~RF_Standalone;
	}

	TObjectPtr<UAbstractSkeletonLabelBinding> NewLabelBinding = NewObject<UAbstractSkeletonLabelBinding>(InParent, Class, Name, FlagsToUse);

	return NewLabelBinding;
}
