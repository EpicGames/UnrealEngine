// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Labels/LabelCollectionAssetDefinition.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonLabelCollection.h"
#include "UAF/AbstractSkeleton/Labels/LabelCollectionEditor.h"

#define LOCTEXT_NAMESPACE "UE::UAF"

FText UAssetDefinition_AbstractSkeletonLabelCollection::GetAssetDisplayName() const
{
	return LOCTEXT("LabelCollection", "UAF Label Collection");
}

FLinearColor UAssetDefinition_AbstractSkeletonLabelCollection::GetAssetColor() const
{
	return FLinearColor(FColor(73, 148, 98));
}

TSoftClassPtr<UObject> UAssetDefinition_AbstractSkeletonLabelCollection::GetAssetClass() const
{
	return UAbstractSkeletonLabelCollection::StaticClass();
}

EAssetCommandResult UAssetDefinition_AbstractSkeletonLabelCollection::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		TArray<UObject*> Assets = OpenArgs.LoadObjects<UObject>();
		
		MakeShared<UE::UAF::Editor::FLabelCollectionEditorToolkit>()->InitEditor(Assets);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AbstractSkeletonLabelCollection::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")) };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
