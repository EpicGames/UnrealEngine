// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Labels/LabelCollectionEditor.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonLabelCollection.h"
#include "UAF/AbstractSkeleton/Labels/SLabelCollection.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UE::UAF::FLabelCollectionEditorToolkit"

namespace UE::UAF::Editor
{
	static FName LabelsTabId("AbstractSkeletonLabelCollection_LabelsTab");

	void FLabelCollectionEditorToolkit::InitEditor(const TArray<UObject*>& InObjects)
	{
		LabelCollection = CastChecked<UAbstractSkeletonLabelCollection>(InObjects[0]);

		const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("AbstractSkeletonLabelCollectionEditorToolkit")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(LabelsTabId, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
			);

		FAssetEditorToolkit::InitAssetEditor(EToolkitMode::Standalone, {}, "AbstractSkeletonLabelCollectionEditorToolkit", Layout, true, true, InObjects);
	}

	void FLabelCollectionEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("AbstractSkeletonLabelsCollectionEditor", "Abstract Skeleton Labels Editor"));

		InTabManager->RegisterTabSpawner(LabelsTabId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&)
			{
				return SNew(SDockTab)
				[
					SNew(SLabelCollection, LabelCollection)
				];
			}))
			.SetDisplayName(LOCTEXT("LabelsTab_DisplayName", "Labels"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Tag"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}

	void FLabelCollectionEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
		InTabManager->UnregisterTabSpawner(LabelsTabId);
	}
}

#undef LOCTEXT_NAMESPACE