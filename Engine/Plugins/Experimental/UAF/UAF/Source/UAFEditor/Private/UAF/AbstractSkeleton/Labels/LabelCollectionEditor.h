// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"

class UAbstractSkeletonLabelCollection;

namespace UE::UAF::Editor
{
	class FLabelCollectionEditorToolkit : public FAssetEditorToolkit
	{
	public:
		void InitEditor(const TArray<UObject*>& InObjects);

		// Begin FAssetEditorToolkit
		void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
		void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

		FName GetToolkitFName() const override { return "AbstractSkeletonLabelCollectionEditor"; }
		FText GetBaseToolkitName() const override { return INVTEXT("Abstract Skeleton Label Collection Editor"); }
		FString GetWorldCentricTabPrefix() const override { return "Abstract Skeleton Label Collection Editor "; }
		FLinearColor GetWorldCentricTabColorScale() const override { return {}; }
		// End FAssetEditorToolkit

	private:
		TSharedRef<SDockTab> SpawnPropertiesTab(const FSpawnTabArgs& Args);

		TWeakObjectPtr<UAbstractSkeletonLabelCollection> LabelCollection;
	};
}