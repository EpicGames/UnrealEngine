// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Labels/LabelCollectionFactory.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonLabelCollection.h"
#include "UObject/Package.h"

UAbstractSkeletonLabelCollectionFactory::UAbstractSkeletonLabelCollectionFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAbstractSkeletonLabelCollection::StaticClass();
}

bool UAbstractSkeletonLabelCollectionFactory::ConfigureProperties()
{
	return true;
}

UObject* UAbstractSkeletonLabelCollectionFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	EObjectFlags FlagsToUse = Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted;
	if (InParent == GetTransientPackage())
	{
		FlagsToUse &= ~RF_Standalone;
	}

	TObjectPtr<UAbstractSkeletonLabelCollection> NewLabelCollection = NewObject<UAbstractSkeletonLabelCollection>(InParent, Class, Name, FlagsToUse);

	return NewLabelCollection;
}
