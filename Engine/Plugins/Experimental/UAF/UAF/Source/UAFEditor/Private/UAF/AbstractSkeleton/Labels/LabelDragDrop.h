// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

class UAbstractSkeletonLabelCollection;

namespace UE::UAF::Editor
{
	class FLabelDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FLabelDragDropOp, FDecoratedDragDropOp)

		static TSharedRef<FLabelDragDropOp> New(const TWeakObjectPtr<const UAbstractSkeletonLabelCollection> InLabelCollection, const FName InLabel)
		{
			TSharedRef<FLabelDragDropOp> Operation = MakeShared<FLabelDragDropOp>();
			Operation->LabelCollection = InLabelCollection;
			Operation->Label = InLabel;

			Operation->Construct();

			return Operation;
		}

		TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SNew(STextBlock)
						.Text(FText::FromName(Label))
				];
		}

		TWeakObjectPtr<const UAbstractSkeletonLabelCollection> LabelCollection;
		FName Label;
	};
}