// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Labels/SLabelBinding.h"

#include "Algo/Count.h"
#include "AssetThumbnail.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonLabelBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonLabelCollection.h"
#include "UAF/AbstractSkeleton/Labels/BoneDragDrop.h"
#include "UAF/AbstractSkeleton/Labels/LabelDragDrop.h"
#include "UAF/AbstractSkeleton/Labels/ILabelsTab.h"
#include "UAF/AbstractSkeleton/Labels/ILabelBinding.h"
#include "UAF/AbstractSkeleton/Labels/ILabelSkeletonTree.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SLabelBinding"

namespace UE::UAF::Editor
{
	void SLabelBinding::Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonLabelBinding> InLabelBinding, TSharedPtr<ILabelsTab> InLabelsTabInterface)
		{
			LabelBinding = InLabelBinding;
			OnTreeRefreshed = InArgs._OnTreeRefreshed;
			LabelsTabInterface = InLabelsTabInterface;

			ChildSlot
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(1.0f, 1.0f))
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(FMargin(1.0f, 1.0f))
							[
								SNew(SPositiveActionButton)
									.OnGetMenuContent(this, &SLabelBinding::CreateImportCollectionWidget)
									.Icon(FAppStyle::Get().GetBrush("Icons.Import"))
									.Text(LOCTEXT("AddButton_Text", "Import Collection"))
							]
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.Padding(FMargin(1.0f, 1.0f))
							[
								SAssignNew(SearchBox, SSearchBox)
									.SelectAllTextWhenFocused(true)
									.OnTextChanged_Lambda([](const FText& InText) { /* TODO: Implement */ })
									.HintText(LOCTEXT("SearchBox_Hint", "Search Labels..."))
							]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(FMargin(1.0f, 1.0f))
					[
						SAssignNew(TreeView, STreeView<FTreeItemPtr>)
							.SelectionMode(ESelectionMode::Single)
							.HighlightParentNodesForSelection(true)
							.TreeItemsSource(&RootItems)
							.OnGenerateRow(this, &SLabelBinding::TreeView_OnGenerateRow)
							.OnGetChildren(this, &SLabelBinding::TreeView_OnGetChildren)
							.OnContextMenuOpening(this, &SLabelBinding::TreeView_OnContextMenuOpening)
					]
			];

			RepopulateTreeData();
			ExpandAllTreeItems();
			BindCommands();
		}

		FReply SLabelBinding::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
		{
			if (CommandList->ProcessCommandBindings(InKeyEvent))
			{
				return FReply::Handled();
			}

			return FReply::Unhandled();
		}

		TSharedRef<ITableRow> SLabelBinding::TreeView_OnGenerateRow(FTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
		{
			return InItem->GenerateRow(OwnerTable);
		}

		void SLabelBinding::TreeView_OnGetChildren(FTreeItemPtr InItem, TArray<FTreeItemPtr>& OutChildren)
		{
			InItem->GetChildren(OutChildren);
		}

		TSharedPtr<SWidget> SLabelBinding::TreeView_OnContextMenuOpening()
		{
			FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

			TArray<FTreeItemPtr> Selection = TreeView->GetSelectedItems();
			if (Selection.Num() > 0)
			{
				FTreeItemPtr Selected = Selection[0]; // Single selection only

				const bool bShowMenu = Selected->GenerateContextMenu(*this, MenuBuilder);

				if (bShowMenu)
				{
					return MenuBuilder.MakeWidget();
				}
			}

			return SNullWidget::NullWidget;
		}

		void SLabelBinding::PostUndo(bool bSuccess)
		{
			RepopulateTreeData();
		}

		void SLabelBinding::PostRedo(bool bSuccess)
		{
			RepopulateTreeData();
		}

		void SLabelBinding::ScrollToLabel(const TObjectPtr<const UAbstractSkeletonLabelCollection> InLabelCollection, const FName InLabel)
		{
			for (const FTreeItemPtr& RootItem : RootItems)
			{
				const TSharedPtr<FTreeItem_LabelCollection> LabelItem = StaticCastSharedPtr<FTreeItem_LabelCollection>(RootItem);
				// All root items are label collection items

				if (LabelItem->LabelCollection == InLabelCollection)
				{
					for (const TSharedPtr<FTreeItem_LabelBinding>& BindingItem : LabelItem->LabelBindings)
					{
						if (BindingItem->Label == InLabel)
						{
							TreeView->RequestScrollIntoView(BindingItem);
							TreeView->SetSelection(BindingItem);
							return;
						}
					}
				}
				
			}
		}

		void SLabelBinding::RepopulateTreeData()
		{
			if (!bRepopulating)
			{
				TGuardValue<bool> RecursionGuard(bRepopulating, true);

				// Rebuild tree data
				RootItems.Reset();

				if (LabelBinding.IsValid())
				{
					for (const TObjectPtr<const UAbstractSkeletonLabelCollection>& LabelCollection : LabelBinding->GetLabelCollections())
					{
						TSharedPtr<FTreeItem_LabelCollection> LabelCollectionItem = MakeShared<FTreeItem_LabelCollection>();
						LabelCollectionItem->LabelCollection = LabelCollection;
						LabelCollectionItem->LabelsTabInterface = LabelsTabInterface;

						for (const FName Label : LabelCollection->GetLabels())
						{
							TSharedPtr<FTreeItem_LabelBinding> LabelBindingItem = MakeShared<FTreeItem_LabelBinding>();
							LabelBindingItem->Label = Label;
							LabelBindingItem->BoneName = LabelBinding->GetLabelBinding(LabelCollection, Label);
							LabelBindingItem->LabelCollection = LabelCollection;
							LabelBindingItem->LabelsTabInterface = LabelsTabInterface;

							LabelCollectionItem->LabelBindings.Add(LabelBindingItem);
						}

						RootItems.Add(LabelCollectionItem);
					}
				}

				TreeView->RequestTreeRefresh();

				for (FTreeItemPtr TreeItem : GetAllTreeItems())
				{
					TreeView->SetItemExpansion(TreeItem, true);
				}

				OnTreeRefreshed.ExecuteIfBound();
			}
		}

		TWeakObjectPtr<UAbstractSkeletonLabelBinding> SLabelBinding::GetLabelBinding() const
		{
			return LabelBinding;
		}

		void SLabelBinding::ExpandAllTreeItems()
		{
			for (FTreeItemPtr TreeItem : GetAllTreeItems())
			{
				TreeView->SetItemExpansion(TreeItem, true);
			}
		}

		void SLabelBinding::BindCommands()
		{
			CommandList = MakeShareable(new FUICommandList);

			CommandList->MapAction(
				FGenericCommands::Get().Delete,
				FExecuteAction::CreateSP(this, &SLabelBinding::OnRemoveSelected)
			);
		}

		TArray<TSharedPtr<SLabelBinding::ITreeItem>> SLabelBinding::GetAllTreeItems()
		{
			TArray<FTreeItemPtr> AllItems;
			AllItems.Append(RootItems);

			for (int32 Index = 0; Index < AllItems.Num(); ++Index)
			{
				AllItems[Index]->GetChildren(AllItems);
			}

			return AllItems;
		}

		void SLabelBinding::OnImportLabelCollection()
		{
		
		}

		TSharedRef<SWidget> SLabelBinding::CreateImportCollectionWidget()
		{
			FMenuBuilder MenuBuilder(true, nullptr);

			const bool bAllowClear = false;
			const bool bAllowCopyPaste = false;

			TArray<const UClass*> AllowedClasses;
			AllowedClasses.Add(UAbstractSkeletonLabelCollection::StaticClass());

			TSharedRef<SWidget> AssetPickerWidget = PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
				FAssetData(),
				bAllowClear,
				bAllowCopyPaste,
				AllowedClasses,
				PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClasses),
				FOnShouldFilterAsset::CreateLambda([this](const FAssetData& InAssetData)
				{
					for (FTreeItemPtr TreeItem : RootItems)
					{
						const TSharedPtr<FTreeItem_LabelCollection> LabelCollectionItem = StaticCastSharedPtr<FTreeItem_LabelCollection>(TreeItem);
						// All root items are label collection items

						if (FAssetData(LabelCollectionItem->LabelCollection.Get()) == InAssetData)
						{
							return true;
						}
					}

					return false;
				}),
				FOnAssetSelected::CreateLambda([this](const FAssetData& InAssetData)
				{
					const FScopedTransaction Transaction(LOCTEXT("AddLabelCollection", "Add Label Collection"));
					LabelBinding->Modify();

					TObjectPtr<UAbstractSkeletonLabelCollection> SetCollection = CastChecked<UAbstractSkeletonLabelCollection>(InAssetData.GetAsset());

					const bool bSuccess = LabelBinding->AddLabelCollection(SetCollection);
					check(bSuccess);

					LabelsTabInterface->RepopulateLabelData();
				}),
				FSimpleDelegate::CreateLambda([]()
				{
					FSlateApplication::Get().DismissAllMenus();
				}));

			MenuBuilder.AddWidget(AssetPickerWidget, FText::GetEmpty(), /*bNoIndent=*/ true);

			return MenuBuilder.MakeWidget();
		}

		void SLabelBinding::OnRemoveSelected()
		{
			TArray<FTreeItemPtr> Selection = TreeView->GetSelectedItems();
			
			if (ensure(Selection.Num() == 1))
			{
				FTreeItemPtr Selected = Selection[0];
				Selected->OnRemove();
			}
		}

		// FTreeItem_LabelCollection

		void SLabelBinding::FTreeItem_LabelCollection::GetChildren(TArray<FTreeItemPtr>& OutChildren)
		{
			OutChildren.Append(LabelBindings);
		}

		TSharedRef<ITableRow> SLabelBinding::FTreeItem_LabelCollection::GenerateRow(const TSharedRef<STableViewBase>& OwnerTable)
		{
			const int32 NumBound = Algo::CountIf(LabelBindings, [](const TSharedPtr<FTreeItem_LabelBinding> InBinding) -> bool
				{
					return InBinding->BoneName != NAME_None;
				});
			const int32 NumUnbound = LabelBindings.Num() - NumBound;

			return SNew(STableRow<FTreeItemPtr>, OwnerTable)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(0.5)
						.VAlign(VAlign_Center)
						.Padding(FMargin(4.0f))
						[
							SNew(SBox)
								.HeightOverride(8.0f)
								[
									SNew(SProgressBar)
										.FillColorAndOpacity(NumUnbound == 0 ? FAppStyle::Get().GetSlateColor("Colors.AccentGreen") : FAppStyle::Get().GetSlateColor("Colors.AccentBlue"))
										.FillImage(FAppStyle::Get().GetBrush("WhiteBrush"))
										.Percent(static_cast<float>(NumBound) / LabelBindings.Num())
								]
						]
						+ SHorizontalBox::Slot()
						.FillWidth(0.5)
						.Padding(FMargin(4.0f))
						[
							SNew(SObjectPropertyEntryBox)
								.AllowedClass(UAbstractSkeletonLabelCollection::StaticClass())
								.ObjectPath_Lambda([&]()
								{
									return LabelCollection->GetPathName();
								})
								.OnObjectChanged_Lambda([](const FAssetData& AssetData)
								{
									// TODO: Implement
								})
								.AllowClear(true)
								.DisplayUseSelected(true)
								.DisplayBrowse(true)
								.DisplayThumbnail(true)
								.ThumbnailPool(MakeShared<FAssetThumbnailPool>(1))
						]
				];
		}

		bool SLabelBinding::FTreeItem_LabelCollection::GenerateContextMenu(SLabelBinding& LabelBindingWidget, FMenuBuilder& MenuBuilder)
		{
			const bool bHasBoundChildren = LabelBindings.ContainsByPredicate([](const TSharedPtr<FTreeItem_LabelBinding> InBinding)
			{
				return InBinding->BoneName != NAME_None;
			});

			if (bHasBoundChildren)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("UnbindAll_Label", "Unbind All"),
					LOCTEXT("UnbindAll_Tooltip", "Unbind all bones and labels"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Unlink"),
					FUIAction(
						FExecuteAction::CreateLambda([this]()
							{
								const FScopedTransaction Transaction(LOCTEXT("UnbindAll", "Unbind All"));
								LabelsTabInterface->GetLabelBindingWidget()->GetLabelBinding()->Modify();

								for (const TSharedPtr<FTreeItem_LabelBinding>& Binding : LabelBindings)
								{
									if (Binding->BoneName != NAME_None)
									{
										const bool bSuccess = LabelsTabInterface->GetLabelBindingWidget()->GetLabelBinding()->UnbindBoneFromLabel(Binding->LabelCollection.Get(), Binding->BoneName, Binding->Label);
										check(bSuccess);
									}
								}

								LabelsTabInterface->GetLabelBindingWidget()->RepopulateTreeData();
								LabelsTabInterface->GetLabelSkeletonTreeWidget()->RepopulateTreeData();
							}
						)
					));
			}

			MenuBuilder.AddSeparator();

			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, LOCTEXT("Remove", "Remove"));

			return true;
		}

		void SLabelBinding::FTreeItem_LabelCollection::OnRemove()
		{
			TSharedPtr<ILabelBindingWidget> LabelBindingInterface = LabelsTabInterface->GetLabelBindingWidget();

			const FScopedTransaction Transaction(LOCTEXT("RemoveLabelCollection", "Remove Label Collection"));
			LabelBindingInterface->GetLabelBinding()->Modify();

			const bool bSuccess = LabelBindingInterface->GetLabelBinding()->RemoveLabelCollection(LabelCollection.Get());
			check(bSuccess);

			LabelBindingInterface->RepopulateTreeData();
			LabelsTabInterface->GetLabelSkeletonTreeWidget()->RepopulateTreeData();
		}

		// FTreeItem_LabelBinding

		void SLabelBinding::FTreeItem_LabelBinding::GetChildren(TArray<FTreeItemPtr>& OutChildren)
		{
			// Do nothing
		}

		TSharedRef<ITableRow> SLabelBinding::FTreeItem_LabelBinding::GenerateRow(const TSharedRef<STableViewBase>& OwnerTable)
		{
			TSharedPtr<SWidget> BoneWidget;

			if (BoneName == NAME_None)
			{
				BoneWidget = SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(FText::FromName(NAME_None));
			}
			else
			{
				BoneWidget = SNew(SHyperlink)
					.Text(FText::FromName(BoneName))
					.OnNavigate_Lambda([this]()
						{
							TSharedPtr<ILabelSkeletonTreeWidget> LabelSkeletonTreeWidget = LabelsTabInterface->GetLabelSkeletonTreeWidget();
							LabelSkeletonTreeWidget->ScrollToBone(BoneName);
						});
			}

			return SNew(STableRow<FTreeItemPtr>, OwnerTable)
				.OnDragDetected_Lambda([this](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
					{
						if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
						{
							const TSharedRef<FLabelDragDropOp> DragDropOp = FLabelDragDropOp::New(LabelCollection, Label);
							return FReply::Handled().BeginDragDrop(DragDropOp);
						}

						return FReply::Unhandled();
					})
				.OnDragEnter_Lambda([this](const FDragDropEvent& DragDropEvent)
				{
					const TSharedPtr<FBoneDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FBoneDragDropOp>();
					if (DragDropOp.IsValid())
					{
						DragDropOp->SetHoveredLabel(Label);
					}
				})
				.OnDragLeave_Lambda([this](const FDragDropEvent& DragDropEvent)
				{
					const TSharedPtr<FBoneDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FBoneDragDropOp>();
					if (DragDropOp.IsValid())
					{
						DragDropOp->SetHoveredLabel(NAME_None);
					}
				})
				.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FTreeItemPtr TargetTreeItem) -> TOptional<EItemDropZone>
					{
						TOptional<EItemDropZone> ReturnedDropZone;

						const TSharedPtr<FBoneDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FBoneDragDropOp>();
						if (DragDropOp.IsValid())
						{
							ReturnedDropZone = EItemDropZone::OntoItem;
						}

						return ReturnedDropZone;
					})
				.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FTreeItemPtr TargetTreeItem) -> FReply
					{
						const TSharedPtr<FBoneDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FBoneDragDropOp>();

						if (DragDropOp.IsValid())
						{
							const FScopedTransaction Transaction(LOCTEXT("BindBoneToLabel", "Bind Bone to Label"));
							DragDropOp->LabelBinding->Modify();

							const bool bSuccess = DragDropOp->LabelBinding->BindBoneToLabel(LabelCollection.Get(), DragDropOp->BoneName, Label);
							check(bSuccess);

							LabelsTabInterface->GetLabelSkeletonTreeWidget()->RepopulateTreeData();
							LabelsTabInterface->GetLabelBindingWidget()->RepopulateTreeData();

							return FReply::Handled();
						}

						return FReply::Unhandled();
					})
				.ShowWires(true)
				.Style(FAppStyle::Get(), "TableView.AlternatingRow")
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(0.5)
						[
							SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(FMargin(4.0f))
								[
									SNew(SImage)
										.ColorAndOpacity(BoneName == NAME_None ? FSlateColor::UseForeground() : FAppStyle::Get().GetSlateColor("Colors.AccentBlue"))
										.Image(FAppStyle::Get().GetBrush("Icons.Tag"))
								]
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(FMargin(4.0f))
								[
									SNew(STextBlock)
										.Text(FText::FromName(Label))
								]
						]
						+ SHorizontalBox::Slot()
						.FillWidth(0.5)
						[
							SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(FMargin(4.0f))
								[
									SNew(SImage)
										.ColorAndOpacity(BoneName == NAME_None ? FSlateColor::UseSubduedForeground() : FAppStyle::Get().GetSlateColor("Colors.AccentBlue"))
										.Image(FAppStyle::Get().GetBrush("SkeletonTree.Bone"))
								]
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(FMargin(4.0f))
								[
									BoneWidget.ToSharedRef()
								]
						]
				];
		}

		bool SLabelBinding::FTreeItem_LabelBinding::GenerateContextMenu(SLabelBinding& InLabelBindingWidget, FMenuBuilder& MenuBuilder)
		{
			if (BoneName != NAME_None)
			{
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("UnbindBone_Label", "Unbind {0}"), FText::FromName(BoneName)),
					LOCTEXT("UnbindBone_Tooltip", "Unbind the selected bone and label"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Unlink"),
					FUIAction(
						FExecuteAction::CreateLambda([this]()
							{
								const bool bSuccess = LabelsTabInterface->GetLabelBindingWidget()->GetLabelBinding()->UnbindBoneFromLabel(LabelCollection.Get(), BoneName, Label);
								check(bSuccess);

								LabelsTabInterface->GetLabelBindingWidget()->RepopulateTreeData();
								LabelsTabInterface->GetLabelSkeletonTreeWidget()->RepopulateTreeData();
							}
						)
					));

				return true;
			}

			return false;
		}

		void SLabelBinding::FTreeItem_LabelBinding::OnRemove()
		{
			if (Label != NAME_None)
			{
				TSharedPtr<ILabelBindingWidget> LabelBindingInterface = LabelsTabInterface->GetLabelBindingWidget();

				const FScopedTransaction Transaction(LOCTEXT("UnbindLabel", "Unbind Label"));
				LabelBindingInterface->GetLabelBinding()->Modify();

				const bool bSuccess = LabelBindingInterface->GetLabelBinding()->UnbindBoneFromLabel(LabelCollection.Get(), BoneName, Label);
				check(bSuccess);

				LabelBindingInterface->RepopulateTreeData();
				LabelsTabInterface->GetLabelSkeletonTreeWidget()->RepopulateTreeData();
			}
		}

}

#undef LOCTEXT_NAMESPACE