// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "UAF/AbstractSkeleton/Labels/ILabelBinding.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class FUICommandList;
class ITableRow;
class SSearchBox;
class STableViewBase;
template<typename> class STreeView;
class UAbstractSkeletonLabelBinding;
class UAbstractSkeletonLabelCollection;

namespace UE::UAF::Editor
{
	class ILabelsTab;

	class SLabelBinding : public SCompoundWidget, public FSelfRegisteringEditorUndoClient, public ILabelBindingWidget
	{
		SLATE_BEGIN_ARGS(SLabelBinding) {}
			SLATE_EVENT(FSimpleDelegate, OnTreeRefreshed)
		SLATE_END_ARGS()

	public:
		struct ITreeItem
		{
			virtual ~ITreeItem() = default;
			virtual void GetChildren(TArray<TSharedPtr<ITreeItem>>& OutChildren) = 0;
			virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) = 0;
			virtual bool GenerateContextMenu(SLabelBinding& SetBindingWidget, FMenuBuilder& Menu) = 0;
			virtual void OnRemove() = 0;
		};

		using FTreeItemPtr = TSharedPtr<ITreeItem>;

		struct FTreeItem_LabelBinding : public ITreeItem
		{
			virtual ~FTreeItem_LabelBinding() override = default;
			virtual void GetChildren(TArray<FTreeItemPtr>& OutChildren) override;
			virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) override;
			virtual bool GenerateContextMenu(SLabelBinding& SetBindingWidget, FMenuBuilder& Menu) override;
			virtual void OnRemove() override;

			FName Label;
			FName BoneName;
			TWeakObjectPtr<const UAbstractSkeletonLabelCollection> LabelCollection;
			TSharedPtr<ILabelsTab> LabelsTabInterface;
		};

		struct FTreeItem_LabelCollection : public ITreeItem
		{
			virtual ~FTreeItem_LabelCollection() override = default;
			virtual void GetChildren(TArray<FTreeItemPtr>& OutChildren) override;
			virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) override;
			virtual bool GenerateContextMenu(SLabelBinding& SetBindingWidget, FMenuBuilder& Menu) override;
			virtual void OnRemove() override;

			TWeakObjectPtr<const UAbstractSkeletonLabelCollection> LabelCollection;
			TArray<TSharedPtr<FTreeItem_LabelBinding>> LabelBindings;
			TSharedPtr<ILabelsTab> LabelsTabInterface;
		};

		void Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonLabelBinding> InSetBinding, TSharedPtr<ILabelsTab> InLabelsTabInterface);

		/** FSelfRegisteringEditorUndoClient interface */
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;

		/** ILabelBindingWidget */
		virtual void ScrollToLabel(const TObjectPtr<const UAbstractSkeletonLabelCollection> InLabelCollection, const FName InLabel) override;
		virtual void RepopulateTreeData() override;
		virtual TWeakObjectPtr<UAbstractSkeletonLabelBinding> GetLabelBinding() const override;

	private:
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

		TSharedRef<ITableRow> TreeView_OnGenerateRow(FTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

		void TreeView_OnGetChildren(FTreeItemPtr InItem, TArray<FTreeItemPtr>& OutChildren);

		TSharedPtr<SWidget> TreeView_OnContextMenuOpening();

		void ExpandAllTreeItems();

		void BindCommands();

		TArray<FTreeItemPtr> GetAllTreeItems();

		void OnImportLabelCollection();

		TSharedRef<SWidget> CreateImportCollectionWidget();

		void OnRemoveSelected();

		TSharedPtr<ILabelsTab> LabelsTabInterface;

	public:
		TWeakObjectPtr<UAbstractSkeletonLabelBinding> LabelBinding;

	private:
		TSharedPtr<STreeView<FTreeItemPtr>> TreeView;

		TArray<FTreeItemPtr> RootItems;

		TSharedPtr<SSearchBox> SearchBox;

		TSharedPtr<FUICommandList> CommandList;

		FSimpleDelegate OnTreeRefreshed;

		bool bRepopulating = false;
	};
}