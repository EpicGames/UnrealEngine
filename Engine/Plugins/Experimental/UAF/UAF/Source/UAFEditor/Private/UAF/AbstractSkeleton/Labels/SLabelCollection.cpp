// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Labels/SLabelCollection.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonLabelCollection.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SLabelCollection"

namespace UE::UAF::Editor
{
	class FLabelCollectionDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FLabelCollectionDragDropOp, FDecoratedDragDropOp)

		enum class EAction
		{
			None,
			SwapLabels,
			MoveLabelAbove,
			MoveLabelBelow
		};

		static TSharedRef<FLabelCollectionDragDropOp> New(const FName InLabel)
		{
			TSharedRef<FLabelCollectionDragDropOp> Operation = MakeShared<FLabelCollectionDragDropOp>();
			Operation->Label = InLabel;
			Operation->Construct();

			return Operation;
		}

		TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			TSharedRef<SHorizontalBox> VBox = SNew(SHorizontalBox);

			// EAction::None
			VBox->AddSlot()
				[
					SNew(SHorizontalBox)
					.Visibility_Lambda([this]()
					{
						return Action == EAction::None ? EVisibility::Visible : EVisibility::Collapsed;
					})
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
							.Text_Lambda([this]()
							{
								return FText::FromName(Label);
							})
					]
				];

			// EAction::SwapLabels
			VBox->AddSlot()
				[
					SNew(SHorizontalBox)
					.Visibility_Lambda([this]()
					{
						return Action == EAction::SwapLabels ? EVisibility::Visible : EVisibility::Collapsed;
					})
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SRichTextBlock)
							.DecoratorStyleSet(&FAppStyle::Get())
							.Text_Lambda([this]()
								{
									return FText::Format(LOCTEXT("SwapLabels", "Swap <RichTextBlock.Bold>{0}</> with <RichTextBlock.Bold>{1}</>"), FText::FromName(Label), ActionLabel);
								})
					]
				];
			
			// EAction::SwapLabels
			VBox->AddSlot()
				[
					SNew(SHorizontalBox)
					.Visibility_Lambda([this]()
					{
						return Action == EAction::MoveLabelAbove ? EVisibility::Visible : EVisibility::Collapsed;
					})
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SRichTextBlock)
							.DecoratorStyleSet(&FAppStyle::Get())
							.Text_Lambda([this]()
								{
									return FText::Format(LOCTEXT("MoveAbove", "Move <RichTextBlock.Bold>{0}</> above <RichTextBlock.Bold>{1}</>"), FText::FromName(Label), ActionLabel);
								})
					]
				];
			
			// EAction::SwapLabels
			VBox->AddSlot()
				[
					SNew(SHorizontalBox)
					.Visibility_Lambda([this]()
					{
						return Action == EAction::MoveLabelBelow ? EVisibility::Visible : EVisibility::Collapsed;
					})
					+ SHorizontalBox::Slot()
					.Padding(3.0f)
					.AutoWidth()
					[
						SNew(SRichTextBlock)
							.DecoratorStyleSet(&FAppStyle::Get())
							.Text_Lambda([this]()
								{
									return FText::Format(LOCTEXT("MoveBelow", "Move <RichTextBlock.Bold>{0}</> below <RichTextBlock.Bold>{1}</>"), FText::FromName(Label), ActionLabel);
								})
					]
				];
			
			return SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				.Padding(FMargin(8.0f, 4.0f))
				[
					VBox
				];
		}
		
		void SetAction(const EAction InAction, const FText InLabel)
		{
			ActionLabel = InLabel;
			Action = InAction;
		}

		void ClearAction()
		{
			ActionLabel = FText();
			Action = EAction::None;
		}
		
		EAction Action = EAction::None;
		
		FText ActionLabel;

		FName Label;
	};

	void SLabelCollection::Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonLabelCollection> InLabelCollection)
	{
		LabelCollection = InLabelCollection;

		ChildSlot
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(1.0f, 1.0f))
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(1.0f, 1.0f))
						[
							SNew(SPositiveActionButton)
								.OnClicked_Lambda([this]()
								{
									OnAddLabel();
									return FReply::Handled();
								})
								.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
								.Text(LOCTEXT("AddButton_Text", "Add Label"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(FMargin(1.0f, 1.0f))
						[
							SAssignNew(SearchBox, SSearchBox)
								.SelectAllTextWhenFocused(true)
								.OnTextChanged_Lambda([this](const FText& InText)
								{
									SearchText = InText;
									ApplySearchFilter();
								})
								.HintText(LOCTEXT("SearchBox_Hint", "Search Labels..."))
						]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(FMargin(1.0f, 1.0f))
				[
					SAssignNew(ListView, SListView<FListItemPtr>)
						.SelectionMode(ESelectionMode::Single)
						.ListItemsSource(&FilteredItems)
						.OnGenerateRow(this, &SLabelCollection::ListView_OnGenerateRow)
						.OnContextMenuOpening(this, &SLabelCollection::ListView_OnContextMenuOpening)
						.OnItemScrolledIntoView(this, &SLabelCollection::ListView_OnItemScrolledIntoView)
						.OnMouseButtonDoubleClick(this, &SLabelCollection::ListView_OnMouseButtonDoubleClick)
						.HeaderRow
						(
							SNew(SHeaderRow)
								+ SHeaderRow::Column(FName("Labels"))
								.FillWidth(0.5f)
								.DefaultLabel(LOCTEXT("ListView_Header", "Labels"))
						)
				]
		];

		RepopulateListData();
		BindCommands();
	}

	void SLabelCollection::PostUndo(bool bSuccess)
	{
		RepopulateListData();
	}

	void SLabelCollection::PostRedo(bool bSuccess)
	{
		RepopulateListData();
	}

	FReply SLabelCollection::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	TSharedRef<ITableRow> SLabelCollection::ListView_OnGenerateRow(FListItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedPtr<SInlineEditableTextBlock> InlineWidget;

		TSharedRef<STableRow<FListItemPtr>> RowWidget = SNew(STableRow<FListItemPtr>, OwnerTable)
			.Style(&FAppStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
			.OnDragLeave_Lambda([this](const FDragDropEvent& DragDropEvent)
			{
				const TSharedPtr<FLabelCollectionDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FLabelCollectionDragDropOp>();
				if (DragDropOp.IsValid())
				{
					DragDropOp->ClearAction();
				}
			})
			.OnDragDetected_Lambda([InItem](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
				{
					if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
					{
						const TSharedRef<FLabelCollectionDragDropOp> DragDropOp = FLabelCollectionDragDropOp::New(InItem->Label);
						return FReply::Handled().BeginDragDrop(DragDropOp);
					}

					return FReply::Unhandled();
				})
			.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FListItemPtr TargetTreeItem) -> TOptional<EItemDropZone>
				{
					using namespace UE::UAF;

					TOptional<EItemDropZone> ReturnedDropZone;

					const TSharedPtr<FLabelCollectionDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FLabelCollectionDragDropOp>();
					if (DragDropOp.IsValid() && DragDropOp->Label != TargetTreeItem->Label)
					{
						if (DropZone == EItemDropZone::OntoItem)
						{
							DragDropOp->SetAction(FLabelCollectionDragDropOp::EAction::SwapLabels, FText::FromName(TargetTreeItem->Label));
						}
						else if (DropZone == EItemDropZone::AboveItem)
						{
							DragDropOp->SetAction(FLabelCollectionDragDropOp::EAction::MoveLabelAbove, FText::FromName(TargetTreeItem->Label));
						}
						else if (DropZone == EItemDropZone::BelowItem)
						{
							DragDropOp->SetAction(FLabelCollectionDragDropOp::EAction::MoveLabelBelow, FText::FromName(TargetTreeItem->Label));
						}
						
						ReturnedDropZone = DropZone;
					}

					return ReturnedDropZone;
				})
			.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FListItemPtr TargetTreeItem) -> FReply
				{
					const TSharedPtr<FLabelCollectionDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FLabelCollectionDragDropOp>();
					if (DragDropOp.IsValid())
					{
						const FScopedTransaction Transaction(LOCTEXT("MoveLabel", "Move Label"));
						LabelCollection->Modify();


						if (DropZone == EItemDropZone::OntoItem)
						{
							TArray<FName>& Labels = LabelCollection->GetMutableLabels();

							const int32 A = Labels.Find(TargetTreeItem->Label);
							const int32 B = Labels.Find(DragDropOp->Label);
						
							// Swap labels
							if (ensure(A != INDEX_NONE && B != INDEX_NONE))
							{
								Labels[A] = DragDropOp->Label;
								Labels[B] = TargetTreeItem->Label;
							}
						}
						else if (DropZone == EItemDropZone::BelowItem)
						{
							LabelCollection->RemoveLabel(DragDropOp->Label);

							TArray<FName>& Labels = LabelCollection->GetMutableLabels();

							const int32 A = Labels.Find(TargetTreeItem->Label);

							Labels.Insert(DragDropOp->Label, A + 1);
						}
						else
						{
							LabelCollection->RemoveLabel(DragDropOp->Label);

							TArray<FName>& Labels = LabelCollection->GetMutableLabels();

							const int32 A = Labels.Find(TargetTreeItem->Label);

							Labels.Insert(DragDropOp->Label, A);
						}

						RepopulateListData();

						return FReply::Handled();
					}

					return FReply::Unhandled();
				})
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(3.0f)
					[
						SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Tag"))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.AutoWidth()
					.Padding(3.0f)
					[
						SAssignNew(InlineWidget, SInlineEditableTextBlock)
							.HighlightText_Lambda([this]() { return SearchText; })
							.Text(FText::FromName(InItem->Label))
							.OnVerifyTextChanged_Lambda([InItem](const FText& InText, FText& OutErrorMessage)
								{
									const FName OldName = InItem->Label;
									const FName NewName = FName(InText.ToString());

									if (InItem->Label == NewName)
									{
										return true;
									}

									check(InItem->CanRenameTo.IsBound());
									if (InItem->CanRenameTo.Execute(OldName, NewName))
									{
										return true;
									}
									else
									{
										OutErrorMessage = LOCTEXT("SetNameTaken", "Name already taken");
										return false;
									}
								})
							.OnTextCommitted_Lambda([InItem](const FText& InText, ETextCommit::Type CommitInfo)
								{
									const FName OldName = InItem->Label;
									const FName NewName = FName(InText.ToString());

									InItem->OnRenamed.ExecuteIfBound(OldName, NewName);
								})
					]

			];

		InItem->OnRequestRename.BindSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

		return RowWidget;
	}

	TSharedPtr<SWidget> SLabelCollection::ListView_OnContextMenuOpening()
	{
		FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

		const bool bIsLabelSelected = !ListView->GetSelectedItems().IsEmpty();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AddLabel_Label", "Add Label"),
			LOCTEXT("AddLabel_Description", "Add a new label"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLabelCollection::OnAddLabel)
			));

		if (bIsLabelSelected)
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		}

		return MenuBuilder.MakeWidget();
	}

	void SLabelCollection::ListView_OnItemScrolledIntoView(FListItemPtr InItem, const TSharedPtr<ITableRow>& InWidget)
	{
		if (ItemToRename)
		{
			ItemToRename->OnRequestRename.ExecuteIfBound();
			ItemToRename = nullptr;
		}
	}

	void SLabelCollection::ListView_OnMouseButtonDoubleClick(FListItemPtr InItem)
	{
		InItem->OnRequestRename.ExecuteIfBound();
	}

	void SLabelCollection::OnAddLabel()
	{
		SearchText = FText();
		
		const FScopedTransaction Transaction(LOCTEXT("AddLabel", "Set Label"));
		LabelCollection->Modify();

		const FText DefaultLabelFormat = LOCTEXT("DefaultLabel", "NewLabel_{0}");

		int32 Suffix = 0;
		FName LabelName = FName(FText::Format(DefaultLabelFormat, Suffix).ToString());
		while (LabelCollection->HasLabel(LabelName))
		{
			++Suffix;
			LabelName = FName(FText::Format(DefaultLabelFormat, Suffix).ToString());
		}

		const bool bSuccess = LabelCollection->AddLabel(LabelName);
		check(bSuccess);

		RepopulateListData();

		const FListItemPtr* const LabelItem = ListItems.FindByPredicate([LabelName](const FListItemPtr& InItem)
		{
			return InItem->Label == LabelName;
		});
		check(LabelItem);

		ItemToRename = *LabelItem;
		ListView->RequestScrollIntoView(*LabelItem);
	}

	void SLabelCollection::OnRenameLabel()
	{
		const TArray<FListItemPtr> SelectedItems = ListView->GetSelectedItems();
		if (!SelectedItems.IsEmpty())
		{
			FListItemPtr LabelItem = SelectedItems[0];

			ItemToRename = LabelItem;
			ListView->RequestScrollIntoView(LabelItem);
		}
	}

	void SLabelCollection::OnRemoveLabel()
	{
		const TArray<FListItemPtr> SelectedItems = ListView->GetSelectedItems();
		if (!SelectedItems.IsEmpty())
		{
			const FScopedTransaction Transaction(LOCTEXT("RemoveLabel", "Remove Label"));
			LabelCollection->Modify();

			const FName LabelToRemove = SelectedItems[0]->Label;

			const bool bSuccess = LabelCollection->RemoveLabel(LabelToRemove);
			check(bSuccess);
		
			RepopulateListData();
		}
	}

	void SLabelCollection::RepopulateListData()
	{
		ListItems.Reset();

		for (const FName& Label : LabelCollection->GetLabels())
		{
			FListItemPtr LabelItem = MakeShared<FListItem>();
			LabelItem->Label = Label;
			LabelItem->OnRenamed = FListItem::FOnRenamed::CreateSP(this, &SLabelCollection::OnItemRenamed);
			LabelItem->CanRenameTo = FListItem::FCanRenameTo::CreateSP(this, &SLabelCollection::CanRenameItem);

			ListItems.Add(MoveTemp(LabelItem));
		}

		ApplySearchFilter();

		ListView->RebuildList();
	}

	void SLabelCollection::BindCommands()
	{
		CommandList = MakeShareable(new FUICommandList);

		CommandList->MapAction(
			FGenericCommands::Get().Rename,
			FExecuteAction::CreateSP(this, &SLabelCollection::OnRenameLabel)
		);
		CommandList->MapAction(
			FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &SLabelCollection::OnRemoveLabel)
		);
	}

	void SLabelCollection::OnItemRenamed(const FName OldLabel, const FName NewLabel)
	{
		if (NewLabel != OldLabel)
		{
			const bool bSuccess = LabelCollection->RenameLabel(OldLabel, NewLabel);
			check(bSuccess);

			RepopulateListData();
		}
	}

	bool SLabelCollection::CanRenameItem(const FName OldLabel, const FName NewLabel) const
	{
		return !LabelCollection->HasLabel(NewLabel);
	}
	
	void SLabelCollection::ApplySearchFilter()
	{
		const FString SearchString = SearchText.ToString();

		FilteredItems = ListItems.FilterByPredicate([&](const FListItemPtr ListItem)
		{
			return ListItem->Label.ToString().Contains(SearchString, ESearchCase::Type::IgnoreCase);
		});

		ListView->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE