// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class ITableRow;
class UAbstractSkeletonLabelCollection;
template<typename> class SListView;
class SSearchBox;
class STableViewBase;

namespace UE::UAF::Editor
{
	class SLabelCollection : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
	{
		SLATE_BEGIN_ARGS(SLabelCollection) {}
		SLATE_END_ARGS()

	public:
		struct FListItem
		{
			FName Label;

			DECLARE_DELEGATE(FOnRequestRename);
			FOnRequestRename OnRequestRename;

			DECLARE_DELEGATE_RetVal_TwoParams(bool, FCanRenameTo, const FName /* Old Name */, const FName /* New Name */);
			FCanRenameTo CanRenameTo;

			DECLARE_DELEGATE_TwoParams(FOnRenamed, const FName /* Old Name */, const FName /* New Name */);
			FOnRenamed OnRenamed;
		};

		using FListItemPtr = TSharedPtr<FListItem>;

		virtual void Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonLabelCollection> InLabelCollection);

		/** FSelfRegisteringEditorUndoClient interface */
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;

	private:
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

		TSharedRef<ITableRow> ListView_OnGenerateRow(FListItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

		TSharedPtr<SWidget> ListView_OnContextMenuOpening();

		void ListView_OnItemScrolledIntoView(FListItemPtr InItem, const TSharedPtr<ITableRow>& InWidget);

		void ListView_OnMouseButtonDoubleClick(FListItemPtr InItem);

		void OnAddLabel();

		void OnRenameLabel();

		void OnRemoveLabel();

		void RepopulateListData();

		void BindCommands();

		void OnItemRenamed(const FName OldLabel, const FName NewLabel);

		bool CanRenameItem(const FName OldLabel, const FName NewLabel) const;

		void ApplySearchFilter();

	private:
		TWeakObjectPtr<UAbstractSkeletonLabelCollection> LabelCollection;

		TSharedPtr<SListView<FListItemPtr>> ListView;

		TArray<FListItemPtr> ListItems;
		
		TArray<FListItemPtr> FilteredItems;

		TSharedPtr<SSearchBox> SearchBox;

		TSharedPtr<FUICommandList> CommandList;

		FListItemPtr ItemToRename;

		FText SearchText;
	};
}