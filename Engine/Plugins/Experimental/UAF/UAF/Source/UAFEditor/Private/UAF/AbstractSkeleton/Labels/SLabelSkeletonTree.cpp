// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Labels/SLabelSkeletonTree.h"

#include "Animation/Skeleton.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPersonaToolkit.h"
#include "IPersonaPreviewScene.h"
#include "ReferenceSkeleton.h"
#include "ScopedTransaction.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonLabelBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonLabelCollection.h"
#include "UAF/AbstractSkeleton/IAbstractSkeletonEditor.h"
#include "UAF/AbstractSkeleton/Labels/ILabelBinding.h"
#include "UAF/AbstractSkeleton/Labels/ILabelsTab.h"
#include "UAF/AbstractSkeleton/Labels/BoneDragDrop.h"
#include "UAF/AbstractSkeleton/Labels/LabelDragDrop.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SLabelSkeletonTree"

namespace UE::UAF::Editor
{

	void SLabelSkeletonTree::Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonLabelBinding> InLabelBinding, TSharedPtr<ILabelsTab> InLabelsTabInterface)
	{
		LabelBinding = InLabelBinding;
		OnTreeRefreshed = InArgs._OnTreeRefreshed;
		LabelsTabInterface = InLabelsTabInterface;

		TWeakPtr<IAbstractSkeletonEditor> AbstractSkeletonEditor = LabelsTabInterface->GetAbstractSkeletonEditor();
		TSharedPtr<IPersonaToolkit> PersonaToolkit = AbstractSkeletonEditor.Pin()->GetPersonaToolkit();
		TSharedPtr<IPersonaPreviewScene> PreviewScene = PersonaToolkit->GetPreviewScene();
		PreviewScene->RegisterOnSelectedBonesChanged(FOnSelectedBonesChanged::CreateSP(this, &SLabelSkeletonTree::HandleBoneSelectionChanged));

		ChildSlot
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(1.0f, 1.0f))
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.Padding(FMargin(1.0f, 1.0f))
						[
							SAssignNew(SearchBox, SSearchBox)
								.SelectAllTextWhenFocused(true)
								.OnTextChanged_Lambda([](const FText& InText) { /* TODO: Implement */ })
								.HintText(LOCTEXT("SearchBox_Hint", "Search Labels..."))
						]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(FMargin(1.0f, 1.0f))
				[
					SAssignNew(TreeView, STreeView<FTreeItemPtr>)
						.SelectionMode(ESelectionMode::Single)
						.HighlightParentNodesForSelection(true)
						.TreeItemsSource(&RootItems)
						.OnGenerateRow(this, &SLabelSkeletonTree::TreeView_OnGenerateRow)
						.OnGetChildren(this, &SLabelSkeletonTree::TreeView_OnGetChildren)
						.OnContextMenuOpening(this, &SLabelSkeletonTree::TreeView_OnContextMenuOpening)
						.OnSelectionChanged_Lambda([this](FTreeItemPtr InItem, ESelectInfo::Type InSelectType)
						{
							TWeakPtr<IAbstractSkeletonEditor> AbstractSkeletonEditor = LabelsTabInterface->GetAbstractSkeletonEditor();
							TSharedPtr<IPersonaToolkit> PersonaToolkit = AbstractSkeletonEditor.Pin()->GetPersonaToolkit();

							if (InItem.IsValid())
							{
								PersonaToolkit->GetPreviewScene()->SetSelectedBone(InItem->BoneName, InSelectType);
							}
							else
							{
								PersonaToolkit->GetPreviewScene()->SetSelectedBone(NAME_None, InSelectType);
							}
						})
				]
		];

		RepopulateTreeData();
		ExpandAllTreeItems();
		BindCommands();
	}

	FReply SLabelSkeletonTree::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	TSharedRef<ITableRow> SLabelSkeletonTree::TreeView_OnGenerateRow(FTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedPtr<SHorizontalBox> HBox;

		TSharedRef<ITableRow> RowWidget = SNew(STableRow<FTreeItemPtr>, OwnerTable)
			.ShowWires(true)
			.Style(FAppStyle::Get(), "TableView.AlternatingRow")
			.OnDragDetected_Lambda([this, InItem](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
				{
					if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
					{
						const TSharedRef<FBoneDragDropOp> DragDropOp = FBoneDragDropOp::New(LabelBinding, InItem->BoneName);
						return FReply::Handled().BeginDragDrop(DragDropOp);
					}

					return FReply::Unhandled();
				})
			.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FTreeItem> TargetTreeItem) -> TOptional<EItemDropZone>
				{
					TOptional<EItemDropZone> ReturnedDropZone;

					const TSharedPtr<FLabelDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FLabelDragDropOp>();
					if (DragDropOp.IsValid())
					{
						ReturnedDropZone = EItemDropZone::OntoItem;
					}

					return ReturnedDropZone;
				})
			.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FTreeItem> TargetTreeItem) -> FReply
				{
					const TSharedPtr<FLabelDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FLabelDragDropOp>();
					
					if (DragDropOp.IsValid())
					{
						const FScopedTransaction Transaction(LOCTEXT("BindBoneToLabel", "Bind Bone to Label"));
						LabelBinding->Modify();

						const bool bSuccess = LabelBinding->BindBoneToLabel(DragDropOp->LabelCollection.Get(), TargetTreeItem->BoneName, DragDropOp->Label);
						check(bSuccess);

						RepopulateTreeData();
						LabelsTabInterface->GetLabelBindingWidget()->RepopulateTreeData();

						return FReply::Handled();
					}

					return FReply::Unhandled();
				})
			[
				SAssignNew(HBox, SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(3.0f))
					[
						SNew(SImage)
							.ColorAndOpacity(InItem->AssignedLabels.IsEmpty() ? FSlateColor::UseForeground() : FAppStyle::Get().GetSlateColor("Colors.AccentBlue"))
							.Image(FAppStyle::Get().GetBrush("SkeletonTree.Bone"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(3.0f))
					[
						SNew(STextBlock)
							.Text(FText::FromName(InItem->BoneName))
					]
			];

		for (const FTreeItem::FBindings Binding : InItem->AssignedLabels)
		{
			HBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(FMargin(2.0f, 0.0f))
					[
						SNew(SImage)
							.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.AccentBlue"))
							.DesiredSizeOverride(FVector2D(16, 16))
							.Image(FAppStyle::Get().GetBrush("Icons.Tag"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(FMargin(2.0f, 0.0f))
					[
						SNew(SHyperlink)
							.Text(FText::FromName(Binding.Label))
							.OnNavigate_Lambda([this, Binding]()
							{
								TSharedPtr<ILabelBindingWidget> LabelBindingWidget = LabelsTabInterface->GetLabelBindingWidget();
								LabelBindingWidget->ScrollToLabel(Binding.LabelCollection.Get(), Binding.Label);
							})
					]
				];
		}

		return RowWidget;
	}

	void SLabelSkeletonTree::TreeView_OnGetChildren(FTreeItemPtr InItem, TArray<FTreeItemPtr>& OutChildren)
	{
		OutChildren.Append(InItem->Children);
	}

	TSharedPtr<SWidget> SLabelSkeletonTree::TreeView_OnContextMenuOpening()
	{
		FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

		TArray<FTreeItemPtr> Selection = TreeView->GetSelectedItems();
		if (Selection.Num() > 0)
		{
			FTreeItemPtr Selected = Selection[0]; // Single selection only

			if (Selected->AssignedLabels.Num() >= 2)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("UnbindAll_Label", "Unbind All"),
					LOCTEXT("UnbindAll_Tooltip", "Unbind all labels"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Unlink"),
					FUIAction(
						FExecuteAction::CreateLambda([this, Selected]()
							{
								const FScopedTransaction Transaction(LOCTEXT("UnbindAll", "Unbind All"));
								LabelBinding->Modify();

								for (const FTreeItem::FBindings Binding : Selected->AssignedLabels)
								{
									const bool bSuccess = LabelBinding->UnbindBoneFromLabel(Binding.LabelCollection.Get(), Selected->BoneName, Binding.Label);
									check(bSuccess);
								}

								RepopulateTreeData();
								LabelsTabInterface->GetLabelBindingWidget()->RepopulateTreeData();
							}
						)
					));

				MenuBuilder.AddSeparator();
			}

			for (const FTreeItem::FBindings Binding : Selected->AssignedLabels)
			{
				MenuBuilder.AddMenuEntry(
					FText::Format(LOCTEXT("UnbindBone_Label", "Unbind {0}"), FText::FromName(Binding.Label)),
					LOCTEXT("UnbindBone_Tooltip", "Unbind the selected bone and label"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Unlink"),
					FUIAction(
						FExecuteAction::CreateLambda([this, Binding, Selected]()
							{
								const FScopedTransaction Transaction(FText::Format(LOCTEXT("UnbindBone_Label", "Unbind {0}"), FText::FromName(Binding.Label)));
								LabelBinding->Modify();

								const bool bSuccess = LabelBinding->UnbindBoneFromLabel(Binding.LabelCollection.Get(), Selected->BoneName, Binding.Label);
								check(bSuccess);

								RepopulateTreeData();
								LabelsTabInterface->GetLabelBindingWidget()->RepopulateTreeData();
							}
						)
					));
			}

			return MenuBuilder.MakeWidget();
		}

		return SNullWidget::NullWidget;
	}

	void SLabelSkeletonTree::PostUndo(bool bSuccess)
	{
		RepopulateTreeData();
	}

	void SLabelSkeletonTree::PostRedo(bool bSuccess)
	{
		RepopulateTreeData();
	}

	void SLabelSkeletonTree::ScrollToBone(const FName InBoneName)
	{
		for (FTreeItemPtr TreeItem : GetAllTreeItems())
		{
			if (TreeItem->BoneName == InBoneName)
			{
				TreeView->RequestScrollIntoView(TreeItem);
				TreeView->SetSelection(TreeItem);
				return;
			}
		}
	}

	void SLabelSkeletonTree::RepopulateTreeData()
	{
		if (!bRepopulating)
		{
			TGuardValue<bool> RecursionGuard(bRepopulating, true);
			
			// Rebuild tree data
			RootItems.Reset();

			if (LabelBinding.IsValid())
			{
				TMap<int32, FTreeItemPtr> TreeItemsMap;

				if (const TObjectPtr<const USkeleton> Skeleton = LabelBinding->GetSkeleton())
				{
					const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
					const int32 BoneCount = RefSkeleton.GetNum();

					for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
					{
						TSharedPtr<FTreeItem> TreeItem = MakeShared<FTreeItem>();
						TreeItem->BoneName = RefSkeleton.GetBoneName(BoneIndex);

						const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);

						if (ParentIndex == INDEX_NONE)
						{
							RootItems.Add(TreeItem);
						}
						else
						{
							if (ensure(TreeItemsMap.Contains(ParentIndex)))
							{
								TreeItemsMap[ParentIndex]->Children.Add(TreeItem);
							}
						}

						TreeItemsMap.Add(BoneIndex, TreeItem);
					}

					for (const FAbstractSkeleton_LabelBinding& Binding : LabelBinding->GetLabelBindings())
					{
						const int32 BoneIndex = RefSkeleton.FindBoneIndex(Binding.BoneName);
						if (ensure(BoneIndex != INDEX_NONE && TreeItemsMap.Contains(BoneIndex)))
						{
							FTreeItem::FBindings& Bindings = TreeItemsMap[BoneIndex]->AssignedLabels.AddDefaulted_GetRef();
							Bindings.Label = Binding.Label;
							Bindings.LabelCollection = Binding.SourceCollection;
						}
					}
				}
			}

			TreeView->RequestTreeRefresh();
			
			for (FTreeItemPtr TreeItem : GetAllTreeItems())
			{
				TreeView->SetItemExpansion(TreeItem, true);
			}

			OnTreeRefreshed.ExecuteIfBound();
		}
	}

	void SLabelSkeletonTree::ExpandAllTreeItems()
	{
		for (FTreeItemPtr TreeItem : GetAllTreeItems())
		{
			TreeView->SetItemExpansion(TreeItem, true);
		}
	}

	void SLabelSkeletonTree::BindCommands()
	{
		CommandList = MakeShareable(new FUICommandList);
	}

	TArray<TSharedPtr<SLabelSkeletonTree::FTreeItem>> SLabelSkeletonTree::GetAllTreeItems()
	{
		TArray<FTreeItemPtr> AllItems;
		AllItems.Append(RootItems);

		for (int32 Index = 0; Index < AllItems.Num(); ++Index)
		{
			AllItems.Append(AllItems[Index]->Children);
		}

		return AllItems;
	}

	void SLabelSkeletonTree::HandleBoneSelectionChanged(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo)
	{
		const FName SelectedBoneName = InBoneNames.IsEmpty() ? NAME_None : InBoneNames[0]; // Single selection only

		for (const FTreeItemPtr& TreeItem : GetAllTreeItems())
		{
			if (TreeItem->BoneName == SelectedBoneName)
			{
				if (!TreeView->IsItemSelected(TreeItem)) // Recursion guard
				{
					TreeView->RequestScrollIntoView(TreeItem);
					TreeView->SetSelection(TreeItem);
					return;
				}

			}
		}
	}
}

#undef LOCTEXT_NAMESPACE