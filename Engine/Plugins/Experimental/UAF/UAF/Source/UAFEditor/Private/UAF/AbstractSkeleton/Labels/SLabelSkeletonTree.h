// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "UAF/AbstractSkeleton/Labels/ILabelSkeletonTree.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class ITableRow;
class STableViewBase;
class SSearchBox;
template<typename> class STreeView;
class UAbstractSkeletonLabelBinding;
class UAbstractSkeletonLabelCollection;

namespace UE::UAF::Editor
{
	class ILabelsTab;

	class SLabelSkeletonTree : public SCompoundWidget, public FSelfRegisteringEditorUndoClient, public ILabelSkeletonTreeWidget
	{
		SLATE_BEGIN_ARGS(SLabelSkeletonTree) {}
			SLATE_EVENT(FSimpleDelegate, OnTreeRefreshed)
		SLATE_END_ARGS()

	public:
		struct FTreeItem
		{
			struct FBindings
			{
				TWeakObjectPtr<const UAbstractSkeletonLabelCollection> LabelCollection;
				FName Label;
			};

			FName BoneName;
			TArray<TSharedPtr<FTreeItem>> Children;
			TArray<FBindings> AssignedLabels;
		};

		using FTreeItemPtr = TSharedPtr<FTreeItem>;

		void Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonLabelBinding> InSetBinding, TSharedPtr<ILabelsTab> InLabelsTabInterface);

		/** FSelfRegisteringEditorUndoClient interface */
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;

		/** ILabelSkeletonTreeWidget */
		virtual void ScrollToBone(const FName InBoneName) override;
		virtual void RepopulateTreeData() override;

	private:
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

		TSharedRef<ITableRow> TreeView_OnGenerateRow(FTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

		void TreeView_OnGetChildren(FTreeItemPtr InItem, TArray<FTreeItemPtr>& OutChildren);

		TSharedPtr<SWidget> TreeView_OnContextMenuOpening();

		void ExpandAllTreeItems();

		void BindCommands();

		TArray<FTreeItemPtr> GetAllTreeItems();

		// Triggered when a bone is selected in the viewport
		void HandleBoneSelectionChanged(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo);

	public:
		TWeakObjectPtr<UAbstractSkeletonLabelBinding> LabelBinding;

	private:
		TSharedPtr<STreeView<FTreeItemPtr>> TreeView;

		TArray<FTreeItemPtr> RootItems;

		TSharedPtr<SSearchBox> SearchBox;

		TSharedPtr<FUICommandList> CommandList;

		FSimpleDelegate OnTreeRefreshed;

		bool bRepopulating = false;

		TSharedPtr<ILabelsTab> LabelsTabInterface;
	};
}