// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Labels/SLabelsTab.h"

#include "Animation/Skeleton.h"
#include "AssetThumbnail.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonLabelBinding.h"
#include "UAF/AbstractSkeleton/Labels/SLabelBinding.h"
#include "UAF/AbstractSkeleton/Labels/SLabelSkeletonTree.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SLabelsTab"

namespace UE::UAF::Editor
{
	FName SLabelsTab::FTabs::SkeletonTreeId("AbstractSkeletonBindingEditor_Labels_SkeletonTreeTab");
	FName SLabelsTab::FTabs::LabelBindingsId("AbstractSkeletonBindingEditor_Labels_BindingsTab");

	TSharedPtr<ILabelBindingWidget> SLabelsTab::GetLabelBindingWidget() const
	{
		return LabelBindingWidget;
	}

	TSharedPtr<ILabelSkeletonTreeWidget> SLabelsTab::GetLabelSkeletonTreeWidget() const
	{
		return LabelSkeletonTreeWidget;
	}

	void SLabelsTab::RepopulateLabelData()
	{
		GetLabelBindingWidget()->RepopulateTreeData();
		GetLabelSkeletonTreeWidget()->RepopulateTreeData();
	}

	void SLabelsTab::Construct(const FArguments& InArgs)
	{
		LabelBinding = InArgs._LabelBinding;
		AbstractSkeletonEditor = InArgs._AbstractSkeletonEditor;

		static const FName LayoutName = "UE::Anim::STF::LabelsTab::SLabelsTab_Layout";

		TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout(LayoutName)
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(FTabs::SkeletonTreeId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(FTabs::LabelBindingsId, ETabState::OpenedTab)
				)
			);

		check(InArgs._ParentTab.IsValid());
		TabManager = FGlobalTabmanager::Get()->NewTabManager(InArgs._ParentTab.ToSharedRef());
		TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateSP(this, &SLabelsTab::HandleTabManagerPersistLayout));

		TabManager->RegisterTabSpawner(FTabs::SkeletonTreeId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
			{
				return SAssignNew(LabelSkeletonTreeTab, SDockTab)
				[
					SAssignNew(LabelSkeletonTreeWidget, SLabelSkeletonTree, LabelBinding, SharedThis(this))
				];
			}))
			.SetDisplayName(LOCTEXT("SkeletonTree_TabDisplayName", "Skeleton Tree"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.SkeletonTree"));

		TabManager->RegisterTabSpawner(FTabs::LabelBindingsId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
			{
				return SAssignNew(LabelBindingTab, SDockTab)
				[
					SAssignNew(LabelBindingWidget, SLabelBinding, LabelBinding, SharedThis(this))
				];
			}))
			.SetDisplayName(LOCTEXT("LabelBindings_TabDisplayName", "Label Bindings"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

		Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout);

		FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);

		ToolbarBuilder.BeginSection("Common");
		{
			ToolbarBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.0f, 0.0f, 24.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("LabelBinding", "Label Binding"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SObjectPropertyEntryBox)
						.AllowedClass(UAbstractSkeletonLabelBinding::StaticClass())
						.ObjectPath_Lambda([&]()
							{
								return LabelBinding->GetPathName();
							})
						.OnObjectChanged_Lambda([this](const FAssetData& AssetData)
							{
								LabelBinding = CastChecked<UAbstractSkeletonLabelBinding>(AssetData.GetAsset());

								LabelSkeletonTreeTab->SetContent(
									SAssignNew(LabelSkeletonTreeWidget, SLabelSkeletonTree, LabelBinding, SharedThis(this)));

								LabelBindingTab->SetContent(
									SAssignNew(LabelBindingWidget, SLabelBinding, LabelBinding, SharedThis(this)));
							})
						.AllowClear(false)
						.DisplayUseSelected(true)
						.DisplayBrowse(true)
						.DisplayThumbnail(true)
						.ThumbnailPool(MakeShared<FAssetThumbnailPool>(1))
				],
				NAME_None,
				true,
				HAlign_Left
			);

			ToolbarBuilder.AddSeparator();

			ToolbarBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(16.0f, 0.0f, 24.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("Skeleton", "Skeleton"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SObjectPropertyEntryBox)
						.AllowedClass(USkeleton::StaticClass())
						.OnIsEnabled_Lambda([&]()
							{
								return LabelBinding.IsValid();
							})
						.ObjectPath_Lambda([&]()
							{
								if (LabelBinding.IsValid() && LabelBinding->GetSkeleton())
								{
									return LabelBinding->GetSkeleton()->GetPathName();
								}
								return FString();
							})
						.OnObjectChanged_Lambda([this](const FAssetData& AssetData)
							{
								if (LabelBinding.IsValid())
								{
									if (TObjectPtr<USkeleton> NewSkeleton = Cast<USkeleton>(AssetData.GetAsset()))
									{
										LabelBinding->SetSkeleton(NewSkeleton);

										LabelSkeletonTreeWidget->RepopulateTreeData();
										LabelBindingWidget->RepopulateTreeData();
									}
								}
							})
						.AllowClear(false)
						.DisplayUseSelected(true)
						.DisplayBrowse(true)
						.DisplayThumbnail(true)
						.ThumbnailPool(MakeShared<FAssetThumbnailPool>(1))
				],
				NAME_None,
				true,
				HAlign_Left
			);
		}
		ToolbarBuilder.EndSection();

		ToolbarBuilder.SetStyle(&FAppStyle::Get(), "SlimToolBar");

		ChildSlot
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						[
							ToolbarBuilder.MakeWidget()
						]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					TabManager->RestoreFrom(Layout, nullptr).ToSharedRef()
				]
		];
	}

	void SLabelsTab::HandleTabManagerPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave)
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToSave);
	}

	TWeakPtr<IAbstractSkeletonEditor> SLabelsTab::GetAbstractSkeletonEditor() const
	{
		return AbstractSkeletonEditor;
	}
}

#undef LOCTEXT_NAMESPACE