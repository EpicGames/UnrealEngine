// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/SCompoundWidget.h"
#include "UAF/AbstractSkeleton/Labels/ILabelsTab.h"

class UAbstractSkeletonLabelBinding;
class SDockTab;

namespace UE::UAF::Editor
{
	class IAbstractSkeletonEditor;
	class SLabelBinding;
	class SLabelSkeletonTree;

	class SLabelsTab : public SCompoundWidget, public ILabelsTab
	{
	public:
		struct FTabs
		{
			static FName SkeletonTreeId;
			static FName LabelBindingsId;
		};

		SLATE_BEGIN_ARGS(SLabelsTab) {}
			SLATE_ARGUMENT(TSharedPtr<SDockTab>, ParentTab)
			SLATE_ARGUMENT(TWeakObjectPtr<UAbstractSkeletonLabelBinding>, LabelBinding)
			SLATE_ARGUMENT(TWeakPtr<IAbstractSkeletonEditor>, AbstractSkeletonEditor)
		SLATE_END_ARGS()

		/** ILabelsTab */
		virtual TSharedPtr<ILabelBindingWidget> GetLabelBindingWidget() const override;
		virtual TSharedPtr<ILabelSkeletonTreeWidget> GetLabelSkeletonTreeWidget() const override;
		virtual void RepopulateLabelData() override;
		virtual TWeakPtr<IAbstractSkeletonEditor> GetAbstractSkeletonEditor() const override;

		virtual void Construct(const FArguments& InArgs);

	private:
		void HandleTabManagerPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave);

	private:
		TWeakObjectPtr<UAbstractSkeletonLabelBinding> LabelBinding;

		TSharedPtr<FTabManager> TabManager;

		TSharedPtr<SDockTab> LabelSkeletonTreeTab;

		TSharedPtr<SLabelSkeletonTree> LabelSkeletonTreeWidget;
		
		TSharedPtr<SDockTab> LabelBindingTab;

		TSharedPtr<SLabelBinding> LabelBindingWidget;

		TWeakPtr<IAbstractSkeletonEditor> AbstractSkeletonEditor;
	};
}