// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/AttributeDragDrop.h"

namespace UE::UAF::Editor
{
	TSharedRef<FAttributeDragDropOp> FAttributeDragDropOp::New(TArray<FAnimationAttributeIdentifier>&& InAttributes)
	{
		TSharedRef<FAttributeDragDropOp> Operation = MakeShared<FAttributeDragDropOp>();
		Operation->Attributes = InAttributes;

		Operation->Construct();

		return Operation;
	}

	TSharedPtr<SWidget> FAttributeDragDropOp::GetDefaultDecorator() const
	{
		return SNew(SBorder)
			.Visibility(EVisibility::Visible)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				SNew(STextBlock)
				.Text(Attributes.Num() == 1
					? FText::FromName(Attributes[0].GetName())
					: FText::Format(NSLOCTEXT("UE::UAF::Editor::FAttributeDragDropOp", "MultipleAttributesTooltip", "{0} attributes"), FText::AsNumber(Attributes.Num())))
			];
	}
}