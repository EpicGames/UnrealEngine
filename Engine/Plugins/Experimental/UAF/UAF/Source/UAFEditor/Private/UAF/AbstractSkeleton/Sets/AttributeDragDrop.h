// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Animation/AnimData/AttributeIdentifier.h"

namespace UE::UAF::Editor
{
	class FAttributeDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FAttributeDragDropOp, FDecoratedDragDropOp)

		static TSharedRef<FAttributeDragDropOp> New(TArray<FAnimationAttributeIdentifier>&& InAttributes);

		TSharedPtr<SWidget> GetDefaultDecorator() const override;

		TArray<FAnimationAttributeIdentifier> Attributes;
	};
}
