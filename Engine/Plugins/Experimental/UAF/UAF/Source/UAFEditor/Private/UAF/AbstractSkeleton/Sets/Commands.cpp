// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/Commands.h"

#include "Framework/Commands/UICommandList.h"

#define LOCTEXT_NAMESPACE "SetBindingEditorCommands"

namespace UE::UAF::Editor
{
	void FSetBindingEditorCommands::RegisterCommands()
	{
		UI_COMMAND(UnbindSelection, "Unbind Selection", "Unbinds the current selection", EUserInterfaceActionType::Button, FInputChord(EKeys::BackSpace));
		UI_COMMAND(SelectWithChildren, "Select With Children", "Sets the current selection to this item and its children", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::A));
	}
}

#undef LOCTEXT_NAMESPACE