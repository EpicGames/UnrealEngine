// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "UAFStyle.h"

class FUICommandInfo;

namespace UE::UAF::Editor
{
	class FSetBindingEditorCommands : public TCommands<FSetBindingEditorCommands>
	{
	public:
		FSetBindingEditorCommands()
			: TCommands<FSetBindingEditorCommands>
			(
				TEXT("SetBindingEditorCommands"),
				NSLOCTEXT("SetBindingEditorCommands", "SetBindingEditorCommands", "Set Binding Editor Commands"),
				NAME_None,
				FUAFStyle::Get().GetStyleSetName()
			)
		{
		}

		virtual void RegisterCommands() override;

		TSharedPtr<FUICommandInfo> UnbindSelection;
		TSharedPtr<FUICommandInfo> SelectWithChildren;
	};

}