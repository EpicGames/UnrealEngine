// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAttributesList.h"

#include "AttributeDragDrop.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PersonaModule.h"
#include "PropertyCustomizationHelpers.h"
#include "SPositiveActionButton.h"
#include "ScopedTransaction.h"
#include "UAF/AbstractSkeleton/Sets/SetDragDrop.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "ToolMenus.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SAttributesList"

using namespace UE::UAF::Editor;

static const FName ColumnId_Set("Set");
static const FName ColumnId_AttributeName("AttributeName");
static const FName ColumnId_AttributeType("AttributeType");

static const FName MenuId_AddAttribute("SAttributesList.AddAttribute");

void SAttributesList::SetSetBinding(TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding)
{
	if (SetBinding.IsValid())
	{
		SetBinding->UnregisterOnBindingsChanged(OnBindingsChangedHandle);
	}

	SetBinding = InSetBinding;

	if (SetBinding.IsValid())
	{
		OnBindingsChangedHandle = SetBinding->RegisterOnBindingsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SAttributesList::HandleBindingsChanged));
	}

	RepopulateListData();
}

void SAttributesList::Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding)
{
	SetBinding = InSetBinding;

	if (SetBinding.IsValid())
	{
		OnBindingsChangedHandle = SetBinding->RegisterOnBindingsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SAttributesList::HandleBindingsChanged));
	}

	RegisterMenus();

	ChildSlot
	[
		SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(1.0f, 1.0f))
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(1.0f, 1.0f))
					[
						SNew(SPositiveActionButton)
							.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
							.Text(LOCTEXT("AddAttributeButton_Label", "Add"))
							.OnGetMenuContent( this, &SAttributesList::CreateAddAttributeWidget)
							.IsEnabled_Lambda([this]()
							{
								return SetBinding.IsValid() && SetBinding->GetSetCollection() && SetBinding->GetSkeleton();
							})
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(FMargin(1.0f, 1.0f))
					[
						SNew(SSearchBox)
							.SelectAllTextWhenFocused(true)
							.OnTextChanged_Lambda([](const FText& InText) { /* TODO: Implement */ })
							.HintText(LOCTEXT("SearchBox_Hint", "Search Sets..."))
					]
			]
			+ SVerticalBox::Slot()
			[
				SAssignNew(ListView, SListView<FListItemPtr>)
					.ListItemsSource(&ListItems)
					.OnGenerateRow(this, &SAttributesList::ListView_OnGenerateRow)
					.OnContextMenuOpening(this, &SAttributesList::ListView_OnContextMenuOpening)
					.HeaderRow(
						SNew(SHeaderRow)
						+ SHeaderRow::Column(ColumnId_Set)
						.DefaultLabel(LOCTEXT("SetColumnLabel", "Set"))
						.FillWidth(0.3f)
						+ SHeaderRow::Column(ColumnId_AttributeName)
						.DefaultLabel(LOCTEXT("AttributeNameColumnLabel", "Attribute Name"))
						.FillWidth(0.35f)
						+ SHeaderRow::Column(ColumnId_AttributeType)
						.DefaultLabel(LOCTEXT("AttributeTypeColumnLabel", "Attribute Type"))
						.FillWidth(0.35f)
					)
			]
	];

	RepopulateListData();
}

void SAttributesList::Tick(const FGeometry& AllocatedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bListDirty)
	{
		RepopulateListData();
		bListDirty = false;
	}
}

void SAttributesList::PostUndo(bool bSuccess)
{
	RepopulateListData();
}

void SAttributesList::PostRedo(bool bSuccess)
{
	RepopulateListData();
}

void SAttributesList::RepopulateListData()
{
	ListItems.Empty();

	if (SetBinding.IsValid())
	{
		for (const FAbstractSkeleton_AttributeBinding& Binding : SetBinding->GetAttributeBindings())
		{
			FListItemPtr BindingItem = MakeShared<FListItem>(Binding.SetName, Binding.Attribute);
			ListItems.Add(BindingItem);
		}
	}

	ListView->RequestListRefresh();
}

class SAttributesListRow : public SMultiColumnTableRow<SAttributesList::FListItemPtr>
{
public:
	using FOnAttributeTypeChanged = TDelegate<void(TNonNullPtr<const UScriptStruct>)>;
	
	SLATE_BEGIN_ARGS(SAttributesListRow) {}
		SLATE_ARGUMENT(SAttributesList::FListItemPtr, Item)
		SLATE_EVENT(FOnCanAcceptDrop, OnCanAcceptDrop)
		SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
		SLATE_EVENT(FOnAttributeTypeChanged, OnAttributeTypeChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, TNonNullPtr<UAbstractSkeletonSetBinding> InSetBinding)
	{
		Item = InArgs._Item;
		OnAttributeTypeChanged = InArgs._OnAttributeTypeChanged;
		SetBinding = InSetBinding;

		const FSuperRowType::FArguments Args = FSuperRowType::FArguments()
			.OnDragDetected(InArgs._OnDragDetected)
			.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
			.OnAcceptDrop(InArgs._OnAcceptDrop)
			.Style(FAppStyle::Get(), "TableView.AlternatingRow");

		SMultiColumnTableRow<SAttributesList::FListItemPtr>::Construct(Args, OwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == ColumnId_Set)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(6, 0)
				[
					SNew(SBox)
						.WidthOverride(8.0f)
						.HeightOverride(8.0f)
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.FilledCircle"))
								.ColorAndOpacity((Item->SetName != NAME_None) ? FLinearColor(FColor(31, 228, 75)) : FLinearColor(FColor(239, 53, 53)))
						]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2, 2)
				[
					SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("ClassIcon.GroupActor"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2, 2)
				[
					SNew(STextBlock)
						.Text(FText::FromName(Item->SetName))
				];
		}
		else if (ColumnName == ColumnId_AttributeName)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2, 2)
				[
					SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("AnimGraph.Attribute.Attributes.Icon"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2, 2)
				[
					SAssignNew(Item->EditableTextBlock, SInlineEditableTextBlock)
					.Text_Lambda([this]()
						{
							return FText::FromName(Item->Attribute.GetName());
						})
					.OnVerifyTextChanged_Lambda([this](const FText& Label, FText& ErrorMessage)
						{
							FName NewAttributeName = FName(Label.ToString());
							
							if (NewAttributeName == Item->Attribute.GetName())
							{
								return true; // Name unchanged
							}

							if (NewAttributeName.IsNone())
							{
								ErrorMessage = LOCTEXT("EmptyAttributeNameErrorMessage", "An attribute name must be non-empty");
								return false;
							}
							
							if (SetBinding->ContainsAttribute(NewAttributeName))
							{
								ErrorMessage = LOCTEXT("RenameClashErrorMessage", "An attribute with that name already exists");
								return false;
							}
							
							return true;
						})
					.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type CommitInfo)
						{
							FName NewAttributeName = FName(InText.ToString());

							// The double call to RemoveAttributeFromSet here is intentional.
							// If the attribute is bound then the first removal unbinds it.
							// The second removal will remove the unbound attribute.

							// TODO: This API is poor and needs changing to be clearer.
							
							if (!Item->SetName.IsNone())
							{
								SetBinding->RemoveAttributeFromSet(Item->Attribute);
							}
							SetBinding->RemoveAttributeFromSet(Item->Attribute);

							FAnimationAttributeIdentifier NewIdentifier(NewAttributeName, INDEX_NONE, NAME_None, Item->Attribute.GetType());
				
							SetBinding->AddAttributeToSet(NewIdentifier, Item->SetName);
						})
				];
		}
		else if (ColumnName == ColumnId_AttributeType)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2, 2)
				[
					SNew(SStructPropertyEntryBox)
						.AllowNone(false)
						.SelectedStruct_Lambda([this]()
							{
								return Item->Attribute.GetType();
							})
						.OnSetStruct_Lambda([this](const UScriptStruct* InNewAttributeType)
							{
								OnAttributeTypeChanged.ExecuteIfBound(InNewAttributeType);
							})
				];
		}
		else 
		{
			return SNullWidget::NullWidget;
		}
	}

private:
	SAttributesList::FListItemPtr Item;
	UAbstractSkeletonSetBinding* SetBinding = nullptr;
	FOnAttributeTypeChanged OnAttributeTypeChanged;
};

SAttributesList::FListItem::FListItem(const FName InSetName, const FAnimationAttributeIdentifier& InAttribute)
	: Attribute(InAttribute)
	, SetName(InSetName)
{
}

TSharedRef<ITableRow> SAttributesList::ListView_OnGenerateRow(FListItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SAttributesListRow, OwnerTable, SetBinding.Get())
		.Item(Item)
		.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FListItemPtr TargetTreeItem) -> TOptional<EItemDropZone>
			{
				TOptional<EItemDropZone> ReturnedDropZone;

				const TSharedPtr<Editor::FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<Editor::FSetDragDropOp>();
				if (DragDropOp.IsValid())
				{
					ReturnedDropZone = EItemDropZone::OntoItem;
				}

				return ReturnedDropZone;
			})
		.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FListItemPtr TargetTreeItem) -> FReply
			{
				const TSharedPtr<Editor::FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<Editor::FSetDragDropOp>();

				if (DragDropOp.IsValid())
				{
					const FScopedTransaction Transaction(LOCTEXT("AddAttributeToSet", "Add Attribute to Set"));
					SetBinding->Modify();

					// TODO: Handle all dragged sets
					if (!DragDropOp->GetDraggedSets().IsEmpty())
					{
						SetBinding->AddAttributeToSet(TargetTreeItem->Attribute, DragDropOp->GetDraggedSets()[0].SetName);
					}
					
					return FReply::Handled();
				}

				return FReply::Unhandled();
			})
		.OnDragDetected_Lambda([this](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
			{
				if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
				{
					TArray<FAnimationAttributeIdentifier> SelectedAttributes;
					SelectedAttributes.Reserve(ListView->GetNumItemsSelected());
					
					for (const FListItemPtr& Selected : ListView->GetSelectedItems())
					{
						SelectedAttributes.Add(Selected->Attribute);
					}
					
					const TSharedRef<FAttributeDragDropOp> DragDropOp = FAttributeDragDropOp::New(MoveTemp(SelectedAttributes));
					return FReply::Handled().BeginDragDrop(DragDropOp);
				}

				return FReply::Unhandled();
			})
		.OnAttributeTypeChanged_Lambda([this, Item](TNonNullPtr<const UScriptStruct> InNewAttributeType)
			{
				// The double call to RemoveAttributeFromSet here is intentional.
				// If the attribute is bound then the first removal unbinds it.
				// The second removal will remove the unbound attribute.

				// TODO: This API is poor and needs changing to be clearer.
				
				if (!Item->SetName.IsNone())
				{
					SetBinding->RemoveAttributeFromSet(Item->Attribute);
				}
				SetBinding->RemoveAttributeFromSet(Item->Attribute);

				// TODO: Will be removing FAnimationAttributeIdentifier here in favour of just (AttributeName, AttributeType)
				// So this const cast will go away
				FAnimationAttributeIdentifier NewIdentifier(Item->Attribute.GetName(), INDEX_NONE, NAME_None, const_cast<UScriptStruct*>(InNewAttributeType.Get()));
				
				SetBinding->AddAttributeToSet(NewIdentifier, Item->SetName);
			});
}

TSharedPtr<SWidget> SAttributesList::ListView_OnContextMenuOpening()
{
	TArray<FListItemPtr> Selection = ListView->GetSelectedItems();
	if (Selection.IsEmpty())
	{
		return SNullWidget::NullWidget;
	}

	FMenuBuilder MenuBuilder(true, nullptr);
	FListItemPtr Selected = Selection[0]; // Single selection only

	MenuBuilder.AddMenuEntry(
			LOCTEXT("RenameAttribute_Label", "Rename"),
			LOCTEXT("RenameAttribute_Tooltip", "Renames the selected attribute"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Rename"),
			FUIAction(
				FExecuteAction::CreateLambda([this, Selected]()
					{
						Selected->EditableTextBlock.Pin()->EnterEditingMode();
					}
				)
			));
	
	if (Selected->SetName.IsNone())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveUnboundAttribute_Label", "Remove Attribute"),
			LOCTEXT("RemoveUnboundAttribute_Tooltip", "Removes the selected attribute"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
			FUIAction(
				FExecuteAction::CreateLambda([this, Selected]()
					{
						const FScopedTransaction Transaction(LOCTEXT("RemoveUnboundAttribute_Label", "Remove Attribute"));
						SetBinding->Modify();

						ensure(SetBinding->RemoveAttributeFromSet(Selected->Attribute));
					}
				)
			));
	}
	else
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("UnbindBoundAttribute_Label", "Unbind Attribute"),
			LOCTEXT("UnbindBoundAttribute_Tooltip", "Unbinds the selected attribute"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Unlink"),
			FUIAction(
				FExecuteAction::CreateLambda([this, Selected]()
					{
						const FScopedTransaction Transaction(LOCTEXT("UnbindBoundAttribute_Label", "Unbind Attribute"));
						SetBinding->Modify();

						ensure(SetBinding->RemoveAttributeFromSet(Selected->Attribute));
					}
				)
			));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("UnbindAndRemoveBoundAttribute_Label", "Unbind and Remove Attribute"),
			LOCTEXT("UnbindAndRemoveBoundAttribute_Tooltip", "Unbinds and removes the selected attribute"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
			FUIAction(
				FExecuteAction::CreateLambda([this, Selected]()
					{
						const FScopedTransaction Transaction(LOCTEXT("UnbindAndRemoveBoundAttribute_Label", "Unbind and Remove Attribute"));
						SetBinding->Modify();

						// First the attribute is removed from the set it is bound to and placed in the 'None' set.
						// Attributes in the 'None' set are all the unbound attributes that the user has added to the list of attributes.
						ensure(SetBinding->RemoveAttributeFromSet(Selected->Attribute));

						// Then removing attributes from the 'None' set actually removes them.
						ensure(SetBinding->RemoveAttributeFromSet(Selected->Attribute));
					}
				)
			));
	}

	return MenuBuilder.MakeWidget();
}

void SAttributesList::RegisterMenus()
{
	if (UToolMenus::Get()->IsMenuRegistered(MenuId_AddAttribute))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(this);
	UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuId_AddAttribute);

	UToolMenu* AddCurveMenu = Menu->AddSubMenu(FToolMenuOwner(), "AddCurve", "AddCurve", LOCTEXT("AddCurve", "Add Curve"));
	UToolMenu* AddAttributeMenu = Menu->AddSubMenu(FToolMenuOwner(), "AddAttribute", "AddAttribute", LOCTEXT("AddAttribute", "Add Attribute"));

	{
		FToolMenuSection& AttributeWidgetSection = AddAttributeMenu->AddSection("WidgetSection");

		AttributeWidgetSection.AddDynamicEntry(
			"NewAttribute",
			FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				if (const UAttributesListMenuContext* Context = Section.FindContext<UAttributesListMenuContext>())
				{
					TSharedPtr<SAttributesList> SetBindingWidget = Context->SetBindingWidget.Pin();
					ensure(SetBindingWidget);
			
					const TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding = SetBindingWidget->SetBinding;
			
					if (SetBinding.IsValid())
					{	
						FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
							"AddNewAttribute",
							LOCTEXT("AddNewAttribute_Label", "Add New Attribute"),
							LOCTEXT("AddNewAttribute_Tooltip", "Adds a new attribute"),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateRaw(SetBindingWidget.Get(), &SAttributesList::HandleAddNewAttribute)),
							EUserInterfaceActionType::Button
						);
						Section.AddEntry(Entry);
					}
				}
			}));
		

		
		AttributeWidgetSection.AddDynamicEntry(
			"AttributePicker",
			FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				if (const UAttributesListMenuContext* Context = Section.FindContext<UAttributesListMenuContext>())
				{
					FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

					TSharedPtr<SAttributesList> SetBindingWidget = Context->SetBindingWidget.Pin();
					ensure(SetBindingWidget);
					
					const TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding = SetBindingWidget->SetBinding;
					
					if (SetBinding.IsValid())
					{	
						FToolMenuEntry WidgetEntry = FToolMenuEntry::InitWidget(
							"AttributePicker",
							PersonaModule.CreateAttributePicker(
								SetBinding->GetSkeleton(),
								FOnAttributesPicked::CreateLambda([SetBinding, WeakSetBindingWidget = Context->SetBindingWidget](const TConstArrayView<FAnimationAttributeIdentifier> InSelectedAttributes)
									{
										const FScopedTransaction Transaction(LOCTEXT("AddAttributeToSet", "Add Attribute to Set"));
										SetBinding->Modify();

										for (const FAnimationAttributeIdentifier& Attribute : InSelectedAttributes)
										{
											SetBinding->AddAttributeToSet(Attribute, NAME_None);
										}

										FSlateApplication::Get().DismissAllMenus();
									}),
								true,
								false),
							FText(),
							true,
							false,
							true
						);

						Section.AddEntry(WidgetEntry);
					}
				}
			}));
	}

	{
		FToolMenuSection& CurveWidgetSection = AddCurveMenu->AddSection("WidgetSection");

		CurveWidgetSection.AddDynamicEntry(
			"CurvePicker",
			FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				if (const UAttributesListMenuContext* Context = Section.FindContext<UAttributesListMenuContext>())
				{
					FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

					TSharedPtr<SAttributesList> SetBindingWidget = Context->SetBindingWidget.Pin();
					ensure(SetBindingWidget);
					
					const TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding = SetBindingWidget->SetBinding;

					if (SetBinding.IsValid())
					{
						FToolMenuEntry WidgetEntry = FToolMenuEntry::InitWidget(
							"CurvePicker",
							PersonaModule.CreateCurvePicker(
								SetBinding->GetSkeleton(),
								FOnCurvePicked::CreateLambda([SetBinding, SetBindingWidget = Context->SetBindingWidget](const FName& InCurve)
									{
										const FScopedTransaction Transaction(LOCTEXT("AddCurvesToSet", "Add Curve to Set"));
										SetBinding->Modify();

										FAnimationAttributeIdentifier CurveAttrId(InCurve, INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct());
										SetBinding->AddAttributeToSet(CurveAttrId, NAME_None);

										FSlateApplication::Get().DismissAllMenus();
									})),
							FText(),
							true,
							false,
							true
						);

						Section.AddEntry(WidgetEntry);
					}
				}
			}));
	}
}

void SAttributesList::HandleAddNewAttribute()
{
	const FScopedTransaction Transaction(LOCTEXT("AddNewAttribute", "Add New Attribute"));

	int32_t Suffix = 0;
	FName NewAttributeName = "NewAttribute0";
	while (SetBinding->ContainsAttribute(NewAttributeName))
	{
		++Suffix;
		NewAttributeName = FName(FString::Format(TEXT("NewAttribute{0}"), { Suffix }));
	}
	
	SetBinding->AddAttributeToSet(FAnimationAttributeIdentifier(NewAttributeName, INDEX_NONE, NAME_None, FFloatAnimationAttribute::StaticStruct()), NAME_None);

	FSlateApplication::Get().DismissAllMenus();
}

void SAttributesList::HandleBindingsChanged()
{
	bListDirty = true;
}

TSharedRef<SWidget> SAttributesList::CreateAddAttributeWidget()
{
	UAttributesListMenuContext* ListContext = NewObject<UAttributesListMenuContext>();
	ListContext->SetBindingWidget = SharedThis(this);
	
	FToolMenuContext MenuContext;
	MenuContext.AddObject(ListContext);

	return UToolMenus::Get()->GenerateWidget(MenuId_AddAttribute, MenuContext);
}

#undef LOCTEXT_NAMESPACE