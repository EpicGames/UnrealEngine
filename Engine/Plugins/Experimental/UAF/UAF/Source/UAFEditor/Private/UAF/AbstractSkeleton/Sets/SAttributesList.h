// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimData/AttributeIdentifier.h"
#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "SAttributesList.generated.h"

class UAbstractSkeletonSetBinding;
class SInlineEditableTextBlock;

namespace UE::UAF::Editor
{

	class SAttributesList : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
	{
	public:
		SLATE_BEGIN_ARGS(SAttributesList) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding);

		void SetSetBinding(TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding);

	public:
		struct FListItem
		{
			FListItem(const FName InSetName, const FAnimationAttributeIdentifier& InAttribute);

			FAnimationAttributeIdentifier Attribute;
			FName SetName;
			TWeakPtr<SInlineEditableTextBlock> EditableTextBlock;
		};

		using FListItemPtr = TSharedPtr<FListItem>;

	private:
		/** SCompoundWidget */
		virtual void Tick(const FGeometry& AllocatedGeometry, const double InCurrentTime, const float InDeltaTime) override;

		/** FSelfRegisteringEditorUndoClient interface */
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;

		void RepopulateListData();

		void RegisterMenus();

		void HandleAddNewAttribute();

		// Called when the bindings inside the set binding asset we're editing have been modified
		void HandleBindingsChanged();

		TSharedRef<ITableRow> ListView_OnGenerateRow(FListItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable);

		TSharedPtr<SWidget> ListView_OnContextMenuOpening();
		
		TSharedRef<SWidget> CreateAddAttributeWidget();

	private:
		TArray<FListItemPtr> ListItems;

		TSharedPtr<SListView<FListItemPtr>> ListView;

		TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding;

		FDelegateHandle OnBindingsChangedHandle;

		bool bListDirty = false;
	};

}

UCLASS()
class UAttributesListMenuContext : public UObject
{
	GENERATED_BODY()
	
public:
	TWeakPtr<UE::UAF::Editor::SAttributesList> SetBindingWidget;
};