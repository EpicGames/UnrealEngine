// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/SSetBinding.h"

#include "AttributeDragDrop.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "SkeletonTreeDragDrop.h"
#include "Styling/StyleColors.h" 
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/AbstractSkeleton/Sets/Commands.h"
#include "UAF/AbstractSkeleton/Sets/SetDragDrop.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SSetBinding"

namespace UE::UAF::Editor
{
	void SSetBinding::Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding)
	{
		SetBinding = InSetBinding;

		if (SetBinding.IsValid())
		{
			OnBindingsChangedHandle = SetBinding->RegisterOnBindingsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SSetBinding::HandleBindingsChanged));
		}

		ChildSlot
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda([this]()
				{
					if (!SetBinding.IsValid())
					{
						return 0;
					}

					if (!SetBinding->GetSetCollection())
					{
						return 1;
					}

					return 2;
				})
				+ SWidgetSwitcher::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Text(LOCTEXT("NoSetBindingSelected", "No Set Binding Selected"))
						]
					]
				]
				+ SWidgetSwitcher::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Text(LOCTEXT("NoSetCollectionSelected", "No Set Collection Selected"))
						]
					]
				]
				+ SWidgetSwitcher::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(1.0f, 1.0f))
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.Padding(FMargin(1.0f, 1.0f))
							[
								SAssignNew(SearchBox, SSearchBox)
									.SelectAllTextWhenFocused(true)
									.OnTextChanged_Lambda([](const FText& InText) { /* TODO: Implement */ })
									.HintText(LOCTEXT("SearchBox_Hint", "Search Sets..."))
							]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(FMargin(1.0f, 1.0f))
					[
						SAssignNew(TreeView, STreeView<TSharedPtr<ITreeItem>>)
							.SelectionMode(ESelectionMode::Multi)
							.HighlightParentNodesForSelection(true)
							.TreeItemsSource(&RootItems)
							.OnGenerateRow(this, &SSetBinding::TreeView_OnGenerateRow)
							.OnGetChildren(this, &SSetBinding::TreeView_OnGetChildren)
							.OnContextMenuOpening(this, &SSetBinding::TreeView_OnContextMenuOpening)
							.HeaderRow
							(
								SNew(SHeaderRow)
								+ SHeaderRow::Column(FName("SetHierarchy"))
								.FillWidth(0.5f)
								.DefaultLabel(LOCTEXT("TreeView_Header", "Set Bindings"))
							)
					]
				]
			];

		RepopulateTreeData();
		ExpandAllTreeItems();
		BindCommands();
	}

	void SSetBinding::Tick(const FGeometry& AllocatedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		if (bTreeDirty)
		{
			RepopulateTreeData();
			bTreeDirty = false;
		}
	}

	FReply SSetBinding::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	TSharedRef<ITableRow> SSetBinding::TreeView_OnGenerateRow(TSharedPtr<ITreeItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->GenerateRow(OwnerTable);
	}

	void SSetBinding::TreeView_OnGetChildren(TSharedPtr<ITreeItem> InItem, TArray<TSharedPtr<ITreeItem>>& OutChildren)
	{
		InItem->GetChildren(OutChildren);
	}

	TSharedPtr<SWidget> SSetBinding::TreeView_OnContextMenuOpening()
	{
		const Editor::FSetBindingEditorCommands& Commands = Editor::FSetBindingEditorCommands::Get();

		FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

		MenuBuilder.AddMenuEntry(Commands.UnbindSelection);

		return MenuBuilder.MakeWidget();
	}

	bool SSetBinding::CanUnbindSelection() const
	{
		TArray<TSharedPtr<ITreeItem>> Selection = TreeView->GetSelectedItems();
		
		for (const TSharedPtr<ITreeItem>& Selected : Selection)
		{
			if (Selected->GetType() == FTreeItem_Bone::StaticGetType() || Selected->GetType() == FTreeItem_Attribute::StaticGetType())
			{
				return true;
			}
			else
			{
				TSharedPtr<FTreeItem_Set> SetItem = StaticCastSharedPtr<FTreeItem_Set>(Selected);
				if (!SetItem->Children.IsEmpty())
				{
					return true;
				}
			}
		}

		return false;
	}

	void SSetBinding::HandleUnbindSelection()
	{
		if (SetBinding.IsValid())
		{
			const FScopedTransaction Transaction(LOCTEXT("UnbindSelection", "Unbind Selection"));
		
			TArray<TSharedPtr<ITreeItem>> Selection = TreeView->GetSelectedItems();

			// Unbind all bones/attributes first
			for (const TSharedPtr<ITreeItem>& Selected : Selection)
			{
				if (Selected->GetType() == FTreeItem_Bone::StaticGetType())
				{
					TSharedPtr<FTreeItem_Bone> BoneItem = StaticCastSharedPtr<FTreeItem_Bone>(Selected);
					SetBinding->RemoveBoneFromSet(BoneItem->Binding.BoneName);
				}
				else if (Selected->GetType() == FTreeItem_Attribute::StaticGetType())
				{
					TSharedPtr<FTreeItem_Attribute> AttributeItem = StaticCastSharedPtr<FTreeItem_Attribute>(Selected);
					SetBinding->RemoveAttributeFromSet(AttributeItem->Binding.Attribute);
				}
			}

			// In a second pass, unbind all sets
			for (const TSharedPtr<ITreeItem>& Selected : Selection)
			{
				if (Selected->GetType() == FTreeItem_Set::StaticGetType())
				{
					TSharedPtr<FTreeItem_Set> SetItem = StaticCastSharedPtr<FTreeItem_Set>(Selected);
					SetBinding->RemoveAllFromSet(SetItem->Set.SetName);
				}
			}
		}
	}

	void SSetBinding::PostUndo(bool bSuccess)
	{
		RepopulateTreeData();
	}

	void SSetBinding::PostRedo(bool bSuccess)
	{
		RepopulateTreeData();
	}

	void SSetBinding::SetSetBinding(TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding)
	{
		if (SetBinding.IsValid())
		{
			SetBinding->UnregisterOnBindingsChanged(OnBindingsChangedHandle);
		}

		SetBinding = InSetBinding;

		if (SetBinding.IsValid())
		{
			OnBindingsChangedHandle = SetBinding->RegisterOnBindingsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SSetBinding::HandleBindingsChanged));
		}

		RepopulateTreeData();
	}

	void SSetBinding::RepopulateTreeData()
	{
		// Save the current expanded state of each tree item
		// - Only Set items can contain children so only Set items can be expanded/collapsed
		TSet<FName> ClosedSetCollections;
		{
			for (TSharedPtr<ITreeItem> TreeItem : GetAllTreeItems())
			{
				if (TreeItem->GetType() == FTreeItem_Set::StaticGetType() && !TreeView->IsItemExpanded(TreeItem))
				{
					TSharedPtr<FTreeItem_Set> SetTreeItem = StaticCastSharedPtr<FTreeItem_Set>(TreeItem);
					ClosedSetCollections.Add(SetTreeItem->Set.SetName);
				}
			}
		}
			
		// Rebuild tree data
		RootItems.Reset();

		int32 SetIndex = 0;

		if (SetBinding.IsValid())
		{
			TMap<FName, TSharedPtr<FTreeItem_Set>> TreeItemMap;
				
			if (SetBinding->GetSetCollection())
			{
				for (const FAbstractSkeletonSet& Set : SetBinding->GetSetCollection()->GetSetHierarchy())
				{
					TSharedPtr<FTreeItem_Set> SetItem = MakeShared<FTreeItem_Set>(*this, Set, false, SetIndex++);

					if (Set.ParentSetName == NAME_None)
					{
						RootItems.Add(SetItem);
					}
					else
					{
						ensure(TreeItemMap.Contains(Set.ParentSetName));
							
						TSharedPtr<FTreeItem_Set> ParentItem = TreeItemMap[Set.ParentSetName];
						ParentItem->Children.Add(SetItem);
					}

					TreeItemMap.Add(Set.SetName, SetItem);
				}
			}

			const USkeleton* Skeleton = SetBinding->GetSkeleton();
			const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

			for (const FAbstractSkeleton_BoneBinding& Binding : SetBinding->GetBoneBindings())
			{
				const int32 BoneIndex = RefSkeleton.FindBoneIndex(Binding.BoneName);
				check(RefSkeleton.IsValidIndex(BoneIndex));

				TSharedPtr<FTreeItem_Bone> BindingItem = MakeShared<FTreeItem_Bone>(*this, Binding, BoneIndex);

				if (TreeItemMap.Contains(Binding.SetName))
				{
					TSharedPtr<FTreeItem_Set> SetItem = TreeItemMap[Binding.SetName];
					SetItem->Children.Insert(BindingItem, 0);
				}
				else
				{
					// This binding binds to a set that no longer exists in the set collection
					// These bindings are not auto-deleted to avoid implicit destructive edits
					// This binding can be fixed by re-adding the set in the set collection or
					// by removing this binding. Show this binding as an orphan.

					TSharedPtr<FTreeItem_Set> DeletedSet = MakeShared<FTreeItem_Set>(*this, FAbstractSkeletonSet(Binding.SetName, NAME_None), true, SetIndex++);
					DeletedSet->Children.Insert(BindingItem, 0);

					TreeItemMap.Add(Binding.SetName, DeletedSet);
					RootItems.Add(DeletedSet);
				}
			}

			for (const FAbstractSkeleton_AttributeBinding& Binding : SetBinding->GetAttributeBindings())
			{
				TSharedPtr<FTreeItem_Attribute> BindingItem = MakeShared<FTreeItem_Attribute>(*this, Binding);

				if (TreeItemMap.Contains(Binding.SetName))
				{
					TSharedPtr<FTreeItem_Set> SetItem = TreeItemMap[Binding.SetName];
					SetItem->Children.Insert(BindingItem, 0);
				}
			}
		}

		for (TSharedPtr<ITreeItem> RootItem : RootItems)
		{
			RootItem->SortChildren();
		}

		TreeView->RebuildList();

		// Restore expanded/collapsed state
		{
			for (TSharedPtr<ITreeItem> TreeItem : GetAllTreeItems())
			{
				if (TreeItem->GetType() == FTreeItem_Set::StaticGetType())
				{
					const TSharedPtr<FTreeItem_Set> SetTreeItem = StaticCastSharedPtr<FTreeItem_Set>(TreeItem);
					const bool bShouldExpand = !ClosedSetCollections.Contains(SetTreeItem->Set.SetName);
					TreeView->SetItemExpansion(TreeItem, bShouldExpand);
				}
			}
		}
	}

	void SSetBinding::HandleBindingsChanged()
	{
		bTreeDirty = true;
	}

	void SSetBinding::ExpandAllTreeItems()
	{
		for (TSharedPtr<ITreeItem> TreeItem : GetAllTreeItems())
		{
			TreeView->SetItemExpansion(TreeItem, true);
		}
	}

	void SSetBinding::BindCommands()
	{
		CommandList = MakeShareable(new FUICommandList);

		const Editor::FSetBindingEditorCommands& Commands = Editor::FSetBindingEditorCommands::Get();

		CommandList->MapAction(Commands.UnbindSelection,
			FExecuteAction::CreateSP(this, &SSetBinding::HandleUnbindSelection),
			FCanExecuteAction::CreateSP(this, &SSetBinding::CanUnbindSelection));
	}

	TArray<TSharedPtr<SSetBinding::ITreeItem>> SSetBinding::GetAllTreeItems()
	{
		TArray<TSharedPtr<ITreeItem>> AllItems;
		AllItems.Append(RootItems);

		for (int32 Index = 0; Index < AllItems.Num(); ++Index)
		{
			AllItems[Index]->GetChildren(AllItems);
		}

		return AllItems;
	}
	
	void SSetBinding::FTreeItem_Set::GetChildren(TArray<TSharedPtr<ITreeItem>>& OutChildren)
	{
		OutChildren.Append(Children);
	}

	TSharedRef<ITableRow> SSetBinding::FTreeItem_Set::GenerateRow(const TSharedRef<STableViewBase>& OwnerTable)
	{
		if (bIsDeletedSet)
		{
			return SNew(STableRow<TSharedPtr<ITreeItem>>, OwnerTable)
				.ToolTipText(LOCTEXT("OrphanSetLabel", "This set no longer exists in the Set Collection and will not be used. Attributes bound to this set should be removed or the set should be recreated."))
				.Style(FAppStyle::Get(), "TableView.AlternatingRow")
				.ShowWires(true)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(3.0f))
					[
						SNew(SImage)
						.ColorAndOpacity(EStyleColor::Error)
						.Image(FAppStyle::Get().GetBrush("ClassIcon.GroupActor"))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.AutoWidth()
					.Padding(3.0f)
					[
						SNew(STextBlock)
						.ColorAndOpacity(EStyleColor::Error)
						.Text(FText::FromName(Set.SetName))
					]
				];
		}
	
		return SNew(STableRow<TSharedPtr<ITreeItem>>, OwnerTable)
			.Style(FAppStyle::Get(), "TableView.AlternatingRow")
			.OnDragDetected_Lambda([this](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
				{
					if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
					{
						const TSharedRef<Editor::FSetDragDropOp> DragDropOp = Editor::FSetDragDropOp::New({ Set });
						return FReply::Handled().BeginDragDrop(DragDropOp);
					}

					return FReply::Unhandled();
				})
			.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<ITreeItem> TargetItem) -> TOptional<EItemDropZone>
				{
					using namespace UE::UAF;

					TOptional<EItemDropZone> ReturnedDropZone;

					if (const TSharedPtr<FSkeletonTreeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSkeletonTreeDragDropOp>())
					{
						ReturnedDropZone = EItemDropZone::OntoItem;
					}

					if (const TSharedPtr<FAttributeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FAttributeDragDropOp>())
					{
						ReturnedDropZone = EItemDropZone::OntoItem;
					}

					return ReturnedDropZone;
				})
			.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<ITreeItem> TargetTreeItem) -> FReply
				{
					if (const TSharedPtr<FSkeletonTreeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSkeletonTreeDragDropOp>())
					{
						TSharedPtr<FTreeItem_Set> TargetSetTreeItem = StaticCastSharedPtr<FTreeItem_Set>(TargetTreeItem);

						if (SetBindingWidget.SetBinding.IsValid())
						{
							const FScopedTransaction Transaction(LOCTEXT("BindAttributeToSet", "Bind Attribute to Set"));
							
							for (const TSharedPtr<SSetsSkeletonTree::FTreeItem>& Item : DragDropOp->Items)
							{
								SetBindingWidget.SetBinding->RemoveBoneFromSet(Item->BoneName);
								SetBindingWidget.SetBinding->AddBoneToSet(Item->BoneName, TargetSetTreeItem->Set.SetName);
							}
						}

						return FReply::Handled();
					}

					if (const TSharedPtr<FAttributeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FAttributeDragDropOp>())
					{
						TSharedPtr<FTreeItem_Set> TargetSetTreeItem = StaticCastSharedPtr<FTreeItem_Set>(TargetTreeItem);

						if (SetBindingWidget.SetBinding.IsValid())
						{
							const FScopedTransaction Transaction(LOCTEXT("BindAttributeToSet", "Bind Attribute to Set"));
							
							for (const FAnimationAttributeIdentifier& Attribute : DragDropOp->Attributes)
							{
								SetBindingWidget.SetBinding->RemoveAttributeFromSet(Attribute);
								SetBindingWidget.SetBinding->AddAttributeToSet(Attribute, TargetSetTreeItem->Set.SetName);
							}
						}

						return FReply::Handled();
					}
					
					return FReply::Unhandled();
				})
			.ShowWires(true)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(3.0f))
					[
						SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("ClassIcon.GroupActor"))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.AutoWidth()
					.Padding(3.0f)
					[
						SNew(STextBlock)
							.Text(FText::FromName(Set.SetName))
					]
			];
	}

	void SSetBinding::FTreeItem_Bone::GetChildren(TArray<TSharedPtr<ITreeItem>>& OutChildren)
	{
		// Bone items don't have children, do nothing
	}

	void SSetBinding::FTreeItem_Attribute::GetChildren(TArray<TSharedPtr<ITreeItem>>& OutChildren)
	{
		// Attribute items don't have children, do nothing
	}

	TSharedRef<ITableRow> SSetBinding::FTreeItem_Bone::GenerateRow(const TSharedRef<STableViewBase>& OwnerTable)
	{
		const FSlateBrush* Icon = FAppStyle::Get().GetBrush("SkeletonTree.Bone");	

		return SNew(STableRow<TSharedPtr<ITreeItem>>, OwnerTable)
			.Style(FAppStyle::Get(), "TableView.AlternatingRow")
			.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<ITreeItem> TargetItem) -> TOptional<EItemDropZone>
				{
					using namespace UE::UAF;

					TOptional<EItemDropZone> ReturnedDropZone;

					const TSharedPtr<FSkeletonTreeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSkeletonTreeDragDropOp>();
					if (DragDropOp.IsValid())
					{
						ReturnedDropZone = EItemDropZone::OntoItem;
					}

					return ReturnedDropZone;
				})
			.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<ITreeItem> TargetTreeItem) -> FReply
				{
					const TSharedPtr<FSkeletonTreeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSkeletonTreeDragDropOp>();
					if (DragDropOp.IsValid() && SetBindingWidget.SetBinding.IsValid())
					{
						const FScopedTransaction Transaction(LOCTEXT("AddAttributeToSet", "Add Attribute to Set"));

						for (const TSharedPtr<SSetsSkeletonTree::FTreeItem>& Item : DragDropOp->Items)
						{
							SetBindingWidget.SetBinding->RemoveBoneFromSet(Item->BoneName);
							SetBindingWidget.SetBinding->AddBoneToSet(Item->BoneName, Binding.SetName);
						}

						return FReply::Handled();
					}
					return FReply::Unhandled();
				})
			.ShowWires(true)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(3.0f))
					[
						SNew(SImage)
							.Image(Icon)
							.ColorAndOpacity(FLinearColor(FColor(31, 228, 75)))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.AutoWidth()
					.Padding(3.0f)
					[
						SNew(STextBlock)
							.Text(FText::FromName(Binding.BoneName))
					]

			];
	}

	TSharedRef<ITableRow> SSetBinding::FTreeItem_Attribute::GenerateRow(const TSharedRef<STableViewBase>& OwnerTable)
	{
		const FSlateBrush* Icon = FAppStyle::Get().GetBrush("AnimGraph.Attribute.Attributes.Icon");

		return SNew(STableRow<TSharedPtr<ITreeItem>>, OwnerTable)
			.Style(FAppStyle::Get(), "TableView.AlternatingRow")
			.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<ITreeItem> TargetItem) -> TOptional<EItemDropZone>
				{
					using namespace UE::UAF;

					TOptional<EItemDropZone> ReturnedDropZone;

					const TSharedPtr<FSkeletonTreeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSkeletonTreeDragDropOp>();
					if (DragDropOp.IsValid())
					{
						ReturnedDropZone = EItemDropZone::OntoItem;
					}

					return ReturnedDropZone;
				})
			.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<ITreeItem> TargetTreeItem) -> FReply
				{
					const TSharedPtr<FSkeletonTreeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSkeletonTreeDragDropOp>();
					if (DragDropOp.IsValid() && SetBindingWidget.SetBinding.IsValid())
					{
						const FScopedTransaction Transaction(LOCTEXT("AddAttributeToSet", "Add Attribute to Set"));

						for (const TSharedPtr<SSetsSkeletonTree::FTreeItem>& Item : DragDropOp->Items)
						{
							SetBindingWidget.SetBinding->RemoveBoneFromSet(Item->BoneName);
							SetBindingWidget.SetBinding->AddBoneToSet(Item->BoneName, Binding.SetName);
						}

						return FReply::Handled();
					}
					return FReply::Unhandled();
				})
			.ShowWires(true)
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(1.0f, 1.0f))
					[
						SNew(SImage)
							.Image(Icon)
							.ColorAndOpacity(FLinearColor(FColor(31, 228, 75)))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.AutoWidth()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
							.Text(FText::FromName(Binding.Attribute.GetName()))
					]

			];
	}

	void SSetBinding::FTreeItem_Set::SortChildren()
	{
		for (const auto& Child : Children)
		{
			Child->SortChildren();
		}

		Algo::StableSort(Children, [](const TSharedPtr<ITreeItem>& A, TSharedPtr<ITreeItem> B) -> bool
			{
				if (A->GetType() != B->GetType())
				{
					return static_cast<uint8>(A->GetType()) < static_cast<uint8>(B->GetType());
				}

				switch (A->GetType())
				{
					case ETreeItem::Set:
					{
						TSharedPtr<FTreeItem_Set> SetA = StaticCastSharedPtr<FTreeItem_Set>(A);
						TSharedPtr<FTreeItem_Set> SetB = StaticCastSharedPtr<FTreeItem_Set>(B);

						return SetA->SetIndex < SetB->SetIndex;
					}
					case ETreeItem::Bone:
					{
						TSharedPtr<FTreeItem_Bone> BoneA = StaticCastSharedPtr<FTreeItem_Bone>(A);
						TSharedPtr<FTreeItem_Bone> BoneB = StaticCastSharedPtr<FTreeItem_Bone>(B);

						return BoneA->BoneIndex < BoneB->BoneIndex;
					}
					case ETreeItem::Attribute:
					{
						TSharedPtr<FTreeItem_Attribute> AttributeA = StaticCastSharedPtr<FTreeItem_Attribute>(A);
						TSharedPtr<FTreeItem_Attribute> AttributeB = StaticCastSharedPtr<FTreeItem_Attribute>(B);

						return AttributeA->Binding.Attribute.GetName().Compare(AttributeB->Binding.Attribute.GetName()) < 0;
					}
				}

				return false;
			});
	}

}

#undef LOCTEXT_NAMESPACE