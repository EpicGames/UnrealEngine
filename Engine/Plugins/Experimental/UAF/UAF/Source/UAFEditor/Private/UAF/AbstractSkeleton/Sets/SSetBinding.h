// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimData/AttributeIdentifier.h"
#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

class FUICommandList;
class SSearchBox;
class UAbstractSkeletonSetBinding;

namespace UE::UAF::Editor
{
	class SSetBinding : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
	{
		SLATE_BEGIN_ARGS(SSetBinding) {}
		SLATE_END_ARGS()

	public:

		// Sorting here determines ordering in the tree view
		enum class ETreeItem : uint8
		{
			Bone,
			Attribute,
			Set
		};

		struct ITreeItem
		{
			ITreeItem(SSetBinding& InSetBindingWidget)
				: SetBindingWidget(InSetBindingWidget)
			{
			}
			
			virtual ~ITreeItem() = default;
			virtual ETreeItem GetType() const = 0;
			virtual void GetChildren(TArray<TSharedPtr<ITreeItem>>& OutChildren) = 0;
			virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) = 0;
			virtual void SortChildren() = 0;

			SSetBinding& SetBindingWidget;
		};

		struct FTreeItem_Set : public ITreeItem
		{
			FTreeItem_Set(SSetBinding& InSetBindingWidget, FAbstractSkeletonSet InSet, bool bInIsDeletedSet, int32 InSetIndex)
				: ITreeItem(InSetBindingWidget)
				, Set(InSet)
				, bIsDeletedSet(bInIsDeletedSet)
				, SetIndex(InSetIndex)
			{
			}
			
			virtual ~FTreeItem_Set() override = default;
			
			static ETreeItem StaticGetType() { return ETreeItem::Set; }
			
			virtual ETreeItem GetType() const override { return StaticGetType(); }
			virtual void GetChildren(TArray<TSharedPtr<ITreeItem>>& OutChildren) override;
			virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) override;
			virtual void SortChildren() override;

			FAbstractSkeletonSet Set;
			TArray<TSharedPtr<ITreeItem>> Children;
			bool bIsDeletedSet;
			int32 SetIndex;
		};

		struct FTreeItem_Bone : public ITreeItem
		{
			FTreeItem_Bone(SSetBinding& InSetBindingWidget, FAbstractSkeleton_BoneBinding InBinding, int32 InBoneIndex)
				: ITreeItem(InSetBindingWidget)
				, Binding(InBinding)
				, BoneIndex(InBoneIndex)
			{
			}
			
			virtual ~FTreeItem_Bone() override = default;

			static ETreeItem StaticGetType() { return ETreeItem::Bone; }
			
			virtual ETreeItem GetType() const override { return StaticGetType(); }
			virtual void GetChildren(TArray<TSharedPtr<ITreeItem>>& OutChildren) override;
			virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) override;
			virtual void SortChildren() override {};

			FAbstractSkeleton_BoneBinding Binding;
			int32 BoneIndex;
		};
		
		struct FTreeItem_Attribute : public ITreeItem
		{
			FTreeItem_Attribute(SSetBinding& InSetBindingWidget, FAbstractSkeleton_AttributeBinding InBinding)
				: ITreeItem(InSetBindingWidget)
				, Binding(InBinding)
			{
			}
			
			virtual ~FTreeItem_Attribute() override = default;

			static ETreeItem StaticGetType() { return ETreeItem::Attribute; }
			
			virtual ETreeItem GetType() const override { return StaticGetType(); }
			virtual void GetChildren(TArray<TSharedPtr<ITreeItem>>& OutChildren) override;
			virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& OwnerTable) override;
			virtual void SortChildren() override {};

			FAbstractSkeleton_AttributeBinding Binding;
		};

		void Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding);

		void SetSetBinding(TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding);

	private:
		/** SCompoundWidget */
		virtual void Tick(const FGeometry& AllocatedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

		/** FSelfRegisteringEditorUndoClient interface */
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;

		void RepopulateTreeData();

		void BindCommands();

		void ExpandAllTreeItems();

		TArray<TSharedPtr<ITreeItem>> GetAllTreeItems();

		// Called when the bindings inside the set binding asset we're editing have been modified
		void HandleBindingsChanged();

		TSharedRef<ITableRow> TreeView_OnGenerateRow(TSharedPtr<ITreeItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

		void TreeView_OnGetChildren(TSharedPtr<ITreeItem> InItem, TArray<TSharedPtr<ITreeItem>>& OutChildren);

		TSharedPtr<SWidget> TreeView_OnContextMenuOpening();

		bool CanUnbindSelection() const;

		void HandleUnbindSelection();

	public:
		TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding;

	private:
		TSharedPtr<STreeView<TSharedPtr<ITreeItem>>> TreeView;

		TArray<TSharedPtr<ITreeItem>> RootItems;

		TSharedPtr<SSearchBox> SearchBox;

		TSharedPtr<FUICommandList> CommandList;

		FDelegateHandle OnBindingsChangedHandle;

		bool bTreeDirty = false;
	};

}
