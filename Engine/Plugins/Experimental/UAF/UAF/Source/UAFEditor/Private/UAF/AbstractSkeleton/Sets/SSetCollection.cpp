// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/SSetCollection.h"

#include "SetDragDrop.h"
#include "SPositiveActionButton.h"
#include "Framework/Commands/GenericCommands.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SSetCollection"

namespace UE::UAF::Editor
{	
	void SSetCollection::Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetCollection> InSetCollection)
	{
		SetCollection = InSetCollection;
		OnSetsChangedHandle = SetCollection->RegisterOnSetsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SSetCollection::OnSetsChanged));

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(1.0f, 1.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(1.0f, 1.0f))
				[
					SNew(SPositiveActionButton)
					.OnClicked_Lambda([this]()
						{
							TreeView->HandleAddSet();
							return FReply::Handled();
						})
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
					.Text(LOCTEXT("AddButton_Text", "Add Set"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(FMargin(1.0f, 1.0f))
				[
					SAssignNew(SearchBox, SSearchBox)
					.SelectAllTextWhenFocused(true)
					.OnTextChanged_Lambda([this](const FText& InText)
					{
						TreeView->SetFilterText(InText);
					})
					.HintText(LOCTEXT("SearchBox_Hint", "Search Sets..."))
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(FMargin(1.0f, 1.0f))
			[
				SAssignNew(TreeView, SSetCollectionTreeView, SetCollection)
			]
		];
	}

	SSetCollection::~SSetCollection()
	{
		SetCollection->UnregisterOnSetsChanged(OnSetsChangedHandle);
	}
	
	void SSetCollection::OnSetsChanged() const
	{
		SearchBox->SetSearchText(FText::GetEmpty());
		TreeView->SetFilterText(FText::GetEmpty());
	}
}

#undef LOCTEXT_NAMESPACE