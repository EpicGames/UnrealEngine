// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/AbstractSkeleton/Sets/SSetCollectionTreeView.h"
#include "Widgets/SCompoundWidget.h"

class SSearchBox;

namespace UE::UAF::Editor
{
	class SSetCollection : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
	{
		SLATE_BEGIN_ARGS(SSetCollection) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetCollection> InSetCollection);
		virtual ~SSetCollection() override;

		void OnSetsChanged() const;
		
	private:
		TWeakObjectPtr<UAbstractSkeletonSetCollection> SetCollection;
		TSharedPtr<SSetCollectionTreeView> TreeView;
		TSharedPtr<SSearchBox> SearchBox;

		FDelegateHandle OnSetsChangedHandle;
	};
}
