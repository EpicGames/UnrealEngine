// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/SSetCollectionTreeView.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "SetDragDrop.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SSetCollectionTreeView"

namespace UE::UAF::Editor
{
	void SSetCollectionTreeView::Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetCollection> InSetCollection)
	{
		SetCollection = InSetCollection;
		OnSetsChangedHandle = SetCollection->RegisterOnSetsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SSetCollectionTreeView::OnSetsChanged));
		
		STreeView<FSetCollectionTreeItemPtr>::FArguments SuperArgs;
		SuperArgs.SelectionMode(ESelectionMode::Multi);
		SuperArgs.HighlightParentNodesForSelection(true);
		SuperArgs.TreeItemsSource(&RootItems);
		SuperArgs.OnGenerateRow(this, &SSetCollectionTreeView::TreeView_OnGenerateRow);
		SuperArgs.OnGetChildren(this, &SSetCollectionTreeView::TreeView_OnGetChildren);
		SuperArgs.OnContextMenuOpening(this, &SSetCollectionTreeView::TreeView_OnContextMenuOpening);
		SuperArgs.OnItemScrolledIntoView(this, &SSetCollectionTreeView::TreeView_OnItemScrolledIntoView);
		SuperArgs.OnMouseButtonDoubleClick(this, &SSetCollectionTreeView::TreeView_OnMouseButtonDoubleClick);
		SuperArgs.HeaderRow
		(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(FName("SetHierarchy"))
			.FillWidth(1.0f)
			.DefaultLabel(LOCTEXT("TreeView_Header", "Set Hierarchy"))
		);

		STreeView<FSetCollectionTreeItemPtr>::Construct(SuperArgs);
		
		BindCommands();

		// Build the tree items on construction
		bNeedsRebuilding = true;
	}

	SSetCollectionTreeView::~SSetCollectionTreeView()
	{
		SetCollection->UnregisterOnSetsChanged(OnSetsChangedHandle);
	}

	TSharedRef<ITableRow> SSetCollectionTreeView::TreeView_OnGenerateRow(FSetCollectionTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedPtr<SInlineEditableTextBlock> InlineWidget;

		TSharedRef<STableRow<FSetCollectionTreeItemPtr>> RowWidget = SNew(STableRow<FSetCollectionTreeItemPtr>, OwnerTable)
			.Visibility_Lambda([InItem]()
			{
				return InItem->bVisible ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.Style(&FAppStyle().GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"))
			.OnDragDetected_Lambda([this](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
				{
					if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
					{
						TArray<FAbstractSkeletonSet> Sets;
						for (const auto& Selected : GetSelectedItems())
						{
							Sets.Add(Selected->Set);
						}
						const TSharedRef<FSetDragDropOp> DragDropOp = FSetDragDropOp::New(MoveTemp(Sets));
						return FReply::Handled().BeginDragDrop(DragDropOp);
					}

					return FReply::Unhandled();
				})
			.OnDragEnter_Lambda([InItem, this](const FDragDropEvent& DragDropEvent)
			{
				const TSharedPtr<FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSetDragDropOp>();
				if (DragDropOp.IsValid())
				{
					DragDropOp->SetOperation(FSetDragDropOp::EOperation::Reparent, InItem->Set.SetName);
				}
			})
			.OnDragLeave_Lambda([this](const FDragDropEvent& DragDropEvent)
			{
				const TSharedPtr<FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSetDragDropOp>();
				if (DragDropOp.IsValid())
				{
					DragDropOp->ClearOperation();
				}
			})
			.OnCanAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FSetCollectionTreeItemPtr TargetTreeItem) -> TOptional<EItemDropZone>
			{
				using namespace UE::UAF;

				TOptional<EItemDropZone> ReturnedDropZone;

				const TSharedPtr<FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSetDragDropOp>();
				if (DragDropOp.IsValid())
				{
					bool bAnyDraggedSetAncestorOfTarget = false;

					for (const FAbstractSkeletonSet& DraggedSet : DragDropOp->GetDraggedSets())
					{
						bAnyDraggedSetAncestorOfTarget |= SetCollection->IsDescendantOf(TargetTreeItem->Set.SetName, DraggedSet.SetName);
					}

					if (!bAnyDraggedSetAncestorOfTarget)
					{
						ReturnedDropZone = EItemDropZone::OntoItem;
					}
				}

				return ReturnedDropZone;
			})
			.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FSetCollectionTreeItemPtr TargetTreeItem) -> FReply
				{
					const TSharedPtr<FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSetDragDropOp>();
					if (DragDropOp.IsValid())
					{
						const FScopedTransaction Transaction(LOCTEXT("ReparentSets", "Reparent Set(s)"));
						SetCollection->Modify();

						for (const FAbstractSkeletonSet& Set : DragDropOp->GetDraggedSets())
						{
							SetCollection->ReparentSet(Set.SetName, TargetTreeItem->Set.SetName);
						}

						return FReply::Handled();
					}

					return FReply::Unhandled();
				})
			.ShowWires(true)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("ClassIcon.GroupActor"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f)
				[
					SAssignNew(InlineWidget, SInlineEditableTextBlock)
					.HighlightText_Lambda([this]()
						{
							return FilterText;
						})
					.Text(FText::FromName(InItem->Set.SetName))
					.OnVerifyTextChanged_Lambda([InItem, this](const FText& InText, FText& OutErrorMessage)
						{
							const FName OldName = InItem->Set.SetName;
							const FName NewName = FName(InText.ToString());

							if (OldName == NewName)
							{
								return true;
							}

							if (NewName.IsNone())
							{
								OutErrorMessage = LOCTEXT("SetNameInvalid", "Set name cannot be 'None'");
								return false;
							}

							const bool bSetNameAlreadyExists = SetCollection->HasSet(NewName);
							if (bSetNameAlreadyExists)
							{
								OutErrorMessage = LOCTEXT("SetNameTaken", "Set name already taken");
								return false;
							}
								
							return true;
						})
					.OnTextCommitted_Lambda([InItem, this](const FText& InText, ETextCommit::Type CommitInfo)
						{
							const FName OldName = InItem->Set.SetName;
							const FName NewName = FName(InText.ToString());

							RenameSet(OldName, NewName);	
						})
				]
			];

		InItem->OnRequestRename.BindSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

		return RowWidget;
	}

	void SSetCollectionTreeView::TreeView_OnGetChildren(FSetCollectionTreeItemPtr InItem, TArray<FSetCollectionTreeItemPtr>& OutChildren)
	{
		OutChildren = InItem->Children;
	}

	TSharedPtr<SWidget> SSetCollectionTreeView::TreeView_OnContextMenuOpening()
	{
		FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AddSet_Label", "Add Set"),
			LOCTEXT("AddSet_Tooltip", "Add a new set to the hierarchy"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
			FUIAction(FExecuteAction::CreateSP(this, &SSetCollectionTreeView::HandleAddSet)));
		
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);

		return MenuBuilder.MakeWidget();
	}

	void SSetCollectionTreeView::TreeView_OnItemScrolledIntoView(FSetCollectionTreeItemPtr InItem, const TSharedPtr<ITableRow>& InWidget)
	{
		if (RequestedSetItemToRename)
		{
			RequestedSetItemToRename->OnRequestRename.ExecuteIfBound();
			RequestedSetItemToRename = nullptr;
		}
	}

	void SSetCollectionTreeView::TreeView_OnMouseButtonDoubleClick(FSetCollectionTreeItemPtr InItem)
	{
		InItem->OnRequestRename.ExecuteIfBound();
	}

	void SSetCollectionTreeView::PostUndo(bool bSuccess)
	{
		bNeedsRebuilding = true;
	}

	void SSetCollectionTreeView::PostRedo(bool bSuccess)
	{
		bNeedsRebuilding = true;
	}

	void SSetCollectionTreeView::RenameSet(const FName OldSetName, const FName NewSetName)
	{
		SetCollection->RenameSet(OldSetName, NewSetName);
	}

	FSetCollectionTreeItemPtr SSetCollectionTreeView::GetTreeItem(const FName SetName)
	{
		for (FSetCollectionTreeItemPtr TreeItem : GetAllTreeItems())
		{
			if (TreeItem->Set.SetName == SetName)
			{
				return TreeItem;
			}
		}

		return nullptr;
	}

	void SSetCollectionTreeView::HandleRemoveSet()
	{
		const TArray<FSetCollectionTreeItemPtr> Selection = GetSelectedItems();
		if (!Selection.IsEmpty())
		{
			const FScopedTransaction Transaction(LOCTEXT("RemoveSets", "Remove Set(s)"));
			SetCollection->Modify();

			for (const FSetCollectionTreeItemPtr& Item : Selection)
			{
				const FName SetToRemove = Item->Set.SetName;
				ensure(SetCollection->RemoveSet(SetToRemove));
			}
		}
	}

	void SSetCollectionTreeView::HandleAddSet()
	{
		const FScopedTransaction Transaction(LOCTEXT("AddSet", "Add Set"));
		SetCollection->Modify();


		const FText DefaultSetNameFormat = LOCTEXT("DefaultSetName", "NewSet_{0}");

		int32 Suffix = 0;
		FName SetName = FName(FText::Format(DefaultSetNameFormat, Suffix).ToString());
		while (SetCollection->HasSet(SetName))
		{
			++Suffix;
			SetName = FName(FText::Format(DefaultSetNameFormat, Suffix).ToString());
		}

		TArray<FSetCollectionTreeItemPtr> Selection = GetSelectedItems();
		const FName ParentSetName = (Selection.Num() == 1) ? Selection[0]->Set.SetName : NAME_None;
		
		SetCollection->AddSet(SetName, ParentSetName);

		// Expand the parent set tree item so we can see the new set in the tree
		FSetCollectionTreeItemPtr ParentTreeItem = GetTreeItem(ParentSetName);
		if (ParentTreeItem)
		{
			SetItemExpansion(ParentTreeItem, true);
		}

		RequestedSetNameToRename = SetName;
	}
	
	void SSetCollectionTreeView::RepopulateTreeData()
	{
		// Make note of the currently selected item, if any
		TArray<FSetCollectionTreeItemPtr> Selected = GetSelectedItems();

		// Make note of all tree items collapsed
		TArray<FName> CollapsedSetNames;

		for (FSetCollectionTreeItemPtr TreeItem : GetAllTreeItems())
		{
			if (!IsItemExpanded(TreeItem) && !TreeItem->Children.IsEmpty())
			{
				CollapsedSetNames.Add(TreeItem->Set.SetName);
			}
		}

		// Rebuild tree data
		RootItems.Reset();

		TMap<FName, FSetCollectionTreeItemPtr> TreeItemMap;

		for (const FAbstractSkeletonSet& Set : SetCollection->GetSetHierarchy())
		{
			FSetCollectionTreeItemPtr SetItem = MakeShared<FSetCollectionTreeItem>();
			SetItem->Set = Set;

			if (Set.ParentSetName == NAME_None)
			{
				RootItems.Add(SetItem);
			}
			else
			{
				if (!TreeItemMap.Contains(Set.ParentSetName))
				{
					FSetCollectionTreeItemPtr ParentItem = MakeShared<FSetCollectionTreeItem>();
					ParentItem->Set = Set;

					TreeItemMap.Add(Set.ParentSetName, ParentItem);
				}

				FSetCollectionTreeItemPtr ParentItem = TreeItemMap[Set.ParentSetName];
				ParentItem->Children.Add(SetItem);

			}

			TreeItemMap.Add(Set.SetName, SetItem);
		}

		// Restore expanded state of tree items and selection
		const FName SelectedSetName = Selected.IsEmpty() ? NAME_None : Selected[0]->Set.SetName;
		for (FSetCollectionTreeItemPtr TreeItem : GetAllTreeItems())
		{
			if (!CollapsedSetNames.Contains(TreeItem->Set.SetName))
			{
				SetItemExpansion(TreeItem, true);
			}

			if (TreeItem->Set.SetName == SelectedSetName)
			{
				SetSelection(TreeItem);
			}
		}

		UpdateTreeItemVisibility();
		RequestTreeRefresh();
	}
	
	FReply SSetCollectionTreeView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	TArray<FSetCollectionTreeItemPtr> SSetCollectionTreeView::GetAllTreeItems()
	{
		TArray<FSetCollectionTreeItemPtr> AllItems;
		AllItems.Append(RootItems);

		for (int32 Index = 0; Index < AllItems.Num(); ++Index)
		{
			AllItems.Append(AllItems[Index]->Children);
		}

		return AllItems;
	}

	FReply SSetCollectionTreeView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		if (TSharedPtr<FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSetDragDropOp>())
		{
			DragDropOp->SetOperation(FSetDragDropOp::EOperation::Reparent, NAME_None);

			return FReply::Handled();
		}
		return FReply::Unhandled();
	}
	
	void SSetCollectionTreeView::OnDragLeave(const FDragDropEvent& DragDropEvent)
	{
		if (TSharedPtr<FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSetDragDropOp>())
		{
			DragDropOp->ClearOperation();
		}
	}
	
	FReply SSetCollectionTreeView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		if (TSharedPtr<FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FSetDragDropOp>())
		{
			const FScopedTransaction Transaction(LOCTEXT("UnparentSets", "Unparent Set(s)"));
			SetCollection->Modify();

			for (const FAbstractSkeletonSet& Set : DragDropOp->GetDraggedSets())
			{
				SetCollection->ReparentSet(Set.SetName, NAME_None);
			}
					
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	void SSetCollectionTreeView::BindCommands()
	{
		CommandList = MakeShareable(new FUICommandList);

		CommandList->MapAction(
			FGenericCommands::Get().Rename,
			FExecuteAction::CreateSP(this, &SSetCollectionTreeView::HandleRenameSet)
		);
	
		CommandList->MapAction(
			FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &SSetCollectionTreeView::HandleRemoveSet)
		);
	}

	void SSetCollectionTreeView::HandleRenameSet()
	{
		TArray<FSetCollectionTreeItemPtr> Selected = GetSelectedItems();
		if (!Selected.IsEmpty())
		{
			RequestedSetNameToRename = Selected[0]->Set.SetName;
		}
	}

	void SSetCollectionTreeView::OnSetsChanged()
	{
		bNeedsRebuilding = true;
	}

	void SSetCollectionTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		STreeView<FSetCollectionTreeItemPtr>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		if (bNeedsRebuilding)
		{
			RepopulateTreeData();
			bNeedsRebuilding = false;
		}

		if (RequestedSetNameToRename != NAME_None)
		{
			RequestedSetItemToRename = GetTreeItem(RequestedSetNameToRename);
			check(RequestedSetItemToRename);
			
			RequestScrollIntoView(RequestedSetItemToRename);
			RequestedSetNameToRename = NAME_None;
		}
	}

	void SSetCollectionTreeView::UpdateTreeItemVisibility()
	{
		const FString FilterString = FilterText.ToString();
		
		const TFunction<void(FSetCollectionTreeItemPtr&)> UpdateItemVisibility = [&](FSetCollectionTreeItemPtr& Item)
			{
				bool bAnyChildVisible = false;
				for (FSetCollectionTreeItemPtr& Child : Item->Children)
				{
					UpdateItemVisibility(Child);
					bAnyChildVisible |= Child->bVisible;
				}

				bool bSelfVisible = bAnyChildVisible || Item->Set.SetName.ToString().Contains(FilterString, ESearchCase::Type::IgnoreCase);
				Item->bVisible = bSelfVisible;
			};

		for (FSetCollectionTreeItemPtr& RootItem : RootItems)
		{
			UpdateItemVisibility(RootItem);
		}
	}
	
	void SSetCollectionTreeView::SetFilterText(const FText& InText)
	{
		FilterText = InText;

		UpdateTreeItemVisibility();
	}
}

#undef LOCTEXT_NAMESPACE