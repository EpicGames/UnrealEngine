// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "Widgets/Views/STreeView.h"

class FUICommandList;

namespace UE::UAF::Editor
{
	struct FSetCollectionTreeItem;
	using FSetCollectionTreeItemPtr = TSharedPtr<FSetCollectionTreeItem>;
	
	struct FSetCollectionTreeItem
	{
		FAbstractSkeletonSet Set;
		TArray<FSetCollectionTreeItemPtr> Children;
		FSimpleDelegate OnRequestRename;
		bool bVisible = true;
	};

	class SSetCollectionTreeView : public STreeView<FSetCollectionTreeItemPtr>, public FSelfRegisteringEditorUndoClient
	{
		SLATE_BEGIN_ARGS(SSetCollectionTreeView)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetCollection> InSetCollection);
		virtual ~SSetCollectionTreeView();

		/** FSelfRegisteringEditorUndoClient interface */
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;

		void BindCommands();
		void RepopulateTreeData();

		void HandleAddSet();
		void HandleRenameSet();
		void HandleRemoveSet();

		void OnSetsChanged();
		
		void ExpandAllTreeItems();
		TArray<FSetCollectionTreeItemPtr> GetAllTreeItems();

		TSharedRef<ITableRow> TreeView_OnGenerateRow(FSetCollectionTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);
		void TreeView_OnGetChildren(FSetCollectionTreeItemPtr InItem, TArray<FSetCollectionTreeItemPtr>& OutChildren);
		TSharedPtr<SWidget> TreeView_OnContextMenuOpening();
		void TreeView_OnItemScrolledIntoView(FSetCollectionTreeItemPtr InItem, const TSharedPtr<ITableRow>& InWidget);
		void TreeView_OnMouseButtonDoubleClick(FSetCollectionTreeItemPtr InItem);

		void RenameSet(const FName OldSetName, const FName NewSetName);

		virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

		void SetFilterText(const FText& InText);

	private:
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);
		virtual void OnDragLeave(const FDragDropEvent& DragDropEvent);
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		
		FSetCollectionTreeItemPtr GetTreeItem(const FName SetName);

		void UpdateTreeItemVisibility();
		
		TArray<FSetCollectionTreeItemPtr> RootItems;

		TWeakObjectPtr<UAbstractSkeletonSetCollection> SetCollection;

		FName RequestedSetNameToRename;
		FSetCollectionTreeItemPtr RequestedSetItemToRename;

		TSharedPtr<FUICommandList> CommandList;

		FDelegateHandle OnSetsChangedHandle;

		bool bNeedsRebuilding = false;

		FText FilterText;
	};
}