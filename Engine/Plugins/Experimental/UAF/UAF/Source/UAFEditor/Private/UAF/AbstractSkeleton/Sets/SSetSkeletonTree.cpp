// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/SSetSkeletonTree.h"

#include "Animation/Skeleton.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ReferenceSkeleton.h"
#include "ScopedTransaction.h"
#include "SetDragDrop.h"
#include "SkeletonTreeDragDrop.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/Sets/Commands.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SSetsSkeletonTree"

namespace UE::UAF::Editor
{
	void SSetsSkeletonTree::Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding)
	{
		SetBinding = InSetBinding;

		if (SetBinding.IsValid())
		{
			OnBindingsChangedHandle = SetBinding->RegisterOnBindingsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SSetsSkeletonTree::HandleBindingsChanged));
		}

		ChildSlot
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(1.0f, 1.0f))
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.Padding(FMargin(1.0f, 1.0f))
							[
								SNew(SSearchBox)
									.SelectAllTextWhenFocused(true)
									.OnTextChanged_Lambda([](const FText& InText) { /* TODO: Implement */ })
									.HintText(LOCTEXT("SearchBox_Hint", "Search Sets..."))
							]
					]
					+ SVerticalBox::Slot()
					[
						SAssignNew(TreeView, STreeView<TSharedPtr<FTreeItem>>)
							.TreeItemsSource(&RootItems)
							.OnGenerateRow(this, &SSetsSkeletonTree::TreeView_OnGenerateRow)
							.OnGetChildren(this, &SSetsSkeletonTree::TreeView_OnGetChildren)
							.OnContextMenuOpening(this, &SSetsSkeletonTree::TreeView_OnContextMenuOpening)
							.HighlightParentNodesForSelection(true)
							.HeaderRow(
								SNew(SHeaderRow)
								+ SHeaderRow::Column("Set")
								.DefaultLabel(LOCTEXT("SetLabel", "Set"))
								.FillWidth(0.3f)
								+ SHeaderRow::Column("Name")
								.DefaultLabel(LOCTEXT("NameLabel", "Name"))
								.FillWidth(0.7f)
							)
					]
			];

		RepopulateTreeData();
		BindCommands();
	}

	void SSetsSkeletonTree::Tick(const FGeometry& AllocatedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		if (bTreeDirty)
		{
			RepopulateTreeData();
			bTreeDirty = false;
		}
	}

	FReply SSetsSkeletonTree::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	void SSetsSkeletonTree::PostUndo(bool bSuccess)
	{
		RepopulateTreeData();
	}

	void SSetsSkeletonTree::PostRedo(bool bSuccess)
	{
		RepopulateTreeData();
	}

	class SSetsSkeletonTreeRow : public SMultiColumnTableRow<TSharedPtr<SSetsSkeletonTree::FTreeItem>>
	{
	public:
		SLATE_BEGIN_ARGS(SSetsSkeletonTreeRow) {}
			SLATE_ARGUMENT(TSharedPtr<SSetsSkeletonTree::FTreeItem>, Item)
			SLATE_EVENT(FOnDragDetected, OnDragDetected)
			SLATE_EVENT(FOnCanAcceptDrop, OnCanAcceptDrop)
			SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
		{
			Item = InArgs._Item;

			const FSuperRowType::FArguments Args = FSuperRowType::FArguments()
				.OnDragDetected(InArgs._OnDragDetected)
				.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
				.OnAcceptDrop(InArgs._OnAcceptDrop)
				.Style(FAppStyle::Get(), "TableView.AlternatingRow");

			SMultiColumnTableRow<TSharedPtr<SSetsSkeletonTree::FTreeItem>>::Construct(Args, OwnerTableView);
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == "Set")
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(6, 0)
					[
						SNew(SBox)
							.WidthOverride(8.0f)
							.HeightOverride(8.0f)
							[
								SNew(SImage)
									.Image(FAppStyle::Get().GetBrush("Icons.FilledCircle"))
									.ColorAndOpacity((Item->BoundSet != NAME_None) ? FLinearColor(FColor(31, 228, 75)) : FLinearColor(FColor(239, 53, 53)))
							]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2, 2)
					[
						SNew(SImage)
							.Visibility(Item->BoundSet == NAME_None ? EVisibility::Collapsed : EVisibility::Visible)
							.Image(FAppStyle::Get().GetBrush("ClassIcon.GroupActor"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2, 2)
					[
						SNew(STextBlock)
							.Visibility(Item->BoundSet == NAME_None ? EVisibility::Collapsed : EVisibility::Visible)
							.Text(FText::FromName(Item->BoundSet))
					];

			}
			else if (ColumnName == "Name")
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SExpanderArrow, SharedThis(this))
							.ShouldDrawWires(true)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2, 2)
					[
						SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("SkeletonTree.Bone"))
							.ColorAndOpacity((Item->BoundSet != NAME_None) ? FLinearColor(FColor(31, 228, 75)) : FSlateColor::UseForeground())
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2, 2)
					[
						SNew(STextBlock)
							.Text(FText::FromName(Item->BoneName))
					];
			}

			return SNullWidget::NullWidget;
		}

	private:
		TSharedPtr<SSetsSkeletonTree::FTreeItem> Item;
	};

	TSharedRef<ITableRow> SSetsSkeletonTree::TreeView_OnGenerateRow(TSharedPtr<FTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(SSetsSkeletonTreeRow, OwnerTable)
			.Item(Item)
			.OnCanAcceptDrop_Lambda([](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FTreeItem> TargetTreeItem) -> TOptional<EItemDropZone>
				{
					using namespace UE::UAF;

					TOptional<EItemDropZone> ReturnedDropZone;

					const TSharedPtr<Editor::FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<Editor::FSetDragDropOp>();
					if (DragDropOp.IsValid())
					{
						ReturnedDropZone = EItemDropZone::OntoItem;
					}

					return ReturnedDropZone;
				})
			.OnAcceptDrop_Lambda([this](const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FTreeItem> TargetTreeItem) -> FReply
				{
					const TSharedPtr<Editor::FSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<Editor::FSetDragDropOp>();
					if (DragDropOp.IsValid())
					{
						TSharedPtr<FTreeItem> TargetSetTreeItem = StaticCastSharedPtr<FTreeItem>(TargetTreeItem);

						const FScopedTransaction Transaction(LOCTEXT("AddAttributeToSet", "Add Attribute to Set"));
						SetBinding->Modify();

						// TODO: Use all dragged sets
						check(!DragDropOp->GetDraggedSets().IsEmpty());
						const bool bSuccess = SetBinding->AddBoneToSet(TargetSetTreeItem->BoneName, DragDropOp->GetDraggedSets()[0].SetName);
						ensure(bSuccess);
						
						return FReply::Handled();
					}
					return FReply::Unhandled();
				})
			.OnDragDetected_Lambda([this](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
				{
					if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
					{
						TArray<TSharedPtr<FTreeItem>> Selection = TreeView->GetSelectedItems();
						const TSharedRef<FSkeletonTreeDragDropOp> DragDropOp = FSkeletonTreeDragDropOp::New(MoveTemp(Selection));
						return FReply::Handled().BeginDragDrop(DragDropOp);
					}

					return FReply::Unhandled();
				});
	}

	void SSetsSkeletonTree::TreeView_OnGetChildren(TSharedPtr<FTreeItem> InParent, TArray<TSharedPtr<FTreeItem>>& OutChildren)
	{
		OutChildren = InParent->Children;
	}

	TSharedPtr<SWidget> SSetsSkeletonTree::TreeView_OnContextMenuOpening()
	{
		const Editor::FSetBindingEditorCommands& Commands = Editor::FSetBindingEditorCommands::Get();

		FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

		MenuBuilder.AddMenuEntry(Commands.UnbindSelection);
		MenuBuilder.AddMenuEntry(Commands.SelectWithChildren);

		return MenuBuilder.MakeWidget();
	}

	TArray<TSharedPtr<SSetsSkeletonTree::FTreeItem>> SSetsSkeletonTree::GetAllDescendentTreeItems(const TArray<TSharedPtr<FTreeItem>> ParentItems)
	{
		TArray<TSharedPtr<FTreeItem>> AllItems;
		AllItems.Append(ParentItems);

		for (int32 Index = 0; Index < AllItems.Num(); ++Index)
		{
			AllItems.Append(AllItems[Index]->Children);
		}

		return AllItems;
	}

	void SSetsSkeletonTree::SetSetBinding(TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding)
	{
		if (SetBinding.IsValid())
		{
			SetBinding->UnregisterOnBindingsChanged(OnBindingsChangedHandle);
		}

		SetBinding = InSetBinding;

		if (SetBinding.IsValid())
		{
			OnBindingsChangedHandle = SetBinding->RegisterOnBindingsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SSetsSkeletonTree::HandleBindingsChanged));
		}

		RepopulateTreeData();
	}

	void SSetsSkeletonTree::RepopulateTreeData()
	{
		// Save expansion state of each tree item to restore later
		TSet<FName> CollapsedBoneNames;
		for (const TSharedPtr<FTreeItem>& TreeItem : GetAllDescendentTreeItems(RootItems))
		{
			if (!TreeView->IsItemExpanded(TreeItem))
			{
				CollapsedBoneNames.Add(TreeItem->BoneName);
			}
		}

		RootItems.Empty();

		if (SetBinding.IsValid() && SetBinding->GetSkeleton())
		{
			const FReferenceSkeleton& RefSkeleton = SetBinding->GetSkeleton()->GetReferenceSkeleton();

			TFunction<TSharedPtr<FTreeItem>(int32, const TSharedPtr<FTreeItem>&)> AddBoneToTree;

			AddBoneToTree = [this, &RefSkeleton, &AddBoneToTree](const int32 BoneIndex, const TSharedPtr<FTreeItem>& Parent) -> TSharedPtr<FTreeItem>
			{
				TSharedPtr<FTreeItem> TreeItem = MakeShared<FTreeItem>();
				TreeItem->BoneName = RefSkeleton.GetBoneName(BoneIndex);
				TreeItem->BoundSet = SetBinding->GetBoneSet(TreeItem->BoneName);
				TreeItem->Parent = Parent;

				TArray<int32> Children;
				RefSkeleton.GetDirectChildBones(BoneIndex, Children);

				for (const int32 ChildBoneIndex : Children)
				{
					TSharedPtr<FTreeItem> ChildTreeItem = AddBoneToTree(ChildBoneIndex, TreeItem);
					TreeItem->Children.Add(ChildTreeItem);
				}

				return TreeItem;
			};
				
			TSharedPtr<FTreeItem> RootBoneItem = AddBoneToTree(0, nullptr);

			RootItems.Add(RootBoneItem);
		}

		TreeView->RebuildList();

		for (TSharedPtr<FTreeItem> TreeItem : GetAllDescendentTreeItems(RootItems))
		{
			const bool bExpanded = !CollapsedBoneNames.Contains(TreeItem->BoneName);
			TreeView->SetItemExpansion(TreeItem, bExpanded);
		}
	}

	void SSetsSkeletonTree::BindCommands()
	{
		CommandList = MakeShareable(new FUICommandList);

		const Editor::FSetBindingEditorCommands& Commands = Editor::FSetBindingEditorCommands::Get();

		CommandList->MapAction(Commands.UnbindSelection,
			FExecuteAction::CreateSP(this, &SSetsSkeletonTree::HandleUnbindSelection),
			FCanExecuteAction::CreateSP(this, &SSetsSkeletonTree::IsSelectionNonEmpty));

		CommandList->MapAction(Commands.SelectWithChildren,
			FExecuteAction::CreateSP(this, &SSetsSkeletonTree::HandleSelectWithChildren),
			FCanExecuteAction::CreateSP(this, &SSetsSkeletonTree::IsSelectionSingle));
	}

	void SSetsSkeletonTree::HandleBindingsChanged()
	{
		bTreeDirty = true;
	}

	void SSetsSkeletonTree::HandleUnbindSelection()
	{
		const FScopedTransaction Transaction(LOCTEXT("UnbindSelection", "Unbind Selection"));
		SetBinding->Modify();

		TArray<TSharedPtr<FTreeItem>> Selection = TreeView->GetSelectedItems();

		for (const TSharedPtr<FTreeItem>& Selected : Selection)
		{
			SetBinding->RemoveBoneFromSet(Selected->BoneName);
		}
	}

	void SSetsSkeletonTree::HandleSelectWithChildren()
	{
		// No transaction as selection is transient

		TArray<TSharedPtr<FTreeItem>> Selection = TreeView->GetSelectedItems();
		check(Selection.Num() == 1);

		TArray<TSharedPtr<FTreeItem>> NewSelection = GetAllDescendentTreeItems(Selection);
		TreeView->SetItemSelection(NewSelection, true);
	}

	bool SSetsSkeletonTree::IsSelectionNonEmpty() const
	{
		return TreeView->GetNumItemsSelected() > 0;
	}

	bool SSetsSkeletonTree::IsSelectionSingle() const
	{
		return TreeView->GetNumItemsSelected() == 1;
	}
}

#undef LOCTEXT_NAMESPACE