// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
template <typename> class STreeView;
class UAbstractSkeletonSetBinding;
class FUICommandList;

namespace UE::UAF::Editor
{
	class SSetsSkeletonTree : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
	{
	public:
		struct FTreeItem
		{
			FName BoneName;
			FName BoundSet;

			TWeakPtr<FTreeItem> Parent;
			TArray<TSharedPtr<FTreeItem>> Children;
		};

		SLATE_BEGIN_ARGS(SSetsSkeletonTree) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding);

		void SetSetBinding(TWeakObjectPtr<UAbstractSkeletonSetBinding> InSetBinding);

	private:
		/** SCompoundWidget */
		virtual void Tick(const FGeometry& AllocatedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

		/** FSelfRegisteringEditorUndoClient interface */
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;

		void RepopulateTreeData();

		void BindCommands();

		// Called when the bindings inside the set binding asset we're editing have been modified
		void HandleBindingsChanged();

		void HandleUnbindSelection();

		void HandleSelectWithChildren();

		bool IsSelectionNonEmpty() const;
		
		bool IsSelectionSingle() const;

		// Returns a list of the provided parents and all of their descendents
		// Beware, the descendents of ParentItems should be non-overlapping!
		TArray<TSharedPtr<FTreeItem>> GetAllDescendentTreeItems(const TArray<TSharedPtr<FTreeItem>> ParentItems);
		
		TSharedRef<ITableRow> TreeView_OnGenerateRow(TSharedPtr<FTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable);
		
		void TreeView_OnGetChildren(TSharedPtr<FTreeItem> InParent, TArray<TSharedPtr<FTreeItem>>& OutChildren);

		TSharedPtr<SWidget> TreeView_OnContextMenuOpening();

	private:
		TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding;

		TArray<TSharedPtr<FTreeItem>> RootItems;

		TSharedPtr<STreeView<TSharedPtr<FTreeItem>>> TreeView;

		FDelegateHandle OnBindingsChangedHandle;

		bool bTreeDirty = false;

		TSharedPtr<FUICommandList> CommandList;
	};

}