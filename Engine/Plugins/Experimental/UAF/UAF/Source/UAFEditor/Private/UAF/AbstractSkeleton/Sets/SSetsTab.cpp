// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/SSetsTab.h"

#include "Animation/Skeleton.h"
#include "AssetThumbnail.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UAF/AbstractSkeleton/Sets/SSetSkeletonTree.h"
#include "UAF/AbstractSkeleton/Sets/SSetBinding.h"
#include "UAF/AbstractSkeleton/Sets/SAttributesList.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SSetsTab"

namespace UE::UAF::Editor
{
	namespace Tabs
	{
		static const FName SkeletonTreeId(TEXT("AbstractSkeletonEditor_Sets_SkeletonTreeTab"));
		static const FName AttributesId(TEXT("AbstractSkeletonEditor_Sets_AttributesTab"));
		static const FName SetBindingsId(TEXT("AbstractSkeletonEditor_Sets_BindingsTab"));
	}

	void SSetsTab::Construct(const FArguments& InArgs)
	{
		SetBinding = InArgs._SetBinding;

		static const FName LayoutName = "UE::Anim::STF::SSetsTab_Layout";

		TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout(LayoutName)
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(Tabs::SkeletonTreeId, ETabState::OpenedTab)
					->AddTab(Tabs::AttributesId, ETabState::OpenedTab)
					->SetForegroundTab(Tabs::SkeletonTreeId)
				)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(Tabs::SetBindingsId, ETabState::OpenedTab)
				)
			);

		check(InArgs._ParentTab.IsValid());
		TabManager = FGlobalTabmanager::Get()->NewTabManager(InArgs._ParentTab.ToSharedRef());
		TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateSP(this, &SSetsTab::HandleTabManagerPersistLayout));

		TabManager->RegisterTabSpawner(Tabs::SkeletonTreeId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
			{
				return SNew(SDockTab)
					.CanEverClose(false)
					[
						SAssignNew(SkeletonTreeWidget, SSetsSkeletonTree, SetBinding)
					];
			}))
			.SetDisplayName(LOCTEXT("SkeletonTree_TabDisplayName", "Skeleton Tree"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.SkeletonTree"));

		TabManager->RegisterTabSpawner(Tabs::AttributesId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
			{
				return SNew(SDockTab)
					.CanEverClose(false)
					[
						SAssignNew(AttributesListWidget, SAttributesList, SetBinding)
					];
			}))
			.SetDisplayName(LOCTEXT("Attributes_TabDisplayName", "Attributes"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimGraph.Attribute.Attributes.Icon"));

		TabManager->RegisterTabSpawner(Tabs::SetBindingsId, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
			{
				return SNew(SDockTab)
					.CanEverClose(false)
					[
						SAssignNew(SetBindingWidget, SSetBinding, SetBinding)
					];
			}))
			.SetDisplayName(LOCTEXT("SetBindings_TabDisplayName", "Set Bindings"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"));

		Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout);

		FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);

		ToolbarBuilder.BeginSection("Common");
		{
			ToolbarBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(16.0f, 0.0f, 24.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("SetBinding", "Set Binding"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0.0f, 0.0f, 16.0f, 0.0f))
				[
					SNew(SObjectPropertyEntryBox)
						.AllowedClass(UAbstractSkeletonSetBinding::StaticClass())
						.ObjectPath_Lambda([&]()
							{
								if (SetBinding.IsValid())
								{
									return SetBinding->GetPathName();
								}
								return FString();
							})
						.OnObjectChanged_Lambda([this](const FAssetData& AssetData)
							{
								// TODO: Update the currently edited objects in the asset editor so saving and showing dirty state is correct
								SetBinding = CastChecked<UAbstractSkeletonSetBinding>(AssetData.GetAsset());

								SkeletonTreeWidget->SetSetBinding(SetBinding);
								AttributesListWidget->SetSetBinding(SetBinding);
								SetBindingWidget->SetSetBinding(SetBinding);
							})
						.AllowClear(false)
						.DisplayUseSelected(true)
						.DisplayBrowse(true)
						.DisplayThumbnail(true)
						.ThumbnailPool(MakeShared<FAssetThumbnailPool>(1))
				],
				NAME_None,
				true,
				HAlign_Left
			);

			ToolbarBuilder.AddSeparator();

			ToolbarBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(16.0f, 0.0f, 24.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("SetCollection", "Set Collection"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SObjectPropertyEntryBox)
						.AllowedClass(UAbstractSkeletonSetCollection::StaticClass())
						.OnIsEnabled_Lambda([&]()
							{
								return SetBinding.IsValid();
							})
						.ObjectPath_Lambda([&]()
							{
								if (SetBinding.IsValid() && SetBinding->GetSetCollection())
								{
									return SetBinding->GetSetCollection()->GetPathName();
								}
								return FString();
							})
						.OnObjectChanged_Lambda([this](const FAssetData& AssetData)
							{
								if (SetBinding.IsValid())
								{
									const FScopedTransaction Transaction(LOCTEXT("SetSetCollection", "Set Set Collection"));
									SetBinding->Modify();

									const bool bSuccess = SetBinding->SetSetCollection(CastChecked<UAbstractSkeletonSetCollection>(AssetData.GetAsset()));
									ensure(bSuccess);
								}
							})
						.AllowClear(false)
						.DisplayUseSelected(true)
						.DisplayBrowse(true)
						.DisplayThumbnail(true)
						.ThumbnailPool(MakeShared<FAssetThumbnailPool>(1))
				],
				NAME_None,
				true,
				HAlign_Left
			);

			ToolbarBuilder.AddWidget(
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(16.0f, 0.0f, 24.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(LOCTEXT("Skeleton", "Skeleton"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SObjectPropertyEntryBox)
						.AllowedClass(USkeleton::StaticClass())
						.OnIsEnabled_Lambda([&]()
							{
								return SetBinding.IsValid();
							})
						.ObjectPath_Lambda([&]()
							{
								if (SetBinding.IsValid() && SetBinding->GetSkeleton())
								{
									return SetBinding->GetSkeleton()->GetPathName();
								}
								return FString();
							})
						.OnObjectChanged_Lambda([this](const FAssetData& AssetData)
							{
								if (SetBinding.IsValid())
								{
									const FScopedTransaction Transaction(LOCTEXT("SetBindingSkeleton", "Set Binding Skeleton"));
									SetBinding->Modify();

									const bool bSuccess = SetBinding->SetSkeleton(CastChecked<USkeleton>(AssetData.GetAsset()));
									ensure(bSuccess);
								}
							})
						.AllowClear(false)
						.DisplayUseSelected(true)
						.DisplayBrowse(true)
						.DisplayThumbnail(true)
						.ThumbnailPool(MakeShared<FAssetThumbnailPool>(1))
				],
				NAME_None,
				true,
				HAlign_Left
			);
		}
		ToolbarBuilder.EndSection();

		ToolbarBuilder.SetStyle(&FAppStyle::Get(), "SlimToolBar");

		ChildSlot
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							[
								ToolbarBuilder.MakeWidget()
							]
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						TabManager->RestoreFrom(Layout, nullptr).ToSharedRef()
					]
			];
	}

	void SSetsTab::HandleTabManagerPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave)
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, LayoutToSave);
	}

}

#undef LOCTEXT_NAMESPACE