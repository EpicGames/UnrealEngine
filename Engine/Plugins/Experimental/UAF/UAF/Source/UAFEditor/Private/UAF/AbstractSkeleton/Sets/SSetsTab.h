// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/SCompoundWidget.h"

class SDockTab;
class UAbstractSkeletonSetBinding;

namespace UE::UAF::Editor
{
	class SSetsSkeletonTree;
	class SAttributesList;
	class SSetBinding;
	
	class SSetsTab : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SSetsTab) {}
			SLATE_ARGUMENT(TSharedPtr<SDockTab>, ParentTab)
			SLATE_ARGUMENT(TWeakObjectPtr<UAbstractSkeletonSetBinding>, SetBinding)
		SLATE_END_ARGS()

		virtual void Construct(const FArguments& InArgs);

	private:
		void HandleTabManagerPersistLayout(const TSharedRef<FTabManager::FLayout>& LayoutToSave);

	private:
		TWeakObjectPtr<UAbstractSkeletonSetBinding> SetBinding;

		TSharedPtr<SSetsSkeletonTree> SkeletonTreeWidget;
		TSharedPtr<SAttributesList> AttributesListWidget;
		TSharedPtr<SSetBinding> SetBindingWidget;

		TSharedPtr<FTabManager> TabManager;
	};
}