// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/SetBindingAssetDefinition.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonEditor.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"

#define LOCTEXT_NAMESPACE "UE::UAF"

FText UAssetDefinition_AbstractSkeletonSetBinding::GetAssetDisplayName() const
{
	return LOCTEXT("SetBinding", "UAF Set Binding");
}

FLinearColor UAssetDefinition_AbstractSkeletonSetBinding::GetAssetColor() const
{
	return FLinearColor(FColor(119, 119, 21));
}

TSoftClassPtr<UObject> UAssetDefinition_AbstractSkeletonSetBinding::GetAssetClass() const
{
	return UAbstractSkeletonSetBinding::StaticClass();
}

EAssetCommandResult UAssetDefinition_AbstractSkeletonSetBinding::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		TArray<UObject*> Assets = OpenArgs.LoadObjects<UObject>();
		
		MakeShared<UE::UAF::Editor::FAbstractSkeletonEditor>()->InitEditor(Assets, OpenArgs.ToolkitHost);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AbstractSkeletonSetBinding::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")) };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
