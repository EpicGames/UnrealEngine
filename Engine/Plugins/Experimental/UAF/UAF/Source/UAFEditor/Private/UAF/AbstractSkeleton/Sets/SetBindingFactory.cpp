// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/SetBindingFactory.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetBinding.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetBindingFactory)

UAbstractSkeletonSetBindingFactory::UAbstractSkeletonSetBindingFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAbstractSkeletonSetBinding::StaticClass();
}

bool UAbstractSkeletonSetBindingFactory::ConfigureProperties()
{
	return true;
}

UObject* UAbstractSkeletonSetBindingFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	EObjectFlags FlagsToUse = Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted;
	if (InParent == GetTransientPackage())
	{
		FlagsToUse &= ~RF_Standalone;
	}

	TObjectPtr<UAbstractSkeletonSetBinding> NewSetBinding = NewObject<UAbstractSkeletonSetBinding>(InParent, Class, Name, FlagsToUse);
	NewSetBinding->AddDefaultAttributes();

	return NewSetBinding;
}
