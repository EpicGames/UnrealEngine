// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/SetCollectionAssetDefinition.h"

#include "UAF/AbstractSkeleton/Sets/SetCollectionEditor.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"

#define LOCTEXT_NAMESPACE "UE::UAF"

FText UAssetDefinition_AbstractSkeletonSetCollection::GetAssetDisplayName() const
{
	return LOCTEXT("SetCollection", "UAF Set Collection");
}

FLinearColor UAssetDefinition_AbstractSkeletonSetCollection::GetAssetColor() const
{
	return FLinearColor(FColor(148, 148, 73));
}

TSoftClassPtr<UObject> UAssetDefinition_AbstractSkeletonSetCollection::GetAssetClass() const
{
	return UAbstractSkeletonSetCollection::StaticClass();
}

EAssetCommandResult UAssetDefinition_AbstractSkeletonSetCollection::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		TArray<UObject*> Assets = OpenArgs.LoadObjects<UObject>();
		MakeShared<UE::UAF::Editor::FSetCollectionEditorToolkit>()->InitEditor(Assets);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AbstractSkeletonSetCollection::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")) };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
