// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "SetCollectionAssetDefinition.generated.h"

UCLASS()
class UAssetDefinition_AbstractSkeletonSetCollection : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	// End UAssetDefinition
};
