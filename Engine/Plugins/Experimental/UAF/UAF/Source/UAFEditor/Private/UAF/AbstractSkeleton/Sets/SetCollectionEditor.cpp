// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/SetCollectionEditor.h"

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "UAF/AbstractSkeleton/Sets/SSetCollection.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::FSetCollectionEditorToolkit"

namespace UE::UAF::Editor
{
	void FSetCollectionEditorToolkit::InitEditor(const TArray<UObject*>& InObjects)
	{
		SetCollection = CastChecked<UAbstractSkeletonSetCollection>(InObjects[0]);

		const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("AbstractSkeletonSetCollectionEditorToolkit")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetHideTabWell(true)
					->AddTab("AbstractSkeletonSetCollection_SetTab", ETabState::OpenedTab)
				)
			);

		InitAssetEditor(EToolkitMode::Standalone, {}, "AbstractSkeletonSetCollectionEditorToolkit", Layout, true, true, InObjects);
	}

	void FSetCollectionEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("AbstractSkeletonSetCollectionEditor", "Abstract Skeleton Set Editor"));
		
		InTabManager->RegisterTabSpawner("AbstractSkeletonSetCollection_SetTab", FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs&)
			{
				return SNew(SDockTab)
					.CanEverClose(false)
					[
						SNew(SSetCollection, SetCollection)
					];
			}))
			.SetDisplayName(LOCTEXT("SetTab_DisplayName", "Abstract Skeleton Set"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}

	void FSetCollectionEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
		InTabManager->UnregisterTabSpawner("AbstractSkeletonSetCollection_SetTab");
	}	
}

#undef LOCTEXT_NAMESPACE