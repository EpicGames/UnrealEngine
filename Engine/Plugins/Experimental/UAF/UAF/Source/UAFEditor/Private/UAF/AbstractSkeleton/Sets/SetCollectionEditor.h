// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "Toolkits/AssetEditorToolkit.h"

namespace UE::UAF::Editor
{
	class FSetCollectionEditorToolkit : public FAssetEditorToolkit
	{
	public:
		void InitEditor(const TArray<UObject*>& InObjects);

		// Begin FAssetEditorToolkit
		void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
		void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

		FName GetToolkitFName() const override { return "AbstractSkeletonSetCollectionEditor"; }
		FText GetBaseToolkitName() const override { return INVTEXT("Abstract Skeleton Set Collection Editor"); }
		FString GetWorldCentricTabPrefix() const override { return "Abstract Skeleton Set Collection Editor "; }
		FLinearColor GetWorldCentricTabColorScale() const override { return {}; }
		// End FAssetEditorToolkit

	private:
		TWeakObjectPtr<UAbstractSkeletonSetCollection> SetCollection;
	};
}
