// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/SetCollectionFactory.h"

#include "UAF/AbstractSkeleton/AbstractSkeletonSetCollection.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetCollectionFactory)

UAbstractSkeletonSetCollectionFactory::UAbstractSkeletonSetCollectionFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAbstractSkeletonSetCollection::StaticClass();
}

bool UAbstractSkeletonSetCollectionFactory::ConfigureProperties()
{
	return true;
}

UObject* UAbstractSkeletonSetCollectionFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	EObjectFlags FlagsToUse = Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted;
	if (InParent == GetTransientPackage())
	{
		FlagsToUse &= ~RF_Standalone;
	}

	TObjectPtr<UAbstractSkeletonSetCollection> NewSetCollection = NewObject<UAbstractSkeletonSetCollection>(InParent, Class, Name, FlagsToUse);

	return NewSetCollection;
}
