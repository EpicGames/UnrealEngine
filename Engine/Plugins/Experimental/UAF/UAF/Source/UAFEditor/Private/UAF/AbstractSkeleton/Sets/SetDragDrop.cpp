// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/AbstractSkeleton/Sets/SetDragDrop.h"

#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::FSetDragDropOp"

namespace UE::UAF::Editor
{
	TSharedRef<FSetDragDropOp> FSetDragDropOp::New(TArray<FAbstractSkeletonSet>&& InSets)
	{
		TSharedRef<FSetDragDropOp> Operation = MakeShared<FSetDragDropOp>();
		Operation->Sets = InSets;

		Operation->Construct();

		return Operation;
	}

	void FSetDragDropOp::Construct()
	{
		FDecoratedDragDropOp::Construct();

		check(!Sets.IsEmpty());
		CachedSetsText = Sets.Num() == 1
			? FText::FromName(Sets[0].SetName)
			: FText::Format(LOCTEXT("MultipleSetsTooltip", "{0} and {1} more"), FText::FromName(Sets[0].SetName), FText::AsNumber(Sets.Num() - 1));
	}

	TSharedPtr<SWidget> FSetDragDropOp::GetDefaultDecorator() const
	{
		return SNew(SBorder)
			.Visibility(EVisibility::Visible)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			.Padding(FMargin(8.0f, 4.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.0f))
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("ClassIcon.GroupActor"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.0f))
				[
					SNew(SRichTextBlock)
					.DecoratorStyleSet(&FAppStyle::Get())
					.Text_Lambda([this]()
					{
						switch (Operation)
						{
						case EOperation::None:
							return CachedSetsText;
						case EOperation::Bind:
							return FText::Format(LOCTEXT("BindSet", "Bind <RichTextBlock.Bold>{0}</> to <RichTextBlock.Bold>{1}</>"), CachedSetsText, FText::FromName(OperationSetName));
						case EOperation::Reparent:
							return OperationSetName != NAME_None
								? FText::Format(LOCTEXT("ParentSet", "Parent <RichTextBlock.Bold>{0}</> to <RichTextBlock.Bold>{1}</>"), CachedSetsText, FText::FromName(OperationSetName))
								: FText::Format(LOCTEXT("Unparent", "Unparent <RichTextBlock.Bold>{0}</>"), CachedSetsText);
						}
						return FText::GetEmpty();
					})
				]
			];
	}

	void FSetDragDropOp::SetOperation(const EOperation InOperation, const FName InOperationSetName)
	{
		Operation = InOperation;
		OperationSetName = InOperationSetName;
	}
	
	void FSetDragDropOp::ClearOperation()
	{
		Operation = EOperation::None;
		OperationSetName = NAME_None;
	}

	const TArray<FAbstractSkeletonSet>& FSetDragDropOp::GetDraggedSets() const
	{
		return Sets;
	}
}

#undef LOCTEXT_NAMESPACE