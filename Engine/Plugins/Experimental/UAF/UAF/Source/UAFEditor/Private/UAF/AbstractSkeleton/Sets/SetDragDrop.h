// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "UAF/AbstractSkeleton/Sets/SSetBinding.h"

namespace UE::UAF::Editor
{
	class FSetDragDropOp : public FDecoratedDragDropOp
	{
	public:
		enum class EOperation : uint8
		{
			None,
			Reparent,
			Bind
		};
		
		DRAG_DROP_OPERATOR_TYPE(FSetDragDropOp, FDecoratedDragDropOp)

		static TSharedRef<FSetDragDropOp> New(TArray<FAbstractSkeletonSet>&& InSets);

		virtual void Construct() override;

		TSharedPtr<SWidget> GetDefaultDecorator() const override;

		void SetOperation(const EOperation InOperation, const FName InOperationSetName);
		void ClearOperation();

		const TArray<FAbstractSkeletonSet>& GetDraggedSets() const;
		
	private:
		TArray<FAbstractSkeletonSet> Sets;

		EOperation Operation = EOperation::None;
		FName OperationSetName;

		FText CachedSetsText;
	};
}