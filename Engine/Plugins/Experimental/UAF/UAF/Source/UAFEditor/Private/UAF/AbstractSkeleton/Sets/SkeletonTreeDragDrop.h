// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SSetSkeletonTree.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

namespace UE::UAF::Editor
{
	class FSkeletonTreeDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FSkeletonTreeDragDropOp, FDecoratedDragDropOp)

		static TSharedRef<FSkeletonTreeDragDropOp> New(TArray<TSharedPtr<SSetsSkeletonTree::FTreeItem>>&& InItems)
		{
			TSharedRef<FSkeletonTreeDragDropOp> Operation = MakeShared<FSkeletonTreeDragDropOp>();
			Operation->Items = MoveTemp(InItems);

			Operation->Construct();

			return Operation;
		}

		TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SNew(STextBlock)
						.Text(Items.Num() == 1 ? FText::FromName(Items[0]->BoneName) :	FText::Format(NSLOCTEXT("UE::UAF::Editor::FSkeletonTreeDragDropOp", "MultipleBonesTooltip", "{0} bones"), FText::AsNumber(Items.Num())))
				];
		}

		TArray<TSharedPtr<SSetsSkeletonTree::FTreeItem>> Items;
	};
}