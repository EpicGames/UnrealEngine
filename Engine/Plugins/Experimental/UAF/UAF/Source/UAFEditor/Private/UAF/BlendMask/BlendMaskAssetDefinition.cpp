// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendMaskAssetDefinition.h"

#include "UAF/BlendMask/UAFBlendMask.h"
#include "UAF/BlendMask/BlendMaskEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendMaskAssetDefinition)

#define LOCTEXT_NAMESPACE "UAFBlendMask"

FText UAssetDefinition_UAFBlendMask::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "UAF Blend Mask");
}

FLinearColor UAssetDefinition_UAFBlendMask::GetAssetColor() const
{
	return FLinearColor(FColor::Yellow);
}

TSoftClassPtr<UObject> UAssetDefinition_UAFBlendMask::GetAssetClass() const
{
	return UUAFBlendMask::StaticClass();
}

EAssetCommandResult UAssetDefinition_UAFBlendMask::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		TArray<UObject*> Assets = OpenArgs.LoadObjects<UObject>();
		MakeShared<UE::UAF::FBlendMaskEditorToolkit>()->InitEditor(Assets);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_UAFBlendMask::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")) };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
