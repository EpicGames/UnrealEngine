// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendMaskEditorToolkit.h"

#include "HierarchyTableEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "BlendMaskEditorToolkit"

namespace UE::UAF
{

FText FBlendMaskEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Blend Mask Editor");
}

void FBlendMaskEditorToolkit::InitEditor(const TArray<UObject*>& InObjects)
{
	BlendMask = CastChecked<UUAFBlendMask>(InObjects[0]);
	
	BlendMask->Table->SetFlags(RF_Transactional);
	BlendMask->UpdateHierarchy();

	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("BlendMaskEditorToolkit")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab("BlendMaskEditorTableTab", ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.3f)
				->AddTab("BlendMaskEditorDetailsTab", ETabState::OpenedTab)
			)
		);

	FAssetEditorToolkit::InitAssetEditor(EToolkitMode::Standalone, {}, "BlendMaskEditor", Layout, true, true, InObjects);

	ExtendToolbar();

	TObjectPtr<USkeleton> Skeleton = BlendMask->GetSkeleton();
	if (Skeleton)
	{
		Skeleton->RegisterOnSkeletonHierarchyChanged(USkeleton::FOnSkeletonHierarchyChanged::CreateRaw(this, &FBlendMaskEditorToolkit::OnSkeletonHierarchyChanged));
	}
}

void FBlendMaskEditorToolkit::OnClose()
{
	TObjectPtr<USkeleton> Skeleton = BlendMask->GetSkeleton();
	if (Skeleton)
	{
		Skeleton->UnregisterOnSkeletonHierarchyChanged(this);
	}
}

void FBlendMaskEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("BlendMaskStandaloneEditor", "Blend Mask Editor"));

	// Table view
	{
		FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");

		TSharedRef<IHierarchyTable> TableWidget = HierarchyTableModule.CreateHierarchyTableWidget(BlendMask->Table);
		HierarchyTableWidgetInterface = TableWidget;

		InTabManager->RegisterTabSpawner("BlendMaskEditorTableTab", FOnSpawnTab::CreateLambda([this, TableWidget](const FSpawnTabArgs&)
			{
				return SNew(SDockTab)
					[
						TableWidget
					];
			}))
			.SetDisplayName(LOCTEXT("BlendMask", "Blend Mask"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}

	// Details panel
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObjects(TArray<UObject*>{ BlendMask });

		InTabManager->RegisterTabSpawner("BlendMaskEditorDetailsTab", FOnSpawnTab::CreateLambda([=](const FSpawnTabArgs&)
			{
				return SNew(SDockTab)
					[
						DetailsView
					];
			}))
			.SetDisplayName(INVTEXT("Details"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}
}

void FBlendMaskEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	InTabManager->UnregisterTabSpawner("BlendMaskEditorTableTab");
	InTabManager->UnregisterTabSpawner("BlendMaskEditorDetailsTab");
}

void FBlendMaskEditorToolkit::OnSkeletonHierarchyChanged()
{
	BlendMask->UpdateHierarchy();
	BlendMask->UpdateCachedData();
}

void FBlendMaskEditorToolkit::ExtendToolbar()
{
	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");
	const TObjectPtr<UHierarchyTable_TableTypeHandler> Handler = HierarchyTableModule.CreateTableHandler(BlendMask->Table);
	if (!ensure(Handler))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(this);

	FName ParentName;
	static const FName MenuName = GetToolMenuToolbarName(ParentName);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);
	Handler->ExtendToolbar(ToolMenu, *HierarchyTableWidgetInterface);
}

}

#undef LOCTEXT_NAMESPACE