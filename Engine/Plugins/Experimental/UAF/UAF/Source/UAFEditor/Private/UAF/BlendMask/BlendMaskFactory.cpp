// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendMaskFactory.h"
#include "MaskProfile/HierarchyTableTypeMask.h"
#include "HierarchyTableEditorModule.h"
#include "SkeletonHierarchyTableType.h"
#include "HierarchyTableEditorModule.h"
#include "Modules/ModuleManager.h"

#include "UAF/BlendMask/UAFBlendMask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendMaskFactory)

#define LOCTEXT_NAMESPACE "UAFBlendMaskFactory"

UUAFBlendMaskFactory::UUAFBlendMaskFactory()
{
	SupportedClass = UUAFBlendMask::StaticClass();
	bCreateNew = true;
}

UObject* UUAFBlendMaskFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	TObjectPtr<UUAFBlendMask> NewBlendMask = NewObject<UUAFBlendMask>(InParent, Class, Name, Flags, Context);
	NewBlendMask->Table = NewObject<UHierarchyTable>(NewBlendMask);

	// TODO: Streamline hierarchy table creation API

	NewBlendMask->Table->Initialize(TableMetadata, FHierarchyTable_ElementType_Mask::StaticStruct());

	check(TableHandler);
	TableHandler->SetHierarchyTable(NewBlendMask->Table);
	TableHandler->ConstructHierarchy();

	return NewBlendMask;
}

bool UUAFBlendMaskFactory::ConfigureProperties()
{
	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");

	TableHandler = HierarchyTableModule.CreateTableHandler(FHierarchyTable_TableType_Skeleton::StaticStruct());
	check(TableHandler);

	TableMetadata = FInstancedStruct(FHierarchyTable_TableType_Skeleton::StaticStruct());

	// Displays window for setting skeleton
	const bool bSuccess = TableHandler->FactoryConfigureProperties(TableMetadata);

	return bSuccess;
}

#undef LOCTEXT_NAMESPACE
