// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "StructUtils/InstancedStruct.h"

#include "BlendMaskFactory.generated.h"

class UHierarchyTable_TableTypeHandler;

UCLASS()
class UUAFBlendMaskFactory : public UFactory
{
	GENERATED_BODY()

public:
	UUAFBlendMaskFactory();

	// UFactory interface
	UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;

private:
	UPROPERTY()
	TObjectPtr<UHierarchyTable_TableTypeHandler> TableHandler;

	FInstancedStruct TableMetadata;
};
