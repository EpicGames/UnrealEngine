// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendProfileAssetDefinition.h"

#include "AssetDefinitionAssetInfo.h"
#include "IAssetStatusInfoProvider.h"
#include "UAFStyle.h"
#include "UAF/BlendProfile/UAFBlendProfile.h"
#include "UAF/BlendProfile/BlendProfileEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendProfileAssetDefinition)

#define LOCTEXT_NAMESPACE "UAFBlendProfile"

FText UAssetDefinition_UAFBlendProfile::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "UAF Blend Profile");
}

FLinearColor UAssetDefinition_UAFBlendProfile::GetAssetColor() const
{
	return FLinearColor(FColor::Yellow);
}

TSoftClassPtr<UObject> UAssetDefinition_UAFBlendProfile::GetAssetClass() const
{
	return UUAFBlendProfile::StaticClass();
}

EAssetCommandResult UAssetDefinition_UAFBlendProfile::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		TArray<UObject*> Assets = OpenArgs.LoadObjects<UObject>();
		MakeShared<UE::UAF::FBlendProfileEditorToolkit>()->InitEditor(Assets);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_UAFBlendProfile::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")) };
	return Categories;
}

void UAssetDefinition_UAFBlendProfile::GetAssetStatusInfo(const TSharedPtr<IAssetStatusInfoProvider>& InAssetStatusInfoProvider, TArray<FAssetDisplayInfo>& OutStatusInfo) const
{
	using namespace UE::UAF;
	
	Super::GetAssetStatusInfo(InAssetStatusInfoProvider, OutStatusInfo);

	const FAssetData AssetData = InAssetStatusInfoProvider->TryGetAssetData();
	if (AssetData.IsValid())
	{
		FString BlendProfileType;
		if (AssetData.GetTagValue(AssetRegistryTag_BlendProfileType, BlendProfileType))
		{
			FAssetDisplayInfo TypeStatus;
			TypeStatus.Priority = FAssetStatusPriority(EStatusSeverity::Info);
			TypeStatus.IsVisible = EVisibility::Visible;
			TypeStatus.StatusIcon = (BlendProfileType == AssetRegistryTag_BlendProfileType_TimeProfile)
				? FUAFStyle::Get().GetBrush("BlendProfiles.TimeProfile")
				: FUAFStyle::Get().GetBrush("BlendProfiles.WeightProfile");
			TypeStatus.StatusDescription = (BlendProfileType == AssetRegistryTag_BlendProfileType_TimeProfile)
				? LOCTEXT("AssetDefinition_UAFBlendProfile_TimeProfileTypeTooltip", "Time Blend Profile")
				: LOCTEXT("AssetDefinition_UAFBlendProfile_WeightProfileTypeTooltip", "Weight Blend Profile");
	
			OutStatusInfo.Add(MoveTemp(TypeStatus));
		}
	}

}

#undef LOCTEXT_NAMESPACE
