// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendProfileFactory.h"
#include "HierarchyTableDefaultTypes.h"
#include "HierarchyTableEditorModule.h"
#include "SkeletonHierarchyTableType.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "SEnumCombo.h"
#include "Editor.h"

#include "UAF/BlendProfile/UAFBlendProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendProfileFactory)

#define LOCTEXT_NAMESPACE "UAFBlendProfileFactory"

UUAFBlendProfileFactory::UUAFBlendProfileFactory()
	: BlendProfileType(EUAFBlendProfileType::WeightFactor)
{
	SupportedClass = UUAFBlendProfile::StaticClass();
	bCreateNew = true;
}

UObject* UUAFBlendProfileFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	TObjectPtr<UUAFBlendProfile> NewBlendProfile = NewObject<UUAFBlendProfile>(InParent, Class, Name, Flags, Context);
	NewBlendProfile->Table = NewObject<UHierarchyTable>(NewBlendProfile);
	NewBlendProfile->Type = BlendProfileType;

	// TODO: Streamline hierarchy table creation API

	NewBlendProfile->Table->Initialize(TableMetadata, FHierarchyTable_ElementType_Float::StaticStruct());

	check(TableHandler);
	TableHandler->SetHierarchyTable(NewBlendProfile->Table);
	TableHandler->ConstructHierarchy();

	if (FHierarchyTableEntryData* RootEntry = NewBlendProfile->Table->GetMutableTableEntry(0))
	{
		if (ensure(RootEntry->IsOverridden()))
		{
			RootEntry->GetMutableValue<FHierarchyTable_ElementType_Float>()->Value = 1.0f;
		}
	}

	return NewBlendProfile;
}

bool UUAFBlendProfileFactory::ConfigureProperties()
{
	return ConfigureBlendProfileType() && ConfigureBlendProfileHierarchy();
}

bool UUAFBlendProfileFactory::ConfigureBlendProfileType()
{
	bool bConfirmClicked = false;

	TSharedPtr<SWindow> Window;
	Window = SAssignNew(Window, SWindow)
		.Title(LOCTEXT("Title", "Choose Blend Profile Type"))
		.ClientSize(FVector2D(400, 400))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SEnumComboBox, StaticEnum<EUAFBlendProfileType>())
						.CurrentValue_Lambda([this]()
							{
								return static_cast<int32>(BlendProfileType);
							})
						.OnEnumSelectionChanged_Lambda([this](int32 InEnumValue, ESelectInfo::Type SelectInfo)
							{
								BlendProfileType = static_cast<EUAFBlendProfileType>(InEnumValue);
							})
				]
			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SButton)
						.OnClicked_Lambda([&]()
							{
								bConfirmClicked = true;
								Window->RequestDestroyWindow();
								return FReply::Handled();
							})
						[
							SNew(STextBlock)
								.Text(LOCTEXT("Confirm", "Confirm"))
						]
				]
		];


	GEditor->EditorAddModalWindow(Window.ToSharedRef());

	return bConfirmClicked;
}

bool UUAFBlendProfileFactory::ConfigureBlendProfileHierarchy()
{
	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");

	TableHandler = HierarchyTableModule.CreateTableHandler(FHierarchyTable_TableType_Skeleton::StaticStruct());
	check(TableHandler);

	TableMetadata = FInstancedStruct(FHierarchyTable_TableType_Skeleton::StaticStruct());

	// Displays window for setting skeleton
	return TableHandler->FactoryConfigureProperties(TableMetadata);
}

#undef LOCTEXT_NAMESPACE
