// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "StructUtils/InstancedStruct.h"
#include "UAF/BlendProfile/UAFBlendProfile.h"

#include "BlendProfileFactory.generated.h"

class UHierarchyTable_TableTypeHandler;

UCLASS()
class UUAFBlendProfileFactory : public UFactory
{
	GENERATED_BODY()

public:
	UUAFBlendProfileFactory();

	// UFactory interface
	UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;

private:
	bool ConfigureBlendProfileType();

	bool ConfigureBlendProfileHierarchy();

	UPROPERTY()
	TObjectPtr<UHierarchyTable_TableTypeHandler> TableHandler;

	EUAFBlendProfileType BlendProfileType;

	FInstancedStruct TableMetadata;
};
