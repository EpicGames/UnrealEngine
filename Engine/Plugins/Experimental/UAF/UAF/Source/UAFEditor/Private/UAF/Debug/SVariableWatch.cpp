// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVariableWatch.h"

#include "EditorModeManager.h"
#include "Component/AnimNextComponent.h"
#include "IGameplayProvider.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "IRewindDebugger.h"
#include "ISinglePropertyView.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "ObjectAsTraceIdProxyArchiveReader.h"
#include "ObjectTrace.h"
#include "RewindDebugger/UAFTrace.h"
#if UAF_TRACE_ENABLED
#include "RewindDebugger/AnimNextProvider.h"
#endif // UAF_TRACE_ENABLED
#include "SceneOutlinerPublicTypes.h"
#include "SSceneOutliner.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Frames.h"
#include "UAF/Workspace/IDebugObjectSelector.h"
#include "UAF/Workspace/RewindDebuggerExtension.h"
#include "Variables/Outliner/VariablesOutlinerColumns.h"
#include "Variables/Outliner/VariablesOutlinerMode.h"
#include "Variables/SVariablesView.h"
#include "Workspace/AnimNextWorkspaceEditorMode.h"

static const FName ColumnId_VariableName(TEXT("VariableName"));
static const FName ColumnId_VariableValue(TEXT("VariableValue"));

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SDebugInstanceTreeRow"

namespace UE::UAF::Editor
{
	class SDebugInstanceTreeRow : public SMultiColumnTableRow<SVariableWatch::FDebugInstanceTreeItemPtr>
    {
    	SLATE_BEGIN_ARGS(SDebugInstanceTreeRow)
    	{}
    	SLATE_END_ARGS()

    public:
    	void Construct(const FArguments& InArgs, SVariableWatch::FDebugInstanceTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTableView, const bool bInHighlight)
    	{
    		Item = InItem;
    		bHighlight = bInHighlight;

    		FSuperRowType::Construct(FSuperRowType::FArguments(), InOwnerTableView);
    	}

    	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnId) override
    	{
			return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.ColorAndOpacity(bHighlight ? FAppStyle::Get().GetSlateColor("Colors.AccentBlue") : FSlateColor::UseForeground())
				.Text(FText::FromString(Item->Asset ? Item->Asset->GetName() : FString()))
			];
    	}

    private:
    	SVariableWatch::FDebugInstanceTreeItemPtr Item;
    	bool bHighlight = false;
    };
    	
    class SVariableWatchRow : public SMultiColumnTableRow<SVariableWatch::FVariablesBoxItemPtr>
	{
		SLATE_BEGIN_ARGS(SVariableWatchRow)
		{}
		SLATE_END_ARGS()
                
	public:
		void Construct(const FArguments& InArgs, SVariableWatch::FVariablesBoxItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<SVariableWatch> InVariableWatch)
		{
			Item = InItem;
			VariableWatch = InVariableWatch;

			FSuperRowType::Construct(FSuperRowType::FArguments(), InOwnerTableView);
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnId) override
		{
			if (InColumnId == ColumnId_VariableName)
			{
				return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(12.0f, 2.0f, 0.0f, 2.0f))
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), TEXT("SmallText"))
					.Text(FText::FromName(*Item))
				];
			}
			
			if (InColumnId == ColumnId_VariableValue)
			{
				FSinglePropertyParams SinglePropertyArgs;
				SinglePropertyArgs.NamePlacement = EPropertyNamePlacement::Hidden;
				SinglePropertyArgs.bHideResetToDefault = true;
				
				FInstancedPropertyBag& PropertyBag = VariableWatch.Pin()->Properties;
				const FName PropName = *Item;
				
				FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
				const TSharedPtr<ISinglePropertyView> SingleStructPropertyView = PropertyEditorModule.CreateSingleProperty(MakeShared<FInstancePropertyBagStructureDataProvider>(PropertyBag), PropName, SinglePropertyArgs);
				SingleStructPropertyView->SetEnabled(false);
                    		
				return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SingleStructPropertyView.ToSharedRef()
				];
			}

			return SNullWidget::NullWidget;
		}

    private:
    	SVariableWatch::FVariablesBoxItemPtr Item;
    	TWeakPtr<SVariableWatch> VariableWatch;
	};

	void SVariableWatch::Construct(const FArguments& InArgs, const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor)
	{
		WorkspaceEditor = InWorkspaceEditor;

		// This widget is constructed before the UAF workspace editor mode is activated so we listen to
		// when its activated and bind to the events we care about
		InWorkspaceEditor.Pin()->GetEditorModeManager().OnEditorModeIDChanged().AddSP(this, &SVariableWatch::HandleWorkspaceEditorModeChanged);
		
		InWorkspaceEditor.Pin()->OnFocusedDocumentChanged().AddSP(this, &SVariableWatch::HandleFocusedDocumentChanged);

		DebugInstanceTreeView = SNew(STreeView<FDebugInstanceTreeItemPtr>)
			.TreeItemsSource(&RootDebugInstances)
			.HeaderRow(
				SNew(SHeaderRow)
				+ SHeaderRow::Column("Instance")
				.DefaultLabel(INVTEXT("Instance"))
			)
			.OnGetChildren(this, &SVariableWatch::InstanceTreeView_OnGetChildren)
			.OnGenerateRow(this, &SVariableWatch::InstanceTreeView_OnGenerateRow)
			.OnSelectionChanged(this, &SVariableWatch::InstanceTreeView_OnSelectionChanged);

		VariablesBox = SNew(SListView<FVariablesBoxItemPtr>)
			.ListItemsSource(&VariablesBoxSource)
			.HeaderRow(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(ColumnId_VariableName)
				.DefaultLabel(LOCTEXT("ColumnLabel_VariableName", "Variable"))
				+ SHeaderRow::Column(ColumnId_VariableValue)
				.DefaultLabel(LOCTEXT("ColumnLabel_VariableValue", "Value"))
			)
			.OnGenerateRow(this, &SVariableWatch::VariablesListView_OnGenerateRow);
		
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				DebugInstanceTreeView.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				VariablesBox.ToSharedRef()
			]
		];
	}
	
	void SVariableWatch::RegenerateDebugInstances(const bool bForceRegenerate)
	{
#if UAF_TRACE_ENABLED
		if (!RewindDebugger)
		{
			return;
		}

		const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

		const double CurrentScrubTime = RewindDebugger->GetScrubTime();
		const double CurrentTraceTime = RewindDebugger->CurrentTraceTime();

		RootDebugInstances.Empty();
		DebugInstances.Empty();

		TraceServices::FFrame Frame;
		{
			const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*RewindDebugger->GetAnalysisSession());
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
			if (!FrameProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, CurrentTraceTime, Frame))
			{
				return;
			}
		}

		const FUAFProvider* UAFProvider = AnalysisSession->ReadProvider<FUAFProvider>("UAFProvider");
		if (!UAFProvider)
		{
			return;
		}

		const UE::Workspace::FWorkspaceDocument FocusedDocument = WorkspaceEditor.Pin()->GetFocusedWorkspaceDocument();
		const FSoftObjectPath FocusedAssetPath = FocusedDocument.Export.GetFirstAssetPath();

		Properties.Reset();
		NativeProperties.Reset();

		const UObject* WorkspaceDebugObject = nullptr;
		UAnimNextWorkspaceEditorMode* EditorMode = Cast<UAnimNextWorkspaceEditorMode>(WorkspaceEditor.Pin()->GetEditorModeManager().GetActiveScriptableMode(UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace));
		if (EditorMode)
		{
			WorkspaceDebugObject = EditorMode->GetDebugObjectSelector()->GetDebugObject().Get();
		}

		{
			TraceServices::FProviderReadScopeLock ProviderReadScope(*UAFProvider);
			UAFProvider->EnumerateDataInterfaces(CurrentScrubTime, [this, FocusedAssetPath, UAFProvider, WorkspaceDebugObject](const TSharedRef<const FDataInterfaceData> DataInterfaceData)
				{
					UObject* Asset = FObjectTrace::GetObjectFromId(DataInterfaceData->AssetId);
					if (FSoftObjectPath(Asset) != FocusedAssetPath)
					{
						return;
					}

					const UObject* OwnerObject = FObjectTrace::GetObjectFromId(DataInterfaceData->OuterObjectId);
					if (!WorkspaceDebugObject || (WorkspaceDebugObject != OwnerObject && !OwnerObject->IsInOuter(WorkspaceDebugObject)))
					{
						return;
					}

					TSharedPtr<FDebugInstance> NewDebugInstance = MakeShared<FDebugInstance>();
					NewDebugInstance->InstanceId = DataInterfaceData->InstanceId;
					NewDebugInstance->HostInstanceId = DataInterfaceData->HostInstanceId;
					NewDebugInstance->Asset = FObjectTrace::GetObjectFromId(DataInterfaceData->AssetId);

					DebugInstances.Add(NewDebugInstance->InstanceId, NewDebugInstance);

					uint64 HostInstanceId = NewDebugInstance->HostInstanceId;
					const FDataInterfaceData* HostInstanceData = UAFProvider->GetDataInterfaceData(HostInstanceId);
					while (HostInstanceData)
					{
						if (!DebugInstances.Contains(HostInstanceId))
						{
							TSharedPtr<FDebugInstance> NewHostDebugInstance = MakeShared<FDebugInstance>();
							NewHostDebugInstance->InstanceId = HostInstanceData->InstanceId;
							NewHostDebugInstance->HostInstanceId = HostInstanceData->HostInstanceId;
							NewHostDebugInstance->Asset = FObjectTrace::GetObjectFromId(HostInstanceData->AssetId);

							DebugInstances.Add(NewHostDebugInstance->InstanceId, NewHostDebugInstance);

							HostInstanceId = NewHostDebugInstance->HostInstanceId;
							HostInstanceData = UAFProvider->GetDataInterfaceData(HostInstanceId);
						}
						else
						{
							break;
						}
					}

					if (!DebugInstances.Contains(HostInstanceId))
					{
						TSharedPtr<FDebugInstance> NewHostDebugInstance = MakeShared<FDebugInstance>();
						NewHostDebugInstance->InstanceId = HostInstanceId;
						NewHostDebugInstance->HostInstanceId = 0;
						NewHostDebugInstance->Asset = FObjectTrace::GetObjectFromId(HostInstanceId);

						DebugInstances.Add(NewHostDebugInstance->InstanceId, NewHostDebugInstance);
						RootDebugInstances.Add(NewHostDebugInstance);
					}
				});
		}

		for (const auto& [InstanceId, DebugTreeItem] : DebugInstances)
		{
			DebugInstanceTreeView->SetItemExpansion(DebugTreeItem, true);
		}
		
		DebugInstanceTreeView->RequestTreeRefresh();
		
		if (SelectedDebugInstanceId != 0)
		{
			const auto DebugInstanceTreeItem = DebugInstances.Find(SelectedDebugInstanceId);
			if (DebugInstanceTreeItem)
			{
				DebugInstanceTreeView->SetSelection(*DebugInstanceTreeItem);
			}
			
		}
		
		OnDebugInstanceChanged();
#endif // UAF_TRACE_ENABLED
	}

	void SVariableWatch::HandleFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument)
	{
		SelectedDebugInstanceId = 0;
	
		RegenerateDebugInstances(true);
	}
	
	void SVariableWatch::HandleWorkspaceDebugObjectChanged(TWeakObjectPtr<UObject> InObject)
	{
		RegenerateDebugInstances(true);
	}
	
	void SVariableWatch::HandleRewindDebuggerUpdate(float DeltaTime, IRewindDebugger* InRewindDebugger)
	{
		RewindDebugger = InRewindDebugger;
		
		RegenerateDebugInstances(false);
	}
	
	void SVariableWatch::OnDebugInstanceChanged()
	{
#if UAF_TRACE_ENABLED
		VariablesBoxSource.Empty();

		const double CurrentTraceTime = RewindDebugger->CurrentTraceTime();

		const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
		const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
		const FUAFProvider* UAFProvider = AnalysisSession->ReadProvider<FUAFProvider>("UAFProvider");
		if (!GameplayProvider || !UAFProvider)
		{
			return;
		}

		TraceServices::FFrame Frame;
		{
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
			if (!TraceServices::ReadFrameProvider(*AnalysisSession).GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, CurrentTraceTime, Frame))
			{
				return;
			}
		}

		{
			TraceServices::FProviderReadScopeLock GameplayProviderReadScope(*GameplayProvider);
			TraceServices::FProviderReadScopeLock UAFProviderReadScope(*UAFProvider);
			const FDataInterfaceData* DataInterfaceData = UAFProvider->GetDataInterfaceData(SelectedDebugInstanceId);
			if (!DataInterfaceData)
			{
				return;
			}
			DataInterfaceData->VariablesTimeline.EnumerateEvents(Frame.StartTime, Frame.EndTime, [UAFProvider, GameplayProvider, AnalysisSession, this](double InStartTime, double InEndTime, uint32 InDepth, const FPropertyVariableData& VariableListData)
				{
					// Most of this reading code is ripped from FAnimNextModuleTrack
					if (VariableListData.ValueType == EPropertyVariableDataType::PropertyBag)
					{
						// First look up property description
						const FPropertyDescriptionData* DescriptionData = UAFProvider->GetPropertyDescriptionData(VariableListData.DescriptionHash);
						if (DescriptionData == nullptr)
						{
							// Can't do anything without a description
							return TraceServices::EEventEnumerate::Continue;
						}

						// Load the property descriptions
						TArray<FPropertyBagPropertyDesc> PropertyDescriptions;
						FMemoryReader DescriptionReader(DescriptionData->PropertyBagData);
						FObjectAsTraceIdProxyArchiveReader DescriptionReaderProxy(DescriptionReader, GameplayProvider);
						DescriptionReaderProxy.UsingCustomVersion(UE::UAF::FUAFTrace::CustomVersionGUID);
						DescriptionReaderProxy << PropertyDescriptions;

						//Properties->Properties.SetNum(Properties->Properties.Num() + 1);
						Properties.AddProperties(PropertyDescriptions);

						// Load the property values
						FMemoryReader Reader(VariableListData.ValueData);
						FObjectAsTraceIdProxyArchiveReader ReaderProxy(Reader, GameplayProvider);

						UPropertyBag* PropertyBagStruct = const_cast<UPropertyBag*>(Properties.GetPropertyBagStruct());
						if (PropertyBagStruct != nullptr)
						{
							PropertyBagStruct->SerializeItem(ReaderProxy, Properties.GetMutableValue().GetMemory(), nullptr);
						}
					}
					else
					{
						FMemoryReader Reader(VariableListData.ValueData);
						FObjectAsTraceIdProxyArchiveReader ReaderProxy(Reader, GameplayProvider);

						FInstancedStruct::StaticStruct()->SerializeItem(ReaderProxy, &NativeProperties, nullptr);
					}

					return TraceServices::EEventEnumerate::Continue;
				});
		}

		// This gives you the schema (like a struct definition)
		const UPropertyBag* BagStruct = Properties.GetPropertyBagStruct();
		if (!BagStruct)
		{
			return;
		}
		
		// Iterate over defined properties
		for (const FPropertyBagPropertyDesc& PropDesc : BagStruct->GetPropertyDescs())
		{
			const FName PropName = PropDesc.Name;
			VariablesBoxSource.Add(MakeShared<FName>(PropName));
		}
		
		VariablesBox->RequestListRefresh();
#endif // UAF_TRACE_ENABLED
	}

	void SVariableWatch::HandleWorkspaceEditorModeChanged(const FEditorModeID& EditorModeId, bool bIsEnteringMode)
	{
		if (EditorModeId == UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace)
		{
			if (bIsEnteringMode)
			{
				UAnimNextWorkspaceEditorMode* EditorMode = Cast<UAnimNextWorkspaceEditorMode>(WorkspaceEditor.Pin()->GetEditorModeManager().GetActiveScriptableMode(UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace));
				if (ensure(EditorMode))
				{
					EditorMode->GetRewindDebuggerExtension()->RegisterOnRewindDebuggerUpdate(
						IUAFWorkspaceRewindDebugger::FOnRewindDebuggerUpdate::FDelegate::CreateSP(this, &SVariableWatch::HandleRewindDebuggerUpdate));

					EditorMode->GetDebugObjectSelector()->RegisterOnDebugObjectChanged(
						IDebugObjectSelector::FOnDebugObjectChanged::FDelegate::CreateSP(this, &SVariableWatch::HandleWorkspaceDebugObjectChanged));
				}
			}
		}
	}

	void SVariableWatch::InstanceTreeView_OnGetChildren(FDebugInstanceTreeItemPtr InItem, TArray<FDebugInstanceTreeItemPtr>& OutChildren)
	{
		for (const auto& [InstanceId, DebugInstance] : DebugInstances)
		{
			if (DebugInstance->HostInstanceId == InItem->InstanceId)
			{
				OutChildren.Add(DebugInstance);
			}
		}
	}

	TSharedRef<ITableRow> SVariableWatch::InstanceTreeView_OnGenerateRow(FDebugInstanceTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		const bool bHighlight = FSoftObjectPath(InItem->Asset) == WorkspaceEditor.Pin()->GetFocusedWorkspaceDocument().Export.GetFirstAssetPath();
		return SNew(SDebugInstanceTreeRow, InItem, OwnerTable, bHighlight);
	}
	
	void SVariableWatch::InstanceTreeView_OnSelectionChanged(FDebugInstanceTreeItemPtr NewSelection, ESelectInfo::Type)
	{
		if (NewSelection.IsValid() && NewSelection->InstanceId != SelectedDebugInstanceId)
		{
			SelectedDebugInstanceId = NewSelection->InstanceId;
		}
		OnDebugInstanceChanged();
	}

	TSharedRef<ITableRow> SVariableWatch::VariablesListView_OnGenerateRow(const FVariablesBoxItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(SVariableWatchRow, InItem, OwnerTable, SharedThis(this));
	}
}

#undef LOCTEXT_NAMESPACE