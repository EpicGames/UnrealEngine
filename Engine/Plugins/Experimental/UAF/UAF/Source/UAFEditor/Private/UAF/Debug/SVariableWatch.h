// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor.h"
#include "Widgets/Views/STreeView.h"
#include "StructUtils/PropertyBag.h"

class IRewindDebugger;

class UUAFRigVMAssetEntry;

namespace UE::Workspace
{
	class IWorkspaceEditor;
	struct FWorkspaceDocument;
}

namespace UE::UAF::Editor
{
	class SVariablesOutliner;

	class SVariableWatch : public SCompoundWidget
	{
	public:
		friend class SVariableWatchRow;
		
		SLATE_BEGIN_ARGS(SVariableWatch)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TWeakPtr<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor);

		struct FDebugInstance
		{
			uint64 InstanceId;
			uint64 HostInstanceId;
			UObject* Asset;
		};

		using FDebugInstanceTreeItemPtr = TSharedPtr<FDebugInstance>;

		using FVariablesBoxItemPtr = TSharedPtr<FName>;
	
	private:
		void HandleWorkspaceEditorModeChanged(const FEditorModeID& EditorModeId, bool bIsEnteringMode);
		
		void HandleRewindDebuggerUpdate(float DeltaTime, IRewindDebugger* RewindDebugger);
		
		void RegenerateDebugInstances(const bool bForceRegenerate);
		void OnDebugInstanceChanged();

		void HandleFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument);
		void HandleWorkspaceDebugObjectChanged(TWeakObjectPtr<UObject> InObject);

		void InstanceTreeView_OnGetChildren(FDebugInstanceTreeItemPtr InItem, TArray<FDebugInstanceTreeItemPtr>& OutChildren);
		TSharedRef<ITableRow> InstanceTreeView_OnGenerateRow(FDebugInstanceTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);
		void InstanceTreeView_OnSelectionChanged(FDebugInstanceTreeItemPtr NewSelection, ESelectInfo::Type);

		TSharedRef<ITableRow> VariablesListView_OnGenerateRow(const FVariablesBoxItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);
		
	private:
		IRewindDebugger* RewindDebugger = nullptr;
	
		FInstancedPropertyBag Properties;
		FInstancedStruct NativeProperties;

	private:
		TWeakPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor;

		TArray<FDebugInstanceTreeItemPtr> RootDebugInstances;
		TMap<uint64, FDebugInstanceTreeItemPtr> DebugInstances;

		TSharedPtr<STreeView<FDebugInstanceTreeItemPtr>> DebugInstanceTreeView;
		
		FDebugInstanceTreeItemPtr SelectedDebugInstance;

	public:
		class FTreeItem
		{
		public:
			TStrongObjectPtr<UUAFRigVMAssetEntry> AssetEntry;
		};

		using FTreeItemPtr = TSharedPtr<FTreeItem>;
		
		TSharedPtr<STreeView<FTreeItemPtr>> TreeView;
		
	private:
		TSharedPtr<SListView<FVariablesBoxItemPtr>> VariablesBox;
		
		TArray<FVariablesBoxItemPtr> VariablesBoxSource;

		TArray<FTreeItemPtr> TreeItems;
		
		uint64 SelectedDebugInstanceId = 0;
	};
}