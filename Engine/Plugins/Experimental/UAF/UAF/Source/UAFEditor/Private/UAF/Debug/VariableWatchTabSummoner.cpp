// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariableWatchTabSummoner.h"
#include "SVariableWatch.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::FVariableWatchTabSummoner"

namespace UE::UAF::Editor
{
	FVariableWatchTabSummoner::FVariableWatchTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp)
		: FWorkflowTabFactory(TEXT("VariableWatch"), StaticCastSharedPtr<FAssetEditorToolkit>(InHostingApp))
	{
		TabLabel = LOCTEXT("TabLabel", "Variable Watch");
		TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults");
		ViewMenuDescription = LOCTEXT("TabMenuDescription", "Variable Watch");
		ViewMenuTooltip = LOCTEXT("TabMenuTooltip", "Track the values of variables.");
		bIsSingleton = true;

		Widget = SNew(SVariableWatch, InHostingApp);
	}

	TSharedRef<SWidget> FVariableWatchTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
	{
		return Widget.ToSharedRef();
	}

	FText FVariableWatchTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
	{
		return ViewMenuTooltip;
	}
}

#undef LOCTEXT_NAMESPACE