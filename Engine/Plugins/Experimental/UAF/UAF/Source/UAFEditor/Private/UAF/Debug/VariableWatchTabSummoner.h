// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IWorkspaceEditor.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

namespace UE::UAF::Editor
{
	class SVariableWatch;
	
	struct FVariableWatchTabSummoner : public FWorkflowTabFactory
	{
	public:
		FVariableWatchTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp);

	private:
		// FWorkflowTabFactory interface
		virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
		virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

		// The widget this tab spawner wraps
		TSharedPtr<SVariableWatch> Widget;
	};
}
