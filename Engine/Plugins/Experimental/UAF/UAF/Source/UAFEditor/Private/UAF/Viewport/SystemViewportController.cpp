// Copyright Epic Games, Inc. All Rights Reserved.

#include "SystemViewportController.h"

#include "AdvancedPreviewScene.h"
#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "IWorkspaceEditor.h"
#include "Component/AnimNextComponent.h"
#include "Engine/PreviewMeshCollection.h"
#include "UAF/Viewport/ViewportSceneDescription.h"
#include "Components/SkeletalMeshComponent.h"

namespace UE::UAF::Editor
{
	void FSystemViewportController::OnEnter(const FViewportContext& InViewportContext)
	{
		UUAFSystem* System = Cast<UUAFSystem>(InViewportContext.OutlinerObject);
		UUAFViewportSceneDescription* UAFViewportSceneDescription = CastChecked<UUAFViewportSceneDescription>(InViewportContext.SceneDescription);
		
		const TObjectPtr<UWorld> PreviewWorld = InViewportContext.PreviewScene->GetWorld();

		const auto AddActor = [&](USkeletalMesh* InSkeletalMesh)
			{
				AActor* PreviewActor = PreviewWorld->SpawnActor<AActor>(FVector::ZeroVector, FRotator::ZeroRotator);
				check(PreviewActor);

				USkeletalMeshComponent* SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(PreviewActor);
				InViewportContext.PreviewScene->AddComponent(SkeletalMeshComponent, FTransform::Identity);
				PreviewActor->SetRootComponent(SkeletalMeshComponent);
				SkeletalMeshComponent->SetEnableAnimation(false);

				SkeletalMeshComponent->SetSkeletalMesh(InSkeletalMesh);

				UUAFComponent* UAFComponent = NewObject<UUAFComponent>(PreviewActor);
				PreviewActor->AddInstanceComponent(UAFComponent);
				UAFComponent->SetAssetFromObject(System);
				UAFComponent->RegisterComponent();
				UAFComponent->InitializeComponent();
			
				PreviewActors.Add(PreviewActor);
			};

		if (UAFViewportSceneDescription->SkeletalMesh)
		{
			AddActor(UAFViewportSceneDescription->SkeletalMesh);
		}

		if (UAFViewportSceneDescription->AdditionalMeshes)
		{
			for (int32 MeshIndex = 0; MeshIndex < UAFViewportSceneDescription->AdditionalMeshes->SkeletalMeshes.Num(); ++MeshIndex)
			{
				const TSoftObjectPtr<USkeletalMesh> SkelMeshSoftPtr = UAFViewportSceneDescription->AdditionalMeshes->SkeletalMeshes[MeshIndex].SkeletalMesh;
				
				if (!SkelMeshSoftPtr.IsNull())
				{
					USkeletalMesh* SkeletalMesh = SkelMeshSoftPtr.LoadSynchronous();
					if (SkeletalMesh)
					{
						AddActor(SkeletalMesh);
					}
				}
			}
		}
	}
	
	void FSystemViewportController::OnExit(FAdvancedPreviewScene* PreviewScene)
	{
		for (AActor* PreviewActor : PreviewActors)
		{
			if (PreviewActor)
			{
				if (PreviewScene && PreviewActor->GetRootComponent())
				{
					PreviewScene->RemoveComponent(PreviewActor->GetRootComponent());
				}
			
				PreviewActor->Destroy();
				PreviewActor = nullptr;
			}
		}
		
		
		PreviewActors.Empty();
	}
}
