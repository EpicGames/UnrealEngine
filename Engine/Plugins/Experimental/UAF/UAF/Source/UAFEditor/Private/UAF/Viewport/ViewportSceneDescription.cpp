// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Viewport/ViewportSceneDescription.h"

#include "PropertyCustomizationHelpers.h"
#include "IWorkspaceEditor.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Component/AnimNextComponent.h"

void UUAFViewportSceneDescription::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UUAFViewportSceneDescription, SkeletalMesh) || PropertyName == GET_MEMBER_NAME_CHECKED(UUAFViewportSceneDescription, AdditionalMeshes))
	{
		BroadcastOnConfigChanged();
	}
}

