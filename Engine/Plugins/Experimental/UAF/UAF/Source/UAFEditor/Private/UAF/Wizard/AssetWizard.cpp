// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Wizard/AssetWizard.h"

#include "AssetToolsModule.h"
#include "Component/AnimNextComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "ContentBrowserModule.h"
#include "EditorModeManager.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/SkeletalMesh.h"
#include "Factories/BlueprintFactory.h"
#include "Framework/Application/SlateApplication.h"
#include "IContentBrowserSingleton.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Module/AnimNextModule.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "TemplateConfig.h"
#include "TemplateDataAsset.h"
#include "UAF/Wizard/STemplatePicker.h"
#include "UObject/SavePackage.h"
#include "Workspace/AnimNextWorkspaceEditorMode.h"
#include "Workspace/AnimNextWorkspaceFactory.h"
#include "ObjectTools.h"
#include "TemplateConfig.h"
#include "IContentBrowserSingleton.h"
#include "UObject/SavePackage.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::FAssetWizard"

namespace UE::UAF::Editor
{
	void FAssetWizard::CreateAssets(const TObjectPtr<const UUAFTemplateDataAsset> Template, const TObjectPtr<const UUAFTemplateConfig> TemplateConfig)
	{
		FScopedSlowTask SlowTask(4.0f, LOCTEXT("SlowTask", "Creating Assets from Template"));
		SlowTask.MakeDialog(false);

		TMap<UObject*, UObject*> TemplateToUserAssetMap;

		// 1. Start by duplicating all the template assets into the user's chosen directory.
		{
			SlowTask.EnterProgressFrame(1.0f, LOCTEXT("CreateAssets", "Duplicating template assets"));

			bool bFailedToInstantiateTemplateAssets = false;
			
			for (int32 TemplateAssetIndex = 0; TemplateAssetIndex < Template->Assets.Num(); ++TemplateAssetIndex)
			{
				TObjectPtr<UObject> TemplateAsset  = Template->Assets[TemplateAssetIndex];
				
				const FString UserAssetName = TemplateConfig->AssetNaming[TemplateAssetIndex];
				
				if (TObjectPtr<UObject> UserAsset = IAssetTools::Get().DuplicateAsset(UserAssetName, TemplateConfig->OutputPath.Path, TemplateAsset))
				{
					TemplateToUserAssetMap.Add(TemplateAsset, UserAsset);
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Asset Wizard: Failed to duplicate asset: %ls", *TemplateAsset->GetFullName());
					bFailedToInstantiateTemplateAssets = true;
					break;
				}
			}

			if (bFailedToInstantiateTemplateAssets)
			{
				for (TPair<UObject*, UObject*> AssetPair : TemplateToUserAssetMap)
				{
					UObject* UserAsset = AssetPair.Value;
					ObjectTools::DeleteSingleObject(UserAsset);
				}
				return;
			}
		}

		// 2. Scan for all references within our new user assets that refer to the template assets and replace them with their user asset counterpart.
		{		
			SlowTask.EnterProgressFrame(1.0f, LOCTEXT("FixingReferences", "Fixing asset references"));

			for (TPair<UObject*, UObject*> AssetPair : TemplateToUserAssetMap)
			{
				UObject* UserAsset = AssetPair.Value;

				FArchiveReplaceObjectRef<UObject> ReplaceAr(UserAsset, TemplateToUserAssetMap);
				UserAsset->Serialize(ReplaceAr);

				UserAsset->MarkPackageDirty();

				FString PackageFileName = FPackageName::LongPackageNameToFilename(
					UserAsset->GetPackage()->GetName(),
					FPackageName::GetAssetPackageExtension()
				);

				FSavePackageArgs Args;
				Args.TopLevelFlags = RF_Public | RF_Standalone;
				Args.Error = GError;

				UPackage::SavePackage(UserAsset->GetPackage(), UserAsset, *PackageFileName, Args);
			}
		}

		// 3. Create a blueprint asset depending on the template config
		{
			SlowTask.EnterProgressFrame(1.0f, LOCTEXT("ConstructingBlueprint", "Constructing blueprint"));

			if (TemplateConfig->BlueprintMode == ETemplateBlueprintMode::CreateNewBlueprint || TemplateConfig->BlueprintMode == ETemplateBlueprintMode::ModifyExistingBlueprint)
			{
				UBlueprint* Blueprint = nullptr;

				if (TemplateConfig->BlueprintMode == ETemplateBlueprintMode::CreateNewBlueprint)
				{
					TObjectPtr<UBlueprintFactory> BlueprintFactory = NewObject<UBlueprintFactory>();

					BlueprintFactory->ParentClass = TemplateConfig->BlueprintClass.Get()
						? TemplateConfig->BlueprintClass.Get()
						: AActor::StaticClass();

					FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
					IAssetTools& AssetTools = AssetToolsModule.Get();

					TObjectPtr<UObject> NewAsset = AssetTools.CreateAsset(TemplateConfig->BlueprintAssetName, TemplateConfig->OutputPath.Path, UBlueprint::StaticClass(), BlueprintFactory);
					Blueprint = Cast<UBlueprint>(NewAsset);

					if (!Blueprint)
					{
						UE_LOGF(LogAnimation, Error, "Asset Wizard: Failed to create blueprint");
					}
				}
				else
				{
					Blueprint = TemplateConfig->BlueprintToModify.LoadSynchronous();

					if (!Blueprint)
					{
						UE_LOGF(LogAnimation, Error, "Asset Wizard: Failed to load blueprint to modify");
					}
				}

				if (Blueprint)
				{
					AActor* Actor = CastChecked<AActor>(Blueprint->GeneratedClass->GetDefaultObject());

					USkeletalMeshComponent* SkelMeshComp = Actor->GetComponentByClass<USkeletalMeshComponent>();
					UUAFComponent* UAFComp = Actor->GetComponentByClass<UUAFComponent>();

					// Create a USkeletalMeshComponent and/or a UAnimNextComponent if either are missing
					{
						USimpleConstructionScript* ConstructionScript = Blueprint->SimpleConstructionScript;
						USCS_Node* DefaultRootNode = ConstructionScript->GetDefaultSceneRootNode();

						if (!SkelMeshComp)
						{
							USCS_Node* SkelMeshCompNode = ConstructionScript->CreateNode(USkeletalMeshComponent::StaticClass(), "Mesh");
							if (DefaultRootNode)
							{
								DefaultRootNode->AddChildNode(SkelMeshCompNode);
							}
							else
							{
								ConstructionScript->AddNode(SkelMeshCompNode);
							}

							SkelMeshComp = CastChecked<USkeletalMeshComponent>(SkelMeshCompNode->ComponentTemplate);
						}

						if (!UAFComp)
						{
							USCS_Node* UAFCompNode = ConstructionScript->CreateNode(UUAFComponent::StaticClass(), "AnimationFramework");
							ConstructionScript->AddNode(UAFCompNode);

							UAFComp = CastChecked<UUAFComponent>(UAFCompNode->ComponentTemplate);
						}
					}

					// Configure components
					{
						check(SkelMeshComp && UAFComp);

						SkelMeshComp->SetSkeletalMesh(TemplateConfig->SkeletalMesh.LoadSynchronous());
						SkelMeshComp->SetEnableAnimation(false);

						FTransform RelativeTransform;
						RelativeTransform.SetTranslation(FVector(0, 0, -90.0));
						RelativeTransform.SetRotation(FQuat::MakeFromEuler(FVector(0, 0, -90.0)));
						SkelMeshComp->SetRelativeTransform(RelativeTransform);

						if (Template->SystemToAssignToComponent)
						{
							UObject** UserAssetToAssignPtr = TemplateToUserAssetMap.Find(Template->SystemToAssignToComponent);
							if (ensureAlwaysMsgf(UserAssetToAssignPtr, TEXT("Asset Wizard: Could not find user system asset counterpart for '%s' to assign to the UAF component. Is the template itself misconfigured?"), *Template->AssetToOpen->GetFullName()))
							{
								UAFComp->SetAssetFromObject(*UserAssetToAssignPtr);
							}
						}
					}

					FKismetEditorUtilities::CompileBlueprint(Blueprint);
				}
			}
		}

		// 4. Open up one of the newly created assets as determined by the template.
		{
			SlowTask.EnterProgressFrame(1.0f, LOCTEXT("Finalizing", "Finalizing"));

			if (Template->AssetToOpen)
			{
				UObject** UserAssetToOpenPtr = TemplateToUserAssetMap.Find(Template->AssetToOpen);
				if (ensureAlwaysMsgf(UserAssetToOpenPtr, TEXT("Asset Wizard: Could not open user asset counterpart for '%s' to open. Is the template itself misconfigured?"), *Template->AssetToOpen->GetFullName()))
				{
					Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
					Workspace::IWorkspaceEditor* WorkspaceEditor = WorkspaceEditorModule.OpenWorkspaceForObject(*UserAssetToOpenPtr, Workspace::EOpenWorkspaceMethod::Default, UAnimNextWorkspaceFactory::StaticClass());

					UAnimNextWorkspaceEditorMode* EditorMode = Cast<UAnimNextWorkspaceEditorMode>(WorkspaceEditor->GetEditorModeManager().GetActiveScriptableMode(UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace));
					EditorMode->HandleCompile();
				}
			}
		}
	}

	void FAssetWizard::ShowTemplatePicker()
	{
		static TWeakPtr<SWindow> AssetWizardWindowWeak;

		TSharedPtr<SWindow> AssetWizardWindow = AssetWizardWindowWeak.Pin();
		if (AssetWizardWindow.IsValid())
		{
			AssetWizardWindow->BringToFront(true);
			return;
		}
			
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

		TArray<FString> SelectedPaths;
		const FContentBrowserItemPath CurrentPath = ContentBrowserModule.Get().GetCurrentPath();

		GetMutableDefault<UUAFTemplateConfig>()->OutputPath.Path = CurrentPath.HasInternalPath() ? CurrentPath.GetInternalPathString() : TEXT("/Game");

		TSharedRef<SWindow> TemplateWindow = SNew(SWindow)
			.Title(LOCTEXT("AssetWizard", "UAF Asset Wizard"))
			.ClientSize(FVector2D(1200, 800))
			.SupportsMinimize(false)
			.SupportsMaximize(false);

		AssetWizardWindowWeak = TemplateWindow;

		TemplateWindow->SetOnWindowClosed(FOnWindowClosed::CreateLambda([](const TSharedRef<SWindow>&)
		{
			AssetWizardWindowWeak.Reset();
		}));

		TemplateWindow->SetContent(
			SNew(STemplatePicker)
			.OnTemplateSelected_Lambda([TemplateWindow](const TObjectPtr<const UUAFTemplateDataAsset> InTemplate, const TObjectPtr<const UUAFTemplateConfig> InTemplateConfig) mutable
			{
				FAssetWizard::CreateAssets(InTemplate, InTemplateConfig);
				FSlateApplication::Get().RequestDestroyWindow(TemplateWindow);
			})
			.OnCancel_Lambda([TemplateWindow]()
			{
				FSlateApplication::Get().RequestDestroyWindow(TemplateWindow);
			})
		);

		FSlateApplication::Get().AddWindow(TemplateWindow);
	}
	
	void FAssetWizard::Launch()
	{
		ShowTemplatePicker();		
	}
}

#undef LOCTEXT_NAMESPACE