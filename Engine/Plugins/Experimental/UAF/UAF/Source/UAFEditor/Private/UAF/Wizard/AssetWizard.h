// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UUAFTemplateDataAsset;
class UUAFTemplateConfig;
class UWorkspace;

namespace UE::UAF::Editor
{
	class FAssetWizard
	{
	public:
		static void Launch();
		
	private:
		static void ShowTemplatePicker();

		static void CreateAssets(const TObjectPtr<const UUAFTemplateDataAsset> Template, const TObjectPtr<const UUAFTemplateConfig> TemplateConfig);
	};
}