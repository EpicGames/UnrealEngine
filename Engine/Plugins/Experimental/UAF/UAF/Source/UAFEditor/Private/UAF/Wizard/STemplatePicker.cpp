// Copyright Epic Games, Inc. All Rights Reserved.

#include "STemplatePicker.h"

#include "UAFStyle.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetThumbnail.h" 
#include "DetailsViewArgs.h"
#include "Engine/Blueprint.h"
#include "Misc/MessageDialog.h"
#include "PropertyEditorModule.h"
#include "SPrimaryButton.h"
#include "Styling/StyleColors.h"
#include "Subsystems/EditorAssetSubsystem.h"
#include "TemplateConfig.h"
#include "TemplateDataAsset.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STileView.h"

#include "Editor/EditorEngine.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::STemplatePicker"

namespace UE::UAF::Editor
{
	void STemplatePicker::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(TemplateConfig);

		for (FTemplateItemPtr TemplatePtr : Templates)
		{
			Collector.AddReferencedObject(TemplatePtr->Template);
		}
	}

	void STemplatePicker::Construct(const FArguments& InArgs)
	{
		OnTemplateSelected = InArgs._OnTemplateSelected;
		
		TSharedRef<SWidget> TemplatesView = SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(TemplatesTileView, STileView<FTemplateItemPtr>)
				.ListItemsSource(&Templates)
				.SelectionMode(ESelectionMode::Single)
				.ClearSelectionOnClick(true)
				.ItemAlignment(EListItemAlignment::LeftAligned)
				.OnGenerateTile_Lambda([](FTemplateItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
				{
					return SNew(SModuleTemplate, OwnerTable)
						.Item(InItem);
				})
				.ItemWidth(111)
				.ItemHeight(162)
				.OnSelectionChanged_Lambda([this](FTemplateItemPtr InItem, ESelectInfo::Type SelectInfo)
				{
					SelectedTemplate = InItem;
					HandleSelectedTemplateChanged();
				})
			];

		TemplateConfig = GetMutableDefault<UUAFTemplateConfig>();
		TemplateConfig->Reset();
		TemplateConfig->OnOutputPathChanged.AddSP(this, &STemplatePicker::HandleOutputPathChanged);
		TemplateConfig->OnBlueprintToModifyChanged.AddSP(this, &STemplatePicker::HandleBlueprintToModifyChanged);
		
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bAllowFavoriteSystem = false;
		DetailsViewArgs.bHideSelectionTip = true;

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		TSharedRef<IDetailsView> DetailsObjView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsObjView->SetObject(TemplateConfig);

		TSharedRef<SWidget> DetailsView = SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				// Panel selector
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Center)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSegmentedControl<EPanel>)
						.IsEnabled_Lambda([this]()
						{
							return SelectedTemplate.IsValid();
						})
						.UniformPadding(FMargin(8.0f, 0.0f))
						.OnValueChanged_Lambda([this](const EPanel InPanel)
						{
							ShowPanel = InPanel;
						})
						.Value_Lambda([this]()
						{
							return ShowPanel;
						})	
						+ SSegmentedControl<EPanel>::Slot(EPanel::Overview)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.Padding(2.0f)
							.AutoWidth()
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("GraphEditor.Animation_16x"))
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.0f)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("Overview", "Overview"))
							]
						]
						+ SSegmentedControl<EPanel>::Slot(EPanel::Details)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.Padding(2.0f)
							.AutoWidth()
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("LevelEditor.Tabs.Details"))
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(2.0f)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("Details", "Details"))
							]
						]
					]
				]
				// Panel
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex_Lambda([this]()
					{
						return static_cast<int32>(ShowPanel);
					})
					+ SWidgetSwitcher::Slot() // EPanel::Overview
					.Padding(8.0f)
					[
						SNew(SVerticalBox)
						// Preview image
						+ SVerticalBox::Slot()
						.HAlign(HAlign_Center)
						.AutoHeight()
						[
							SNew(SScaleBox)
							.Stretch(EStretch::ScaleToFill)
							[
								SNew(SImage)
								.DesiredSizeOverride(FVector2D(600,300))
								.Image_Lambda([this]() -> const FSlateBrush*
								{
									if (SelectedTemplate && SelectedTemplate->Template->DetailsImage.IsSet())
									{
										return &SelectedTemplate->Template->DetailsImage;
									}

									return FAppStyle::GetNoBrush();
								})
							]
						]
						// Template Name
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(FMargin(0.0f, 16.0f, 0.0f, 2.0f))
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.TextStyle(FAppStyle::Get(), "LargeText")
							.Text_Lambda([this]()
							{
								return SelectedTemplate.IsValid()
									? SelectedTemplate->Template->Title
									: FText();
							})
						]
						// Template Description
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(FMargin(0.0f, 2.0f, 0.0f, 0.0f))
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text_Lambda([this]()
							{
								return SelectedTemplate.IsValid()
									? SelectedTemplate->Template->Description
									: FText();
							})
						]
						// Documentation URL
						+ SVerticalBox::Slot()
						.Padding(0.0f, 24.0f)
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SButton)
								.Cursor(EMouseCursor::Hand)
								.Visibility_Lambda([this]()
								{
									return SelectedTemplate.IsValid() && !SelectedTemplate->Template->DocumentationUrl.IsEmpty()
										? EVisibility::Visible
										: EVisibility::Collapsed;
								})
								.ButtonStyle(FCoreStyle::Get(), "NoBorder")
								.OnClicked_Lambda([this]()
								{
									if (SelectedTemplate.IsValid() && !SelectedTemplate->Template->DocumentationUrl.IsEmpty())
									{
										FPlatformProcess::LaunchURL(*SelectedTemplate->Template->DocumentationUrl, nullptr, nullptr);
									}
									return FReply::Handled();
								})
								.ContentPadding(0)
								[
									SNew(SHorizontalBox)

									+ SHorizontalBox::Slot()
									.AutoWidth()
									[
										SNew(STextBlock)
										.Text(LOCTEXT("Documentation", "Documentation"))
										.ColorAndOpacity(FLinearColor(0.2f, 0.4f, 1.0f))
									]

									+ SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(FMargin(4, 0, 0, 0))
									[
										SNew(SImage)
										.Image(FUAFStyle::Get().GetBrush("AssetWizard.OpenExternal"))
										.ColorAndOpacity(FLinearColor(0.2f, 0.4f, 1.0f))
									]
								]
							]
						]
						// Tags
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SAssignNew(TagsHBox, SHorizontalBox)
						]
					]
					+ SWidgetSwitcher::Slot() // EPanel::Details
					[
						SNew(SVerticalBox)
						.Visibility_Lambda([this]()
						{
							return SelectedTemplate.IsValid() ? EVisibility::Visible : EVisibility::Hidden;		
						})
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							DetailsObjView
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SExpandableArea)
							.AreaTitle(LOCTEXT("IncludedAssets", "Included Assets"))
							.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
							.BodyBorderImage(FAppStyle::GetBrush("Brushes.Recessed"))
							.HeaderPadding(4.0f)
							.Padding(2.0f)
							.AllowAnimatedTransition(false)
							.InitiallyCollapsed(false)
							.BodyContent()
							[
								SAssignNew(IncludedAssetsListView, SListView<TSharedPtr<FAssetMetadata>>)
								.SelectionMode(ESelectionMode::None)
								.ListItemsSource(&IncludedAssets)
								.OnGenerateRow_Lambda([this](TSharedPtr<FAssetMetadata> InAsset, const TSharedRef<STableViewBase>& InTableView)
								{
									TSharedPtr<FAssetThumbnailPool> Pool = UThumbnailManager::Get().GetSharedThumbnailPool();
									TSharedPtr<FAssetThumbnail> AssetThumbnail = MakeShareable(new FAssetThumbnail(InAsset->AssetData.GetAsset(), 32, 32, Pool));

									FAssetThumbnailConfig AssetThumbnailConfig;
									AssetThumbnailConfig.ShowAssetBorder = true;
																	
									return SNew(STableRow<TSharedPtr<FAssetMetadata>>, InTableView)
									.Content()
									[
										SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.FillWidth(0.5f)
										.Padding(4.0f)
										[
											SNew(SHorizontalBox)
											+ SHorizontalBox::Slot()
											.AutoWidth()
											[
												AssetThumbnail->MakeThumbnailWidget(AssetThumbnailConfig)

											]
											+ SHorizontalBox::Slot()
											.AutoWidth()
											.VAlign(VAlign_Center)
											.Padding(4.0f)
											[
												SNew(SVerticalBox)
												+ SVerticalBox::Slot()
												.AutoHeight()
												[
													SNew(STextBlock)
													.Text(FText::FromName(InAsset->AssetData.AssetName))
												]
												+ SVerticalBox::Slot()
												.AutoHeight()
												[
													SNew(STextBlock)
													.TextStyle(&FAppStyle::Get(), "SmallText.Subdued")
													.Text(InAsset->AssetData.GetClass()->GetDisplayNameText())
												]
											]
										]
										+ SHorizontalBox::Slot()
										.FillWidth(0.5f)
										.Padding(4.0f)
										.VAlign(VAlign_Center)
										[
											SNew(SEditableTextBox)
											.Text_Lambda([InAsset]
											{
												return FText::FromString(InAsset->AssetRename);
											})
											.OnTextCommitted_Lambda([InAsset](const FText& InText, ETextCommit::Type)
											{
												InAsset->AssetRename = InText.ToString();
											})
											.OnVerifyTextChanged_Lambda([this, InAsset](const FText& InNewText, FText& OutErrorMessage)
											{
												UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();

												const FString AssetPath = FPaths::Combine(TemplateConfig->OutputPath.Path, InNewText.ToString());

												if (EditorAssetSubsystem->DoesAssetExist(AssetPath))
												{
													OutErrorMessage = LOCTEXT("AssetAlreadyExists", "Name clashes with an existing asset");
													return false;
												}

												for (TSharedPtr<FAssetMetadata> OtherAsset : IncludedAssets)
												{
													if (InAsset != OtherAsset && OtherAsset->AssetRename.Equals(InNewText.ToString()))
													{
														OutErrorMessage = LOCTEXT("AssetNameClash", "Name clashes with another template asset");
														return false;
													}
												}

												if (TemplateConfig->BlueprintMode == ETemplateBlueprintMode::CreateNewBlueprint && TemplateConfig->BlueprintAssetName.Equals(InNewText.ToString()))
												{
													OutErrorMessage = LOCTEXT("AssetNameClashWithBlueprint", "Name clashes with blueprint asset name");
													return false;
												}
												
												return true;
											})
										]
									];
								})
							]
						]
					]
				]
			];

		TSharedRef<SWidget> HeaderWidget = SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(16.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Header","Create a bundle preconfigured UAF assets by selecting a template to generate from."))
			];

		TSharedRef<SWidget> FooterWidget = SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(16.0f)
				[
					SNullWidget::NullWidget
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SImage)
						.Visibility_Lambda([this]()
						{
							return ErrorText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
						})
						.ColorAndOpacity(FStyleColors::Error)
						.Image(FAppStyle::GetBrush("Icons.Error"))
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(FMargin(4.0f, 0.0f, 8.0f, 0.0f))
					[
						SNew(STextBlock)
						.ColorAndOpacity(FStyleColors::Error)
						.Text_Lambda([this]()
						{
							return ErrorText;
						})
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoWidth()
					.Padding(FMargin(16.0f, 16.0f, 8.0f, 16.0f))
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("CreateAssets", "Create Assets"))
						.IsEnabled(this, &STemplatePicker::CanCreateAssets)
						.OnClicked_Lambda([this]()
						{
							HandleUseTemplate();
							return FReply::Handled();
						})
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoWidth()
					.Padding(FMargin(8.0f, 16.0f, 16.0f, 16.0f))
					[
						SNew(SButton)
						.Text(LOCTEXT("Cancel","Cancel"))
						.OnClicked_Lambda([InArgs]()
						{
							InArgs._OnCancel.ExecuteIfBound();
							return FReply::Handled();
						})
					]
				]
			];
		
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(8.0f, 8.0f, 8.0f, 0.0f)
			.AutoHeight()
			[
				HeaderWidget
			]
			+ SVerticalBox::Slot()
			.Padding(4.0f)
			.FillHeight(1.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(4.0f)
				.FillWidth(1.0f)
				[
					SNew(SSplitter)
					.Orientation(Orient_Horizontal)
					+ SSplitter::Slot()
					.Value(0.5)
					[
						TemplatesView
					]
					+ SSplitter::Slot()
					.Value(0.5)
					[
						DetailsView
					]
				]
			]
			+ SVerticalBox::Slot()
			.Padding(8.0f, 0.0f, 8.0f, 8.0f)
			.AutoHeight()
			[
				FooterWidget
			]
		];

		RefreshModuleTemplates();

		FTemplateItemPtr* DefaultSelectedTemplate = Templates.FindByPredicate([](const FTemplateItemPtr TemplateItem)
		{
			return TemplateItem->Template->GetPathName() == TEXT("/UAF/Templates/BasicCharacter/DA_DefaultCharacter.DA_DefaultCharacter");
		});

		if (DefaultSelectedTemplate)
		{
			SelectedTemplate = *DefaultSelectedTemplate;
			TemplatesTileView->SetSelection(SelectedTemplate);
		}
		
		HandleSelectedTemplateChanged();
	}
		
	void STemplatePicker::HandleUseTemplate()
	{
		bool bSuccess = TemplateConfig->TryUpdateDefaultConfigFile();

		for (const TSharedPtr<FAssetMetadata>& AssetMetadata : IncludedAssets)
		{
			TemplateConfig->AssetNaming.Add(AssetMetadata->AssetRename);
		}
		
		OnTemplateSelected.ExecuteIfBound(SelectedTemplate->Template, TemplateConfig);
	}

	void STemplatePicker::HandleOutputPathChanged()
	{
		UpdateUserAssetNames();
	}

	void STemplatePicker::HandleBlueprintToModifyChanged()
	{
		if (!TemplateConfig->BlueprintToModify.IsNull())
		{
			UBlueprint* BP = TemplateConfig->BlueprintToModify.LoadSynchronous();
			if (!BP->GeneratedClass->IsChildOf<AActor>())
			{
				// Restricting the blueprint picker to only allow Actor-derived BPs requires loading the asset to verify the class inheritance which
				// is too slow so the check is performed at selection time instead.
				FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, LOCTEXT("InvalidBlueprintSelected", "The selected blueprint to modify must derive from Actor to be able to contain the components necessary for UAF to function."));
				TemplateConfig->BlueprintToModify.Reset();
			}
		}
	}

	void STemplatePicker::UpdateUserAssetNames()
	{
		UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();

		const auto GetNewUserAssetName = [&, this](const FString BaseAssetName)
			{
				if (!EditorAssetSubsystem->DoesAssetExist(FPaths::Combine(TemplateConfig->OutputPath.Path, BaseAssetName)))
				{
					return BaseAssetName;
				}

				FString OutAssetName = BaseAssetName;
				int32 Suffix = 1;

				while (EditorAssetSubsystem->DoesAssetExist(FPaths::Combine(TemplateConfig->OutputPath.Path, OutAssetName)))
				{
					OutAssetName = BaseAssetName + FString::FromInt(Suffix++);
				}

				return OutAssetName;
			};

		
		IncludedAssets.Empty();

		if (SelectedTemplate.IsValid())
		{
			check(SelectedTemplate->Template);
			
			TemplateConfig->BlueprintAssetName = GetNewUserAssetName(SelectedTemplate->Template->DefaultBlueprintAssetName);

			for (const TObjectPtr<UObject>& Asset : SelectedTemplate->Template->Assets)
			{
				TSharedPtr<FAssetMetadata> AssetMetadata = MakeShared<FAssetMetadata>();
				AssetMetadata->AssetData = FAssetData(Asset);

				const FString BaseAssetName = AssetMetadata->AssetData.AssetName.ToString();
				AssetMetadata->AssetRename = GetNewUserAssetName(BaseAssetName);
				
				IncludedAssets.Add(AssetMetadata);
			}
		}

		IncludedAssetsListView->RequestListRefresh();
	}
	
	void STemplatePicker::HandleSelectedTemplateChanged()
	{
		TagsHBox->ClearChildren();

		if (SelectedTemplate.IsValid())
		{
			TemplateConfig->BlueprintAssetName = SelectedTemplate->Template->DefaultBlueprintAssetName;
			
			for (const FText& InTag : SelectedTemplate->Template->Tags)
			{
				TagsHBox->AddSlot()
				.Padding(0.0f, 4.0f, 4.0f, 4.0f)
				.AutoWidth()
				[
					SNew(SBorder)
					.BorderImage(FUAFStyle::Get().GetBrush("AssetWizard.TemplateTag.OuterBorder"))
					.BorderBackgroundColor(FLinearColor::White)
					.Padding(1.0f)
					[
						SNew(SBorder)
						.BorderImage(FUAFStyle::Get().GetBrush("AssetWizard.TemplateTag.InnerBorder"))
						.Padding(12.0f, 2.0f)
						[
							SNew(STextBlock)
							.Text(InTag)
						]
					]
				];
			}
		}
		
		UpdateUserAssetNames();
	}
	
	void STemplatePicker::RefreshModuleTemplates()
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		TArray<FAssetData> TemplateAssets;
		AssetRegistry.GetAssetsByClass(UUAFTemplateDataAsset::StaticClass()->GetClassPathName(), TemplateAssets);

		for (const FAssetData& AssetData : TemplateAssets)
		{
			TObjectPtr<UUAFTemplateDataAsset> ModuleTemplateDataAsset = CastChecked<UUAFTemplateDataAsset>(AssetData.GetAsset());

			TSharedPtr<FTemplate> ModuleTemplate = Templates.Add_GetRef(MakeShared<FTemplate>(ModuleTemplateDataAsset));
			ModuleTemplate->Template = ModuleTemplateDataAsset;
		}

		TemplatesTileView->RequestListRefresh();
	}

	bool STemplatePicker::CanCreateAssets() const
	{
		ErrorText = FText::GetEmpty();

		if (!SelectedTemplate.IsValid())
		{
			return false;
		}
		
		switch (TemplateConfig->BlueprintMode)
		{
		case ETemplateBlueprintMode::CreateNewBlueprint:
			if (TemplateConfig->BlueprintAssetName.IsEmpty())
			{
				ErrorText = LOCTEXT("InvalidBlueprintAssetName", "Invalid blueprint asset name");
				return false;
			}
			if (TemplateConfig->BlueprintClass.Get() == nullptr)
			{
				ErrorText = LOCTEXT("InvalidBlueprintClass", "Invalid blueprint class");
				return false;
			}
			if (TemplateConfig->SkeletalMesh.IsNull())
			{
				ErrorText = LOCTEXT("InvalidSkeletalMesh", "Invalid skeletal mesh");
				return false;
			}
			break;
		case ETemplateBlueprintMode::ModifyExistingBlueprint:
			if (TemplateConfig->BlueprintToModify.IsNull())
			{
				ErrorText = LOCTEXT("InvalidBlueprintToModify", "Invalid blueprint to modify");
				return false;
			}
			break;
		}

		if (TemplateConfig->BlueprintMode == ETemplateBlueprintMode::CreateNewBlueprint)
		{
			for (TSharedPtr<FAssetMetadata> Asset : IncludedAssets)
			{
				if (Asset->AssetRename.Equals(TemplateConfig->BlueprintAssetName))
				{
					ErrorText = LOCTEXT("InvalidAssetNames", "Blueprint asset name clashes with template asset name");
					return false;
				}
			}

			UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();
			if (EditorAssetSubsystem->DoesAssetExist(FPaths::Combine(TemplateConfig->OutputPath.Path, TemplateConfig->BlueprintAssetName)))
			{
				ErrorText = LOCTEXT("ClashingBlueprintAssetName", "Blueprint asset name clashes with an existing asset");
				return false;
			}
		}

		return true;
	}
	
	void SModuleTemplate::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
	{
		Item = InArgs._Item;

		STableRow::FArguments TableRowArguments;
		TableRowArguments._SignalSelectionMode = ETableRowSignalSelectionMode::Instantaneous;
		
		STableRow::Construct(
			TableRowArguments
			.Style(FAppStyle::Get(), "ProjectBrowser.TableRow")
			.Padding(2.0f)
			.Content()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
				.Clipping(EWidgetClipping::ClipToBounds)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot() // Thumbnail
						.AutoHeight()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SBox)
							.WidthOverride(102.0f)
							.HeightOverride(102.0f)
							[
								SNew(SBorder)
								.Padding(0.0f)
								.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.ThumbnailAreaBackground"))
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								[
									SNew(SImage)
									.Image(FUAFStyle::Get().GetBrush("AssetWizard.Template"))
								]
							]
						]
						// Name
						+ SVerticalBox::Slot()
						[
							SNew(SBorder)
							.VAlign(VAlign_Top)
							.Padding(FMargin(3.0f, 3.0f))
							.BorderImage_Lambda
							(
								[this]()
								{
									const bool bIsSelected = IsSelected();
									const bool bIsRowHovered = IsHovered();

									if (bIsSelected && bIsRowHovered)
									{
										static const FName SelectedHover("ProjectBrowser.ProjectTile.NameAreaSelectedHoverBackground");
										return FAppStyle::Get().GetBrush(SelectedHover);
									}
									if (bIsSelected)
									{
										static const FName Selected("ProjectBrowser.ProjectTile.NameAreaSelectedBackground");
										return FAppStyle::Get().GetBrush(Selected);
									}
									if (bIsRowHovered)
									{
										static const FName Hovered("ProjectBrowser.ProjectTile.NameAreaHoverBackground");
										return FAppStyle::Get().GetBrush(Hovered);
									}

									return FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.NameAreaBackground");
								}
							)
							[
								SNew(STextBlock)
								.Font(FAppStyle::Get().GetFontStyle("ProjectBrowser.ProjectTile.Font"))
								.AutoWrapText(true)
								.Text(Item->Template->Title)
								.ColorAndOpacity_Lambda
								(
									[this]()
									{
										const bool bIsSelected = IsSelected();
										const bool bIsRowHovered = IsHovered();

										if (bIsSelected || bIsRowHovered)
										{
											return FStyleColors::White;
										}

										return FSlateColor::UseForeground();
									}
								)
							]
						]
					]
					+ SOverlay::Slot()
					[
						SNew(SImage)
						.Visibility(EVisibility::HitTestInvisible)
						.Image_Lambda
						(
							[this]()
							{
								const bool bIsSelected = IsSelected();
								const bool bIsRowHovered = IsHovered();

								if (bIsSelected && bIsRowHovered)
								{
									static const FName SelectedHover("ProjectBrowser.ProjectTile.SelectedHoverBorder");
									return FAppStyle::Get().GetBrush(SelectedHover);
								}
								if (bIsSelected)
								{
									static const FName Selected("ProjectBrowser.ProjectTile.SelectedBorder");
									return FAppStyle::Get().GetBrush(Selected);
								}
								if (bIsRowHovered)
								{
									static const FName Hovered("ProjectBrowser.ProjectTile.HoverBorder");
									return FAppStyle::Get().GetBrush(Hovered);
								}

								return FStyleDefaults::GetNoBrush();
							}
						)
					]
				]
			],
			OwnerTable);
	}
}

#undef LOCTEXT_NAMESPACE
