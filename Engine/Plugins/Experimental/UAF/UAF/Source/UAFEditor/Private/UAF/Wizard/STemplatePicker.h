// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "TemplateDataAsset.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STileView.h"

class UUAFTemplateDataAsset;
class UUAFTemplateConfig;

namespace UE::UAF::Editor
{
	DECLARE_DELEGATE_TwoParams(FOnTemplateSelected, const TObjectPtr<const UUAFTemplateDataAsset>, const TObjectPtr<const UUAFTemplateConfig>);

	class STemplatePicker : public SCompoundWidget, public FGCObject
	{
	public:
		friend class SModuleTemplate;
		
		SLATE_BEGIN_ARGS(STemplatePicker) {}
			SLATE_EVENT(FOnTemplateSelected, OnTemplateSelected)
			SLATE_EVENT(FSimpleDelegate, OnCancel)
		SLATE_END_ARGS()

		/** FGCObject interface */
		virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("STemplatePicker");
		}

		void Construct(const FArguments& InArgs);

	private:
		void RefreshModuleTemplates();

		void HandleSelectedTemplateChanged();

		void HandleUseTemplate();
		
		void HandleOutputPathChanged();

		void HandleBlueprintToModifyChanged();

		void UpdateUserAssetNames();

		bool CanCreateAssets() const;
		
		enum class EPanel
		{
			Overview,
			Details
		};

		struct FTemplate
		{
			TObjectPtr<UUAFTemplateDataAsset> Template;
		};
	
		using FTemplateItemPtr = TSharedPtr<FTemplate>;
		
		TSharedPtr<STileView<FTemplateItemPtr>> TemplatesTileView;

		TArray<FTemplateItemPtr> Templates;

		FTemplateItemPtr SelectedTemplate;

		TSharedPtr<SHorizontalBox> TagsHBox;
		
		EPanel ShowPanel = EPanel::Details;

		struct FAssetMetadata
		{
			FString AssetRename;
			FAssetData AssetData;
		};
		
		TArray<TSharedPtr<FAssetMetadata>> IncludedAssets;
		
		FOnTemplateSelected OnTemplateSelected;

		TObjectPtr<UUAFTemplateConfig> TemplateConfig;
		
		TSharedPtr<SListView<TSharedPtr<FAssetMetadata>>> IncludedAssetsListView;

		mutable FText ErrorText;
	};

	class SModuleTemplate : public STableRow<TSharedPtr<STemplatePicker::FTemplate>>
	{
	public:
		SLATE_BEGIN_ARGS(SModuleTemplate) {}
		SLATE_ARGUMENT(TSharedPtr<STemplatePicker::FTemplate>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable);

	private:
		TSharedPtr<STemplatePicker::FTemplate> Item;
	};
}