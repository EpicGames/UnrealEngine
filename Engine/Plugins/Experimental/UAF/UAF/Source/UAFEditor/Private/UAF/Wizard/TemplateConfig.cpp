// Copyright Epic Games, Inc. All Rights Reserved.

#include "TemplateConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateConfig)

void UUAFTemplateConfig::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UUAFTemplateConfig, OutputPath))
	{
		OnOutputPathChanged.Broadcast();
	}
	else if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UUAFTemplateConfig, BlueprintToModify))
	{
		OnBlueprintToModifyChanged.Broadcast();
	}
}

void UUAFTemplateConfig::Reset()
{
	OnOutputPathChanged.Clear();
	OnBlueprintToModifyChanged.Clear();
	AssetNaming.Empty();
}