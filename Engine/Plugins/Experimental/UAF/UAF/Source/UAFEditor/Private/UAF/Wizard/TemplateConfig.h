// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Pawn.h"
#include "Templates/SubclassOf.h"

#include "TemplateConfig.generated.h"

class USkeletalMesh;
class UBlueprint;

UENUM()
enum class ETemplateBlueprintMode
{
	CreateNewBlueprint,
	ModifyExistingBlueprint,
	DoNothing,
};

UCLASS(Config = EditorPerProjectUserSettings)
class UUAFTemplateConfig :	public UObject
{
	GENERATED_BODY()

public:
	// Whether the template should also set up a new Blueprint for this template
	UPROPERTY(Config, EditAnywhere, Category = "Config")
	ETemplateBlueprintMode BlueprintMode = ETemplateBlueprintMode::CreateNewBlueprint;

	UPROPERTY(Config, EditAnywhere, Category = "Config", Meta = (EditCondition = "BlueprintMode == ETemplateBlueprintMode::CreateNewBlueprint", EditConditionHides))
	FString BlueprintAssetName;
	
	UPROPERTY(Config, EditAnywhere, Category = "Config", Meta = (EditCondition = "BlueprintMode == ETemplateBlueprintMode::CreateNewBlueprint", EditConditionHides))
	TSubclassOf<AActor> BlueprintClass = APawn::StaticClass();
	
	UPROPERTY(Config, EditAnywhere, Category = "Config", Meta = (EditCondition = "BlueprintMode == ETemplateBlueprintMode::CreateNewBlueprint", EditConditionHides))
	TSoftObjectPtr<USkeletalMesh> SkeletalMesh;

	UPROPERTY(EditAnywhere, Category = "Config", Meta = (EditCondition = "BlueprintMode == ETemplateBlueprintMode::ModifyExistingBlueprint", EditConditionHides, DisallowCreateNew))
	TSoftObjectPtr<UBlueprint> BlueprintToModify;

	UPROPERTY(EditAnywhere, Category = "Output", Meta = (ContentDir))
	FDirectoryPath OutputPath;

	// UObject
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	FSimpleMulticastDelegate OnOutputPathChanged;
	FSimpleMulticastDelegate OnBlueprintToModifyChanged;

	void Reset();

public:
	TArray<FString> AssetNaming;
};