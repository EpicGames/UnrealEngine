// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DataAsset.h"
#include "Styling/SlateBrush.h"

#include "TemplateDataAsset.generated.h"

class UUAFSystem;

UCLASS()
class UUAFTemplateDataAsset : public UPrimaryDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Default)
	FText Title;

	UPROPERTY(EditAnywhere, Category = Default)
	FText Description;
	
	UPROPERTY(EditAnywhere, Category = Default)
	FString DocumentationUrl;

	UPROPERTY(EditAnywhere, Category = Default)
	TArray<FText> Tags;

	UPROPERTY(EditAnywhere, Category = Default)
	FSlateBrush ThumbnailImage;

	UPROPERTY(EditAnywhere, Category = Default)
	FSlateBrush DetailsImage;

	// These are the assets that form this template. Each of these will be duplicated into the user's chosen directory (with references patched).
	UPROPERTY(EditAnywhere, Category = Default)
	TArray<TObjectPtr<UObject>> Assets;

	// This is the asset that will be opened automatically once the wizard has finished creating the assets for this template.
	UPROPERTY(EditAnywhere, Category = Default)
	TObjectPtr<UObject> AssetToOpen;

	// This is the UAF system that we will assign to the UAF component if the user is configuring a blueprint.
	UPROPERTY(EditAnywhere, Category = Default)
	TObjectPtr<UUAFSystem> SystemToAssignToComponent;

	UPROPERTY(EditAnywhere, Category = Default)
	FString DefaultBlueprintAssetName;
};