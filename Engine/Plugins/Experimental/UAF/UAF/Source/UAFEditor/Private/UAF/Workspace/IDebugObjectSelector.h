// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF::Editor
{
	class IDebugObjectSelector
	{
	public:
		virtual ~IDebugObjectSelector() = default;
		
		using FOnDebugObjectChanged = TMulticastDelegate<void(TWeakObjectPtr<UObject>)>;

		virtual TWeakObjectPtr<UObject> GetDebugObject() const = 0;

		virtual FDelegateHandle RegisterOnDebugObjectChanged(FOnDebugObjectChanged::FDelegate InDelegate) = 0;

		virtual bool UnregisterOnDebugObjectChanged(const FDelegateHandle InHandle) = 0;
	};
}