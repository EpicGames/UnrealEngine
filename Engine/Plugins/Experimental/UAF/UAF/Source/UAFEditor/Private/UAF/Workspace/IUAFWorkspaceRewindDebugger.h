// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::UAF::Editor
{
	class IUAFWorkspaceRewindDebugger
	{
	public:
		virtual ~IUAFWorkspaceRewindDebugger() = default;
		
		using FOnRewindDebuggerUpdate = TMulticastDelegate<void(float, class IRewindDebugger*)>;

		virtual FDelegateHandle RegisterOnRewindDebuggerUpdate(FOnRewindDebuggerUpdate::FDelegate InDelegate) = 0;
		virtual bool UnregisterOnRewindDebuggerUpdate(const FDelegateHandle InHandle) = 0;
	};
}