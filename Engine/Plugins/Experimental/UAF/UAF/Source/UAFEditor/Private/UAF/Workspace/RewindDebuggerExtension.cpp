// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Workspace/RewindDebuggerExtension.h"

#include "IRewindDebugger.h"

namespace UE::UAF::Editor
{
	FString FRewindDebuggerWorkspaceExtension::GetName()
	{
		return TEXT("UAF Workspace Extension");
	}
	
	void FRewindDebuggerWorkspaceExtension::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
	{
		if (RewindDebugger->GetScrubTime() != LastScrubTime)
		{
			LastScrubTime = RewindDebugger->GetScrubTime();

			// Forward this update to workspace components
			OnRewindDebuggerUpdate.Broadcast(DeltaTime, RewindDebugger);
		}
	}

	FDelegateHandle FRewindDebuggerWorkspaceExtension::RegisterOnRewindDebuggerUpdate(FOnRewindDebuggerUpdate::FDelegate InDelegate)
	{
		return OnRewindDebuggerUpdate.Add(InDelegate);
	}
	
	bool FRewindDebuggerWorkspaceExtension::UnregisterOnRewindDebuggerUpdate(const FDelegateHandle InHandle)
	{
		return OnRewindDebuggerUpdate.Remove(InHandle);
	}
}