// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRewindDebuggerExtension.h"
#include "UAF/Workspace/IUAFWorkspaceRewindDebugger.h"

class IRewindDebugger;

namespace UE::UAF::Editor
{
	class IWorkspaceEditor;

	// FRewindDebuggerWorkspaceExtension exists to allow any systems interacting
	// with the workspace editor to query a centralized rewind debugger extension
	// instead of having to define and register their own.
	class FRewindDebuggerWorkspaceExtension : public IRewindDebuggerExtension, public IUAFWorkspaceRewindDebugger
	{
	public:
		virtual ~FRewindDebuggerWorkspaceExtension() override = default;

		// IUAFWorkspaceRewindDebugger
		virtual FDelegateHandle RegisterOnRewindDebuggerUpdate(FOnRewindDebuggerUpdate::FDelegate InDelegate) override;
		virtual bool UnregisterOnRewindDebuggerUpdate(const FDelegateHandle InHandle) override;

	private:
		// IRewindDebuggerExtension
		virtual FString GetName() override;
		virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
		
	private:
		FOnRewindDebuggerUpdate OnRewindDebuggerUpdate;
		float LastScrubTime = 0.0f;
	};
}
