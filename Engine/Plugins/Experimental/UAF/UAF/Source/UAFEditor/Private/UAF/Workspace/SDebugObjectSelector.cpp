// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAF/Workspace/SDebugObjectSelector.h"

#include "EditorModeManager.h"
#include "Widgets/Input/STextComboBox.h"

#define LOCTEXT_NAMESPACE "UE::UAF::Editor::SDebugObjectSelector"

namespace UE::UAF::Editor
{
	void SDebugObjectSelector::Construct(const FArguments& InArgs)
	{
		GetDebugObjectsDelegate = InArgs._GetDebugObjects;
		
		ChildSlot
		[
			SNew(SComboBox<TSharedPtr<FBlueprintDebugObjectInstance>>)
			.OptionsSource(&DebugObjects)
			.OnComboBoxOpening(this, &SDebugObjectSelector::ComboBox_OnComboBoxOpening)
			.OnSelectionChanged(this, &SDebugObjectSelector::ComboBox_OnSelectionChanged)
			.OnGenerateWidget(this, &SDebugObjectSelector::ComboBox_OnGenerateWidget)
			.ContentPadding(FMargin(0.f, 4.f))
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return Selected.IsValid()
						? FText::FromString(Selected->ObjectLabel)
						: LOCTEXT("NoDebugObjectSelected", "No debug object selected");
				})
			]
		];
	}

	TSharedRef<SWidget> SDebugObjectSelector::ComboBox_OnGenerateWidget(TSharedPtr<FBlueprintDebugObjectInstance> InItem)
	{
		FString ItemString;
		FString ItemTooltip;

		if (InItem.IsValid())
		{
			ItemString = InItem->ObjectLabel;
			ItemTooltip = InItem->ObjectPath;
		}

		return SNew(STextBlock)
			.Text(FText::FromString(*ItemString))
			.ToolTipText(FText::FromString(*ItemTooltip));
	}

	void SDebugObjectSelector::ComboBox_OnSelectionChanged(TSharedPtr<FBlueprintDebugObjectInstance> NewSelection, ESelectInfo::Type SelectInfo)
	{
		Selected = NewSelection;
		OnDebugObjectChangedDelegate.Broadcast(Selected ? Selected->ObjectPtr : TWeakObjectPtr<UObject>());
	}
	
	void SDebugObjectSelector::ComboBox_OnComboBoxOpening()
	{
		RegenerateDebugObjects();
	}
	
	void SDebugObjectSelector::RegenerateDebugObjects()
	{
		TArray<UObject*> DebugObjectPtrs;
		GetDebugObjectsDelegate.ExecuteIfBound(DebugObjectPtrs);
		
		DebugObjects.Empty();
		for (UObject* DebugObject : DebugObjectPtrs)
		{
			DebugObjects.Add(MakeShared<FBlueprintDebugObjectInstance>(DebugObject, DebugObject->GetFullName()));
		}
	}
	
	TWeakObjectPtr<UObject> SDebugObjectSelector::GetDebugObject() const
	{
		return Selected.IsValid() ? Selected->ObjectPtr : nullptr;
	}

	FDelegateHandle SDebugObjectSelector::RegisterOnDebugObjectChanged(FOnDebugObjectChanged::FDelegate InDelegate)
	{
		return OnDebugObjectChangedDelegate.Add(InDelegate);
	}
	
	bool SDebugObjectSelector::UnregisterOnDebugObjectChanged(const FDelegateHandle InHandle)
	{
		return OnDebugObjectChangedDelegate.Remove(InHandle);
	}
}

#undef LOCTEXT_NAMESPACE