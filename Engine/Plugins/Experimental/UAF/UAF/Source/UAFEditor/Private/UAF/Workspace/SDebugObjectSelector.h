// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SBlueprintEditorSelectedDebugObjectWidget.h"
#include "UAF/Workspace/IDebugObjectSelector.h"

namespace UE::UAF::Editor
{
	class SDebugObjectSelector : public SCompoundWidget, public IDebugObjectSelector
	{
	public:
		DECLARE_DELEGATE_OneParam(FGetDebugObjects, TArray<UObject*>&)
		
		SLATE_BEGIN_ARGS(SDebugObjectSelector)
		{}
			SLATE_EVENT(FGetDebugObjects, GetDebugObjects)
		SLATE_END_ARGS()
    
		void Construct(const FArguments& InArgs);

		// IDebugObjectSelector
    	virtual TWeakObjectPtr<UObject> GetDebugObject() const override;
		virtual FDelegateHandle RegisterOnDebugObjectChanged(FOnDebugObjectChanged::FDelegate InDelegate) override;
		virtual bool UnregisterOnDebugObjectChanged(const FDelegateHandle InHandle) override;
    		
	private:
		TSharedRef<SWidget> ComboBox_OnGenerateWidget(TSharedPtr<FBlueprintDebugObjectInstance> InItem);
		void ComboBox_OnSelectionChanged(TSharedPtr<FBlueprintDebugObjectInstance> NewSelection, ESelectInfo::Type SelectInfo);
		void ComboBox_OnComboBoxOpening();
		
		void RegenerateDebugObjects();

		FOnDebugObjectChanged OnDebugObjectChangedDelegate;
		FGetDebugObjects GetDebugObjectsDelegate;
		
		TArray<TSharedPtr<FBlueprintDebugObjectInstance>> DebugObjects;
		TSharedPtr<FBlueprintDebugObjectInstance> Selected;
	};
}