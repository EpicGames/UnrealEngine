// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFStyle.h"
#include "Brushes/SlateBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/CoreStyle.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateTypes.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"

#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( FUAFStyle::InResources(RelativePath), __VA_ARGS__)
#define PLUGIN_BOX_BRUSH_BRUSH( RelativePath, ... ) FSlateBoxBrush( FUAFStyle::InResources(RelativePath), __VA_ARGS__)

FUAFStyle::FUAFStyle()
	: FSlateStyleSet(TEXT("UAFStyle"))
{
	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	SetClassIcons();
	SetVariablesOutlinerStyles();
	SetTraitStyles();
	SetAssetWizardStyles();
	SetSetBindingStyles();
	SetBlendProfileStyles();

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FUAFStyle::~FUAFStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

void FUAFStyle::SetClassIcons()
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	
	Set("ClassIcon.UAFSharedVariables", new IMAGE_PLUGIN_BRUSH_SVG("Icons/Archive_16.svg", Icon16x16));
	Set("ClassIcon.UAFSystem", new IMAGE_BRUSH_SVG("Starship/Common/Modules", Icon16x16 ));
	Set("ClassIcon.UAFAnimGraph", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_BlueprintEditor_EventGraph", Icon16x16));
	Set("ClassIcon.AnimNextStateTree", new IMAGE_BRUSH_SVG(TEXT("UMG/TreeView"), Icon16x16));
	Set("ClassIcon.AbstractSkeletonLabelCollection", new IMAGE_PLUGIN_BRUSH_SVG("Icons/LabelCollection_16.svg", Icon16x16));
	Set("ClassIcon.AbstractSkeletonLabelBinding", new IMAGE_PLUGIN_BRUSH_SVG("Icons/LabelBinding_16.svg", Icon16x16));
	Set("ClassIcon.AbstractSkeletonSetCollection", new IMAGE_PLUGIN_BRUSH_SVG("Icons/SetCollection_16.svg", Icon16x16));
	Set("ClassIcon.AbstractSkeletonSetBinding", new IMAGE_PLUGIN_BRUSH_SVG("Icons/SetBinding_16.svg", Icon16x16));
}

void FUAFStyle::SetVariablesOutlinerStyles()
{
	Set("VariablesOutliner.SharedVariablesPill.Border", new PLUGIN_BOX_BRUSH_BRUSH("NamespaceBorder.png", FMargin(4.0f / 16.0f)));

	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");
	Set("VariablesOutliner.SharedVariablesPill.Text", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 8))
		.SetColorAndOpacity(FSlateColor::UseForeground()));

	const FVector2D Icon14x14(14.0f, 14.0f);
	const FButtonStyle CloseButtonStyle = FButtonStyle()
		.SetNormal(CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon14x14, FStyleColors::Foreground))
		.SetPressed(CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon14x14, FStyleColors::Foreground))
		.SetHovered(CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon14x14, FStyleColors::ForegroundHover));
	Set("VariablesOutliner.SharedVariablesPill.CloseButton", CloseButtonStyle);
}

void FUAFStyle::SetTraitStyles()
{
	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FComboButtonStyle& ComboButtonStyle = FCoreStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton");
	
	// Trait details labels
	Set("AnimNext.TraitDetails.Label", FTextBlockStyle(NormalText)
		.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
		.SetFontSize(7));

	Set("AnimNext.TraitDetails.Background", new FSlateRoundedBoxBrush(FStyleColors::Hover, 6.f));

	// Add trait combo button
	const FButtonStyle AddButtonStyle = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Hover, 3.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Primary, 3.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Primary, 3.0f))
		.SetNormalForeground(FStyleColors::Transparent)
		.SetHoveredForeground(FStyleColors::Hover)
		.SetPressedForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(6, 2, 6, 2))
		.SetPressedPadding(FMargin(6, 3, 6, 1));

	Set("AnimNext.TraitDetails.AddButton", FComboButtonStyle(ComboButtonStyle).SetButtonStyle(AddButtonStyle));
}

void FUAFStyle::SetAssetWizardStyles()
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);
	
	Set("AssetWizard.OpenExternal", new FSlateVectorImageBrush(InResources("Icons/Wizard/OpenExternal.svg"),Icon16x16));
	Set("AssetWizard.Template", new FSlateVectorImageBrush(InResources("Icons/Wizard/Template.svg"),Icon64x64));

	Set("AssetWizard.TemplateTag.OuterBorder", new FSlateRoundedBoxBrush(FStyleColors::AccentGray, 4.0f));
	Set("AssetWizard.TemplateTag.InnerBorder", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f));

	Set("AssetWizard.TemplateFolderBackground", new FSlateRoundedBoxBrush(EStyleColor::Panel, 4.0f));
	Set("AssetWizard.TemplateFolderHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f));
	Set("AssetWizard.TemplateFolderSelectedBackground", new FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f));
	Set("AssetWizard.TemplateFolderHoverSelectedBackground", new FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f));
}

void FUAFStyle::SetSetBindingStyles()
{
	const FVector2D Icon16x16(16.0f, 16.0f);

	Set("SetBindingEditorCommands.UnbindSelection", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Unlinked", Icon16x16));
	Set("SetBindingEditorCommands.SelectWithChildren", new IMAGE_PLUGIN_BRUSH_SVG("Icons/SelectAll_16.svg", Icon16x16));
}

void FUAFStyle::SetBlendProfileStyles()
{
	const FVector2D Icon16x16(16.0f, 16.0f);

	Set("BlendProfiles.TimeProfile", new IMAGE_BRUSH_SVG("Starship/Common/Realtime", Icon16x16));
	Set("BlendProfiles.WeightProfile", new FSlateVectorImageBrush(InResources("Icons/Weight_16.svg"),Icon16x16));
}

FUAFStyle& FUAFStyle::Get()
{
	static FUAFStyle Instance;
	return Instance;
}

FString FUAFStyle::InResources(const FString& RelativePath)
{
	static const FString ResourcesDir = IPluginManager::Get().FindPlugin(TEXT("UAF"))->GetBaseDir() / TEXT("Resources");
	return ResourcesDir / RelativePath;
}