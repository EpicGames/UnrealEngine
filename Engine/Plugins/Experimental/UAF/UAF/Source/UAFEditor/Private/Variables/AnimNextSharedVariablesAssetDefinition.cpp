// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextSharedVariablesAssetDefinition.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "ContentBrowserMenuContexts.h"
#include "Editor.h"
#include "IWorkspaceEditorModule.h"
#include "ToolMenus.h"
#include "UncookedOnlyUtils.h"
#include "UObject/SavePackage.h"
#include "Workspace/AnimNextWorkspaceFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextSharedVariablesAssetDefinition)

#define LOCTEXT_NAMESPACE "AnimNextAssetDefinitions"

EAssetCommandResult UAssetDefinition_AnimNextSharedVariables::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace UE::Workspace;

	for (UUAFSharedVariables* Asset : OpenArgs.LoadObjects<UUAFSharedVariables>())
	{
		IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::Get().LoadModuleChecked<IWorkspaceEditorModule>("WorkspaceEditor");
		WorkspaceEditorModule.OpenWorkspaceForObject(Asset, EOpenWorkspaceMethod::Default, UAnimNextWorkspaceFactory::StaticClass());
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
