// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Variables/AnimNextSharedVariables.h"
#include "AssetDefinitionDefault.h"
#include "AnimNextSharedVariablesAssetDefinition.generated.h"

#define LOCTEXT_NAMESPACE "AnimNextAssetDefinitions"

UCLASS()
class UAssetDefinition_AnimNextSharedVariables : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition interface
	virtual FText GetAssetDisplayName() const override { return LOCTEXT("UAFSharedVariables", "UAF Shared Variables"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(128,64,32)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UUAFSharedVariables::StaticClass(); }
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("UAFSubMenu", "Animation Framework")) };
		return Categories;
	}
	virtual bool ShouldSaveExternalPackages() const override { return true; }
};

#undef LOCTEXT_NAMESPACE