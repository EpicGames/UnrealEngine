// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/AnimNextSharedVariablesFactory.h"

#include "UAFCompilationScope.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextSharedVariablesFactory)

UUAFSharedVariablesFactory::UUAFSharedVariablesFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UUAFSharedVariables::StaticClass();
}

bool UUAFSharedVariablesFactory::ConfigureProperties()
{
	return true;
}

UObject* UUAFSharedVariablesFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	EObjectFlags FlagsToUse = Flags | RF_Public | RF_Standalone | RF_Transactional | RF_LoadCompleted;
	if(InParent == GetTransientPackage())
	{
		FlagsToUse &= ~RF_Standalone;
	}

	UUAFSharedVariables* NewDataInterface = NewObject<UUAFSharedVariables>(InParent, Class, Name, FlagsToUse);

	// Create internal editor data
	UUAFSharedVariables_EditorData* EditorData = NewObject<UUAFSharedVariables_EditorData>(NewDataInterface, TEXT("EditorData"), RF_Transactional);
	NewDataInterface->EditorData = EditorData;
	EditorData->Initialize(/*bRecompileVM*/false);

	// Compile the initial skeleton
	UE::UAF::UncookedOnly::Compilation::RequestAssetCompilation(NewDataInterface);
	
	check(!EditorData->bErrorsDuringCompilation);

	return NewDataInterface;
}
