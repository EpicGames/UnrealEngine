// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextVariableSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextVariableSettings)

UAnimNextVariableSettings::UAnimNextVariableSettings()
{
}

const FAnimNextParamType& UAnimNextVariableSettings::GetLastVariableType() const
{
	return LastVariableType;
}

void UAnimNextVariableSettings::SetLastVariableType(const FAnimNextParamType& InLastVariableType)
{
	LastVariableType = InLastVariableType;
}

FName UAnimNextVariableSettings::GetLastVariableName() const
{
	return LastVariableName;
}

void UAnimNextVariableSettings::SetLastVariableName(FName InLastVariableName)
{
	LastVariableName = InLastVariableName;
}

SPinTypeSelector::ESelectorType UAnimNextVariableSettings::GetVariablesViewDefaultPinSelectorType() const
{
	return static_cast<SPinTypeSelector::ESelectorType>(VariablesViewPinSelectorType);
}

void UAnimNextVariableSettings::SetVariablesViewDefaultPinSelectorType(SPinTypeSelector::ESelectorType InLastSelectorType)
{
	VariablesViewPinSelectorType = static_cast<uint32>(InLastSelectorType);
	SaveConfig();
}
