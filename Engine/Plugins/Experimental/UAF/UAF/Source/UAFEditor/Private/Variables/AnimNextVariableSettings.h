// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPinTypeSelector.h"
#include "Engine/DeveloperSettings.h"
#include "Param/ParamType.h"
#include "AnimNextVariableSettings.generated.h"

UCLASS(Config = EditorPerProjectUserSettings)
class UAnimNextVariableSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	UAnimNextVariableSettings();

public:
	// Get the type of the last variable that we created
	const FAnimNextParamType& GetLastVariableType() const;

	// Set the type of the last variable that we created
	void SetLastVariableType(const FAnimNextParamType& InLastVariableType);

	// Get the name of the last variable that we created
	FName GetLastVariableName() const;

	// Set the name of the last variable that we created
	void SetLastVariableName(FName InLastVariableName);
	
	SPinTypeSelector::ESelectorType GetVariablesViewDefaultPinSelectorType() const;
	void SetVariablesViewDefaultPinSelectorType(SPinTypeSelector::ESelectorType InLastSelectorType);
private:
	UPROPERTY(Transient)
	FAnimNextParamType LastVariableType = FAnimNextParamType::GetType<bool>();

	UPROPERTY(Transient)
	FName LastVariableName = FName("NewVariable");

	UPROPERTY(VisibleAnywhere, Category="UAF", Config)
	uint32 VariablesViewPinSelectorType = static_cast<uint32>(SPinTypeSelector::ESelectorType::Partial);
};