// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariablesOutlinerCategoryItem.h"

#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "ISceneOutliner.h"
#include "ScopedTransaction.h"
#include "UncookedOnlyUtils.h"
#include "Common/Outliner/OutlinerDragDropOps.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "Styling/StarshipCoreStyle.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "VariablesOutlinerCategoryItem"

namespace UE::UAF::Editor
{

const FSceneOutlinerTreeItemType FVariablesOutlinerCategoryItem::Type(&FOutlinerCategoryItem::Type);

FVariablesOutlinerCategoryItem::FVariablesOutlinerCategoryItem(const FOutlinerCategoryItem::FItemData& InItemData) : FOutlinerCategoryItem(FVariablesOutlinerCategoryItem::Type, InItemData)
{
}

void FVariablesOutlinerCategoryItem::Rename(const FText& InNewName) const
{
	if(UUAFRigVMAsset* Owner = WeakOwner.Get())
	{
		if (UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(Owner))
		{
			FScopedTransaction Transaction(LOCTEXT("RenameCategory", "Rename category"));
			
			FString NewCategoryPath = ParentCategoryName;
			if (!NewCategoryPath.IsEmpty())
			{
				NewCategoryPath.Append(TEXT("|"));
			}
			NewCategoryPath.Append(InNewName.ToString());

			EditorData->RenameCategory(CategoryPath, NewCategoryPath);
		}
	}
}

bool FVariablesOutlinerCategoryItem::ValidateName(const FText& InNewName, FText& OutErrorMessage) const
{
	if (InNewName.IsEmpty())
	{		
		OutErrorMessage = LOCTEXT("NameEmpty", "Category name cannot be empty");
		return false;
	}

	if(UUAFRigVMAsset* Owner = WeakOwner.Get())
	{
		if (UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(Owner))
		{
			FString NewCategoryPath = ParentCategoryName;
			if (!NewCategoryPath.IsEmpty())
			{
				NewCategoryPath.Append(TEXT("|"));
			}
			NewCategoryPath.Append(InNewName.ToString());
		
			if (EditorData->VariableAndFunctionCategories.Contains(NewCategoryPath) && CategoryPath != NewCategoryPath
			)
			{
				OutErrorMessage = LOCTEXT("NameExistsError", "Category name already exists in this asset");
				return false;
			}
			
			return true;
		}
	}
	return false;
}
}

#undef LOCTEXT_NAMESPACE // "VariablesOutlinerCategoryItem"
