// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"
#include "Common/Outliner/OutlinerCategoryItem.h"

class UUAFRigVMAsset;

namespace UE::UAF::Editor
{

struct FVariablesOutlinerCategoryItem : FOutlinerCategoryItem
{
	static const FSceneOutlinerTreeItemType Type;

	FVariablesOutlinerCategoryItem(const FOutlinerCategoryItem::FItemData& IntemData);

	virtual void Rename(const FText& InNewName) const override final;
	virtual bool ValidateName(const FText& InNewName, FText& OutErrorMessage) const override final;
};

}
