// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariablesOutlinerColumns.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "DetailLayoutBuilder.h"
#include "EditorUtils.h"
#include "IAnimNextRigVMExportInterface.h"
#include "Variables/IAnimNextRigVMVariableInterface.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "ISinglePropertyView.h"
#include "PropertyBagDetails.h"
#include "ScopedTransaction.h"
#include "SPinTypeSelector.h"
#include "UncookedOnlyUtils.h"
#include "VariablesOutlinerEntryItem.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Images/SImage.h"
#include "Variables/AnimNextUniversalObjectLocatorBindingData.h"
#include "Variables/AnimNextVariableSettings.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Variables/Outliner/VariablesOutlinerMode.h"
#include "Common/Outliner/OutlinerAssetItem.h"

#define LOCTEXT_NAMESPACE "VariablesOutlinerColumns"

namespace UE::UAF::Editor
{

FLazyName VariablesOutlinerType("Type");

FName FVariablesOutlinerTypeColumn::GetID()
{
	return VariablesOutlinerType;
}

FVariablesOutlinerTypeColumn::FVariablesOutlinerTypeColumn(ISceneOutliner& SceneOutliner) : WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared()))
{
	SelectorType = GetDefault<UAnimNextVariableSettings>()->GetVariablesViewDefaultPinSelectorType();
}

SHeaderRow::FColumn::FArguments FVariablesOutlinerTypeColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.ManualWidth(this, &FVariablesOutlinerTypeColumn::GetColumnWidth)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Left)
		.VAlignCell(VAlign_Center)
		.SortMode(EColumnSortMode::None)
		.ShouldGenerateEmptyWidgetForSpacing(false)
		.HeaderComboVisibility(EHeaderComboVisibility::Ghosted)
		[
			SNew(SBox)
			.WidthOverride(16.0f)
			.HeightOverride(16.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon")))
				.ToolTipText(LOCTEXT("TypeTooltip", "Type of this entry"))
			]
		]
		.OnGetMenuContent_Lambda([this]()
		{
			const bool bCloseAfterSelection = true;
			FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);
			MenuBuilder.BeginSection("", LOCTEXT("VariablesOutlinerTypeColumnContextMenu", "Variable Type Column") );
			{
				MenuBuilder.AddMenuEntry(
						LOCTEXT("CompactView", "Compact"),
						LOCTEXT("CompactView_ToolTip", "Displays the Variable Type using Compact representation."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([this]()
						{
							SelectorType = SPinTypeSelector::ESelectorType::Compact;
							GetMutableDefault<UAnimNextVariableSettings>()->SetVariablesViewDefaultPinSelectorType(SelectorType);
							WeakSceneOutliner.Pin()->FullRefresh();
						}), FCanExecuteAction(), FGetActionCheckState::CreateLambda([this]() {return SelectorType == SPinTypeSelector::ESelectorType::Compact ? ECheckBoxState::Checked : ECheckBoxState::Undetermined;})),
						NAME_None,
						EUserInterfaceActionType::Check
				);

				MenuBuilder.AddMenuEntry(
						LOCTEXT("PartialView", "Partial"),
						LOCTEXT("PartialView_ToolTip", "Displays the Variable Type using Partial representation."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([this]()
						{
							SelectorType = SPinTypeSelector::ESelectorType::Partial;
							GetMutableDefault<UAnimNextVariableSettings>()->SetVariablesViewDefaultPinSelectorType(SelectorType);
							WeakSceneOutliner.Pin()->FullRefresh();					
						}), FCanExecuteAction(), FGetActionCheckState::CreateLambda([this]() {return SelectorType == SPinTypeSelector::ESelectorType::Partial ? ECheckBoxState::Checked : ECheckBoxState::Undetermined;})),
						NAME_None,
						EUserInterfaceActionType::Check
				);

				MenuBuilder.AddMenuEntry(
						LOCTEXT("FullView", "Full"),
						LOCTEXT("FullView_ToolTip", "Displays the Variable Type using Full representation."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([this]()
						{
							SelectorType = SPinTypeSelector::ESelectorType::Full;
							GetMutableDefault<UAnimNextVariableSettings>()->SetVariablesViewDefaultPinSelectorType(SelectorType);
							WeakSceneOutliner.Pin()->FullRefresh();
						}), FCanExecuteAction(), FGetActionCheckState::CreateLambda([this]() {return SelectorType == SPinTypeSelector::ESelectorType::Full ? ECheckBoxState::Checked : ECheckBoxState::Undetermined;})),
						NAME_None,
						EUserInterfaceActionType::Check
				);
			}
			MenuBuilder.EndSection();

			return MenuBuilder.MakeWidget();
		});
}

const TSharedRef<SWidget> FVariablesOutlinerTypeColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef Item, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	const FOutlinerAssetItem* AssetItem = Item->CastTo<FOutlinerAssetItem>();
	if (AssetItem)
	{
		return SNullWidget::NullWidget;
	}

	// Return a dummy box widget, with ShouldGenerateEmptyWidgetForSpacing settings this will prevent us from skipping outside of the header, avoiding alignment issues
	const FVariablesOutlinerEntryItem* TreeItem = Item->CastTo<FVariablesOutlinerEntryItem>();
	if (TreeItem == nullptr)
	{
		return SNew(SBox);
	}

	return
		SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.IsEnabled(!TreeItem->WeakSharedVariablesEntry.IsValid() || !TreeItem->PropertyPath.IsPathToFieldEmpty())
			[
				SNew(SPinTypeSelector, FGetPinTypeTree::CreateStatic(&Editor::FUtils::GetFilteredVariableTypeTree))
				.TargetPinType_Lambda([WeakItem = TWeakPtr<ISceneOutlinerTreeItem>(Item)]()
				{
					TSharedPtr<ISceneOutlinerTreeItem> PinnedItem = WeakItem.Pin();
					if (!PinnedItem->IsValid())
					{
						return FEdGraphPinType();
					}

					const FVariablesOutlinerEntryItem* VariablesItem = PinnedItem->CastTo<FVariablesOutlinerEntryItem>();
					
					if(const IUAFRigVMVariableInterface* Variable = Cast<IUAFRigVMVariableInterface>(VariablesItem->WeakEntry.Get()))
					{
						return UncookedOnly::FUtils::GetPinTypeFromParamType(Variable->GetType());
					}
					else if (const FProperty* Property = VariablesItem->PropertyPath.Get())
					{
						FEdGraphPinType PinType;
						GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(Property, PinType);
						return PinType;
					}

					return FEdGraphPinType();
				})
				.OnPinTypeChanged_Lambda([WeakEntry = TreeItem->WeakEntry](const FEdGraphPinType& PinType)
				{
					if(IUAFRigVMVariableInterface* Variable = Cast<IUAFRigVMVariableInterface>(WeakEntry.Get()))
					{
						const FAnimNextParamType ParamType = UncookedOnly::FUtils::GetParamTypeFromPinType(PinType);
						if(ParamType.IsValid())
						{
							FScopedTransaction Transaction(LOCTEXT("SetTypeTransaction", "Set Variable Type"));

							UncookedOnly::FUtils::SetVariableType(Cast<UAnimNextVariableEntry>(Variable), ParamType, true, true);
							UAnimNextVariableSettings* Settings = GetMutableDefault<UAnimNextVariableSettings>();
							Settings->SetLastVariableType(ParamType);
						}
					}
				})
				.Schema(GetDefault<UPropertyBagSchema>())
				.bAllowArrays(true)
				.TypeTreeFilter(ETypeTreeFilter::None)
				.ReadOnly_Lambda([&Row, this]() {return !Row.IsHovered(); })
				.SelectorType(SelectorType)
		];
}

float FVariablesOutlinerTypeColumn::GetColumnWidth() const
{
	switch (SelectorType)
	{
	case SPinTypeSelector::ESelectorType::Partial:
		return 128.f;
		
	case SPinTypeSelector::ESelectorType::Full:
		return 164.f;
			
	case SPinTypeSelector::ESelectorType::Compact:
	default:
		return 48.f;
	}
}

FLazyName VariablesOutlinerValue("Value");

FName FVariablesOutlinerValueColumn::GetID()
{
	return VariablesOutlinerValue;
}

SHeaderRow::FColumn::FArguments FVariablesOutlinerValueColumn::ConstructHeaderRowColumn()
{
	return
		SHeaderRow::Column(GetColumnID())
		.FillWidth(1.0f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.ShouldGenerateEmptyWidgetForSpacing(false)
		[
			SNew(SBox) 
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ValueLabel", "Value"))
				.ToolTipText(LOCTEXT("ValueTooltip", "Value of the variable"))
			]
		];
}

class SVariablesOutlinerValue : public SCompoundWidget, public FNotifyHook
{
	SLATE_BEGIN_ARGS(SVariablesOutlinerValue) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FVariablesOutlinerEntryItem& InTreeItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakTreeItem = StaticCastSharedRef<FVariablesOutlinerEntryItem>(InTreeItem.AsShared());

		if(UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(InTreeItem.WeakEntry.Get()))
		{
			if(UUAFRigVMAssetEditorData* EditorData = VariableEntry->GetTypedOuter<UUAFRigVMAssetEditorData>())
			{
				EditorData->ModifiedDelegate.AddSP(this, &SVariablesOutlinerValue::HandleModified);
			}
		}
		if(const UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(InTreeItem.WeakSharedVariablesEntry.Get()))
		{
			if(UUAFRigVMAssetEditorData* EditorData = SharedVariablesEntry->GetTypedOuter<UUAFRigVMAssetEditorData>())
			{
				EditorData->ModifiedDelegate.AddSP(this, &SVariablesOutlinerValue::HandleModified);
			}
		}

		ChildSlot
		[
			SAssignNew(WidgetContainer, SBox)
		];

		BuildValueWidget();
	}

	bool bPropertySetGuard = false;

	void BuildValueWidget()
	{
		TSharedPtr<FVariablesOutlinerEntryItem> TreeItem = WeakTreeItem.Pin();
		if(!TreeItem.IsValid())
		{
			return;
		}

		UAnimNextVariableEntry* VariableEntry = TreeItem->WeakEntry.Get();
		if(VariableEntry == nullptr)
		{
			return;
		}

		TSharedPtr<SWidget> ValueWidget = SNullWidget::NullWidget;
		FInstancedPropertyBag& PropertyBag = VariableEntry->GetMutablePropertyBag();
		const FName ValueName = IUAFRigVMVariableInterface::ValueName;

		TSharedRef<SHorizontalBox> ValueHBox = SNew(SHorizontalBox);
		const FPropertyBagPropertyDesc* PropertyDesc = PropertyBag.FindPropertyDescByName(ValueName);
		if (PropertyDesc != nullptr)
		{
			if (PropertyDesc->ContainerTypes.IsEmpty()) // avoid trying to inline containers
			{
				FSinglePropertyParams SinglePropertyArgs;
				SinglePropertyArgs.NamePlacement = EPropertyNamePlacement::Hidden;
				SinglePropertyArgs.NotifyHook = this;
				SinglePropertyArgs.bHideResetToDefault = true;

				FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
				
				const TSharedPtr<ISinglePropertyView> SingleStructPropertyView = PropertyEditorModule.CreateSingleProperty(MakeShared<FInstancePropertyBagStructureDataProvider>(PropertyBag), ValueName, SinglePropertyArgs);
				if (SingleStructPropertyView.IsValid())
				{
					ValueHBox->AddSlot()
					.AutoWidth()
					[
						SNew(SBox)
						.HAlign(HAlign_Left)
						.Visibility_Lambda([WeakEntry = TWeakObjectPtr<UAnimNextVariableEntry>(VariableEntry)]() -> EVisibility
						{
							if (UAnimNextVariableEntry* Entry = WeakEntry.Get())
							{
								return Entry->GetBinding().IsValid() ? EVisibility::Collapsed : EVisibility::Visible;
							}
							return EVisibility::Collapsed;
						})
						[
							SingleStructPropertyView.ToSharedRef()
						]
					];

					const TSharedPtr<ISinglePropertyView> LocatorPropertyView = PropertyEditorModule.CreateSingleProperty(VariableEntry, GET_MEMBER_NAME_CHECKED(UAnimNextVariableEntry, Binding), SinglePropertyArgs);

					LocatorPropertyView.Get()->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this, LocatorPropertyView, WeakEntry = TWeakObjectPtr<UAnimNextVariableEntry>(VariableEntry)]()
					{
						if (bPropertySetGuard)
						{
							return;
						}

						TGuardValue<bool> Guard(bPropertySetGuard, true);						
						if (UAnimNextVariableEntry* Entry = WeakEntry.Get())
						{
							Entry->BroadcastModified(EAnimNextEditorDataNotifType::VariableBindingChanged);
						}

						LocatorPropertyView->GetPropertyHandle()->NotifyPostChange(EPropertyChangeType::ValueSet);
					}));
					
					ValueHBox->AddSlot()
					.AutoWidth()
					[
						SNew(SBox)
						.HAlign(HAlign_Left)
						.Visibility_Lambda([WeakEntry = TWeakObjectPtr<UAnimNextVariableEntry>(VariableEntry)]() -> EVisibility
						{
							if (UAnimNextVariableEntry* Entry = WeakEntry.Get())
							{
								return Entry->GetBinding().IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
							}

							return EVisibility::Collapsed;
						})
						[
							VariableEntry ? LocatorPropertyView.ToSharedRef() : SNullWidget::NullWidget
						]
					];

					ValueHBox->AddSlot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SNew(SCheckBox)
							.Style( &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
							.Padding(FMargin(4.f, 2.f))
							.IsChecked_Lambda([WeakEntry = TWeakObjectPtr<UAnimNextVariableEntry>(VariableEntry)]() -> ECheckBoxState
							{
								if (UAnimNextVariableEntry* Entry = WeakEntry.Get())
								{
									return Entry->GetBinding().IsValid() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								}

								return ECheckBoxState::Undetermined;
							})
							[
								SNew(SImage)
								.Image_Lambda([WeakEntry = TWeakObjectPtr<UAnimNextVariableEntry>(VariableEntry)]() -> const FSlateBrush*
								{
									if (UAnimNextVariableEntry* Entry = WeakEntry.Get())
									{
										return Entry->GetBinding().IsValid() ? FAppStyle::Get().GetBrush("Icons.Link") : FAppStyle::Get().GetBrush("Icons.Unlink");
									}

									return FAppStyle::GetBrush(TEXT("ClassIcon.Default"));
								})
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
							.OnCheckStateChanged_Lambda([LocatorPropertyView, WeakEntry = TWeakObjectPtr<UAnimNextVariableEntry>(VariableEntry)](ECheckBoxState InState)
							{
								if (UAnimNextVariableEntry* Entry = WeakEntry.Get())
								{
									if (InState == ECheckBoxState::Checked)
									{
										FScopedTransaction Transaction(LOCTEXT("SetVariableBinding", "Set Variable Binding"));
										Entry->SetBindingType(FAnimNextUniversalObjectLocatorBindingData::StaticStruct(), true);
										LocatorPropertyView->GetPropertyHandle()->NotifyPostChange(EPropertyChangeType::ValueSet);
									}
									else if (InState == ECheckBoxState::Unchecked)
									{
										FScopedTransaction Transaction(LOCTEXT("ClearVariableBinding", "Clear Variable Binding"));
										Entry->SetBindingType(nullptr, true);
										LocatorPropertyView->GetPropertyHandle()->NotifyPostChange(EPropertyChangeType::ValueSet);
									}

									Entry->BroadcastModified(EAnimNextEditorDataNotifType::VariableBindingChanged);
								}
							})
					];
				}
			}
		}

		ValueWidget = ValueHBox;
		WidgetContainer->SetContent(ValueWidget.ToSharedRef());
	}

	// FNotifyHook interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;

	void HandleModified(UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType, UObject* InSubject)
	{
		if( InType != EAnimNextEditorDataNotifType::VariableDefaultValueChanged &&
			InType != EAnimNextEditorDataNotifType::VariableTypeChanged &&
			InType != EAnimNextEditorDataNotifType::UndoRedo)
		{
			return;
		}
		
		TSharedPtr<FVariablesOutlinerEntryItem> TreeItem = WeakTreeItem.Pin();
		if(!TreeItem.IsValid())
		{
			return;
		}

		UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(TreeItem->WeakEntry.Get());
		const UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(TreeItem->WeakSharedVariablesEntry.Get());
		if(VariableEntry != InSubject && SharedVariablesEntry != InSubject)
		{
			return;
		}

		BuildValueWidget();
	}

	TWeakPtr<FVariablesOutlinerEntryItem> WeakTreeItem;
	TSharedPtr<SBox> WidgetContainer;
	FInstancedPropertyBag InternalPropertyBag;
};

void SVariablesOutlinerValue::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	TSharedPtr<FVariablesOutlinerEntryItem> TreeItem = WeakTreeItem.Pin();
	if(!TreeItem.IsValid())
	{
		return;
	}

	UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(TreeItem->WeakEntry.Get());
	if(VariableEntry == nullptr)
	{
		return;
	}

	UUAFRigVMAssetEditorData* EditorData = VariableEntry->GetTypedOuter<UUAFRigVMAssetEditorData>();
	if (EditorData == nullptr)
	{
		return;
	}

	VariableEntry->MarkPackageDirty();
	VariableEntry->BroadcastModified(EAnimNextEditorDataNotifType::VariableDefaultValueChanged);
}

const TSharedRef<SWidget> FVariablesOutlinerValueColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef Item, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	const FOutlinerAssetItem* AssetItem = Item->CastTo<FOutlinerAssetItem>();
	if (AssetItem)
	{
		return SNullWidget::NullWidget;
	}

	// Return a dummy box widget, with ShouldGenerateEmptyWidgetForSpacing settings this will prevent us from skipping outside of the header, avoiding alignment issues
	FVariablesOutlinerEntryItem* TreeItem = Item->CastTo<FVariablesOutlinerEntryItem>();
	if (TreeItem == nullptr)
	{
		return SNew(SBox);
	}
	IUAFRigVMVariableInterface* Variable = Cast<IUAFRigVMVariableInterface>(TreeItem->WeakEntry.Get());
	if(Variable == nullptr)
	{
		return SNew(SBox);
	}
	
	return SNew(SVariablesOutlinerValue, *TreeItem, WeakSceneOutliner.Pin().ToSharedRef().Get(), Row);
}

}

#undef LOCTEXT_NAMESPACE // "VariablesOutlinerColumns"