// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutliner.h"
#include "ISceneOutlinerColumn.h"
#include "SceneOutlinerFwd.h"
#include "SPinTypeSelector.h"
#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SHeaderRow.h"

class SWidget;

namespace UE::UAF::Editor
{

class FVariablesOutlinerTypeColumn : public ISceneOutlinerColumn
{
public:
	static FName GetID();

	FVariablesOutlinerTypeColumn(ISceneOutliner& SceneOutliner);

	// ISceneOutlinerColumn interface
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return false; }

protected:
	float GetColumnWidth() const;
	
private:
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;
	SPinTypeSelector::ESelectorType SelectorType;
};

class FVariablesOutlinerValueColumn : public ISceneOutlinerColumn
{
public:
	static FName GetID();

	FVariablesOutlinerValueColumn(ISceneOutliner& SceneOutliner)
		: WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) 
		{}

	// ISceneOutlinerColumn interface
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return false; }

private:
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;
};



}
