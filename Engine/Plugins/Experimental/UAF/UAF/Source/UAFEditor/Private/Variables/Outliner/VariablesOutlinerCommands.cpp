// Copyright Epic Games, Inc. All Rights Reserved.


#include "VariablesOutlinerCommands.h"

#define LOCTEXT_NAMESPACE "VariablesOutlinerCommands"

void FVariablesOutlinerCommands::RegisterCommands()
{
	UI_COMMAND(AddNewVariable, "Add Variable", "Adds a single new variable to the selected asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNewVariables, "Add Variable(s)", "Adds variables to assets.\nIf multiple assets are selected, then variables will be added to each.\nIf no assets are selected and there are multiple assets, variables will be added to all assets.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(CreateSharedVariablesAssets, "Create Shared Variables", "Creates a new Shared Variables asset, adding and moving all of the selected set of variables.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
