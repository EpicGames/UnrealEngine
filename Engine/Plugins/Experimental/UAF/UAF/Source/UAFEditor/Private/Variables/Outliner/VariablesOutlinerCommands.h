// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "VariablesOutlinerCommands"

class FVariablesOutlinerCommands : public TCommands<FVariablesOutlinerCommands>
{
public:

	FVariablesOutlinerCommands() : TCommands<FVariablesOutlinerCommands>( TEXT("VariablesOutliner"), LOCTEXT("VariablesOutlinerCommands", "Variables Outliner Commands"), NAME_None, FAppStyle::GetAppStyleSetName() )
	{
	}
	
	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> AddNewVariable;
	TSharedPtr<FUICommandInfo> AddNewVariables;
	TSharedPtr<FUICommandInfo> CreateSharedVariablesAssets;
};

#undef LOCTEXT_NAMESPACE