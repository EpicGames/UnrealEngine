// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariablesOutlinerEntryItem.h"

#include "ISceneOutliner.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "EditorUtils.h"
#include "ScopedTransaction.h"
#include "UncookedOnlyUtils.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "UObject/Package.h"
#include "Widgets/Views/SListView.h"
#include "Common/Outliner/OutlinerDragDropOps.h"
#include "Entries/AnimNextVariableEntry.h"

#define LOCTEXT_NAMESPACE "VariablesOutlinerTreeItem"

namespace UE::UAF::Editor
{

const FSceneOutlinerTreeItemType FVariablesOutlinerEntryItem::Type(&FOutlinerEntryItem::Type);

FVariablesOutlinerEntryItem::FVariablesOutlinerEntryItem(const FVariablesOutlinerEntryItem::FItemData& InItemData)
: FOutlinerEntryItem(FVariablesOutlinerEntryItem::Type, { InItemData.Entry ? InItemData.Entry ->GetTypedOuter<UUAFRigVMAsset>() : nullptr, InItemData.SortValue })
	, WeakEntry(InItemData.Entry)
	, PropertyPath(InItemData.Property)
	, bStructOwner(InItemData.Property != nullptr)
{
}

bool FVariablesOutlinerEntryItem::IsValid() const
{
	return WeakEntry.Get() != nullptr || PropertyPath.Get() != nullptr;
}

FSceneOutlinerTreeItemID FVariablesOutlinerEntryItem::GetID() const
{
	if (UAnimNextVariableEntry* Entry = WeakEntry.Get())
	{
		return HashCombine(GetTypeHash(Entry), GetTypeHash(WeakSharedVariablesEntry.Get()));
	}
	else
	{
		return HashCombine(GetTypeHash(PropertyPath), GetTypeHash(WeakSharedVariablesEntry.Get()));
	}
}

FString FVariablesOutlinerEntryItem::GetDisplayString() const
{
	if(UUAFRigVMAssetEntry* Entry = WeakEntry.Get())
	{
		return Entry->GetDisplayName().ToString();
	}
	else if (const FProperty* Property = PropertyPath.Get())
	{
		return Property->GetName();
	}

	return FString();
}

FString FVariablesOutlinerEntryItem::GetPackageName() const
{
	if(UUAFRigVMAssetEntry* Entry = WeakEntry.Get())
	{
		return Entry->GetPackage()->GetName();
	}
	else if (const FProperty* Property = PropertyPath.Get())
	{
		const UObject* Owner = Property->GetOwner<UObject>();
		const UPackage* Package = Owner ? Owner->GetPackage() : nullptr;
		if (Package != nullptr)
		{
			return Package->GetName();
		}
	}
	
	return ISceneOutlinerTreeItem::GetPackageName();
}

void FVariablesOutlinerEntryItem::Rename(const FText& InNewName) const
{
	if(UUAFRigVMAssetEntry* Entry = WeakEntry.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("RenameVariable", "Rename variable"));
		const FName NewName = FName(*InNewName.ToString());
		UncookedOnly::FUtils::RenameVariable(CastChecked<UAnimNextVariableEntry>(Entry), NewName);
	}
}

bool FVariablesOutlinerEntryItem::ValidateName(const FText& InNewName, FText& OutErrorMessage) const
{
	return FUtils::IsValidParameterNameForEntry(WeakEntry.Get(), InNewName, OutErrorMessage);
}

void FVariablesOutlinerEntryItem::SetAccessSpecifier(const EAnimNextExportAccessSpecifier& InSpecifier) const
{
	IUAFRigVMExportInterface* Export = Cast<IUAFRigVMExportInterface>(WeakEntry.Get());
	if(Export == nullptr)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SetAccessSpecifierTransaction", "Set Access Specifier"));
	Export->SetExportAccessSpecifier(InSpecifier);
}

EAnimNextExportAccessSpecifier FVariablesOutlinerEntryItem::GetAccessSpecifier() const
{
	IUAFRigVMExportInterface* Export = Cast<IUAFRigVMExportInterface>(WeakEntry.Get());
	if(Export == nullptr)
	{
		return EAnimNextExportAccessSpecifier::Public;
	}

	return Export->GetExportAccessSpecifier();
}

bool FVariablesOutlinerEntryItem::CanSetAccessSpecifier() const
{
	if (UAnimNextVariableEntry* Entry = WeakEntry.Get())
	{
		if (UUAFRigVMAsset* OuterAsset = Entry->GetTypedOuter<UUAFRigVMAsset>())
		{
			// Disable toggling public/private for pure UUAFSharedVariables objects				
			return ExactCast<UUAFSharedVariables>(OuterAsset) == nullptr;	
		}
	}

	return false;
}

FStringView FVariablesOutlinerEntryItem::GetCategoryPath() const
{
	if (UAnimNextVariableEntry* Entry = WeakEntry.Get())
	{
		return Entry->GetVariableCategory();
	}
	
	return FStringView();
}

bool FVariablesOutlinerEntryItem::IsReadOnly() const
{
	return !PropertyPath.IsPathToFieldEmpty() || WeakEntry.Get() == nullptr;
}
}

#undef LOCTEXT_NAMESPACE // "VariablesOutlinerTreeItem"
