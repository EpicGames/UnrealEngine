// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"
#include "Common/Outliner/OutlinerEntryItem.h"

class UUAFSharedVariablesEntry;
class UAnimNextVariableEntry;

namespace UE::UAF::Editor
{

struct FVariablesOutlinerEntryItem : FOutlinerEntryItem
{
	static const FSceneOutlinerTreeItemType Type;
	
	struct FItemData 
	{
		UAnimNextVariableEntry* Entry;
		const FProperty* Property;
		int32 SortValue;
	};

	FVariablesOutlinerEntryItem() = delete;
	FVariablesOutlinerEntryItem(const FVariablesOutlinerEntryItem::FItemData& InItemData);

	// Begin ISceneOutlinerTreeItem overrides
	virtual bool IsValid() const override;
	virtual FSceneOutlinerTreeItemID GetID() const override;
	virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override { return true; }
	virtual FString GetPackageName() const override;
	// End ISceneOutlinerTreeItem overrides

	// Renames the item to the specified name
	virtual void Rename(const FText& InNewName) const override;

	// Validates the new item name
	virtual bool ValidateName(const FText& InNewName, FText& OutErrorMessage) const override;

	virtual void SetAccessSpecifier(const EAnimNextExportAccessSpecifier& InSpecifier) const override;
	virtual EAnimNextExportAccessSpecifier GetAccessSpecifier() const override;
	virtual bool CanSetAccessSpecifier() const override;
	virtual FStringView GetCategoryPath() const override;
	virtual bool IsReadOnly() const override;

	bool HasStructOwner() const { return bStructOwner; }

	// Ptr to the underlying entry if this variable came from an asset
	TWeakObjectPtr<UAnimNextVariableEntry> WeakEntry;

	// The data interface entry this entry is from, if any
	TWeakObjectPtr<const UUAFSharedVariablesEntry> WeakSharedVariablesEntry;

	// Ptr to the property if this variable came from a struct
	TFieldPath<const FProperty> PropertyPath;

	bool bStructOwner = false;
};

}
