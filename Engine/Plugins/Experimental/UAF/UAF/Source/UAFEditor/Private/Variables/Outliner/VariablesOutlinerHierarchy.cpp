// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariablesOutlinerHierarchy.h"

#include "AnimNextRigVMAsset.h"
#include "ISceneOutlinerMode.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "VariablesOutlinerCategoryItem.h"
#include "VariablesOutlinerStructSharedVariablesItem.h"
#include "VariablesOutlinerMode.h"
#include "VariablesOutlinerEntryItem.h"
#include "VariablesOutlinerRigVMAssetSharedVariablesItem.h"
#include "Common/Outliner/OutlinerAssetItem.h"
#include "Common/Outliner/SCommonOutliner.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Net/RepLayout.h"
#include "Variables/SVariablesView.h"
#include "Variables/AnimNextSharedVariables.h"
#include "VerseVM/VVMNativeRef.h"

namespace UE::UAF::Editor
{

FVariablesOutlinerHierarchy::FVariablesOutlinerHierarchy(ISceneOutlinerMode* Mode)
	: FOutlinerHierarchy(Mode)
{
}

FSceneOutlinerTreeItemPtr FVariablesOutlinerHierarchy::FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate)
{
	uint32 ParentHash = INDEX_NONE;
	FSoftObjectPath OwnerPath;
	FStringView CategoryPath;
	
	if (const FVariablesOutlinerEntryItem* EntryItem = Item.CastTo<FVariablesOutlinerEntryItem>())
	{
		const UUAFSharedVariablesEntry* SharedVariablesEntry = EntryItem->WeakSharedVariablesEntry.Get();
		
		if(SharedVariablesEntry != nullptr)
		{
			OwnerPath = SharedVariablesEntry->GetObjectPath();
		}
		else
		{
			UUAFRigVMAssetEntry* Entry = EntryItem->WeakEntry.Get();
			if (Entry == nullptr)
			{
				return nullptr;
			}

			UUAFRigVMAsset* Asset = Entry->GetTypedOuter<UUAFRigVMAsset>();
			if (Asset == nullptr)
			{
				return nullptr;
			}

			TSoftObjectPtr<UUAFRigVMAsset> SoftObjectPtr(Asset);
			OwnerPath = Asset;
			
		}

		uint32 CategoryHash = INDEX_NONE;

		// Now we have the parent asset hash, determine if we need to be child-ed to a category
		if (UAnimNextVariableEntry* VariableEntry = EntryItem->WeakEntry.Get())
		{
			if (!VariableEntry->GetVariableCategory().IsEmpty())
			{
				CategoryPath = VariableEntry->GetVariableCategory();
				CategoryHash = GetTypeHash(CategoryPath);
			}
		}

		check(OwnerPath.IsValid());
		ParentHash = CategoryHash != INDEX_NONE ? HashCombine(GetTypeHash(OwnerPath), CategoryHash) : GetTypeHash(OwnerPath);
	}

	if (const FVariablesOutlinerCategoryItem* CategoryItem = Item.CastTo<FVariablesOutlinerCategoryItem>())
	{
		if (UUAFRigVMAsset* Asset = CategoryItem->WeakOwner.Get())
		{
			OwnerPath = Asset;
			const uint32 AssetHash = GetTypeHash(OwnerPath);
			const FStringView ParentCategory = CategoryItem->ParentCategoryName;
			CategoryPath = ParentCategory;
			ParentHash = ParentCategory.Len() ? HashCombine(AssetHash, GetTypeHash(ParentCategory)) : AssetHash;
		}
	}

	if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentHash))
	{
		return *ParentItem;
	}

	TSoftObjectPtr<UUAFRigVMAsset> SoftAsset = Cast<UUAFRigVMAsset>(OwnerPath.ResolveObject());

	if (!CategoryPath.IsEmpty())
	{
		UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(SoftAsset.Get());
		const int32 CategoryIndex = EditorData->VariableAndFunctionCategories.IndexOfByKey(CategoryPath);
		ensure(CategoryIndex != INDEX_NONE);
		
		FStringView ParentCategoryName;
		FStringView SubCategoryName;

		int32 Index = INDEX_NONE;
		if (CategoryPath.FindLastChar('|', Index))
		{
			ParentCategoryName = CategoryPath.Left(Index);
			SubCategoryName = CategoryPath.RightChop(Index+1);
		}
		else
		{
			SubCategoryName = CategoryPath;
		}
		
		return Mode->CreateItemFor<FVariablesOutlinerCategoryItem>(FOutlinerCategoryItem::FItemData{UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(EditorData), CategoryIndex, SubCategoryName, ParentCategoryName, CategoryPath}, bCreate);
	}
	else if (OwnerPath.IsValid())
	{
		TArray<const FSoftObjectPath> SharedVariableSourcePaths;
		
		if (SoftAsset.IsValid())
		{
			RetrieveSharedAssetPaths(SoftAsset, SharedVariableSourcePaths);
			const FSceneOutlinerTreeItemPtr AssetItem = Mode->CreateItemFor<FOutlinerAssetItem>(FOutlinerAssetItem::FItemData{SoftAsset, 0, SharedVariableSourcePaths}, bCreate);
			
			UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(), LogAnimation, Warning, "* FOutlinerAssetItem: %ls %i", *SoftAsset.GetAssetName(), 0);
			return AssetItem;
		}
		else if (const UScriptStruct* Struct = Cast<UScriptStruct>(OwnerPath.ResolveObject()))
		{
			return Mode->CreateItemFor<FVariablesOutlinerStructSharedVariablesItem>(Struct, bCreate);
		}
		else if (TScriptInterface<IRigVMRuntimeAssetInterface> RigVMAssetInterface = OwnerPath.ResolveObject())
		{
			return Mode->CreateItemFor<FVariablesOutlinerRigVMAssetSharedVariablesItem>(RigVMAssetInterface, bCreate);
		}
	}

	return nullptr;
}

void FVariablesOutlinerHierarchy::CreateItemsForAsset(const UUAFRigVMAssetEditorData* EditorData, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	TArray<FSceneOutlinerTreeItemPtr> ParentItems;
	EditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>([this, /*&OutSharedVariableSourcePaths,*/ &ParentItems, /*Depth,*/ &OutItems](UUAFSharedVariablesEntry* InSharedVariablesEntry)
	{
		SCommonOutliner* Outliner = static_cast<FCommonOutlinerMode* const>(Mode)->GetOutliner();

		switch (InSharedVariablesEntry->Type)
		{
		case EAnimNextSharedVariablesType::Asset:
			{
				if (const UUAFSharedVariables* SharedVariables = InSharedVariablesEntry->GetAsset())
				{
					// Would have expected shared variable (assets) to be part of the outer export chain, so should have been handled as regular assets.
					if (!Outliner->GetAssetsView().Contains(SharedVariables))
					{	
						TArray<const FSoftObjectPath> SharedVariableSourcePaths;
						ProcessAsset( const_cast<UUAFSharedVariables*>(SharedVariables), 0, ParentItems, SharedVariableSourcePaths);
					}
				}				
			}
			break;

		case EAnimNextSharedVariablesType::Struct:
			{				
				if (const UScriptStruct* Struct = InSharedVariablesEntry->GetStruct())
				{
					if (WeakProcessedStructs.Contains(Struct))
					{
						return true;
					}

					WeakProcessedStructs.Add(Struct);

					if (FSceneOutlinerTreeItemPtr Item = Mode->CreateItemFor<FVariablesOutlinerStructSharedVariablesItem>(Struct))
					{
						OutItems.Add(Item);
					}
					
					UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(), LogAnimation, Warning, "~ FVariablesOutlinerStructSharedVariablesItem: %ls %i", *InSharedVariablesEntry->GetObjectPath().ToString(), 0);
					
					int32 Index = 0;
					for (TFieldIterator<FProperty> It(Struct); It; ++It)
					{
						const FProperty* Property = *It;
						if ((Property->GetPropertyFlags() & CPF_NativeAccessSpecifierPublic) == 0)
						{
							continue;
						}
						
						if (FSceneOutlinerTreeItemPtr Item = Mode->CreateItemFor<FVariablesOutlinerEntryItem>(FVariablesOutlinerEntryItem::FItemData{nullptr, Property, Index++}))
						{
							FVariablesOutlinerEntryItem* EntryItem = Item->CastTo<FVariablesOutlinerEntryItem>();
							EntryItem->WeakSharedVariablesEntry = InSharedVariablesEntry;
							OutItems.Add(Item);
						}
					}
				}
			}
			break;

		case EAnimNextSharedVariablesType::RigVMAsset:
			{
				if (const IRigVMRuntimeAssetInterface* RigVMAsset = InSharedVariablesEntry->GetRigVMAsset().GetInterface())
				{
					const UObject* RigVMAssetObject = InSharedVariablesEntry->GetRigVMAsset().GetObject();
					check(RigVMAssetObject);
					if (WeakProcessedRigVMAssets.Contains(RigVMAssetObject))
					{
						return true;
					}

					WeakProcessedRigVMAssets.Add(RigVMAssetObject);
					
					if (FSceneOutlinerTreeItemPtr Item = Mode->CreateItemFor<FVariablesOutlinerRigVMAssetSharedVariablesItem>(InSharedVariablesEntry->GetRigVMAsset()))
					{
						OutItems.Add(Item);
					}

					TArray<FRigVMExternalVariable> ExternalVariables = const_cast<IRigVMRuntimeAssetInterface*>(RigVMAsset)->GetExternalVariables();
					for (int32 i = 0; i < ExternalVariables.Num(); i++)
					{
						// We only add the public variables to the list
						if (ExternalVariables[i].IsPublic())
						{
							if (FSceneOutlinerTreeItemPtr Item = Mode->CreateItemFor<FVariablesOutlinerEntryItem>(FVariablesOutlinerEntryItem::FItemData{nullptr, ExternalVariables[i].GetProperty(), i}))
							{
								FVariablesOutlinerEntryItem* EntryItem = Item->CastTo<FVariablesOutlinerEntryItem>();
								EntryItem->WeakSharedVariablesEntry = InSharedVariablesEntry;
								EntryItem->SetAccessSpecifier(EAnimNextExportAccessSpecifier::Public);
								OutItems.Add(Item);
							}
						}
					}
				}
			}
		}		
		
		return true;
	});

	OutItems.Append(ParentItems);

	TSet<FString> ParsedCategories;
	for (int32 CategoryIndex = 0; CategoryIndex < EditorData->VariableAndFunctionCategories.Num(); ++CategoryIndex)
	{
		const FString& CategoryName = EditorData->VariableAndFunctionCategories[CategoryIndex];
		TArray<FString> SubCategories;
		CategoryName.ParseIntoArray(SubCategories, TEXT("|"));

		FString RebuildCategories;
		for (const FString& SubCategoryName : SubCategories)
		{
			const FString ParentCategoryName = RebuildCategories;

			if (!RebuildCategories.IsEmpty())
			{
				RebuildCategories.Append(TEXT("|"));
			}
			RebuildCategories.Append(SubCategoryName);
			
			const FString FullCategoryName = RebuildCategories;

			if (!ParsedCategories.Contains(FullCategoryName))
			{
				if (TSharedPtr<FVariablesOutlinerCategoryItem> Item = StaticCastSharedPtr<FVariablesOutlinerCategoryItem>(Mode->CreateItemFor<FVariablesOutlinerCategoryItem>(FVariablesOutlinerCategoryItem::FItemData{UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(EditorData), CategoryIndex, SubCategoryName, ParentCategoryName, FullCategoryName})))
				{
					ParsedCategories.Add(FullCategoryName);
					OutItems.AddUnique(Item);

					UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(),LogAnimation, Warning, "+ FVariablesOutlinerCategoryItem: %ls  %i", *FullCategoryName, CategoryIndex);
				}
			}
		}
	}

	constexpr int32 NumberOfVariableSortValueEntriesInCategory = 1024;
	
	{
		int32 Index = 1;
		EditorData->ForEachEntryOfType<UAnimNextVariableEntry>([this, &OutItems, &Index, EditorData](UAnimNextVariableEntry* InVariable)
		{
			const int32 CategoryIndex = EditorData->VariableAndFunctionCategories.IndexOfByKey(InVariable->Category);
			if (FSceneOutlinerTreeItemPtr Item = Mode->CreateItemFor<FVariablesOutlinerEntryItem>(
				FVariablesOutlinerEntryItem::FItemData
				{
					CastChecked<UAnimNextVariableEntry>(InVariable),
					nullptr,
					(CategoryIndex != INDEX_NONE ? ((CategoryIndex + 1) * NumberOfVariableSortValueEntriesInCategory) : NumberOfVariableSortValueEntriesInCategory) + Index++
				}))
			{
				OutItems.Add(Item);
				
				UE_CLOGF(GOutputHierarchyDebugInformationCvar.GetValueOnAnyThread(), LogAnimation, Warning, "\t- FVariablesOutlinerEntryItem: %ls %i %ls", *InVariable->GetEntryName().ToString(), Item->CastTo<FVariablesOutlinerEntryItem>()->SortValue, InVariable->GetVariableCategory().GetData());
			}
			return true;
		});
	}
}
}
