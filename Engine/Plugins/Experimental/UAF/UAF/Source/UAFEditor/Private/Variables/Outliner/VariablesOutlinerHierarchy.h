// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerHierarchy.h"
#include "Common/Outliner/OutlinerHierarchy.h"

class UUAFRigVMAssetEditorData;
class UUAFRigVMAsset;

namespace UE::UAF::Editor
{

class FVariablesOutlinerHierarchy : public FOutlinerHierarchy
{
public:
	FVariablesOutlinerHierarchy(ISceneOutlinerMode* Mode);
	FVariablesOutlinerHierarchy(const FVariablesOutlinerHierarchy&) = delete;
	FVariablesOutlinerHierarchy& operator=(const FVariablesOutlinerHierarchy&) = delete;

	// Begin ISceneOutlinerHierarchy overrides
	virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;
	// End ISceneOutlinerHierarchy overrides

	virtual void CreateItemsForAsset(const UUAFRigVMAssetEditorData* EditorData, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
};

}