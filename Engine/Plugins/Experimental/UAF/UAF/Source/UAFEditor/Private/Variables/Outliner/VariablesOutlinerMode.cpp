// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariablesOutlinerMode.h"

#include "ISourceControlModule.h"
#include "WorkspaceItemMenuContext.h"
#include "VariablesOutlinerHierarchy.h"
#include "VariablesOutlinerEntryItem.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "IWorkspaceEditor.h"
#include "AnimNextEditorModule.h"
#include "UAFStyle.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "Framework/Commands/GenericCommands.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "AnimNextRigVMAsset.h"
#include "ContentBrowserModule.h"
#include "FileHelpers.h"
#include "IWorkspaceEditorModule.h"
#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "VariablesOutlinerCommands.h"
#include "VariablesOutlinerStructSharedVariablesItem.h"
#include "VariablesOutlinerCategoryItem.h"
#include "Common/Outliner/OutlinerDragDropOps.h"
#include "Common/AnimNextAssetFindReplaceVariables.h"
#include "Common/GraphEditorSchemaActions.h"
#include "Common/Outliner/OutlinerAssetItem.h"
#include "Common/Outliner/OutlinerCommands.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Variables/SVariablesView.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Styling/SlateIconFinder.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Variables/AnimNextSharedVariablesFactory.h"
#include "Variables/AnimNextVariableSettings.h"
#include "Widgets/Docking/SDockTab.h"
#include "UAFCompilationScope.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Common/Outliner/OutlinerItemMenuContexts.h"
#include "Common/Outliner/SCommonOutliner.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "VariablesOutlinerMode"

namespace UE::UAF::Editor
{
void FVariablesOutlinerMode::RegisterToolMenus()
{
	FCommonOutlinerMode::RegisterToolMenus();
	
	UToolMenus* ToolMenus = UToolMenus::Get();
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);
		
		if (UToolMenu* Menu = ToolMenus->ExtendMenu(ContextToolMenuName))
		{
			FToolMenuSection& AssetSection = Menu->AddDynamicSection("DynamicAssetSection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				const UCommonOutlinerItemMenuContext* MenuContext = InMenu->FindContext<UCommonOutlinerItemMenuContext>();			
				const UVariablesOutlinerItemMenuContext* VariablesContext = InMenu->FindContext<UVariablesOutlinerItemMenuContext>();
				if (MenuContext == nullptr || VariablesContext == nullptr)
				{
					return;
				}
				
				TSharedPtr<SCommonOutliner> SharedOutliner = MenuContext->WeakOutliner.Pin();
				if (SharedOutliner.IsValid() && MenuContext->WeakEditorDatas.Num() == 1)
				{
					FToolMenuSection& AssetsSection = InMenu->AddSection(ContextToolAssetSectionName, LOCTEXT("AssetsSectionLabel", "Assets"));
				
					const FVariablesOutlinerMode* Outliner = static_cast<const FVariablesOutlinerMode*>(SharedOutliner->GetMode());
					AssetsSection.AddMenuEntry("AddNewCategory",  LOCTEXT("AddCategoryLabel", "Add New Category"), LOCTEXT("AddCategoryTooltip", "Adds a new Category to the selected Asset"), FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.PlusCircle"), FUIAction(FExecuteAction::CreateRaw(Outliner, &FVariablesOutlinerMode::AddCategory, MenuContext->WeakEditorDatas[0])));
				}
			}));
			
			
			FToolMenuSection& VariablesSection = Menu->AddDynamicSection("Variables", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				const UAssetEditorToolkitMenuContext* EditorContext = InMenu->FindContext<UAssetEditorToolkitMenuContext>();
				const UCommonOutlinerItemMenuContext* OutlinerContext = InMenu->FindContext<UCommonOutlinerItemMenuContext>();
				const UVariablesOutlinerItemMenuContext* MenuContext = InMenu->FindContext<UVariablesOutlinerItemMenuContext>();
				if(EditorContext == nullptr || OutlinerContext == nullptr || MenuContext == nullptr)
				{
					return;
				}
				
				FToolMenuSection& VariablesSection = InMenu->AddSection("Variables", LOCTEXT("VariablesSectionLabel", "Variables"));

				/** Variable specific actions */			
				{
					// Populates entries for adding of different variable types/sources	
					PopulateNewVariableToolMenuEntries(InMenu, false);
					
					FToolMenuEntry& Entry = VariablesSection.AddMenuEntryWithCommandList(FVariablesOutlinerCommands::Get().CreateSharedVariablesAssets, OutlinerContext->WeakCommandList.Pin(), TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FUAFStyle::Get().GetStyleSetName(), "ClassIcon.AnimNextSharedVariables"));
					Entry.InsertPosition = FToolMenuInsert("AddNewVariable", EToolMenuInsertType::After);
				}
			}), FToolMenuInsert(FCommonOutlinerMode::ContextToolAssetSectionName, EToolMenuInsertType::After));
		}
		
		
		if (UToolMenu* Menu = ToolMenus->ExtendMenu(AddEntryContextMenuName))
		{
			Menu->AddDynamicSection("VariablesDynamic", FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
			{
				const UCommonOutlinerItemMenuContext* OutlinerContext = InMenu->FindContext<UCommonOutlinerItemMenuContext>();
				const UVariablesOutlinerItemMenuContext* MenuContext = InMenu->FindContext<UVariablesOutlinerItemMenuContext>();
				if(MenuContext == nullptr)
				{
					return;
				}
				
				TSharedPtr<SCommonOutliner> SharedOutliner = OutlinerContext->WeakOutliner.Pin();
				if (SharedOutliner.IsValid() && OutlinerContext->WeakEditorDatas.Num() == 1)
				{
					FToolMenuSection& AssetSection = InMenu->AddSection(ContextToolAssetSectionName, LOCTEXT("AssetSectionLabel", "Asset"));
					{
						const FVariablesOutlinerMode* Outliner = static_cast<const FVariablesOutlinerMode*>(SharedOutliner->GetMode());
						AssetSection.AddMenuEntry("AddNewCategory",  LOCTEXT("AddCategoryLabel", "Add New Category"), LOCTEXT("AddCategoryTooltip", "Adds a new Category to the selected Asset"), FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.PlusCircle"), FUIAction(FExecuteAction::CreateRaw(Outliner, &FVariablesOutlinerMode::AddCategory, OutlinerContext->WeakEditorDatas[0])));
					}
				}
			
				PopulateNewVariableToolMenuEntries(InMenu, false);			
			}));
		}
	}
}

void FVariablesOutlinerMode::InitToolMenuContext(FToolMenuContext& InOutContext) const
{
	FCommonOutlinerMode::InitToolMenuContext(InOutContext);
	
	UVariablesOutlinerItemMenuContext* MenuContext = NewObject<UVariablesOutlinerItemMenuContext>();
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = GetOutliner()->GetSelectedItems();
	for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
	{
		if (const FVariablesOutlinerEntryItem* EntryItem = Item->CastTo<FVariablesOutlinerEntryItem>())
		{
			UUAFRigVMAssetEntry* Entry = EntryItem->WeakEntry.Get();
			if(Entry == nullptr)
			{
				continue;
			}

			MenuContext->WeakEntries.Add(Entry);
		}
	}
	
	InOutContext.AddObject(MenuContext);
}

FVariablesOutlinerMode::FVariablesOutlinerMode(SCommonOutliner* InOutliner, const TSharedRef<Workspace::IWorkspaceEditor>& InWorkspaceEditor)
	: FCommonOutlinerMode(InOutliner, InWorkspaceEditor)
{
	static const FName MenuName("VariablesOutliner.AddVariablesMenu");
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		FToolMenuOwnerScoped ToolMenuOwnerScope(this);

		if (UToolMenu* Menu = ToolMenus->RegisterMenu(MenuName))
		{
			Menu->AddDynamicSection(TEXT("Variables"), FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				PopulateNewVariableToolMenuEntries(InMenu, false);				
			}));
		}
	}
	
	RegisterDragDropOperationConstructor<FVariablesOutlinerEntryItem>([](const FSceneOutlinerTreeItemPtr& InItem)-> TSharedPtr<FDragDropOperation>
	{
		FVariablesOutlinerEntryItem* VariableItem = InItem->CastTo<FVariablesOutlinerEntryItem>();
		check(VariableItem);
		
		if(UAnimNextVariableEntry* Entry = Cast<UAnimNextVariableEntry>(VariableItem->WeakEntry.Get()))
		{
			const UUAFSharedVariables* Asset = nullptr;
			if (VariableItem->WeakSharedVariablesEntry.Get())
			{
				Asset = VariableItem->WeakSharedVariablesEntry->GetAsset();
			}
			else
			{
				Asset = Entry->GetTypedOuter<UUAFSharedVariables>();
			}
	
			if (Asset)
			{
				TSharedPtr<FAnimNextSchemaAction_Variable> Action = MakeShared<FAnimNextSchemaAction_Variable>(Entry->GetVariableName(), Asset, Entry->GetType(), FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Deferred);

				return FVariableDragDropOp::New(Action, StaticCastSharedRef<FVariablesOutlinerEntryItem>(VariableItem->AsShared()));
			}
		}
		else if (const FProperty* Property = VariableItem->PropertyPath.Get())
		{
			if (const UObject* Owner = Property->GetOwner<UObject>())
			{
				TSharedPtr<FAnimNextSchemaAction_Variable> Action = MakeShared<FAnimNextSchemaAction_Variable>(Property->GetFName(), Owner, FAnimNextParamType::FromProperty(Property), FAnimNextSchemaAction_Variable::EVariableAccessorChoice::Deferred);
			
				TSharedPtr<FVariableDragDropOp> Op = FVariableDragDropOp::New(Action, StaticCastSharedRef<FVariablesOutlinerEntryItem>(VariableItem->AsShared()));
				return Op;
			}
		}
		
		return nullptr;
	});
	
	RegisterDragDropOperationParser<FVariableDragDropOp>([](FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& InDragDropOperation)
	{
		check(InDragDropOperation.IsOfType<FVariableDragDropOp>());
		const FVariableDragDropOp& VariableDragDropOp = static_cast<const FVariableDragDropOp&>(InDragDropOperation);

		OutPayload.DraggedItems.Add(VariableDragDropOp.WeakItem.Pin());
	});
	
	RegisterDragDropOperator<FVariablesOutlinerEntryItem, FOutlinerCategoryItem, FVariableDragDropOp>([](const ISceneOutlinerTreeItem& DragOverTreeItem, TSharedRef<FDragDropOperation> DragDropOperation, FSceneOutlinerDragValidationInfo& OutValidationInfo) -> FReply
	{
		TSharedRef<FVariableDragDropOp> VariableDropOp = StaticCastSharedRef<FVariableDragDropOp>(DragDropOperation);			
		const FOutlinerCategoryItem* DragOverItem = DragOverTreeItem.CastTo<FOutlinerCategoryItem>();
		
		if (VariableDropOp->WeakItem.IsValid())
		{
			TSharedPtr<FVariablesOutlinerEntryItem> DropVariableItem = VariableDropOp->WeakItem.Pin();
			
			if (DropVariableItem->HasStructOwner())
			{
				const FText FormattedMessage = LOCTEXT("StructItemFeedback", "Cannot make changes to Struct Entries");						
				OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FormattedMessage);
				
				return FReply::Handled();
			}
			
			if (UAnimNextVariableEntry* VariableEntry = DropVariableItem->WeakEntry.Get())
			{
				const FText OtherVariableNameText = FText::FromName(VariableEntry->GetVariableName());
				const FText ThisVariableCategoryText = FText::FromString(DragOverItem->GetDisplayString());

				const UUAFRigVMAssetEditorData* VariableEditorData = VariableEntry->GetTypedOuter<UUAFRigVMAssetEditorData>();
				const UUAFRigVMAssetEditorData* CategoryEditorData = UncookedOnly::FUtils::GetEditorData(DragOverItem->WeakOwner.Get());

				if (VariableEditorData == CategoryEditorData)
				{
					if (VariableEntry->GetVariableCategory() == DragOverItem->CategoryPath)
					{
						const FText FormattedMessage = FText::Format(LOCTEXT("VariableAlreadyPartOfCategoryFormat", "{0} is already part of Category {1}"), OtherVariableNameText, ThisVariableCategoryText);
						
						OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, FormattedMessage);
					}
					else if (!VariableEntry->GetVariableCategory().IsEmpty())
					{
						const FText FormattedMessage = FText::Format(LOCTEXT("MoveVariableToCategoryFormat", "Moving {0} to Category {1}"), OtherVariableNameText, ThisVariableCategoryText);
						OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
					}
					else
					{
						const FText FormattedMessage = FText::Format(LOCTEXT("AddVariableToCategoryFormat", "Add {0} to Category {1}"), OtherVariableNameText, ThisVariableCategoryText);
						OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
					}
				}
				else
				{
					const FText ThisAssetText = FText::FromString(DragOverItem->GetDisplayString());
					const FText FormattedMessage = FText::Format(LOCTEXT("MoveVariableToCategoryAndAssetFormat", "Moving {0} to Category {1} in {2}"), OtherVariableNameText, ThisVariableCategoryText, ThisAssetText);
						OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);		
				}
				
				return FReply::Handled();
			}
		}
		
	
		return FReply::Unhandled();
	},
	[this](ISceneOutlinerTreeItem& DroppedOnTreeItem, TSharedRef<const FDragDropOperation> DragDropOperation)
	{			
		TSharedRef<const FVariableDragDropOp> VariableDropOp = StaticCastSharedRef<const FVariableDragDropOp>(DragDropOperation);
		const FOutlinerCategoryItem* DroppedOnCategory = DroppedOnTreeItem.CastTo<FOutlinerCategoryItem>();			
		TSharedPtr<FVariablesOutlinerEntryItem> DropVariableItem = VariableDropOp->WeakItem.Pin();
		
		if (DropVariableItem.IsValid())
		{
			if (UAnimNextVariableEntry* VariableEntry = DropVariableItem->WeakEntry.Get())
			{
				UUAFRigVMAsset* VariableAsset = VariableEntry->GetTypedOuter<UUAFRigVMAsset>();
				UUAFRigVMAsset* CategoryAsset = DroppedOnCategory->WeakOwner.Get();

				if (CategoryAsset)
				{
					const FText OtherVariableNameText = FText::FromName(VariableEntry->GetVariableName());
					const FText ThisVariableCategoryText = FText::FromString(DroppedOnCategory->GetDisplayString());
					const FText FormattedMessage = FText::Format(LOCTEXT("MoveVariableToCategoryFormat", "Moving {0} to Category {1}"), OtherVariableNameText, ThisVariableCategoryText);	
					FScopedTransaction Transaction(FormattedMessage);

					UAnimNextVariableEntry* NewVariableEntry = VariableEntry;
						
					if (VariableAsset != CategoryAsset)
					{
						const FName VariableEntryName = VariableEntry->GetVariableName();
						UncookedOnly::FUtils::MoveVariableToAsset(VariableEntry, CategoryAsset, true, true);
						NewVariableEntry = Cast<UAnimNextVariableEntry>(UncookedOnly::FUtils::GetEditorData(CategoryAsset)->FindEntry(VariableEntryName));
					}
						
					check(NewVariableEntry);
						
					// Only add if not already part of category
					if (NewVariableEntry->GetVariableCategory() != DroppedOnCategory->CategoryPath)
					{
						NewVariableEntry->SetVariableCategory(DroppedOnCategory->CategoryPath);
					}
					
					return true;
				}
			}
		}
		
		return false;
	});	
	
	RegisterDragDropOperator<FVariablesOutlinerEntryItem, FVariablesOutlinerEntryItem, FVariableDragDropOp>([](const ISceneOutlinerTreeItem& DragOverTreeItem, TSharedRef<FDragDropOperation> DragDropOperation, FSceneOutlinerDragValidationInfo& OutValidationInfo) -> FReply
	{
		TSharedRef<FVariableDragDropOp> VariableDropOp = StaticCastSharedRef<FVariableDragDropOp>(DragDropOperation);			
		const FVariablesOutlinerEntryItem* DragOverItem = DragOverTreeItem.CastTo<FVariablesOutlinerEntryItem>();			
		TSharedPtr<FVariablesOutlinerEntryItem> DropVariableItem = VariableDropOp->WeakItem.Pin();

		if (DragOverItem && DropVariableItem.IsValid())
		{
			const UAnimNextVariableEntry* ThisEntry = DragOverItem->WeakEntry.Get();

			if (DragOverItem->HasStructOwner() || DropVariableItem->HasStructOwner())
			{
				const FText FormattedMessage = LOCTEXT("StructItemFeedback", "Cannot make changes to Struct Entries");						
				OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FormattedMessage);
			}
			else if (ThisEntry)
			{
				if (const UAnimNextVariableEntry* OtherEntry = DropVariableItem->WeakEntry.Get())
				{
					const FText OtherVariableNameText = FText::FromName(OtherEntry->GetVariableName());					
					const FText OtherVariableCategoryText = FText::FromStringView(OtherEntry->GetVariableCategory());
					const FText ThisVariableNameText = FText::FromName(ThisEntry->GetVariableName());
					const FText ThisVariableCategoryText = FText::FromStringView(ThisEntry->GetVariableCategory());
					
					if (ThisEntry && ThisEntry != OtherEntry)
					{
						UUAFRigVMAsset* ThisAsset = DragOverItem->WeakEntry.Get()->GetTypedOuter<UUAFRigVMAsset>();
						const UUAFRigVMAsset* OtherAsset = DropVariableItem->WeakEntry.Get()->GetTypedOuter<UUAFRigVMAsset>();
						if (ThisAsset && ThisAsset == OtherAsset)
						{
							if (ThisEntry->GetVariableCategory() != OtherEntry->GetVariableCategory())
							{
								if (ThisEntry->GetVariableCategory().IsEmpty())
								{
									// Remove from category
									const FText FormattedMessage = FText::Format(LOCTEXT("RemoveVariableToCategoryFormat", "Remove {0} from Category {1}"), OtherVariableNameText, OtherVariableCategoryText);
									OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
								}
								else
								{
									// Add to category
									const FText FormattedMessage = FText::Format(LOCTEXT("AddVariableToCategoryFormat", "Add {0} to Category {1}"), OtherVariableNameText, ThisVariableCategoryText);
									OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
								}
							}
							else
							{
								// Reorder
								const FText FormattedMessage = FText::Format(LOCTEXT("ReorderVariableFormat", "Reorder {0} before {1}"), OtherVariableNameText, ThisVariableNameText);
								OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
							}
						}
						else if (ThisAsset)
						{
							const FText VariableNameText = FText::FromName(OtherEntry->GetVariableName());
							const FText AssetNameText = FText::FromName(ThisAsset->GetFName());
							
							// [TODO] should we only allow move variables "up" the asset chain? If so validate that according to the workspace export chain, that VariableOuter is a child of Asset
							const FText FormattedMessage = FText::Format(LOCTEXT("MoveVariableToAssetFormat", "Move {0} to {1}"), VariableNameText, AssetNameText);
							
							OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
						}
					}
					else
					{
						// Reorder
						const FText FormattedMessage = LOCTEXT("ReorderSelfVariableLabel", "Cannot Reorder Variable before itself");
						OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FormattedMessage);
					}
				}					
			}
			
			return FReply::Handled();
		}
	
		return FReply::Unhandled();
	},
	[this](ISceneOutlinerTreeItem& DroppedOnTreeItem, TSharedRef<const FDragDropOperation> DragDropOperation)
	{
		TSharedRef<const FVariableDragDropOp> VariableDropOp = StaticCastSharedRef<const FVariableDragDropOp>(DragDropOperation);			
		const FVariablesOutlinerEntryItem* DroppedOnItem = DroppedOnTreeItem.CastTo<FVariablesOutlinerEntryItem>();			
		TSharedPtr<FVariablesOutlinerEntryItem> DropVariableItem = VariableDropOp->WeakItem.Pin();

		if (DroppedOnItem)
		{
			const UAnimNextVariableEntry* ThisEntry = DroppedOnItem->WeakEntry.Get();
			TSharedPtr<FVariablesOutlinerEntryItem> EntryItem = VariableDropOp->WeakItem.Pin();

			if (ThisEntry && EntryItem.IsValid())
			{
				if (UAnimNextVariableEntry* OtherEntry = EntryItem->WeakEntry.Get())
				{
					const FText OtherVariableNameText = FText::FromName(OtherEntry->GetVariableName());					
					const FText OtherVariableCategoryText = FText::FromStringView(OtherEntry->GetVariableCategory());
					const FText ThisVariableNameText = FText::FromName(ThisEntry->GetVariableName());
					const FText ThisVariableCategoryText = FText::FromStringView(ThisEntry->GetVariableCategory());
					
					if (ThisEntry && ThisEntry != OtherEntry)
					{
						UUAFRigVMAsset* ThisAsset = DroppedOnItem->WeakEntry.Get()->GetTypedOuter<UUAFRigVMAsset>();
						const UUAFRigVMAsset* OtherAsset = EntryItem->WeakEntry.Get()->GetTypedOuter<UUAFRigVMAsset>();
						if (ThisAsset && ThisAsset == OtherAsset)
						{
							UAnimNextVariableEntry* VariableEntry = EntryItem->WeakEntry.Get();
							const UAnimNextVariableEntry* ThisVariableEntry = DroppedOnItem->WeakEntry.Get();
								
							if (VariableEntry && ThisVariableEntry)
							{	
								if (ThisEntry->GetVariableCategory() != OtherEntry->GetVariableCategory())
								{
									UUAFRigVMAssetEditorData* VariableEditorData = OtherEntry->GetTypedOuter<UUAFRigVMAssetEditorData>();
									UUAFRigVMAssetEditorData* ThisEditorData = ThisEntry->GetTypedOuter<UUAFRigVMAssetEditorData>();																			
									if (VariableEditorData && ThisEditorData)
									{
										if (VariableEditorData == ThisEditorData)
										{
											// Add or remove to category of this variable entry
											const bool bRemovingFromCategory = ThisEntry->GetVariableCategory().IsEmpty();
																				
											const FText AddFormattedMessage = FText::Format(LOCTEXT("AddVariableToCategoryFormat", "Add {0} to Category {1}"), OtherVariableNameText, ThisVariableCategoryText);
											const FText RemoveFormattedMessage = FText::Format(LOCTEXT("RemoveVariableToCategoryFormat", "Remove {0} from Category {1}"), OtherVariableNameText, OtherVariableCategoryText);
												
											FScopedTransaction Transaction(bRemovingFromCategory ? RemoveFormattedMessage : AddFormattedMessage);
											OtherEntry->SetVariableCategory(ThisEntry->GetVariableCategory(), true);		
											// Also reorder variables such that dropped item is above dropped-on item
											VariableEditorData->ReorderVariable(VariableEntry, ThisVariableEntry);
											
											return true;
										}
									}
								}	
								else
								{
									const FText FormattedMessage = FText::Format(LOCTEXT("ReorderVariableFormat", "Reorder {0} before {1}"), OtherVariableNameText, ThisVariableNameText);
									FScopedTransaction Transaction(FormattedMessage);
								
									UUAFRigVMAssetEditorData* VariableEditorData = VariableEntry->GetTypedOuter<UUAFRigVMAssetEditorData>();
									UUAFRigVMAssetEditorData* ThisEditorData = ThisVariableEntry->GetTypedOuter<UUAFRigVMAssetEditorData>();
									if (VariableEditorData && VariableEditorData == ThisEditorData)
									{
										VariableEditorData->ReorderVariable(VariableEntry, ThisVariableEntry);
										return true;
									}
								}	
							}	
						}
						else if (ThisAsset)
						{
							if (UAnimNextVariableEntry* VariableEntry = EntryItem->WeakEntry.Get())
							{
								const FText VariableNameText = FText::FromName(OtherEntry->GetVariableName());
								const FText AssetNameText = FText::FromName(ThisAsset->GetFName());
																	
								const FText FormattedMessage = FText::Format(LOCTEXT("MoveVariableToAssetFormatTransaction", "Moving {0} to {1}"), VariableNameText, AssetNameText);
								FScopedTransaction Transaction(FormattedMessage);
								UncookedOnly::FUtils::MoveVariableToAsset(VariableEntry, ThisAsset, true, true);
								
								return true;
							}
						}
					}
				}					
			}
		}
	
		return false;
	});
	
	RegisterDragDropOperator<FVariablesOutlinerEntryItem, FVariablesOutlinerStructSharedVariablesItem, FVariableDragDropOp>([](const ISceneOutlinerTreeItem& DragOverTreeItem, TSharedRef<FDragDropOperation> DragDropOperation, FSceneOutlinerDragValidationInfo& OutValidationInfo) -> FReply
	{
		const FText FormattedMessage = LOCTEXT("StructItemFeedback", "Cannot make changes to Struct Entries");						
		OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FormattedMessage);			
		return FReply::Handled();
	},
	[this](ISceneOutlinerTreeItem& DroppedOnTreeItem, TSharedRef<const FDragDropOperation> DragDropOperation){ return false; }
	);
	
	RegisterDragDropOperator<FVariablesOutlinerEntryItem, FOutlinerAssetItem, FVariableDragDropOp>([](const ISceneOutlinerTreeItem& DragOverTreeItem, TSharedRef<FDragDropOperation> DragDropOperation, FSceneOutlinerDragValidationInfo& OutValidationInfo) -> FReply
	{	
		TSharedRef<FVariableDragDropOp> VariableDropOp = StaticCastSharedRef<FVariableDragDropOp>(DragDropOperation);			
		const FOutlinerAssetItem* DragOverItem = DragOverTreeItem.CastTo<FOutlinerAssetItem>();			
		TSharedPtr<FVariablesOutlinerEntryItem> DropVariableItem = VariableDropOp->WeakItem.Pin();
		
		if (DropVariableItem->HasStructOwner())
		{
			const FText FormattedMessage = LOCTEXT("StructItemFeedback", "Cannot make changes to Struct Entries");						
			OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FormattedMessage);
			return FReply::Handled();
		}
		
		if (const UUAFRigVMAsset* Asset = DragOverItem->SoftAsset.Get())
		{
			if (DropVariableItem.IsValid())
			{
				if (const UAnimNextVariableEntry* VariableEntry = DropVariableItem->WeakEntry.Get())
				{
					const FText VariableNameText = FText::FromName(VariableEntry->GetVariableName());
					const FText AssetNameText = FText::FromName(Asset->GetFName());
				
					const UUAFRigVMAsset* VariableOuter = VariableEntry->GetTypedOuter<UUAFRigVMAsset>();
					if (VariableOuter != Asset)
					{
						// [TODO] should we only allow move variables "up" the asset chain? If so validate that according to the workspace export chain, that VariableOuter is a child of Asset
						const FText FormattedMessage = FText::Format(LOCTEXT("MoveVariableToAssetFormat", "Move {0} to {1}"), VariableNameText, AssetNameText);
						OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);	
					}
					else
					{
						const FStringView CategoryPath = VariableEntry->GetVariableCategory();
						if (CategoryPath.IsEmpty())
						{
							const FText FormattedMessage = FText::Format(LOCTEXT("VariableAlreadyPartOfAssetFormat", "{0} is already part of {1}"), VariableNameText, AssetNameText);								
							OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::Incompatible, FormattedMessage);	
						}
						else
						{
							const FText FormattedMessage = FText::Format(LOCTEXT("RemoveVariableFromCategoryFormat", "Removing {0} from Category {1}"), VariableNameText, FText::FromStringView(CategoryPath));	
							OutValidationInfo = FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FormattedMessage);
						}							
					}
					
					return FReply::Handled();
				}
			}
		}
		
		return FReply::Unhandled();
	},
	[this](ISceneOutlinerTreeItem& DroppedOnTreeItem, TSharedRef<const FDragDropOperation> DragDropOperation)
	{
		if (DragDropOperation->IsOfType<FVariableDragDropOp>() && DroppedOnTreeItem.IsA<FOutlinerAssetItem>())
		{
			TSharedRef<const FVariableDragDropOp> VariableDropOp = StaticCastSharedRef<const FVariableDragDropOp>(DragDropOperation);			
			const FOutlinerAssetItem* DroppedOnItem = DroppedOnTreeItem.CastTo<FOutlinerAssetItem>();			
			TSharedPtr<FVariablesOutlinerEntryItem> DropVariableItem = VariableDropOp->WeakItem.Pin();
			
			if (UUAFRigVMAsset* Asset = DroppedOnItem->SoftAsset.Get())
			{
				if (DropVariableItem.IsValid())
				{
					if (UAnimNextVariableEntry* VariableEntry = DropVariableItem->WeakEntry.Get())
					{
						const UUAFRigVMAsset* VariableOuter = VariableEntry->GetTypedOuter<UUAFRigVMAsset>();
						const FText VariableNameText = FText::FromName(VariableEntry->GetVariableName());
						
						if (VariableOuter != Asset)
						{								
							const FText AssetNameText = FText::FromName(Asset->GetFName());						
							const FText FormattedMessage = FText::Format(LOCTEXT("MoveVariableToAssetFormatTransaction", "Moving {0} to {1}"), VariableNameText, AssetNameText);
							FScopedTransaction Transaction(FormattedMessage);
							UncookedOnly::FUtils::MoveVariableToAsset(VariableEntry, Asset, true, true);
							
							return true;
						}
						else
						{
							const FStringView CategoryPath = VariableEntry->GetVariableCategory();
							if (!CategoryPath.IsEmpty())
							{
								const FText FormattedMessage = FText::Format(LOCTEXT("RemoveVariableFromCategoryFormat", "Removing {0} from Category {1}"), VariableNameText, FText::FromStringView(CategoryPath));	
								
								FScopedTransaction Transaction(FormattedMessage);
								VariableEntry->SetVariableCategory(TEXT(""));
								
								return true;
							}							
						}
					}
				}
			}
		}
		
		return false;
	});
}

bool FVariablesOutlinerMode::UpdateOperationDecorator(const FDragDropEvent& Event, const FSceneOutlinerDragValidationInfo* ValidationInfo) const
{
	TSharedPtr<FVariableDragDropOp> GraphDropOp = Event.GetOperationAs<FVariableDragDropOp>();
	if (GraphDropOp.IsValid())
	{
		if (ValidationInfo)
		{
			const FSlateBrush* ValidHoverIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
			const FSlateBrush* InvalidSymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));			
			GraphDropOp->SetSimpleFeedbackMessage(ValidationInfo->IsValid() ? ValidHoverIcon : InvalidSymbol, FLinearColor::White, ValidationInfo->ValidationText);		
		}
		else
		{
			GraphDropOp->HoverTargetChanged();
		}
		
		return true;
	}
	
	return false;
}

void FVariablesOutlinerMode::ReorderCategoryItem(TSharedPtr<FOutlinerCategoryItem> EntryItem, TSharedPtr<FOutlinerCategoryItem> BeforeItem) const
{
	const UUAFRigVMAssetEditorData* ThisEditorData = UncookedOnly::FUtils::GetEditorData(EntryItem->WeakOwner.Get());
	 UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(BeforeItem->WeakOwner.Get());
	if (EditorData && ThisEditorData)
	{
		if (EditorData == ThisEditorData)
		{		
			EditorData->ReorderCategory(EntryItem->CategoryPath, BeforeItem->CategoryPath);
		}
	}
}

void FVariablesOutlinerMode::ReorderCategory(const FString& CategoryPath, const FString& BeforeCategoryPath, UUAFRigVMAsset* Asset) const
{
	if (UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(Asset))
	{
		EditorData->ReorderCategory(CategoryPath, BeforeCategoryPath);
	}
}

void FVariablesOutlinerMode::MoveCategoryToAsset(const FString& CategoryPath, UUAFRigVMAsset* FromAsset, UUAFRigVMAsset* ToAsset) const
{
	UUAFRigVMAssetEditorData* ThisEditorData = UncookedOnly::FUtils::GetEditorData(FromAsset);
	UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(ToAsset);

	UE::UAF::UncookedOnly::FCompilationScope CompilerResults(LOCTEXT("MoveCategoryToAsset", "Move Category To Asset"), MakeConstArrayView({ FromAsset, ToAsset }));
	
	TArray<FString> CategoriesToMove;
	Algo::TransformIf(ThisEditorData->VariableAndFunctionCategories, CategoriesToMove, [&CategoryPath](const FString& CategoryName)
	{
		return CategoryName.StartsWith(CategoryPath);
	},
	[](const FString& CategoryName)
	{
		return CategoryName;
	});	
	
	TArray<UAnimNextVariableEntry*> EntriesToMove;
	ThisEditorData->ForEachEntryOfType<UAnimNextVariableEntry>([&EntriesToMove, &CategoriesToMove](UAnimNextVariableEntry* Entry)
	{
		if (CategoriesToMove.Contains(Entry->GetVariableCategory()))
		{
			EntriesToMove.Add(Entry);
		}
	
		return true;
	});
	
	for (const FString& CategoryName : CategoriesToMove)
	{	
		EditorData->AddCategory(CategoryName);
		ThisEditorData->RemoveCategory(CategoryName);
	}
	
	if (EntriesToMove.Num())
	{
		for (UAnimNextVariableEntry* ToMoveEntry : EntriesToMove)
		{
			if (UAnimNextVariableEntry* NewEntry = UncookedOnly::FUtils::MoveVariableToAsset(ToMoveEntry, UncookedOnly::FUtils::GetAsset(EditorData)))
			{
				NewEntry->SetVariableCategory(ToMoveEntry->GetVariableCategory());
			}
		}
	}
}

void FVariablesOutlinerMode::Rebuild()
{
	Hierarchy = CreateHierarchy();
}

void FVariablesOutlinerMode::HandleItemSelection(const FSceneOutlinerItemSelection& Selection) const
{
	if (TSharedPtr<Workspace::IWorkspaceEditor> WorkspaceEditor = WeakWorkspaceEditor.Pin())
	{
		TArray<FSceneOutlinerTreeItemPtr> SelectedItems;
		Selection.Get(SelectedItems);
		TArray<UObject*> EntriesToShow;
		EntriesToShow.Reserve(SelectedItems.Num());
		for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
		{
			if (FVariablesOutlinerEntryItem* VariablesItem = Item->CastTo<FVariablesOutlinerEntryItem>())
			{
				if(UAnimNextVariableEntry* VariableEntry = VariablesItem->WeakEntry.Get())
				{
					EntriesToShow.Add(VariableEntry);
				}
			}
		}

		WorkspaceEditor->SetDetailsObjects(EntriesToShow);
	}
}

bool FVariablesOutlinerMode::CanRenameItem(const ISceneOutlinerTreeItem& Item) const
{
	if (const FVariablesOutlinerEntryItem* EntryItem = Item.CastTo<FVariablesOutlinerEntryItem>())
	{
		return !EntryItem->HasStructOwner();
	}
	else if (Item.IsA<FOutlinerAssetItem>())
	{
		return true;
	}
	else if (Item.IsA<FVariablesOutlinerCategoryItem>())
	{
		return true;
	}
	
	return false;
}

void FVariablesOutlinerMode::BindCommands(const TSharedRef<FUICommandList>& OutCommandList)
{
	FCommonOutlinerMode::BindCommands(OutCommandList);
	
	CommandList->MapAction(		
		FVariablesOutlinerCommands::Get().CreateSharedVariablesAssets,
		FExecuteAction::CreateRaw(this, &FVariablesOutlinerMode::CreateSharedVariablesAssets),
		FCanExecuteAction::CreateRaw(this, &FVariablesOutlinerMode::CanCreateSharedVariablesAssets),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateRaw(this, &FVariablesOutlinerMode::CanCreateSharedVariablesAssets)
	);
}

TUniquePtr<ISceneOutlinerHierarchy> FVariablesOutlinerMode::CreateHierarchy()
{
	return MakeUnique<FVariablesOutlinerHierarchy>(this);
}

void FVariablesOutlinerMode::Rename() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();

		if (ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
		{
			SceneOutliner->SetPendingRenameItem(ItemToRename);
			SceneOutliner->ScrollItemIntoView(ItemToRename);
		}
	}
}

bool FVariablesOutlinerMode::CanRename() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();
		return ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract();
	}
	return false;
}

void FVariablesOutlinerMode::Delete() const
{
	int32 NumEntries = 0;
	TMap<UUAFRigVMAssetEditorData*, TArray<UUAFRigVMAssetEntry*>> EntriesToDeletePerAsset;
	
	TMap<UUAFRigVMAssetEditorData*, TArray<FString>> CategoryPathsToDeletePerAsset;
	
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();
	for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
	{
		if (const FVariablesOutlinerEntryItem* VariablesItem = Item->CastTo<FVariablesOutlinerEntryItem>())
		{
			if (!VariablesItem->HasStructOwner())
			{
				UAnimNextVariableEntry* VariableEntry = VariablesItem->WeakEntry.Get();
				const UUAFSharedVariablesEntry* SharedVariablesEntry = VariablesItem->WeakSharedVariablesEntry.Get();
				if(VariableEntry == nullptr || SharedVariablesEntry != nullptr)	// Cant delete variables in other data interfaces
				{
					continue;
				}

				UUAFRigVMAssetEditorData* EditorData = VariableEntry->GetTypedOuter<UUAFRigVMAssetEditorData>();
				if(EditorData == nullptr)
				{
					continue;
				}

				TArray<UUAFRigVMAssetEntry*>& EntriesToDelete = EntriesToDeletePerAsset.FindOrAdd(EditorData);
				EntriesToDelete.Add(VariableEntry);
				NumEntries++;
			}
		}
		else if (const FVariablesOutlinerCategoryItem* CategoryItem = Item->CastTo<FVariablesOutlinerCategoryItem>())
		{
			if (UUAFRigVMAsset* Asset = CategoryItem->WeakOwner.Get())
			{
				UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(Asset);
				CategoryPathsToDeletePerAsset.FindOrAdd(EditorData).Add(CategoryItem->CategoryPath);
				NumEntries++;

				EditorData->ForEachEntryOfType<UAnimNextVariableEntry>([CategoryPath = CategoryItem->CategoryPath, EditorData, &EntriesToDeletePerAsset, &NumEntries, &CategoryPathsToDeletePerAsset](UAnimNextVariableEntry* Entry)
				{
					if (Entry->GetVariableCategory().StartsWith(CategoryPath))
					{
						CategoryPathsToDeletePerAsset.FindOrAdd(EditorData).AddUnique(Entry->GetVariableCategory().GetData());
						EntriesToDeletePerAsset.FindOrAdd(EditorData).Add(Entry);
						NumEntries++;
					}
					
					return true;
				});
			}		
		}
	}

	if(EntriesToDeletePerAsset.Num() || CategoryPathsToDeletePerAsset.Num())
	{
		// [TODO] prompt user when deletion impacts assets not part of the current workspace (local context), as these have a hidden (read unknown) impact
		FScopedTransaction Transaction(FText::Format(LOCTEXT("DeleteVariablesFormat", "Delete {0}|plural(one=item, other=items)"), NumEntries));
		for(const TPair<UUAFRigVMAssetEditorData*, TArray<UUAFRigVMAssetEntry*>>& EntriesPair : EntriesToDeletePerAsset)
		{
			for (UUAFRigVMAssetEntry* Entry : EntriesPair.Value)
			{
				UncookedOnly::FUtils::DeleteVariable(CastChecked<UAnimNextVariableEntry>(Entry), true, true);
			}
		}

		for(const TPair<UUAFRigVMAssetEditorData*, TArray<FString>>& CategoriesPair : CategoryPathsToDeletePerAsset)
		{
			for (const FString& CategoryPath : CategoriesPair.Value)
			{
				CategoriesPair.Key->RemoveCategory(CategoryPath);
			}
		}
	}	
}

bool FVariablesOutlinerMode::CanDelete() const
{
	TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SceneOutliner->GetSelectedItems();

	bool bCanDelete = false;
	for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
	{
		if (const FVariablesOutlinerEntryItem* VariablesItem = Item->CastTo<FVariablesOutlinerEntryItem>())
		{
			if (!VariablesItem->HasStructOwner())
			{
				bCanDelete = true;
			}
		}
		
		if (const FVariablesOutlinerCategoryItem* CategoryItem = Item->CastTo<FVariablesOutlinerCategoryItem>())
		{
			bCanDelete = true;
		}
	}
	
	return bCanDelete;
}

bool RecursivelyFindVariableItem(const FVariablesOutlinerCategoryItem& Category)
{	
	const TSet<TWeakPtr<ISceneOutlinerTreeItem>>& Children = Category.GetChildren();
	for (const TWeakPtr<ISceneOutlinerTreeItem>& ChildItem : Children)
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> SharedChild = ChildItem.Pin())
		{
			if (SharedChild->IsA<FVariablesOutlinerEntryItem>())
			{
				return true;
			}
			
			if (FVariablesOutlinerCategoryItem* ChildCategoryItem = SharedChild->CastTo<FVariablesOutlinerCategoryItem>())
			{
				if (RecursivelyFindVariableItem(*ChildCategoryItem))
				{
					return true;
				}
			}
		}
	}
	
	return false;
}

bool FVariablesOutlinerMode::CanCopy() const
{
	/**
	 *	- Variable(s) from a graph, module of shared variables. (from multiple assets at the same time)
	 *	- Entire category (by category selection only)
	 *	- Variables from structs
	 */
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	const bool bSelectionContainsVariable = Selection.Has<FVariablesOutlinerEntryItem>();
	const bool bSelectionContainsCategory = Selection.Has<FVariablesOutlinerCategoryItem>();	
	const bool bSelectedCategoryContainsVariables = bSelectionContainsCategory && [&Selection]()-> bool
	{
		bool bContainsVariable = false;
		Selection.ForEachItem<FVariablesOutlinerCategoryItem>([&bContainsVariable](const FVariablesOutlinerCategoryItem& Category)
		{
			bContainsVariable |= RecursivelyFindVariableItem(Category);
			return !bContainsVariable;
		});
		
		return bContainsVariable;
	}();
	
	const bool bSelectionContainsAssetItem = Selection.Has<FOutlinerAssetItem>();
	const bool bSelectionContainsStructItem = Selection.Has<FVariablesOutlinerStructSharedVariablesItem>();
	
	return !bSelectionContainsAssetItem && !bSelectionContainsStructItem && (bSelectionContainsVariable || (bSelectionContainsCategory && bSelectedCategoryContainsVariables));
}

void FVariablesOutlinerMode::Copy() const
{
	FVariablesOutlinerClipboardData ClipboardData;

	auto CopyVariableEntry = [&ClipboardData](const UAnimNextVariableEntry* Entry, const UUAFRigVMAssetEditorData* EditorData)
	{
		FVariableClipboardData VariableData;
		VariableData.SoftSourceObjectPath = EditorData;

		VariableData.ParameterName = Entry->GetVariableName();
		VariableData.Type = Entry->GetType();
		if (Entry->Binding.IsValid())
		{
			VariableData.Binding = Entry->Binding;
		}
		VariableData.Comment = Entry->Comment;
		VariableData.Category = Entry->Category;
		VariableData.Access = Entry->GetExportAccessSpecifier();

		Entry->GetDefaultValueString(VariableData.DefaultValue);

		ClipboardData.Variables.AddUnique(VariableData);
	};

	auto CopyStructProperty = [&ClipboardData](const FProperty* Property)
	{		
		if (const UScriptStruct* Struct = Property->GetOwner<UScriptStruct>())
		{
			FVariableClipboardData VariableData;
			VariableData.SoftSourceObjectPath = Struct;
			
			FStructOnScope DefaultStruct;
			DefaultStruct.Initialize(Struct);
			VariableData.ParameterName = Property->GetFName();
			VariableData.Type = FAnimNextParamType::FromProperty(Property);			
			VariableData.Access = EAnimNextExportAccessSpecifier::Public;

			Property->ExportTextItem_InContainer(VariableData.DefaultValue, DefaultStruct.GetStructMemory(), nullptr, nullptr, 0);
			
			ClipboardData.Variables.AddUnique(VariableData);
		}
	};
	
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	Selection.ForEachItem<FVariablesOutlinerCategoryItem>([&CopyVariableEntry, &ClipboardData](const FVariablesOutlinerCategoryItem& SelectedItem)
	{
		const FString& CategoryPath = SelectedItem.CategoryPath;
		if (UUAFRigVMAsset* Asset = SelectedItem.WeakOwner.Get())
		{
			if (UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(Asset))
			{
				EditorData->ForEachEntryOfType<UAnimNextVariableEntry>([&CopyVariableEntry, &CategoryPath, &EditorData](const UAnimNextVariableEntry* VariableEntry)
				{
					// Category is stored as set of concatenated (sub)categories, so this will cover _all_ variables
					if (VariableEntry->GetVariableCategory().StartsWith(CategoryPath))
					{
						CopyVariableEntry(VariableEntry, EditorData);
					}
					return true;
				});
			}

			ClipboardData.bCategorySelectedForOperation = true;
		}
	});

	Selection.ForEachItem<FVariablesOutlinerEntryItem>([&CopyVariableEntry, &CopyStructProperty](const FVariablesOutlinerEntryItem& SelectedItem)
	{
		if (UAnimNextVariableEntry* Entry = SelectedItem.WeakEntry.Get())
		{
			if (UUAFRigVMAssetEditorData* EditorData = CastChecked<UUAFRigVMAssetEditorData>(Entry->GetOuter()))
			{
				CopyVariableEntry(Entry, EditorData);
			}
		}
		else if (SelectedItem.bStructOwner)
		{
			if (const FProperty* Property = SelectedItem.PropertyPath.Get())
			{
				CopyStructProperty(Property);
			}
		}
	});
	
	ensure(ClipboardData.Variables.Num());
	{		
		FString ClipboardDataString;
		FVariablesOutlinerClipboardData::StaticStruct()->ExportText(ClipboardDataString, &ClipboardData, nullptr, nullptr, 0, nullptr);	
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardDataString);
	}
}

bool FVariablesOutlinerMode::CanCut() const
{
	/**
	 *	- Variable(s) from a graph, module of shared variables. (from multiple assets at the same time?) BUT not from Structs
	 *	- Entire category (by category selection only)
	*/
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	const bool bSelectionContainsVariable = Selection.Has<FVariablesOutlinerEntryItem>();
	const bool bSelectionContainsCategory = Selection.Has<FVariablesOutlinerCategoryItem>();
	const bool bSelectedCategoryContainsVariables = bSelectionContainsCategory && [&Selection]()-> bool
	{
		bool bContainsVariable = false;
		Selection.ForEachItem<FVariablesOutlinerCategoryItem>([&bContainsVariable](const FVariablesOutlinerCategoryItem& Category)
		{
			bContainsVariable |= RecursivelyFindVariableItem(Category);
			return !bContainsVariable;
		});
	
		return bContainsVariable;
	}();
	const bool bSelectionContainsAssetItem = Selection.Has<FOutlinerAssetItem>();

	bool bSelectionContainsVariableFromStruct = false;
	if (bSelectionContainsVariable)
	{
		TArray<FVariablesOutlinerEntryItem*> SelectedItems;
		Selection.Get<FVariablesOutlinerEntryItem>(SelectedItems);

		for (FVariablesOutlinerEntryItem* Item : SelectedItems)
		{
			bSelectionContainsVariableFromStruct |= Item->HasStructOwner();
		}
	}
	if (bSelectionContainsCategory)
	{
		TArray<FVariablesOutlinerCategoryItem*> SelectedItems;
		Selection.Get<FVariablesOutlinerCategoryItem>(SelectedItems);

		for (FVariablesOutlinerCategoryItem* Item : SelectedItems)
		{
			bSelectionContainsVariableFromStruct |= Item->WeakOwner.Get() == nullptr;
		}
	}

	return !bSelectionContainsAssetItem && !bSelectionContainsVariableFromStruct && (bSelectionContainsVariable || (bSelectionContainsCategory && bSelectedCategoryContainsVariables));
}

void FVariablesOutlinerMode::Cut() const
{
	FVariablesOutlinerClipboardData ClipboardData;
	ClipboardData.bCutOperation = true;

	auto CutVariableEntry = [&ClipboardData](UAnimNextVariableEntry* Entry)
	{
		UUAFRigVMAssetEditorData* EditorData = CastChecked<UUAFRigVMAssetEditorData>(Entry->GetOuter());
		
		FVariableClipboardData& VariableData = ClipboardData.Variables.AddDefaulted_GetRef();
		VariableData.SoftSourceObjectPath = EditorData;

		VariableData.ParameterName = Entry->GetVariableName();
		VariableData.Type = Entry->GetType();
		if (Entry->Binding.IsValid())
		{
			VariableData.Binding = Entry->Binding;
		}
		VariableData.Comment = Entry->Comment;
		VariableData.Category = Entry->Category;
		VariableData.Access = Entry->GetExportAccessSpecifier();
		Entry->GetDefaultValueString(VariableData.DefaultValue);

		EditorData->RemoveEntry(Entry);
	};

	FScopedTransaction Transaction(LOCTEXT("CutVariables", "Cut Variable(s)"));
	TArray<UAnimNextVariableEntry*> EntriesToCut;
	TArray<UUAFRigVMAssetEditorData*> EditorDatas;
	
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	Selection.ForEachItem<FVariablesOutlinerCategoryItem>([&CutVariableEntry, &ClipboardData, &EntriesToCut, &EditorDatas](const FVariablesOutlinerCategoryItem& SelectedItem)
	{
		const FString& CategoryPath = SelectedItem.CategoryPath;
		if (UUAFRigVMAsset* Asset = SelectedItem.WeakOwner.Get())
		{
			if (UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData(Asset))
			{
				EditorData->ForEachEntryOfType<UAnimNextVariableEntry>([&CutVariableEntry, &CategoryPath, &EntriesToCut](UAnimNextVariableEntry* VariableEntry)
				{
					// Category is stored as set of concatenated (sub)categories, so this will cover _all_ variables
					if (VariableEntry->GetVariableCategory().StartsWith(CategoryPath))
					{
						EntriesToCut.Add(VariableEntry);
					}
					return true;
				});

				EditorDatas.AddUnique(EditorData);
				EditorData->RemoveCategory(CategoryPath);
			}

			ClipboardData.bCategorySelectedForOperation = true;
		}
	});
	
	
	TMap<UUAFRigVMAssetEditorData*, TArray<FString>> CategoriesToCheck;
	Selection.ForEachItem<FVariablesOutlinerEntryItem>([&EntriesToCut, &EditorDatas, &CategoriesToCheck](const FVariablesOutlinerEntryItem& SelectedItem)
	{
		if (UAnimNextVariableEntry* Entry = SelectedItem.WeakEntry.Get())
		{
			const FStringView Category = Entry->GetVariableCategory();
			if (!Category.IsEmpty())
			{
				CategoriesToCheck.FindOrAdd(UncookedOnly::FUtils::GetEditorData(SelectedItem.WeakOwner.Get())).Add(FString(Category));
			}
		
			EntriesToCut.Add(Entry);
			EditorDatas.AddUnique(Entry->GetTypedOuter<UUAFRigVMAssetEditorData>());
		}
	});

	TArray<TUniquePtr<FRigVMControllerCompileBracketScope>> RigVMScopes;
	for (UUAFRigVMAssetEditorData* EditorData : EditorDatas)
	{
		// Standalone SharedVariableAssets don't have a graph, and thus no controller
		if (URigVMController* Controller = EditorData->GetOrCreateController())
		{
			RigVMScopes.Add(MakeUnique<FRigVMControllerCompileBracketScope>(Controller));
		}
	}

	for (UAnimNextVariableEntry* VariableEntry : EntriesToCut)
	{
		CutVariableEntry(VariableEntry);
	}
	
	// Handle categories for cut variables (as the might end up empty)
	for (TPair<UUAFRigVMAssetEditorData*, TArray<FString>> Pair : CategoriesToCheck)
	{	
		TArray<FString>& Categories = Pair.Value;
		// Remove all non-empty categories
		Pair.Key->ForEachEntryOfType<UAnimNextVariableEntry>([&Categories](UAnimNextVariableEntry* VariableEntry)
		{
			Categories.RemoveAll([CategoryPath = VariableEntry->GetVariableCategory()](const FString& PathToCheck) { return PathToCheck == CategoryPath; });
			return true;
		});
		
		// Now remove all empty categories
		for (const FString& CategoryPath : Categories)
		{
			Pair.Key->RemoveCategory(CategoryPath);
		}
	}
	
	ensure(ClipboardData.Variables.Num());
	{
		FString ClipboardDataString;
		FVariablesOutlinerClipboardData::StaticStruct()->ExportText(ClipboardDataString, &ClipboardData, nullptr, nullptr, 0, nullptr);	
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardDataString);
	}
}

void FVariablesOutlinerMode::CheckOverlappingVariableNamesRecursively(const UUAFRigVMAssetEditorData* EditorData, const TArray<FName>& BaseVariableNames, TMap<FName, FSoftObjectPath>& OutOverlappingVariableNames) const
{
	EditorData->ForEachEntryOfType<UAnimNextVariableEntry>([&BaseVariableNames, &OutOverlappingVariableNames, &EditorData](const UAnimNextVariableEntry* Entry)
	{
		if (BaseVariableNames.Contains(Entry->GetVariableName()))
		{
			OutOverlappingVariableNames.Add(Entry->GetVariableName(), EditorData->GetOuter());
		}
		
		return true;
	});	
	
	EditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>([this, &BaseVariableNames, &OutOverlappingVariableNames](const UUAFSharedVariablesEntry* Entry)
	{
		if (const UUAFSharedVariables* SharedVariables = Entry->Asset.Get())
		{
			if (const UUAFRigVMAssetEditorData* SharedVariablesEditorData = UncookedOnly::FUtils::GetEditorData<const UUAFRigVMAssetEditorData>(SharedVariables))
			{
				CheckOverlappingVariableNamesRecursively(SharedVariablesEditorData, BaseVariableNames, OutOverlappingVariableNames);
			}
		}
		
		return true;
	});
}

bool FVariablesOutlinerMode::CanPaste() const
{
	/**
	 *	- Asset (not a struct) selected as target
	 *	- (Sub)Category selected as target
	 */
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	const bool bSelectionIsCategory = Selection.Has<FVariablesOutlinerCategoryItem>();
	const bool bSelectionIsAsset = Selection.Has<FOutlinerAssetItem>();

	UUAFRigVMAsset* TargetAsset = nullptr;
	if (bSelectionIsCategory)
	{
		TArray<FVariablesOutlinerCategoryItem*> SelectedItems;
		Selection.Get<FVariablesOutlinerCategoryItem>(SelectedItems);

		if (SelectedItems.Num() == 1)
		{
			const FVariablesOutlinerCategoryItem& SelectedCategory = *SelectedItems[0];
			TargetAsset = SelectedCategory.WeakOwner.Get();
		}
	}
	else
	{
		TArray<FOutlinerAssetItem*> SelectedItems;
		Selection.Get<FOutlinerAssetItem>(SelectedItems);

		// This should have been enforced by CanPaste
		if (SelectedItems.Num() == 1)
		{
			const FOutlinerAssetItem& SelectedAsset = *SelectedItems[0];
			TargetAsset = SelectedAsset.SoftAsset.Get();
		}
	}

	FVariablesOutlinerClipboardData ClipboardData;
	if (TargetAsset)
	{
		FString ClipboardDataString;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardDataString);

		FVariablesOutlinerClipboardData::StaticStruct()->ImportText(*ClipboardDataString, &ClipboardData, nullptr, 0, nullptr, FVariablesOutlinerClipboardData::StaticStruct()->GetName());
	}

	const bool bClipboardHasValidPasteData = ClipboardData.Variables.Num() != 0;
	return bClipboardHasValidPasteData;
}

void FVariablesOutlinerMode::Paste() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	const bool bSelectionIsCategory = Selection.Has<FVariablesOutlinerCategoryItem>();
	const bool bSelectionIsAsset = Selection.Has<FOutlinerAssetItem>();

	const FVariablesOutlinerCategoryItem* SelectedCategoryPtr = nullptr;

	UUAFRigVMAsset* TargetAsset = nullptr;
	if (bSelectionIsCategory)
	{
		TArray<FVariablesOutlinerCategoryItem*> SelectedItems;
		Selection.Get<FVariablesOutlinerCategoryItem>(SelectedItems);

		// This should have been enforced by CanPaste
		check(SelectedItems.Num() == 1);

		SelectedCategoryPtr = SelectedItems[0];
		TargetAsset = SelectedCategoryPtr->WeakOwner.Get();
	}
	else if (bSelectionIsAsset)
	{
		TArray<FOutlinerAssetItem*> SelectedItems;
		Selection.Get<FOutlinerAssetItem>(SelectedItems);

		// This should have been enforced by CanPaste
		check(SelectedItems.Num() == 1);

		const FOutlinerAssetItem& SelectedAsset = *SelectedItems[0];
		TargetAsset = SelectedAsset.SoftAsset.Get();
	}

	if (TargetAsset)
	{
		FString ClipboardDataString;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardDataString);

		FVariablesOutlinerClipboardData ClipboardData;

		FVariablesOutlinerClipboardData::StaticStruct()->ImportText(*ClipboardDataString, &ClipboardData, nullptr, 0, nullptr, FVariablesOutlinerClipboardData::StaticStruct()->GetName());
		
		// Verify that to-be-pasted variables don't overlap with any existing within the asset, or its upward dependencies
		{
			TArray<FName> PasteVariableNames;
			Algo::Transform(ClipboardData.Variables, PasteVariableNames, [](const FVariableClipboardData& VariablePair)
			{
				return VariablePair.ParameterName;
			});
		
			TMap<FName, FSoftObjectPath> OverlappingVariableNames;
			UUAFRigVMAssetEditorData* TargetEditorData = UncookedOnly::FUtils::GetEditorData(TargetAsset);
			CheckOverlappingVariableNamesRecursively(TargetEditorData, PasteVariableNames, OverlappingVariableNames);
			
			if (OverlappingVariableNames.Num())
			{
				FString OverlappingVariableNameInfo;
				for (const TPair<FName, FSoftObjectPath>& Pair : OverlappingVariableNames)
				{
					if (OverlappingVariableNameInfo.Len())
					{
						OverlappingVariableNameInfo.Append(TEXT("\n"));
					}
					OverlappingVariableNameInfo.Append(FString::Printf(TEXT("%s: %s"), *Pair.Key.ToString(), *Pair.Value.GetAssetName()));
				}

				const FText NotificationText = FText::Format(LOCTEXT("FailedPasteOverlappingNames", "Asset or references already contain Variable Name(s):\n{0}"), FText::FromString(OverlappingVariableNameInfo));
				
				FNotificationInfo Notification(NotificationText);
				Notification.ExpireDuration = 5.0f;
				FSlateNotificationManager::Get().AddNotification(Notification);	

				return;
			}
		}

		UUAFRigVMAssetEditorData* TargetEditorData = UncookedOnly::FUtils::GetEditorData(TargetAsset);
		if (TargetEditorData && ClipboardData.Variables.Num())
		{
			FScopedTransaction Transaction(LOCTEXT("PasteVariables", "Paste Variable(s)"));
			
			// Standalone SharedVariableAssets don't have a graph, and thus no controller
			TUniquePtr<FRigVMControllerCompileBracketScope> Scope;
			if (URigVMController* Controller = TargetEditorData->GetOrCreateController())
			{
				Scope = MakeUnique<FRigVMControllerCompileBracketScope>(Controller);
			}
			
			TargetEditorData->Modify();

			const bool bShouldRetainCopiedCategories = ClipboardData.bCategorySelectedForOperation;
			
			for (FVariableClipboardData& VariableData : ClipboardData.Variables)
			{
				const FSoftObjectPath& SourceObjectPath = VariableData.SoftSourceObjectPath;
				UAnimNextVariableEntry* NewVariable = TargetEditorData->AddVariable(VariableData.ParameterName, VariableData.Type, VariableData.DefaultValue);
				NewVariable->SetExportAccessSpecifier(VariableData.Access);

				// In case the user explicitly selected categories when copy/cutting
				if (bShouldRetainCopiedCategories && !VariableData.Category.IsEmpty())
				{
					// Maintain the categories in case of pasting on an asset
					if (SelectedCategoryPtr == nullptr)
					{
						TargetEditorData->AddCategory(VariableData.Category);
						NewVariable->SetVariableCategory(VariableData.Category);
					}
					// If pasting on a category (make the pasted category a child category)
					else
					{
						const FString BaseCategoryPath = SelectedCategoryPtr->CategoryPath;
						const FString NewCategoryPath = BaseCategoryPath + TEXT("|") + VariableData.Category;

						TargetEditorData->AddCategory(NewCategoryPath);
						NewVariable->SetVariableCategory(NewCategoryPath);
					}
				}
				// Otherwise, in case the user explicitly selected a category to paste in - set it as the category for all pasted variables
				else if (bSelectionIsCategory && SelectedCategoryPtr)
				{
					NewVariable->SetVariableCategory(SelectedCategoryPtr->CategoryPath);
				}

				if(VariableData.Binding.IsValid())
				{
					NewVariable->SetBinding(MoveTemp(VariableData.Binding.BindingData));
				}

				NewVariable->Comment = VariableData.Comment;

				if (ClipboardData.bCutOperation)
				{
					if (UUAFRigVMAssetEditorData* CurrentEditorData = Cast<UUAFRigVMAssetEditorData>(SourceObjectPath.ResolveObject()))
					{
						UUAFRigVMAssetEditorData* NewEditorData = TargetEditorData;
						
						const FAnimNextSoftVariableReference FindReference = FAnimNextSoftVariableReference::FromName(VariableData.ParameterName, UncookedOnly::FUtils::GetAsset(CurrentEditorData));
						const FAnimNextSoftVariableReference ReplaceReference = FAnimNextSoftVariableReference::FromName(VariableData.ParameterName, UncookedOnly::FUtils::GetAsset(NewEditorData));

						UncookedOnly::FUtils::ReplaceVariableReferencesAcrossProject(FindReference, ReplaceReference);
					}
				}
			}
		}
	}
}


void FVariablesOutlinerMode::AddCategory(TWeakObjectPtr<UUAFRigVMAssetEditorData> InWeakEditorData) const
{
	if (UUAFRigVMAssetEditorData* EditorData = InWeakEditorData.Get())
	{
		auto GetCategoryName = [&EditorData]() -> FString
		{
			auto NameExists = [ExistingNames = EditorData->VariableAndFunctionCategories](FString InName)
			{
				for(FString AdditionalName : ExistingNames)
				{
					if(AdditionalName == InName)
					{
						return true;
					}
				}

				return false;
			};
			
			const FString BaseCategoryName(TEXT("Category"));

			if(!NameExists(BaseCategoryName))
			{
				// Early out - name is valid
				return BaseCategoryName;
			}

			int32 PostFixIndex = 0;
			TStringBuilder<128> StringBuilder;
			while(true)
			{
				StringBuilder.Reset();
				StringBuilder.Append(BaseCategoryName);
				StringBuilder.Appendf(TEXT("_%d"), PostFixIndex++);

				const FString TestString(StringBuilder.ToString()); 
				if(!NameExists(TestString))
				{
					return TestString;
				}
			}
		};

		FScopedTransaction Transaction(LOCTEXT("AddNewCategory", "Adding New Category"));
		const FString NewCategoryName = GetCategoryName();
		check(EditorData->AddCategory(NewCategoryName));
		
		if (SceneOutliner)
		{
			const FSoftObjectPath OwnerPath = EditorData->GetTypedOuter<UUAFRigVMAsset>();
			const FSceneOutlinerTreeItemID CategoryID = HashCombine(GetTypeHash(OwnerPath), GetTypeHash(NewCategoryName));
			SceneOutliner->OnItemAdded(CategoryID, SceneOutliner::ENewItemAction::Select | SceneOutliner::ENewItemAction::Rename);
		}
	}
}

void FVariablesOutlinerMode::Duplicate() const
{	
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num())
	{		
		FScopedTransaction Transaction(FText::Format(LOCTEXT("DuplicateVariableFormat", "Duplicate {0}|plural(one=variable, other=variables)"), Selection.Num()));

		Selection.ForEachItem<FVariablesOutlinerEntryItem>([](const FVariablesOutlinerEntryItem& EntryItem)
		{
			if (UAnimNextVariableEntry* VariableEntry = EntryItem.WeakEntry.Get())
			{
				if (UUAFRigVMAssetEditorData* EditorData = VariableEntry->GetTypedOuter<UUAFRigVMAssetEditorData>())
				{	
					const FName VariableName = UncookedOnly::FUtils::GetValidVariableName(EditorData, VariableEntry->GetVariableName());
					FString DefaultValueString;
					VariableEntry->GetDefaultValueString(DefaultValueString);
					if (UAnimNextVariableEntry* DuplicatedVariable = EditorData->AddVariable(VariableName, VariableEntry->GetType(), DefaultValueString))
					{
						if (!VariableEntry->GetVariableCategory().IsEmpty())
						{
							DuplicatedVariable->SetVariableCategory(VariableEntry->GetVariableCategory());
						}

						if(VariableEntry->GetBinding().IsValid())
						{
							TInstancedStruct<FAnimNextVariableBindingData> BindingCopy = VariableEntry->Binding.BindingData;
							DuplicatedVariable->SetBinding(MoveTemp(BindingCopy));
						}					
						DuplicatedVariable->SetExportAccessSpecifier(VariableEntry->GetExportAccessSpecifier());
					}
				}
			}
		});
	}
}	

bool FVariablesOutlinerMode::CanDuplicate() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();

	bool bCanDuplicate = true;
	Selection.ForEachItem([&bCanDuplicate](const FSceneOutlinerTreeItemPtr& Item)
	{
		const FVariablesOutlinerEntryItem* EntryItem = Item->CastTo<FVariablesOutlinerEntryItem>();
		if (EntryItem == nullptr || EntryItem->HasStructOwner())
		{
			bCanDuplicate = false;
		}
	});
	
	return bCanDuplicate;
}

void FVariablesOutlinerMode::FindReferences(ESearchScope InSearchScope) const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToFind = Selection.SelectedItems[0].Pin();
		if (const FVariablesOutlinerEntryItem* EntryItem = ItemToFind->CastTo<FVariablesOutlinerEntryItem>())
		{
			if (TSharedPtr<SDockTab> FindAndReplaceTab = WeakWorkspaceEditor.Pin()->GetTabManager()->TryInvokeTab(FTabId(UAF::Editor::FindAndReplaceTabName)))
			{
				TSharedRef<IAnimAssetFindReplace> FindAndReplace = StaticCastSharedRef<IAnimAssetFindReplace>(FindAndReplaceTab->GetContent());
				FindAndReplace->SetCurrentProcessor(UAnimNextAssetFindReplaceVariables::StaticClass());
				UAnimNextAssetFindReplaceVariables* AnimNextFindAndReplaceProcessor = FindAndReplace->GetProcessor<UAnimNextAssetFindReplaceVariables>();

				if (TSharedPtr<Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin())
				{
					AnimNextFindAndReplaceProcessor->SetWorkspaceEditor(SharedWorkspaceEditor.ToSharedRef());
				}
				
				UAnimNextVariableEntry* VariableEntry = EntryItem->WeakEntry.Get();
				AnimNextFindAndReplaceProcessor->SetSearchScope(InSearchScope);		
				AnimNextFindAndReplaceProcessor->SetFindReferenceFromEntry(VariableEntry);				
			}
		}		
	}
}

bool FVariablesOutlinerMode::CanFindReferences() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToFind = Selection.SelectedItems[0].Pin();
		if (const FVariablesOutlinerEntryItem* EntryItem = ItemToFind->CastTo<FVariablesOutlinerEntryItem>())
		{
			return true;
		}
	}
	
	return false;
}

bool FVariablesOutlinerMode::IsFindReferencesVisible(ESearchScope InSearchScope) const
{
	return CanFindReferences();
}

void FVariablesOutlinerMode::PopulateNewVariableToolMenuEntries(UToolMenu* InMenu, bool bAddSeparator)
{
	const UCommonOutlinerItemMenuContext* OutlinerContext = InMenu->FindContext<UCommonOutlinerItemMenuContext>();
	const UVariablesOutlinerItemMenuContext* MenuContext = InMenu->FindContext<UVariablesOutlinerItemMenuContext>();
	if(OutlinerContext == nullptr || MenuContext == nullptr)
	{
		return;
	}

	TMap<TWeakObjectPtr<UUAFRigVMAssetEditorData>, FString> AssetsAndCategoriesToAddTo;
	if (OutlinerContext->WeakEditorDatas.Num())
	{
		if (UUAFRigVMAssetEditorData* EditorData = OutlinerContext->WeakEditorDatas[0].Get())
		{
			AssetsAndCategoriesToAddTo.Add(EditorData, TEXT(""));
		}
	}
	else
	{
		TSharedPtr<SCommonOutliner> SharedOutliner = OutlinerContext->WeakOutliner.Pin();
		if (SharedOutliner.IsValid())
		{
			TArray<FSceneOutlinerTreeItemPtr> SelectedItems = SharedOutliner->GetSelectedItems();
		
			for(const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
			{
				const FOutlinerAssetItem* AssetItem = Item->CastTo<FOutlinerAssetItem>();
				const FVariablesOutlinerCategoryItem* CategoryItem = Item->CastTo<FVariablesOutlinerCategoryItem>();
				if (AssetItem == nullptr && CategoryItem == nullptr)
				{
					continue;
				}

				UUAFRigVMAsset* Asset = nullptr;
				FString Category;
		
				if (CategoryItem)
				{
					Asset = CategoryItem->WeakOwner.Get();
					Category = CategoryItem->GetDisplayString();
				}
				else
				{
					Asset = AssetItem->SoftAsset.Get();
				}

				if(Asset == nullptr)
				{
					continue;
				}

				UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
				if(EditorData == nullptr)
				{
					continue;
				}

				AssetsAndCategoriesToAddTo.Add(EditorData, Category);
			}
		}
	}
	
	if (AssetsAndCategoriesToAddTo.Num())
	{
		constexpr bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, TSharedPtr<FUICommandList>());

		FToolMenuSection& Section = InMenu->AddSection("Variables", LOCTEXT("VariablesSectionLabel", "Variables"));
		Section.AddMenuEntry("AddNewVariable", LOCTEXT("VariableLabel", "Add Variable"), LOCTEXT("VariableTooltip", "Adds a new Variable"),FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.PlusCircle"),
			FUIAction(FExecuteAction::CreateLambda([WeakOutliner = OutlinerContext->WeakOutliner, AssetsAndCategoriesToAddTo]()
			{
				for (auto& Pair : AssetsAndCategoriesToAddTo)
				{
					if (UUAFRigVMAssetEditorData* EditorData = Pair.Key.Get())
                    {
                        UAnimNextVariableSettings* Settings = GetMutableDefault<UAnimNextVariableSettings>();
                        FScopedTransaction Transaction(LOCTEXT("AddVariable", "Add Variable"));

                        UncookedOnly::FCompilationScope CompileScope(LOCTEXT("AddVariable", "Add Variable"), UncookedOnly::FUtils::GetAsset(EditorData));
                        
                        const FName VariableName = UncookedOnly::FUtils::GetValidVariableName(EditorData, Settings->GetLastVariableName());		

						if (UAnimNextVariableEntry* VariableEntry = EditorData->AddVariable(VariableName, Settings->GetLastVariableType()))
						{
							VariableEntry->SetVariableCategory(Pair.Value);
							Settings->SetLastVariableName(VariableName);

							// Prompt user to rename added variable in outliner
							TSharedPtr<SCommonOutliner> SharedOutliner = WeakOutliner.Pin();
							if (SharedOutliner.IsValid())
							{
								// FVariablesOutlinerEntryItem::SharedVariablesEntry is only populated for Struct based SharedVariables
								const UUAFSharedVariablesEntry* SharedVariablesEntry = nullptr;
								FSceneOutlinerTreeItemID EntryID = HashCombine(GetTypeHash(VariableEntry), GetTypeHash(SharedVariablesEntry));
								SharedOutliner->OnItemAdded(EntryID, SceneOutliner::ENewItemAction::Select | SceneOutliner::ENewItemAction::Rename);
							}
						}
                    }
				}
			}))
		);

		if (AssetsAndCategoriesToAddTo.Num() == 1)
		{
			Section.AddSubMenu("AddSharedVariables", LOCTEXT("SharedVariableLabel", "Add Shared Variables"), LOCTEXT("SharedVariableTooltip", "Includes added SharedVariables"),
			FNewMenuDelegate::CreateLambda([AssetsAndCategoriesToAddTo](FMenuBuilder& SubMenuBuilder)
				{
					TArray<FAssetData> ExistingSharedVariableReferences;
					for (auto& Pair : AssetsAndCategoriesToAddTo)
					{
						if (UUAFRigVMAssetEditorData* EditorData = Pair.Key.Get())
						{
							EditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>([&ExistingSharedVariableReferences](const UUAFSharedVariablesEntry* SharedVariablesEntry) -> bool
							{
								if (SharedVariablesEntry->GetType() == EAnimNextSharedVariablesType::Asset)
								{
									ExistingSharedVariableReferences.Add(SharedVariablesEntry->GetAsset());
								}
								return true;
							});
						}
					}

					FAssetPickerConfig AssetPickerConfig;
					AssetPickerConfig.Filter.bRecursiveClasses = true;
					AssetPickerConfig.Filter.ClassPaths.Add(UUAFSharedVariables::StaticClass()->GetClassPathName());
					AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
					AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([AssetsAndCategoriesToAddTo](const FAssetData& InAssetData)
					{
						FSlateApplication::Get().DismissAllMenus();
						if(UUAFSharedVariables* SharedVariables = Cast<UUAFSharedVariables>(InAssetData.GetAsset()))
						{							
							FScopedTransaction Transaction(LOCTEXT("AddSharedVariable", "Add Shared Variables"));
							for (auto& Pair : AssetsAndCategoriesToAddTo)
							{
								if (UUAFRigVMAssetEditorData* EditorData = Pair.Key.Get())
								{
									EditorData->AddSharedVariables(SharedVariables);
								}
							}
						}
					});
					AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([ExistingSharedVariableReferences](const FAssetData& InAssetData)
					{
						FAnimNextAssetRegistryExports Exports;
						UncookedOnly::FUtils::GetExportedVariablesForAsset(InAssetData, Exports);
						if(Exports.Exports.Num() == 0 && InAssetData.GetClass(EResolveClass::Yes) != UUAFSharedVariables::StaticClass())
						{
							return true;
						}

						// Filter out already referenced assets
						return ExistingSharedVariableReferences.ContainsByPredicate([InAssetData](const FAssetData& ExistingAssetData)
						{
							return ExistingAssetData == InAssetData;
						});
					});

					FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
					TSharedPtr<SWidget> Widget = SNew(SBox)
					.WidthOverride(300.0f)
					.HeightOverride(400.0f)
					[
						ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
					];

					SubMenuBuilder.AddWidget(Widget.ToSharedRef(), FText::GetEmpty(), true);
				}),
				false,
				FSlateIcon(FUAFStyle::Get().GetStyleSetName(), "ClassIcon.AnimNextSharedVariables")
			);


			Section.AddSubMenu("AddNativeSharedVariables",LOCTEXT("NativeSharedVariableLabel", "Add Native Shared Variables"), LOCTEXT("NativeSharedVariableTooltip", "Includes added Native SharedVariables"),
		FNewMenuDelegate::CreateLambda([AssetsAndCategoriesToAddTo](FMenuBuilder& SubMenuBuilder)
				{

					TArray<FSoftObjectPath> ExistingSharedVariableStructPaths;
					TArray<FAssetData> ExistingSharedVariableReferences;
					for (auto& Pair : AssetsAndCategoriesToAddTo)
					{
						if (UUAFRigVMAssetEditorData* EditorData = Pair.Key.Get())
						{
							EditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>([&ExistingSharedVariableStructPaths](const UUAFSharedVariablesEntry* SharedVariablesEntry) -> bool
							{
								if (SharedVariablesEntry->GetType() == EAnimNextSharedVariablesType::Struct)
								{
									ExistingSharedVariableStructPaths.Add(SharedVariablesEntry->GetObjectPath());
								}
								return true;
							});
						}
					}
			
					class FStructFilter : public IStructViewerFilter
					{
					public:

						FStructFilter(const TArray<FSoftObjectPath>& InFilteredStructPaths) : FilteredStructPaths(InFilteredStructPaths) {}
						
						virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override
						{
							if (InStruct->IsA<UUserDefinedStruct>())
							{
								return false;
							}

							if (!InStruct->HasMetaData(TEXT("BlueprintType")) || InStruct->HasMetaData(TEXT("Hidden")))
							{
								return false;
							}

							if (FilteredStructPaths.Contains(InStruct))
							{
								return false;
							}

							return true;
						}

						virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<class FStructViewerFilterFuncs> InFilterFuncs) override
						{
							return false;
						};

						TArray<FSoftObjectPath> FilteredStructPaths;
					};

					FStructViewerModule& StructViewerModule = FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer");

					FStructViewerInitializationOptions Options;
					Options.Mode = EStructViewerMode::StructPicker;
					Options.StructFilter = MakeShared<FStructFilter>(ExistingSharedVariableStructPaths);

					TSharedPtr<SWidget> Widget = SNew(SBox)
						.WidthOverride(300.0f)
						.HeightOverride(400.0f)
						[
							StructViewerModule.CreateStructViewer(Options, FOnStructPicked::CreateLambda([AssetsAndCategoriesToAddTo](const UScriptStruct* InStruct)
							{
								FSlateApplication::Get().DismissAllMenus();

								if(InStruct)
								{
									FScopedTransaction Transaction(LOCTEXT("AddNativeSharedVariable", "Add Native Shared Variables"));
									for (auto& Pair : AssetsAndCategoriesToAddTo)
									{
										if (UUAFRigVMAssetEditorData* EditorData = Pair.Key.Get())
										{									
											EditorData->AddSharedVariablesStruct(InStruct);
										}
									}
								}
							}))
						];

					SubMenuBuilder.AddWidget(Widget.ToSharedRef(), FText::GetEmpty(), true);
				}),
				false,
				FSlateIconFinder::FindIconForClass(UUserDefinedStruct::StaticClass())
			);

						// Note we refer to them as control rig shared variables even though technically any rig vm asset will display here
			Section.AddSubMenu("AddRigVMAssetSharedVariables",LOCTEXT("RigVMAssetSharedVariableLabel", "Add Control Rig Shared Variables"), LOCTEXT("RigVMAssetSharedVariableTooltip", "Includes added Control Rig Asset SharedVariables"),
		FNewMenuDelegate::CreateLambda([AssetsAndCategoriesToAddTo](FMenuBuilder& SubMenuBuilder)
				{

					TArray<FSoftObjectPath> ExistingSharedVariableRigVMPaths;
					for (auto& Pair : AssetsAndCategoriesToAddTo)
					{
						if (UUAFRigVMAssetEditorData* EditorData = Pair.Key.Get())
						{
							EditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>([&ExistingSharedVariableRigVMPaths](const UUAFSharedVariablesEntry* SharedVariablesEntry) -> bool
							{
								if (SharedVariablesEntry->GetType() == EAnimNextSharedVariablesType::RigVMAsset)
								{
									ExistingSharedVariableRigVMPaths.Add(SharedVariablesEntry->GetObjectPath());
								}
								return true;
							});
						}
					}
			
					FAssetPickerConfig AssetPickerConfig;
					AssetPickerConfig.Filter.bRecursiveClasses = true;
					AssetPickerConfig.Filter.ClassPaths.Add(URigVMEditorAssetInterface::StaticClass()->GetClassPathName());
					AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
					AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([AssetsAndCategoriesToAddTo](const FAssetData& InAssetData)
					{
						FSlateApplication::Get().DismissAllMenus();
						if(const IRigVMEditorAssetInterface* RigVMEditorAsset = Cast<IRigVMEditorAssetInterface>(InAssetData.GetAsset()))
						{
							if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = RigVMEditorAsset->GetRuntimeAssetInterface())
							{
								FScopedTransaction Transaction(LOCTEXT("AddSharedVariable", "Add Shared Variables"));
								for (auto& Pair : AssetsAndCategoriesToAddTo)
								{
									if (UUAFRigVMAssetEditorData* EditorData = Pair.Key.Get())
									{
										EditorData->AddSharedVariablesRigVMAsset(RigVMEditorAsset->GetRuntimeAssetInterface().GetInterface());
									}
								}
							}
						}
					});
					AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([ExistingSharedVariableRigVMPaths](const FAssetData& InAssetData)
					{
						FSoftObjectPath AssetPath = InAssetData.GetSoftObjectPath();
						if (InAssetData.IsInstanceOf<UBlueprint>())
						{
							// We want to check against generated classes for blueprints
							AssetPath.SetPath(AssetPath.GetAssetPath().ToString() + TEXT("_C"));
						}
							
						// Filter out already referenced assets
						return ExistingSharedVariableRigVMPaths.ContainsByPredicate([AssetPath](const FSoftObjectPath& ExistingRigVMAssetPath)
						{
							return ExistingRigVMAssetPath == AssetPath;
						});
					});

					FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
					TSharedPtr<SWidget> Widget = SNew(SBox)
					.WidthOverride(300.0f)
					.HeightOverride(400.0f)
					[
						ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
					];

					SubMenuBuilder.AddWidget(Widget.ToSharedRef(), FText::GetEmpty(), true);
				}),
				false,
				FSlateIcon(FUAFStyle::Get().GetStyleSetName(), "ClassIcon.AnimNextSharedVariables")
			);
		}

		if (bAddSeparator)
		{
			Section.AddSeparator("PostAddVariableSeparator");
		}
	}
}

bool FVariablesOutlinerMode::IsAsset() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();	
	return Selection.SelectedItems.ContainsByPredicate([](const TWeakPtr<ISceneOutlinerTreeItem>& WeakItem) { return WeakItem.IsValid() && WeakItem.Pin()->IsA<FOutlinerAssetItem>(); });	
}

void FVariablesOutlinerMode::CreateSharedVariablesAssets() const
{
	ensure(CanCreateSharedVariablesAssets());
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();

	TArray<UAnimNextVariableEntry*> EntriesToMove;
	Selection.ForEachItem<FVariablesOutlinerEntryItem>([&EntriesToMove](const FVariablesOutlinerEntryItem& VariableItem)
	{
		if (UAnimNextVariableEntry* VariableEntry = VariableItem.WeakEntry.Get())
		{
			EntriesToMove.Add(VariableEntry);
		}
	});

	TArray<UUAFRigVMAssetEditorData*> UniqueOuterEditorDatas;
	
	TArray<FOutlinerAssetItem*> AssetItems;
	Selection.Get<FOutlinerAssetItem>(AssetItems);
	if (AssetItems.Num() == 1)
	{
		UUAFRigVMAsset* Asset = AssetItems[0]->SoftAsset.LoadSynchronous();
		UniqueOuterEditorDatas.AddUnique(UncookedOnly::FUtils::GetEditorData(Asset));
	}

	// Prompt user for asset path
	FString SaveObjectPath;
	{			
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		{
			// Populate default path to match currently focussed assets
			TSharedPtr<Workspace::IWorkspaceEditor> SharedWorkspaceEditor = WeakWorkspaceEditor.Pin();
			if (SharedWorkspaceEditor.IsValid())
			{
				const Workspace::FWorkspaceDocument& Document = SharedWorkspaceEditor->GetFocusedWorkspaceDocument();
				if (UObject* FocussedObject = Document.GetObject())
				{
					SaveAssetDialogConfig.DefaultPath = FPaths::GetPath(FocussedObject->GetPackage()->GetPathName());
				}
			}

			SaveAssetDialogConfig.DefaultAssetName = TEXT("NewSharedVariables");
			SaveAssetDialogConfig.AssetClassNames.Add(UUAFSharedVariables::StaticClass()->GetClassPathName());
			SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
			SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveLiveLinkPresetDialogTitle", "Add new Shared Variables Asset");
		}

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
	}

	if (!SaveObjectPath.IsEmpty())
	{
		const FString PackagePath = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		const FString AssetName = FPaths::GetBaseFilename(PackagePath, true);
		
		const FText FormattedMessage = FText::Format(LOCTEXT("CreateSharedVariablesAsset", "Creating new Shared Variables Asset {0}"), FText::FromString(AssetName));
		FScopedTransaction Transaction(FormattedMessage);			
		
		UPackage* Package = CreatePackage(*PackagePath);
		Package->Modify();
			
		UUAFSharedVariablesFactory* Factory = NewObject<UUAFSharedVariablesFactory>(GetTransientPackage());
		UUAFSharedVariables* NewSharedVariables = CastChecked<UUAFSharedVariables>(Factory->FactoryCreateNew(UUAFSharedVariables::StaticClass(), Package, *AssetName, RF_Public | RF_Standalone | RF_Transactional, nullptr, nullptr, NAME_None));
		check(NewSharedVariables);
		
		for (UAnimNextVariableEntry* Entry : EntriesToMove)
		{
			// Force variable to be public as its being moved to SharedVariables asset
			Entry->SetExportAccessSpecifier(EAnimNextExportAccessSpecifier::Public);

			UniqueOuterEditorDatas.AddUnique(Entry->GetTypedOuter<UUAFRigVMAssetEditorData>());
			UncookedOnly::FUtils::MoveVariableToAsset(Entry, NewSharedVariables);
		}

		for (UUAFRigVMAssetEditorData* OuterEditorData : UniqueOuterEditorDatas)
		{
			OuterEditorData->AddSharedVariables(NewSharedVariables);
		}
		
		// mark asset dirty 
		FAssetRegistryModule::AssetCreated(NewSharedVariables);
		NewSharedVariables->MarkPackageDirty();
	}
}

bool FVariablesOutlinerMode::CanCreateSharedVariablesAssets() const
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	const bool bSelectionOnlyContainsVariables = Selection.Num() == Selection.Num<FVariablesOutlinerEntryItem>();
	const bool bSelectionOnlyContainsSingleAsset = Selection.Num() == 1 && Selection.Num<FOutlinerAssetItem>() == 1;

	bool bSelectionContainsNativeSharedVariablesStructProperty = false;

	// Prevent moving from an assets which would leave it empty (removing all variables)
	TMap<const UUAFRigVMAssetEditorData*, int32> EditorDataToNumRemainingVariables;
	Selection.ForEachItem<FVariablesOutlinerEntryItem>([&EditorDataToNumRemainingVariables, &bSelectionContainsNativeSharedVariablesStructProperty](const FVariablesOutlinerEntryItem& Item)
	{
		if (const UAnimNextVariableEntry* VariableEntry = Item.WeakEntry.Get())
		{
			if (const UUAFRigVMAssetEditorData* EditorData = CastChecked<UUAFRigVMAssetEditorData>(VariableEntry->GetOuter()))
			{
				if (int32* ExistingEntryValue = EditorDataToNumRemainingVariables.Find(EditorData))
				{
					--(*ExistingEntryValue);
				}
				else
				{
					int32 NumVariables = 0;
					EditorData->ForEachEntryOfType<UAnimNextVariableEntry>([&NumVariables](const UAnimNextVariableEntry* Entry)
					{
						++NumVariables;
						return true;
					});

					EditorDataToNumRemainingVariables.Add(EditorData, NumVariables - 1);
				}
			}
		}
		else if (const FSceneOutlinerTreeItemPtr& Parent = Item.GetParent(); Parent->IsA<FVariablesOutlinerStructSharedVariablesItem>())
		{
			bSelectionContainsNativeSharedVariablesStructProperty = true;
			return false;
		}

		return true;
	});

	if (bSelectionContainsNativeSharedVariablesStructProperty)
	{
		return false;
	}

	TArray<int32> NumRemainVariablesArray;
	
	EditorDataToNumRemainingVariables.GenerateValueArray(NumRemainVariablesArray);
	const bool bMoveWillLeaveEmptyAsset = NumRemainVariablesArray.Contains(0);
	
	return (bSelectionOnlyContainsVariables && !bMoveWillLeaveEmptyAsset) || bSelectionOnlyContainsSingleAsset;
}

}

#undef LOCTEXT_NAMESPACE // "FVariablesOutlinerMode"
