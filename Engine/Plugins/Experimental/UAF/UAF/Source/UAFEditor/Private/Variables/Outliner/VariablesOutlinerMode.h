// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextRigVMAssetEditorData.h"
#include "ISceneOutlinerMode.h"
#include "Common/AnimNextAssetFindReplaceVariables.h"
#include "Common/Outliner/CommonOutlinerMode.h"

#include "VariablesOutlinerMode.generated.h"

class UUAFRigVMAssetEditorData;
class FVariablesOutlinerMode;
namespace UE::Workspace
{
	class IWorkspaceEditor;
}

USTRUCT()
struct FVariableClipboardData
{
	GENERATED_BODY()
	
	UPROPERTY()
	FSoftObjectPath SoftSourceObjectPath;

	UPROPERTY()
	EAnimNextExportAccessSpecifier Access = EAnimNextExportAccessSpecifier::Private;

	UPROPERTY()
	FName ParameterName;

	UPROPERTY()
	FAnimNextParamType Type;

	UPROPERTY()
	FAnimNextVariableBinding Binding;

	UPROPERTY()
	FString Comment;

	UPROPERTY()
	FString Category;

	UPROPERTY()
	FString DefaultValue;

	bool operator==(const FVariableClipboardData& Other) const
	{
		return ParameterName == Other.ParameterName
			&& Type == Other.Type
			&& Binding.BindingData == Other.Binding.BindingData
			&& Access == Other.Access
			&& Comment == Other.Comment
			&& Category == Other.Category
			&& SoftSourceObjectPath == Other.SoftSourceObjectPath
			&& DefaultValue == Other.DefaultValue;
	}
};

USTRUCT()
struct FVariablesOutlinerClipboardData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FVariableClipboardData> Variables;

	UPROPERTY()
	bool bCutOperation = false;

	UPROPERTY()
	bool bCategorySelectedForOperation = false;
};

namespace UE::UAF::Editor
{
class FVariablesOutlinerMode : public FCommonOutlinerMode
{
public:
	FVariablesOutlinerMode(SCommonOutliner* InOutliner, const TSharedRef<UE::Workspace::IWorkspaceEditor>& InWorkspaceEditor);
	
	// Begin FCommonOutlinerMode overrides
	virtual void HandleItemSelection(const FSceneOutlinerItemSelection& Selection) const override;
	virtual void RegisterToolMenus() override;
	virtual void InitToolMenuContext(FToolMenuContext& InOutContext) const override;
	virtual void ReorderCategoryItem(TSharedPtr<FOutlinerCategoryItem> EntryItem, TSharedPtr<FOutlinerCategoryItem> BeforeItem) const override;
	virtual void ReorderCategory(const FString& CategoryName, const FString& BeforeCategoryName, UUAFRigVMAsset* Asset) const override;
	virtual void MoveCategoryToAsset(const FString& CategoryPath, UUAFRigVMAsset* FromAsset, UUAFRigVMAsset* ToAsset) const override;
	// End FCommonOutlinerMode overrides

	// Begin ISceneOutlinerMode overrides
	virtual void Rebuild() override;
	virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override;
	virtual void BindCommands(const TSharedRef<FUICommandList>& OutCommandList) override;
	virtual bool UpdateOperationDecorator(const FDragDropEvent& Event, const FSceneOutlinerDragValidationInfo* ValidationInfo) const override;	
	static void PopulateNewVariableToolMenuEntries(UToolMenu* InMenu, bool bAddSeparator);
protected:
	virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;
	// End ISceneOutlinerMode overrides


	// QQ begin/end overrides
	virtual void Rename() const override;
	virtual bool CanRename() const override;

	virtual void Delete() const override;
	virtual bool CanDelete() const override;

	virtual void Copy() const override;
	virtual bool CanCopy() const override;

	virtual void Cut() const override;
	virtual bool CanCut() const override;

	virtual void Paste() const override;
	virtual bool CanPaste() const override;

	void CheckOverlappingVariableNamesRecursively(const UUAFRigVMAssetEditorData* EditorData, const TArray<FName>& BaseVariableNames, TMap<FName, FSoftObjectPath>& OutOverlappingVariableNames) const;
	virtual void AddCategory(TWeakObjectPtr<UUAFRigVMAssetEditorData> InWeakEditorData) const override;

	virtual void Duplicate() const override;
	virtual bool CanDuplicate() const override;

	virtual void FindReferences(ESearchScope InSearchScope) const override;
	virtual bool CanFindReferences() const override;
	virtual bool IsFindReferencesVisible(ESearchScope InSearchScope) const override;

	bool IsAsset() const;

	void CreateSharedVariablesAssets() const;
	bool CanCreateSharedVariablesAssets() const;
private:
	friend class FVariablesOutlinerHierarchy;
};

}
