// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariablesOutlinerRigVMAssetSharedVariablesItem.h"

#include "ISceneOutliner.h"
#include "Styling/SlateColor.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "AnimNextEditorModule.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Common/Outliner/OutlinerDragDropOps.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StarshipCoreStyle.h"
#include "UObject/Package.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "VariablesOutlinerTreeItem"

namespace UE::UAF::Editor
{

const FSceneOutlinerTreeItemType FVariablesOutlinerRigVMAssetSharedVariablesItem::Type(&FOutlinerItem::Type);

class SVariablesOutlinerRigVMSharedVariablesLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SVariablesOutlinerRigVMSharedVariablesLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FVariablesOutlinerRigVMAssetSharedVariablesItem& InTreeItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
		TreeItem = StaticCastSharedRef<FVariablesOutlinerRigVMAssetSharedVariablesItem>(InTreeItem.AsShared());

		FText AssetName;
		if(const IRigVMRuntimeAssetInterface* RigVMAssetInterface = InTreeItem.RigVMAsset.GetInterface())
		{
			AssetName = FText::FromString(RigVMAssetInterface->GetEditorOnlyData()->GetPathName());
		}
		else
		{
			AssetName = LOCTEXT("UnknownAssetName", "Unknown Asset");
		}
		
		ChildSlot
		[
			SNew(SHorizontalBox)
			.ToolTipText(FText::Format(LOCTEXT("ImportedVariablesFormat", "Shared variables from '{0}'"), AssetName))
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 2.0f)
			[
				SNew(SImage)
				.Image(FSlateIconFinder::FindIconBrushForClass(UUserDefinedStruct::StaticClass()))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 2.0f)
			[
				SNew(STextBlock)
				.Font(FStyleFonts::Get().NormalBold)
				.Text(this, &SVariablesOutlinerRigVMSharedVariablesLabel::GetDisplayText)
				.HighlightText(SceneOutliner.GetFilterHighlightText())
				.ColorAndOpacity(this, &SVariablesOutlinerRigVMSharedVariablesLabel::GetForegroundColor)
			]
		];
	}
	
	FText GetDisplayText() const
	{
		if (const TSharedPtr<FVariablesOutlinerRigVMAssetSharedVariablesItem> Item = TreeItem.Pin())
		{
			return FText::FromString(Item->GetDisplayString());
		}
		return FText();
	}

	virtual FSlateColor GetForegroundColor() const override
	{
		TOptional<FLinearColor> BaseColor;
		if (TreeItem.IsValid())
		{
			BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItem.Pin());
		}
		return BaseColor.IsSet() ? BaseColor.GetValue() : FSlateColor::UseForeground();
	}


	TWeakPtr<FVariablesOutlinerRigVMAssetSharedVariablesItem> TreeItem;
	TSharedPtr<SInlineEditableTextBlock> TextBlock;
};

FVariablesOutlinerRigVMAssetSharedVariablesItem::FVariablesOutlinerRigVMAssetSharedVariablesItem(const TScriptInterface<const IRigVMRuntimeAssetInterface>& InRigVMAsset)
: FOutlinerItem(FVariablesOutlinerRigVMAssetSharedVariablesItem::Type, {nullptr, 0})
	, RigVMAsset(InRigVMAsset)
{
}

bool FVariablesOutlinerRigVMAssetSharedVariablesItem::IsValid() const
{
	return RigVMAsset != nullptr;
}

FSceneOutlinerTreeItemID FVariablesOutlinerRigVMAssetSharedVariablesItem::GetID() const
{
	if (RigVMAsset)
	{
		const FSoftObjectPath Path = RigVMAsset.GetObject()->GetPathName();
		return GetTypeHash(Path);
	}
	
	return GetTypeHash(RigVMAsset);
}

FString FVariablesOutlinerRigVMAssetSharedVariablesItem::GetDisplayString() const
{
	if(RigVMAsset == nullptr)
	{
		return FString();
	}

	return RigVMAsset.GetObject()->GetName();
}

TSharedRef<SWidget> FVariablesOutlinerRigVMAssetSharedVariablesItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SVariablesOutlinerRigVMSharedVariablesLabel, *this, Outliner, InRow);
}

FString FVariablesOutlinerRigVMAssetSharedVariablesItem::GetPackageName() const
{
	if(RigVMAsset == nullptr)
	{
		return ISceneOutlinerTreeItem::GetPackageName();
	}

	return RigVMAsset.GetObject()->GetPackage()->GetName();
}

}

#undef LOCTEXT_NAMESPACE // "VariablesOutlinerTreeItem"
