// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"
#include "Common/Outliner/OutlinerEntryItem.h"

class IRigVMRuntimeAssetInterface;
class UUAFSharedVariablesEntry;

namespace UE::UAF::Editor
{

class IWorkspaceOutlinerItemDetails;

struct FVariablesOutlinerRigVMAssetSharedVariablesItem : FOutlinerItem
{
	static const FSceneOutlinerTreeItemType Type;

	FVariablesOutlinerRigVMAssetSharedVariablesItem(const TScriptInterface<const IRigVMRuntimeAssetInterface>& InRigVMAsset);

	// Begin ISceneOutlinerTreeItem overrides
	virtual bool IsValid() const override;
	virtual FSceneOutlinerTreeItemID GetID() const override;
	virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override { return true; }
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	virtual FString GetPackageName() const override;
	// End ISceneOutlinerTreeItem overrides

	TScriptInterface<const IRigVMRuntimeAssetInterface> RigVMAsset;
};

}
