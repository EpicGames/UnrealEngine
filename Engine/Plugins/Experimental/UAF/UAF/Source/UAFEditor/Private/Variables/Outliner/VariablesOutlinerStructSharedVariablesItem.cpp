// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariablesOutlinerStructSharedVariablesItem.h"

#include "ISceneOutliner.h"
#include "Styling/SlateColor.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "AnimNextEditorModule.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Common/Outliner/OutlinerDragDropOps.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StarshipCoreStyle.h"
#include "UObject/Package.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "VariablesOutlinerTreeItem"

namespace UE::UAF::Editor
{

const FSceneOutlinerTreeItemType FVariablesOutlinerStructSharedVariablesItem::Type(&FOutlinerItem::Type);

class SVariablesOutlinerStructSharedVariablesLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SVariablesOutlinerStructSharedVariablesLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FVariablesOutlinerStructSharedVariablesItem& InTreeItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
		TreeItem = StaticCastSharedRef<FVariablesOutlinerStructSharedVariablesItem>(InTreeItem.AsShared());

		FText AssetName;
		if(const UScriptStruct* Struct = InTreeItem.Struct.Get())
		{
			AssetName = FText::FromString(Struct->GetPathName());
		}
		else
		{
			AssetName = LOCTEXT("UnknownAssetName", "Unknown Asset");
		}
		
		ChildSlot
		[
			SNew(SHorizontalBox)
			.ToolTipText(FText::Format(LOCTEXT("ImportedVariablesFormat", "Shared variables from '{0}'"), AssetName))
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 2.0f)
			[
				SNew(SImage)
				.Image(FSlateIconFinder::FindIconBrushForClass(UUserDefinedStruct::StaticClass()))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 2.0f)
			[
				SNew(STextBlock)
				.Font(FStyleFonts::Get().NormalBold)
				.Text(this, &SVariablesOutlinerStructSharedVariablesLabel::GetDisplayText)
				.HighlightText(SceneOutliner.GetFilterHighlightText())
				.ColorAndOpacity(this, &SVariablesOutlinerStructSharedVariablesLabel::GetForegroundColor)
			]
		];
	}
	
	FText GetDisplayText() const
	{
		if (const TSharedPtr<FVariablesOutlinerStructSharedVariablesItem> Item = TreeItem.Pin())
		{
			return FText::FromString(Item->GetDisplayString());
		}
		return FText();
	}

	virtual FSlateColor GetForegroundColor() const override
	{
		TOptional<FLinearColor> BaseColor;
		if (TreeItem.IsValid())
		{
			BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItem.Pin());
		}
		return BaseColor.IsSet() ? BaseColor.GetValue() : FSlateColor::UseForeground();
	}


	TWeakPtr<FVariablesOutlinerStructSharedVariablesItem> TreeItem;
	TSharedPtr<SInlineEditableTextBlock> TextBlock;
};

FVariablesOutlinerStructSharedVariablesItem::FVariablesOutlinerStructSharedVariablesItem(const TObjectPtr<const UScriptStruct> InStruct)
: FOutlinerItem(FVariablesOutlinerStructSharedVariablesItem::Type, {nullptr, 0})
	, Struct(InStruct)
{
}

bool FVariablesOutlinerStructSharedVariablesItem::IsValid() const
{
	return Struct != nullptr;
}

FSceneOutlinerTreeItemID FVariablesOutlinerStructSharedVariablesItem::GetID() const
{
	if (Struct)
	{
		const FSoftObjectPath Path = Struct;
		return GetTypeHash(Path);
	}
	
	return GetTypeHash(Struct);
}

FString FVariablesOutlinerStructSharedVariablesItem::GetDisplayString() const
{
	if(Struct == nullptr)
	{
		return FString();
	}

	return Struct->GetName();
}

TSharedRef<SWidget> FVariablesOutlinerStructSharedVariablesItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SVariablesOutlinerStructSharedVariablesLabel, *this, Outliner, InRow);
}

FString FVariablesOutlinerStructSharedVariablesItem::GetPackageName() const
{
	if(Struct == nullptr)
	{
		return ISceneOutlinerTreeItem::GetPackageName();
	}

	return Struct->GetPackage()->GetName();
}

}

#undef LOCTEXT_NAMESPACE // "VariablesOutlinerTreeItem"
