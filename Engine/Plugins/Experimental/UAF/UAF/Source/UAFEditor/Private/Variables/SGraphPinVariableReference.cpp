// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGraphPinVariableReference.h"
#include "EditorUtils.h"
#include "K2Node_CallFunction.h"
#include "Variables/SVariablePickerCombo.h"
#include "ScopedTransaction.h"
#include "UncookedOnlyUtils.h"
#include "Component/AnimNextComponent.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Param/ParamCompatibility.h"
#include "Param/ParamUtils.h"
#include "RigVMModel/RigVMPin.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMModel/RigVMController.h"
#include "Variables/AnimNextVariableReference.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SGraphPinVariableReference"

namespace UE::UAF::Editor
{

void SGraphPinVariableReference::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	Node = InPin->GetOwningNode();

	if (InArgs._FilterType.IsValid())
	{
		FilterType = InArgs._FilterType;
	}
	else
	{
		const FString AllowedType = Node->GetPinMetaData(InPin->GetFName(),"AllowedType");
		FilterType = FAnimNextParamType::FromString(AllowedType);
	}

	bIsContextSensitive = Node->GetPinMetaData(InPin->GetFName(),"ShowAll") != TEXT("true");

	if (UUAFRigVMAsset* Asset = Node->GetTypedOuter<UUAFRigVMAsset>())
	{
		WeakFilterAsset = Asset;
	}

	check(CastChecked<UScriptStruct>(InPin->PinType.PinSubCategoryObject.Get()) == FAnimNextVariableReference::StaticStruct());
	if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(Node))
	{
		ModelPin = RigVMEdGraphNode->GetModelPinFromPinPath(InPin->GetName());
		ensure(ModelPin);
		RigVMEdGraphNode->GetModel()->OnModified().AddSP(this, &SGraphPinVariableReference::HandleGraphModified);
	}

	SGraphPin::Construct(SGraphPin::FArguments(), InPin);
}

TSharedRef<SWidget> SGraphPinVariableReference::GetDefaultValueWidget()
{
	FVariablePickerArgs Args;

	UpdateCachedData();

	Args.OnVariablePicked = FOnVariablePicked::CreateLambda([this](const FAnimNextSoftVariableReference& InVariableReference, const FAnimNextParamType& InType)
	{
		{
			FScopedTransaction Transaction(LOCTEXT("SelectVariable", "Select Variable"));

			FAnimNextVariableReference Reference = FAnimNextVariableReference(InVariableReference);
			FString ValueAsString;
			{
				FAnimNextVariableReference::StaticStruct()->ExportText(ValueAsString, &Reference, nullptr, nullptr, PPF_None, nullptr);
			}

			GraphPinObj->Modify();
			GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, ValueAsString);
		}

		UpdateCachedData();
	});

	if (WeakFilterAsset.Get() != nullptr)
	{
		OnIsContextSensitiveDelegate = FOnIsContextSensitive::CreateLambda([this]()
		{
			return bIsContextSensitive;
		});

		Args.OnIsContextSensitive = &OnIsContextSensitiveDelegate;
		Args.OnContextSensitivityChanged = FOnContextSensitivityChanged::CreateLambda([this](bool bInIsContextSensitive)
		{
			bIsContextSensitive = bInIsContextSensitive;
		});

		Args.OnGetIncludeExcludeFilter = FOnGetIncludeExcludeFilter::CreateLambda([this](EAnimNextExportedVariableFlags& OutFlagInclusionFilter, EAnimNextExportedVariableFlags& OutFlagExclusionFilter)
		{
			// If we are context-sensitive and we have an asset, we allow private variable references from that asset
			if (bIsContextSensitive && WeakFilterAsset.Get())
			{
				OutFlagInclusionFilter = EAnimNextExportedVariableFlags::Declared;
				OutFlagExclusionFilter = EAnimNextExportedVariableFlags::Referenced;
			}
			else
			{
				OutFlagInclusionFilter = EAnimNextExportedVariableFlags::Declared | EAnimNextExportedVariableFlags::Public;
				OutFlagExclusionFilter = EAnimNextExportedVariableFlags::Referenced;
			}
		});
	}

	Args.OnFilterVariable = FOnFilterVariable::CreateLambda([this](const FAnimNextSoftVariableReference& InVariableReference)
	{
		const UUAFRigVMAsset* FilterAsset = WeakFilterAsset.Get();
		if (FilterAsset == nullptr || !bIsContextSensitive)
		{
			return EFilterVariableResult::Include;
		}

		if (FilterAsset != nullptr)
		{
			FSoftObjectPath FilterAssetPath(FilterAsset);
			if (InVariableReference.GetSoftObjectPath() == FilterAssetPath)
			{
				return EFilterVariableResult::Include;
			}
		}

		return EFilterVariableResult::Exclude;
	});

	if (FilterType.IsValid())
	{
		Args.CompatibleTypes = { FilterType };
	}

	Args.OnFilterVariableType = FOnFilterVariableType::CreateLambda([this](const FAnimNextParamType& InParamType) -> EFilterVariableResult
	{
		if(InParamType.IsValid())
		{
			const FRigVMTemplateArgumentType RigVMType = InParamType.ToRigVMTemplateArgument();
			if(!RigVMType.IsValid() || FRigVMRegistry::Get().GetTypeIndex(RigVMType) == INDEX_NONE)
			{
				return EFilterVariableResult::Exclude;
			}
		}

		if(ModelPin && ModelPin->IsLinked())
		{
			const FAnimNextParamType Type = FAnimNextParamType::FromRigVMTemplateArgument(ModelPin->GetTemplateArgumentType());
			if(!Type.IsValid() || !FParamUtils::GetCompatibility(Type, InParamType).IsCompatibleWithDataLoss())
			{
				return EFilterVariableResult::Exclude;
			}
		}

		return EFilterVariableResult::Include;
	});

	return
		SNew(SBox)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		[
			SAssignNew(PickerCombo, SVariablePickerCombo)
			.PickerArgs(Args)
			.OnGetVariableReference_Lambda([this]()
			{
				return CachedVariableReference;
			})
			.OnGetVariableType_Lambda([this]()
			{
				return CachedType;
			})
		];
}

void SGraphPinVariableReference::UpdateCachedData()
{
	TStringBuilder<256> DefaultValueBuilder;
	if(ModelPin)
	{
		DefaultValueBuilder.Append(ModelPin->GetDefaultValue());
	}
	else if(GraphPinObj)
	{
		DefaultValueBuilder.Append(GraphPinObj->DefaultValue);
	}

	if(DefaultValueBuilder.Len() > 0)
	{
		FAnimNextVariableReference VariableReference;
		FAnimNextVariableReference::StaticStruct()->ImportText(DefaultValueBuilder.ToString(), &VariableReference, nullptr, PPF_None, nullptr, FAnimNextVariableReference::StaticStruct()->GetName());
		CachedVariableReference = FAnimNextSoftVariableReference(VariableReference);
	}

	if (!CachedVariableReference.IsNone())
	{
		CachedType = UncookedOnly::FUtils::FindVariableType(CachedVariableReference);
	}

	if(PickerCombo.IsValid())
	{
		PickerCombo->RequestRefresh();
	}
}

void SGraphPinVariableReference::HandleGraphModified(ERigVMGraphNotifType InType, URigVMGraph* InGraph, UObject* InSubject)
{
	switch (InType)
	{
	case ERigVMGraphNotifType::PinDefaultValueChanged:
		UpdateCachedData();
		break;
	default:
		break;
	}
}

}

#undef LOCTEXT_NAMESPACE
