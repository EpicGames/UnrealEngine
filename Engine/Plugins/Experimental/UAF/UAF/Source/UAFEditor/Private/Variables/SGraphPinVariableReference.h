// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "Param/ParamType.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "Variables/VariablePickerArgs.h"

class URigVMGraph;
enum class ERigVMGraphNotifType : uint8;
class URigVMPin;
class URigVMEdGraphNode;

namespace UE::UAF::Editor
{
	class SVariablePickerCombo;
}

namespace UE::UAF::Editor
{

// A pin widget that allows picking AnimNext variable references
class SGraphPinVariableReference : public SGraphPin
{
	SLATE_BEGIN_ARGS(SGraphPinVariableReference)
		: _GraphNode(nullptr)
	{}

	SLATE_ARGUMENT(UEdGraphNode*, GraphNode)

	SLATE_ARGUMENT(FAnimNextParamType, FilterType)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

private:
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;

	void UpdateCachedData();

	void HandleGraphModified(ERigVMGraphNotifType InType, URigVMGraph* InGraph, UObject* InSubject);

	UEdGraphNode* Node = nullptr;

	URigVMPin* ModelPin = nullptr;

	FAnimNextParamType FilterType;

	FAnimNextSoftVariableReference CachedVariableReference;

	FAnimNextParamType CachedType;

	TSharedPtr<SVariablePickerCombo> PickerCombo;

	TWeakObjectPtr<UUAFRigVMAsset> WeakFilterAsset;

	FOnIsContextSensitive OnIsContextSensitiveDelegate;

	bool bIsContextSensitive = true;
};

}
