// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVariablePicker.h"

#include "UncookedOnlyUtils.h"
#include "Param/ParamType.h"
#include "Param/ParamUtils.h"
#include "Param/ParamCompatibility.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/PropertyViewer/SPropertyViewer.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SVariablePicker"

namespace UE::UAF::Editor
{

void SVariablePicker::Construct(const FArguments& InArgs)
{
	Args = InArgs._Args;

	FieldIterator = MakeUnique<FFieldIterator>();
	FieldIterator->Outer = this;
	FieldIterator->CompatibleTypes = Args.CompatibleTypes;
	const bool bAutoExpand = Args.CompatibleTypes.Num() > 0;
	FieldIterator->bAllowStructExpansion = Args.bAllowStructExpansion || bAutoExpand;
	FieldExpander.bAllowStructExpansion  = Args.bAllowStructExpansion || bAutoExpand;

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(400.0f)
		.HeightOverride(400.0f)
		.Padding(2.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(PropertyViewer, UE::PropertyViewer::SPropertyViewer)
				.OnSelectionChanged(this, &SVariablePicker::HandleFieldPicked)
				.OnGenerateContainer(this, &SVariablePicker::HandleGenerateContainer)
				.FieldIterator(FieldIterator.Get())
				.FieldExpander(&FieldExpander)
				.bShowSearchBox(true)
				.bFocusSearchBox(Args.bFocusSearchWidget)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0, 2.0f, 0.0f, 0.0f)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("ContextSensitiveTooltip", "Whether to only display variables that are valid in the current context"))
				.Visibility_Lambda([this]()
				{
					return Args.OnIsContextSensitive && Args.OnIsContextSensitive->IsBound() ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.IsChecked_Lambda([this]()
				{
					if (Args.OnIsContextSensitive && Args.OnIsContextSensitive->IsBound())
					{
						return Args.OnIsContextSensitive->Execute() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InState)
				{
					Args.OnContextSensitivityChanged.ExecuteIfBound(InState == ECheckBoxState::Checked);
					RefreshEntries();
				})
				.Content()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ContextSensitive", "Context Sensitive"))
				]
			]
		]
	];

	RefreshEntries();
}

void SVariablePicker::RefreshEntries()
{
	PropertyViewer->RemoveAll();
	CachedContainers.Reset();
	ContainerMap.Reset();

	EAnimNextExportedVariableFlags FlagInclusionFilter = Args.FlagInclusionFilter;
	EAnimNextExportedVariableFlags FlagExclusionFilter = Args.FlagExclusionFilter;
	Args.OnGetIncludeExcludeFilter.ExecuteIfBound(FlagInclusionFilter, FlagExclusionFilter);

	// Add variables exposed on assets
	TMap<FAssetData, FAnimNextAssetRegistryExports> Exports;
	if(UncookedOnly::FUtils::GetExportsOfTypeFromAssetRegistry<FAnimNextVariableDeclarationData>(Exports))
	{
		for(const TPair<FAssetData, FAnimNextAssetRegistryExports>& ExportPair : Exports)
		{
			if(ExportPair.Value.Exports.Num() > 0)
			{
				// Add a placeholder struct for this asset's properties
				TArray<FPropertyBagPropertyDesc> PropertyDescs;
				PropertyDescs.Reserve(ExportPair.Value.Exports.Num());
				for(const FAnimNextExport& Export : ExportPair.Value.Exports)
				{
					if (const FAnimNextVariableDeclarationData* VariableEntry = Export.Data.GetPtr<FAnimNextVariableDeclarationData>())
					{
						const bool bHasInclusionFlags = EnumHasAllFlags(static_cast<EAnimNextExportedVariableFlags>(VariableEntry->Flags), FlagInclusionFilter);
						const bool bHasExclusionFlags = EnumHasAnyFlags(static_cast<EAnimNextExportedVariableFlags>(VariableEntry->Flags), FlagExclusionFilter);
						
						if(bHasInclusionFlags && !bHasExclusionFlags && Export.Identifier != NAME_None && VariableEntry->Type.IsValid())
						{
							const FAnimNextSoftVariableReference VariableReference = FAnimNextSoftVariableReference::FromName(Export.Identifier, ExportPair.Key.ToSoftObjectPath());
							if (!Args.OnFilterVariable.IsBound() || Args.OnFilterVariable.Execute(VariableReference) == EFilterVariableResult::Include)
							{
								bool bPassesTypeFilter = IsDirectlyCompatible(VariableEntry->Type);

								if (!bPassesTypeFilter && FieldIterator->bAllowStructExpansion &&
									VariableEntry->Type.GetValueType() == EPropertyBagPropertyType::Struct)
								{
									if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(VariableEntry->Type.GetValueTypeObject()))
									{
										TSet<const UScriptStruct*> Visited;
										bPassesTypeFilter = HasCompatibleLeaf(ScriptStruct, Visited);
									}
								}

								if (bPassesTypeFilter && Args.OnFilterVariableType.IsBound())
								{
									bPassesTypeFilter = (Args.OnFilterVariableType.Execute(VariableEntry->Type) == EFilterVariableResult::Include);
								}

								if (bPassesTypeFilter)
								{
									PropertyDescs.Emplace(Export.Identifier, VariableEntry->Type.GetContainerType(), VariableEntry->Type.GetValueType(), VariableEntry->Type.GetValueTypeObject());
								}
							}
						}
					}
				}

				if(PropertyDescs.Num() > 0)
				{
					const FText DisplayName = FText::FromName(ExportPair.Key.AssetName);
					const FText TooltipText = FText::FromString(ExportPair.Key.GetObjectPathString());
					FContainerInfo& ContainerInfo = CachedContainers.Emplace_GetRef(DisplayName, TooltipText, ExportPair.Key, MakeUnique<FInstancedPropertyBag>());
					ContainerInfo.PropertyBag->AddProperties(PropertyDescs);
					UE::PropertyViewer::SPropertyViewer::FHandle Handle = PropertyViewer->AddContainer(ContainerInfo.PropertyBag.Get()->GetPropertyBagStruct(), DisplayName);
					ContainerMap.Add(Handle, CachedContainers.Num() - 1);
				}
			}
		}
	}

}

bool SVariablePicker::GetFieldInfo(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, const FFieldVariant& InField, FAnimNextSoftVariableReference& OutVariableReference, FAnimNextParamType& OutType) const
{
	if(const int32* ContainerIndexPtr = ContainerMap.Find(InHandle))
	{
		const FContainerInfo& ContainerInfo = CachedContainers[*ContainerIndexPtr];
		if(FProperty* Property = InField.Get<FProperty>())
		{
			FSoftObjectPath ObjectPath = ContainerInfo.AssetData.ToSoftObjectPath();
			UClass* AssetClass = ContainerInfo.AssetData.GetClass(EResolveClass::Yes);
			
			if (AssetClass != nullptr && AssetClass->IsChildOf(UBlueprint::StaticClass()))
			{
				// Shortcut to avoid loading asset
				// We know that the generated class of a blueprint will be appended with _C
				// We want the path to be to the runtime asset (i.e. the generated class) not the blueprint
				FString GeneratedClassName = ContainerInfo.AssetData.GetObjectPathString() + TEXT("_C");
				ObjectPath = GeneratedClassName;
			}

			OutVariableReference = FAnimNextSoftVariableReference::FromName(Property->GetFName(), ObjectPath);
			OutType = FAnimNextParamType::FromProperty(Property);

			return true;
		}
		
		OutVariableReference = FAnimNextSoftVariableReference();
		OutType = FAnimNextParamType();
	}

	return false;
}

void SVariablePicker::HandleFieldPicked(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, TArrayView<const FFieldVariant> InFields, ESelectInfo::Type InSelectionType)
{
	if(InFields.Num() == 1)
	{
		FAnimNextParamType Type;
		FAnimNextSoftVariableReference SoftVariableReference;
		if(GetFieldInfo(InHandle, InFields.Last(), SoftVariableReference, Type))
		{
			if (!IsDirectlyCompatible(Type))
			{
				return;  // not directly compatible — user must expand and pick a sub-property
			}

			const FAnimNextSoftVariableReference VariableReference = FAnimNextSoftVariableReference::FromName(SoftVariableReference.GetName(), SoftVariableReference.GetSoftObjectPath());
			if(ensure(Type.IsValid() && !VariableReference.IsNone()))
			{
				Args.OnVariablePicked.ExecuteIfBound(VariableReference, Type);
			}
		}
	}
	else if (InFields.Num() > 1 && Args.OnSubPropertyVariablePicked.IsBound())
	{
		// Sub-property path: InFields[0] = container variable, rest = path to leaf
		FAnimNextParamType TopType;
		FAnimNextSoftVariableReference TopVarRef;
		if (GetFieldInfo(InHandle, InFields[0], TopVarRef, TopType))
		{
			TArray<FName> PathNames;
			FAnimNextParamType LeafType;
			for (int32 i = 1; i < InFields.Num(); ++i)
			{
				if (FProperty* Prop = InFields[i].Get<FProperty>())
				{
					PathNames.Add(Prop->GetFName());
					LeafType = FAnimNextParamType::FromProperty(Prop);
				}
			}
			if (PathNames.Num() > 0 && LeafType.IsValid())
			{
				if (!IsDirectlyCompatible(LeafType))
				{
					return;  // intermediate struct node — user must expand further
				}

				Args.OnSubPropertyVariablePicked.Execute(TopVarRef, PathNames, LeafType);
			}
		}
	}
}

TSharedRef<SWidget> SVariablePicker::HandleGenerateContainer(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, TOptional<FText> InDisplayName)
{
	if(int32* ContainerIndexPtr = ContainerMap.Find(InHandle))
	{
		if(CachedContainers.IsValidIndex(*ContainerIndexPtr))
		{
			FContainerInfo& ContainerInfo = CachedContainers[*ContainerIndexPtr];

			return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("ClassIcon.Object"))
			]
			+SHorizontalBox::Slot()
			.Padding(4.0f)
			[
				SNew(STextBlock)
				.Text(ContainerInfo.DisplayName)
				.ToolTipText(ContainerInfo.TooltipText)
			];
		}
	}

	return SNullWidget::NullWidget;
}

TArray<FFieldVariant> SVariablePicker::FFieldIterator::GetFields(const UStruct* Struct, const FName FieldName, const UStruct* ContainerStruct) const
{
	auto PassesFilterChecks = [this](const FProperty* InProperty) -> bool
	{
		if (!InProperty)
		{
			return false;
		}
		FAnimNextParamType Type = FAnimNextParamType::FromProperty(InProperty);
		if (!Type.IsValid())
		{
			return false;
		}
		if (!Outer->IsDirectlyCompatible(Type))
		{
			return false;
		}
		if (Outer->Args.OnFilterVariableType.IsBound())
		{
			return Outer->Args.OnFilterVariableType.Execute(Type) == EFilterVariableResult::Include;
		}
		return true;
	};

	TArray<FFieldVariant> Result;
	for (TFieldIterator<FProperty> PropertyIt(Struct); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if(PassesFilterChecks(Property))
		{
			Result.Add(FFieldVariant(Property));
		}
		else if (bAllowStructExpansion)
		{
			// Include struct-type properties that contain any compatible leaf at any depth (handles nested structs like FTransform -> FVector -> float)
			if (const FStructProperty* StructProp = CastField<FStructProperty>(Property))
			{
				TSet<const UScriptStruct*> Visited;
				if (Outer->HasCompatibleLeaf(StructProp->Struct, Visited))
				{
					Result.Add(FFieldVariant(Property));
				}
			}
		}
	}

	return Result;
}

TOptional<const UClass*> SVariablePicker::FFieldExpander::CanExpandObject(const FObjectPropertyBase* Property, const UObject* Instance) const
{
	return TOptional<const UClass*>();
}

bool SVariablePicker::FFieldExpander::CanExpandScriptStruct(const FStructProperty* StructProperty) const
{
	return bAllowStructExpansion;
}

TOptional<const UStruct*> SVariablePicker::FFieldExpander::GetExpandedFunction(const UFunction* Function) const
{
	return TOptional<const UStruct*>();
}

bool SVariablePicker::IsDirectlyCompatible(const FAnimNextParamType& TestType) const
{
	if (Args.CompatibleTypes.IsEmpty())
	{
		return true;
	}
	for (const FAnimNextParamType& Compat : Args.CompatibleTypes)
	{
		if (FParamUtils::GetCompatibility(Compat, TestType).IsCompatibleWithDataLoss())
		{
			return true;
		}
	}
	return false;
}

bool SVariablePicker::HasCompatibleLeaf(const UScriptStruct* Struct, TSet<const UScriptStruct*>& Visited) const
{
	if (!Struct || Visited.Contains(Struct))
	{
		return false;
	}
	Visited.Add(Struct);
	for (TFieldIterator<FProperty> PropIt(Struct); PropIt; ++PropIt)
	{
		FAnimNextParamType SubType = FAnimNextParamType::FromProperty(*PropIt);
		if (SubType.IsValid() && IsDirectlyCompatible(SubType))
		{
			return true;
		}
		if (const FStructProperty* SP = CastField<FStructProperty>(*PropIt))
		{
			if (HasCompatibleLeaf(SP->Struct, Visited))
			{
				return true;
			}
		}
	}
	return false;
}

}

#undef LOCTEXT_NAMESPACE