// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/PropertyViewer/IFieldExpander.h"
#include "Framework/PropertyViewer/IFieldIterator.h"
#include "Widgets/SCompoundWidget.h"
#include "Variables/VariablePickerArgs.h"
#include "Widgets/PropertyViewer/SPropertyViewer.h"
#include "Widgets/Views/STreeView.h"
#include "StructUtils/PropertyBag.h"
#include "AssetRegistry/AssetData.h"

class IPropertyRowGenerator;
class IStructureDataProvider;
class SSearchBox;

namespace UE::UAF::Editor
{

class SVariablePicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SVariablePicker) {}

	SLATE_ARGUMENT(FVariablePickerArgs, Args)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	void RefreshEntries();

	bool GetFieldInfo(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, const FFieldVariant& InField, FAnimNextSoftVariableReference& OutVariableReference, FAnimNextParamType& OutType) const;

	void HandleFieldPicked(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, TArrayView<const FFieldVariant> InFields, ESelectInfo::Type InSelectionType);

	TSharedRef<SWidget> HandleGenerateContainer(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, TOptional<FText> InDisplayName);

	// Returns true if TestType is directly compatible with any entry in Args.CompatibleTypes.
	// Returns true unconditionally when CompatibleTypes is empty.
	bool IsDirectlyCompatible(const FAnimNextParamType& TestType) const;

	// Returns true if Struct has any directly-compatible leaf at any depth.
	bool HasCompatibleLeaf(const UScriptStruct* Struct, TSet<const UScriptStruct*>& Visited) const;

private:
	friend class SVariablePickerRow;

	FVariablePickerArgs Args;
	
	TSharedPtr<UE::PropertyViewer::SPropertyViewer> PropertyViewer;

	FText FilterText;

	TSharedPtr<SSearchBox> SearchBox;

	struct FFieldIterator : UE::PropertyViewer::IFieldIterator
	{
		FFieldIterator() = default;

		virtual TArray<FFieldVariant> GetFields(const UStruct* Struct, const FName FieldName, const UStruct* ContainerStruct) const override;

		TArray<FAnimNextParamType> CompatibleTypes;
		SVariablePicker* Outer = nullptr;
		const UStruct* CurrentStruct = nullptr;
		bool bAllowStructExpansion = false;
	};

	struct FFieldExpander : UE::PropertyViewer::IFieldExpander
	{
		virtual TOptional<const UClass*> CanExpandObject(const FObjectPropertyBase* Property, const UObject* Instance) const override;
		virtual bool CanExpandScriptStruct(const FStructProperty* StructProperty) const override;
		virtual TOptional<const UStruct*> GetExpandedFunction(const UFunction* Function) const override;
		bool bAllowStructExpansion = false;
	} FieldExpander;

	TUniquePtr<FFieldIterator> FieldIterator;

	struct FContainerInfo
	{
		FContainerInfo(const FText& InDisplayName, const FText& InTooltipText, const FAssetData& InAssetData, TUniquePtr<FInstancedPropertyBag>&& InPropertyBag)
			: DisplayName(InDisplayName)
			, TooltipText(InTooltipText)
			, PropertyBag(MoveTemp(InPropertyBag))
			, AssetData(InAssetData)
		{}

		FText DisplayName;
		FText TooltipText;
		TUniquePtr<FInstancedPropertyBag> PropertyBag;
		FAssetData AssetData;
	};
	
	TArray<FContainerInfo> CachedContainers;

	TMap<UE::PropertyViewer::SPropertyViewer::FHandle, int32> ContainerMap;
};

}