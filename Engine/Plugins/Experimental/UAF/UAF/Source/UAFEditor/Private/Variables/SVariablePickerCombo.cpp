// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/SVariablePickerCombo.h"
#include "SVariablePicker.h"
#include "UncookedOnlyUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "SVariablePickerCombo"

namespace UE::UAF::Editor
{

void SVariablePickerCombo::Construct(const FArguments& InArgs)
{
	OnGetVariableReferenceDelegate = InArgs._OnGetVariableReference;
	OnGetVariableTypeDelegate = InArgs._OnGetVariableType;
	VariableName = InArgs._VariableName;
	VariableTooltip = InArgs._VariableTooltip;

	FOnVariablePicked OnVariablePicked = InArgs._PickerArgs.OnVariablePicked;
	FOnSubPropertyVariablePicked OnSubPropertyVariablePicked = InArgs._PickerArgs.OnSubPropertyVariablePicked;

	PickerArgs = InArgs._PickerArgs;

	PickerArgs.OnVariablePicked = FOnVariablePicked::CreateSPLambda(this, [this, OnVariablePicked](const FAnimNextSoftVariableReference& InVariableReference, const FAnimNextParamType& InType)
	{
		FSlateApplication::Get().DismissAllMenus();

		// Forward to the original delegate
		OnVariablePicked.ExecuteIfBound(InVariableReference, InType);

		RequestRefresh();
	});

	PickerArgs.OnSubPropertyVariablePicked = FOnSubPropertyVariablePicked::CreateSPLambda(this, [this, OnSubPropertyVariablePicked](const FAnimNextSoftVariableReference& InTopVarRef, const TArray<FName>& InSubPropertyPath, const FAnimNextParamType& InLeafType)
	{
		FSlateApplication::Get().DismissAllMenus();

		// Forward to the original delegate
		OnSubPropertyVariablePicked.ExecuteIfBound(InTopVarRef, InSubPropertyPath, InLeafType);

		RequestRefresh();
	});

	PickerArgs.bFocusSearchWidget = true;

	ChildSlot
	[
		SNew(SComboButton)
		.ToolTipText_Lambda([this]()
		{
			return CachedVariableNameTooltipText;
		})
		.OnGetMenuContent_Lambda([this]()
		{
			return
				SNew(SVariablePicker)
				.Args(PickerArgs);
		})
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.Padding(0.0f, 2.0f, 2.0f, 2.0f)
			[
				SNew(SImage)
				.Image_Lambda([this]()
				{
					return Icon;
				})
				.ColorAndOpacity_Lambda([this]()
				{
					return IconColor;
				})
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(0.0f, 2.0f, 0.0f, 2.0f)
			[
				SNew(STextBlock)
				.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
				.Text_Lambda([this]()
				{
					return CachedVariableNameText;
				})
			]
		]
	];

	RequestRefresh();
}

void SVariablePickerCombo::RequestRefresh()
{
	if(!bRefreshRequested)
	{
		bRefreshRequested = true;
		RegisterActiveTimer(1.0f/60.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
		{
			if(OnGetVariableReferenceDelegate.IsBound())
			{
				CachedVariableReference = OnGetVariableReferenceDelegate.Execute();
			}

			if(OnGetVariableTypeDelegate.IsBound())
			{
				VariableType = OnGetVariableTypeDelegate.Execute();
			}

			CachedVariableNameText = VariableName.IsSet() ? VariableName.Get() : FText::FromName(CachedVariableReference.GetName());
			CachedVariableNameTooltipText = VariableTooltip.IsSet() ? VariableTooltip.Get() : FText::Format(LOCTEXT("VariableTooltipFormat", "{0}\n{1}"), FText::FromName(CachedVariableReference.GetName()), FText::FromString(CachedVariableReference.GetSoftObjectPath().ToString()));
			PinType = UncookedOnly::FUtils::GetPinTypeFromParamType(VariableType);
			Icon = FBlueprintEditorUtils::GetIconFromPin(PinType, true);
			IconColor = GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);

			bRefreshRequested = false;
			return EActiveTimerReturnType::Stop;
		}));
	}
}

}

#undef LOCTEXT_NAMESPACE