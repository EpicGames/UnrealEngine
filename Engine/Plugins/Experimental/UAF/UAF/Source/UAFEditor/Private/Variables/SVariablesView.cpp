// Copyright Epic Games, Inc. All Rights Reserved.

#include "SVariablesView.h"

#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "AnimNextRigVMAsset.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SAssetDropTarget.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerSourceControlColumn.h"
#include "ScopedTransaction.h"
#include "SSceneOutliner.h"
#include "Outliner/VariablesOutlinerColumns.h"
#include "Outliner/VariablesOutlinerMode.h"
#include "Outliner/VariablesOutlinerEntryItem.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "Common/Outliner/OutlinerColumns.h"
#include "Common/Outliner/SCommonOutliner.h"
#include "Entries/AnimNextSharedVariablesEntry.h"

#define LOCTEXT_NAMESPACE "SVariablesView"

namespace UE::UAF::Editor
{

const FLazyName VariablesTabName("VariablesTab");
void SVariablesView::Construct(const FArguments& InArgs, TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor)
{
	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.OutlinerIdentifier = TEXT("AnimNextVariablesOutliner");
	InitOptions.bShowHeaderRow = true;
	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn(), false, 0.5f));
	InitOptions.ColumnMap.Add(FVariablesOutlinerTypeColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShared<FVariablesOutlinerTypeColumn>(InSceneOutliner); }), false,
		TOptional<float>(), TAttribute<FText>(), EHeaderComboVisibility::Ghosted));
	InitOptions.ColumnMap.Add(FVariablesOutlinerValueColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 20, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShared<FVariablesOutlinerValueColumn>(InSceneOutliner); }), true, 0.5f));
	InitOptions.ColumnMap.Add(FSceneOutlinerSourceControlColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 30, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShared<FSceneOutlinerSourceControlColumn>(InSceneOutliner); }), true));
	InitOptions.ColumnMap.Add(FOutlinerAccessSpecifierColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 40, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShared<FOutlinerAccessSpecifierColumn>(InSceneOutliner); }), false));
	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([this, WeakWorkspaceEditor=InWorkspaceEditor.ToWeakPtr()](SSceneOutliner* InOutliner) { check(WeakWorkspaceEditor.IsValid()); return new FVariablesOutlinerMode(static_cast<SCommonOutliner*>(InOutliner), WeakWorkspaceEditor.Pin().ToSharedRef()); });

	VariablesOutliner = SNew(UE::UAF::Editor::SCommonOutliner, InitOptions)
	.BindAssetDelegate_Lambda([WeakThis=StaticCastWeakPtr<SVariablesView>(AsWeak())](UUAFRigVMAssetEditorData* EditorData)
	{
		if (WeakThis.IsValid())
		{
			TSharedPtr<SVariablesView> SharedView = WeakThis.Pin();
			EditorData->ModifiedDelegate.AddSP(SharedView.Get(), &SVariablesView::OnEditorDataModified);
		}
	})
	.UnbindAssetDelegate_Lambda([WeakThis=StaticCastWeakPtr<SVariablesView>(AsWeak())](UUAFRigVMAssetEditorData* EditorData)
	{
		if (WeakThis.IsValid())
		{
			TSharedPtr<SVariablesView> SharedView = WeakThis.Pin();
			EditorData->ModifiedDelegate.RemoveAll(SharedView.Get());
		}
	});
	VariablesOutliner->SetEnabled(MakeAttributeSP(VariablesOutliner.Get(), &SCommonOutliner::HasAssets));

	InWorkspaceEditor->OnFocusedDocumentChanged().AddSP(this, &SVariablesView::HandleFocusedDocumentChanged);
	const Workspace::FWorkspaceDocument& Document = InWorkspaceEditor->GetFocusedWorkspaceDocument();
	HandleFocusedDocumentChanged(Document);

	ChildSlot
	[
		VariablesOutliner.ToSharedRef()
	];
}

void SVariablesView::SetExportDirectly(const FWorkspaceOutlinerItemExport& InExport) const
{
	if (VariablesOutliner.IsValid())
	{
		VariablesOutliner->SetExport(InExport);
	}
}

void SVariablesView::HandleFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument) const
{
	const FWorkspaceOutlinerItemExport& Export = InDocument.Export;
	VariablesOutliner->SetExport(Export);
}

void SVariablesView::OnEditorDataModified(UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType, UObject* InSubject) const
{
	switch(InType)
	{
	case EAnimNextEditorDataNotifType::UndoRedo:
	case EAnimNextEditorDataNotifType::EntryAdded:
	case EAnimNextEditorDataNotifType::VariableCategoryChanged:
	case EAnimNextEditorDataNotifType::CategoryAdded:
	case EAnimNextEditorDataNotifType::CategoryChanged:
	case EAnimNextEditorDataNotifType::CategoryRemoved:	
	case EAnimNextEditorDataNotifType::EntryRemoved:
		VariablesOutliner->UpdateAssets();
		VariablesOutliner->FullRefresh();
		break;
	default:
		break;
	}
}

FAnimNextVariablesTabSummoner::FAnimNextVariablesTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp)
	: FWorkflowTabFactory(VariablesTabName, StaticCastSharedPtr<FAssetEditorToolkit>(InHostingApp))
{
	TabLabel = LOCTEXT("AnimNextVariablesTabLabel", "Variables");
	TabIcon = FSlateIcon("EditorStyle", "LevelEditor.Tabs.Outliner");
	ViewMenuDescription = LOCTEXT("AnimNextVariablesTabMenuDescription", "Variables");
	ViewMenuTooltip = LOCTEXT("AnimNextVariablesTabToolTip", "Shows the Variables tab.");
	bIsSingleton = true;

	VariablesView = SNew(SVariablesView, InHostingApp.ToSharedRef());

	const Workspace::FWorkspaceDocument& Document = InHostingApp->GetFocusedWorkspaceDocument();
	VariablesView->VariablesOutliner->SetExport(Document.Export);
}

TSharedRef<SWidget> FAnimNextVariablesTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return VariablesView.ToSharedRef();
}

FText FAnimNextVariablesTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return ViewMenuTooltip;
}

}

#undef LOCTEXT_NAMESPACE
