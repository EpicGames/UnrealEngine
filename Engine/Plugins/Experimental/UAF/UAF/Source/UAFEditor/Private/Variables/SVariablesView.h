// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "SSceneOutliner.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "Editor/Persona/Private/SAnimAssetFindReplace.h"
#include "Widgets/SCompoundWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

enum class EAnimNextEditorDataNotifType : uint8;
class UUAFRigVMAsset;
struct FWorkspaceOutlinerItemExport;
class SPositiveActionButton;
class UUAFRigVMAssetEditorData;

namespace UE::Workspace
{
	class IWorkspaceEditor;
	struct FWorkspaceDocument;
}

namespace UE::UAF::Editor
{
class SCommonOutliner;

extern const FLazyName VariablesTabName;

class SVariablesView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SVariablesView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor);

	void SetExportDirectly(const FWorkspaceOutlinerItemExport& InExport) const;
private:
	void HandleFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument) const;	
	void OnEditorDataModified(UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType, UObject* InSubject) const;

private:
	friend struct FAnimNextVariablesTabSummoner;
	TSharedPtr<SCommonOutliner> VariablesOutliner;
};

struct FAnimNextVariablesTabSummoner : public FWorkflowTabFactory
{
public:
	FAnimNextVariablesTabSummoner(TSharedPtr<UE::Workspace::IWorkspaceEditor> InHostingApp);

private:
	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

	// The widget this tab spawner wraps
	TSharedPtr<SVariablesView> VariablesView;
};


};