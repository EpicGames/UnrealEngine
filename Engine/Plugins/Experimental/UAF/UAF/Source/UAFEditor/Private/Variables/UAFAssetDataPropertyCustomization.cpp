// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFAssetDataPropertyCustomization.h"

#include "AnimNextConfig.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailDragDropHandler.h"
#include "InstancedStructDetails.h"
#include "PropertyCustomizationHelpers.h"
#include "RigVMBlueprintLegacy.h"
#include "SAssetDropTarget.h"
#include "ScopedTransaction.h"
#include "SInstancedStructPicker.h"
#include "StructUtilsMetadata.h"
#include "Engine/Blueprint.h"
#include "Settings/EditorStyleSettings.h"
#include "UAF/UAFAssetData.h"
#include "UAF/UAFAssetFactory.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ParamPropertyCustomization"

class UEditorStyleSettings;

bool FUAFAssetDataPropertyIdentifier::IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(PropertyHandle.GetProperty()))
	{
		if (StructProperty->Struct == FInstancedStruct::StaticStruct())
		{
			const FString& BaseStructName = PropertyHandle.GetMetaData(UE::StructUtils::Metadata::BaseStructName);
			UScriptStruct* Struct = UClass::TryFindTypeSlow<UScriptStruct>(BaseStructName);
			const UStruct* AssetDataStruct = FUAFAssetData::StaticStruct();
			return Struct->IsChildOf(AssetDataStruct);
		}
	}

	return false;
}

FUAFAssetDataPropertyCustomization::FUAFAssetDataPropertyCustomization()
{
}

void FUAFAssetDataPropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow,
                                                         IPropertyTypeCustomizationUtils& CustomizationUtils)
{

	const FString& BaseStructName = StructPropertyHandle->GetMetaData(UE::StructUtils::Metadata::BaseStructName);
	TWeakObjectPtr<UScriptStruct> Struct = UClass::TryFindTypeSlow<UScriptStruct>(BaseStructName);

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.f)
		.VAlign(VAlign_Center)
		[
			SNew(SAssetDropTarget)
			.OnAreAssetsAcceptableForDrop_Lambda([Struct](TArrayView<FAssetData> AssetDatas)-> bool
			{
					if (AssetDatas.Num() > 0 && Struct.IsValid())
					{
						return ValidateAssetDataForDrop(Struct.Get(), AssetDatas[0]);
					}
					return false;
			})
			.OnAssetsDropped_Lambda([StructPropertyHandle](const FDragDropEvent& DragDropEvent, TArrayView<FAssetData> InAssets)
			{
				if (InAssets.Num() > 0)
				{
					TrySetAsset(StructPropertyHandle, InAssets[0]);
				}
			})
			[
				// Todo: handle allowed structs via meta filter
				SNew(SInstancedStructPicker, StructPropertyHandle, CustomizationUtils.GetPropertyUtilities())	
			]
		]
		.IsEnabled(StructPropertyHandle->IsEditable());
}



void FUAFAssetDataPropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedRef<FInstancedStructDataDetails> DataDetails = MakeShared<FInstancedStructDataDetails>(PropertyHandle);
	ChildBuilder.AddCustomBuilder(DataDetails);
}

bool FUAFAssetDataPropertyCustomization::ValidateAssetDataForDrop(UScriptStruct* BaseAssetType, const FAssetData& AssetData)
{
	const UObject* Object = AssetData.GetAsset();
	if (Object == nullptr)
	{
		return true;
	}
	
	TArray<UClass*> AllowedClasses = UE::UAF::FAssetDataFactory::GetRegisteredObjectClasses(BaseAssetType);
	const UClass* CandidateClass = Object->GetClass();

	return AllowedClasses.ContainsByPredicate([CandidateClass](const UClass* Class) -> bool
	{
		return CandidateClass->IsChildOf(Class);
	});
}

void FUAFAssetDataPropertyCustomization::TrySetAsset(TSharedPtr<IPropertyHandle> StructProperty, const FAssetData& InAsset)
{
	// Set the property
	UObject* Object = InAsset.GetAsset();
	if (Object == nullptr)
	{
		// Clear property and return
		StructProperty->ResetToDefault();
		return;
	}

	TInstancedStruct<FUAFAssetData> AssetData = UE::UAF::FAssetDataFactory::CreateUAFAssetDataFromObject(Object);
	if (AssetData.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("OnStructPicked", "Set Struct"));

		TArray<UObject*> OuterObjects;
		OuterObjects.Reserve(StructProperty->GetNumOuterObjects());
		StructProperty->GetOuterObjects(OuterObjects);
		
		if (OuterObjects.IsEmpty())
		{
			StructProperty->NotifyPreChange();

			StructProperty->EnumerateRawData([AssetData](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
			{
				if (FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(RawData))
				{
					// Todo: how to avoid copy here?
					InstancedStruct->InitializeAs(AssetData.GetScriptStruct(), AssetData.GetMemory());
				}
				return true;
			});

			StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
			StructProperty->NotifyFinishedChangingProperties();
		}
		else
		{
			FInstancedStruct InstancedStruct;
			// Todo: avoid a copy here?
			InstancedStruct.InitializeAs(AssetData.GetScriptStruct(), AssetData.GetMemory());

			TArray<FString> NewValues;
			NewValues.Reserve(OuterObjects.Num());
			for (UObject* OuterObject : OuterObjects)
			{
				if (!OuterObject)
				{
					continue;
				}

				InstancedStruct.ExportTextItem(
					NewValues.Emplace_GetRef(),
					FInstancedStruct(),
					OuterObject,
					PPF_None,
					OuterObject->GetOutermost());
			}

			// This will internally handle propagation for values within default/archetype objects
			StructProperty->SetPerObjectValues(NewValues, EPropertyValueSetFlags::NotTransactable);
		}

		// After the type has changed, let's expand, so that the user can edit the newly appeared child properties
		StructProperty->SetExpanded(true);
	}
}

#undef LOCTEXT_NAMESPACE
