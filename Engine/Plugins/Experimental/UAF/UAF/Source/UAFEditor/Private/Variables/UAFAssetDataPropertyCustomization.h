// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedStructDetails.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "UAF/UAFAssetData.h"

class FUAFAssetDataPropertyIdentifier : public IPropertyTypeIdentifier
{
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& PropertyHandle) const override;
};

class FUAFAssetDataPropertyCustomization : public IPropertyTypeCustomization
{
public:
	FUAFAssetDataPropertyCustomization();
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	static bool ValidateAssetDataForDrop(UScriptStruct* BaseAssetType, const FAssetData& AssetData);
	static void TrySetAsset(TSharedPtr<IPropertyHandle> AssetPropertyHandle, const FAssetData& InAsset);
};
