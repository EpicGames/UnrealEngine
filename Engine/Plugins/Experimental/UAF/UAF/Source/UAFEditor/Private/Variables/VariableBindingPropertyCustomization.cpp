// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariableBindingPropertyCustomization.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "DetailWidgetRow.h"
#include "EdGraphSchema_K2.h"
#include "EditorUtils.h"
#include "IAnimNextUncookedOnlyModule.h"
#include "PropertyHandle.h"
#include "SInstancedStructPicker.h"
#include "UncookedOnlyUtils.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Modules/ModuleManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Variables/IVariableBindingType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "ParamPropertyCustomization"

namespace UE::UAF::Editor
{

void FVariableBindingPropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;
	BindingDataHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextVariableBinding, BindingData));
	check(BindingDataHandle.IsValid());

	PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FVariableBindingPropertyCustomization::OnBindingChanged));
	PropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FVariableBindingPropertyCustomization::OnBindingChanged));

	TOptional<FAnimNextParamType> CommonType;

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);
	for(UObject* Object : OuterObjects)
	{
		if(UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(Object))
		{
			if(!CommonType.IsSet())
			{
				CommonType = VariableEntry->GetType();
			}
			else if(CommonType.GetValue() != VariableEntry->GetType())
			{
				// No common type, so use an invalid type
				CommonType = FAnimNextParamType();
				break;
			}
			
			if (UUAFRigVMAssetEditorData* EditorData = VariableEntry->GetTypedOuter<UUAFRigVMAssetEditorData>())
			{
				TWeakObjectPtr<UAnimNextVariableEntry> WeakEntry(VariableEntry);
				EditorData->OnModified().AddSP(this, &FVariableBindingPropertyCustomization::OnEditorDataModified, WeakEntry);
			}
		}
	}

	Type = CommonType.IsSet() ? CommonType.GetValue() : FAnimNextParamType();

	InHeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SAssignNew(ContainerWidget, SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0.0f, 0.0f, 5.0f, 0.0f)
		.AutoWidth()
		[
			SAssignNew(ValueWidget, SComboButton)
			.ForegroundColor_Lambda([this]()
			{
				return bIsBindingValid ? FSlateColor::UseForeground() : FLinearColor::Red;
			})
			.Visibility_Lambda([this]()
			{
				return bShowBindingSelector ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.ToolTipText_Lambda([this]()
			{
				return TooltipText;
			})
			.MenuContent()
			[
				CreateBindingWidget()
			]
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(0.0f, 2.0f, 2.0f, 2.0f)
				[
					SNew(SImage)
					.Image_Lambda([this]()
					{
						return Icon;
					})
					.ColorAndOpacity_Lambda([this]()
					{
						return IconColor;
					})
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(0.0f, 2.0f, 0.0f, 2.0f)
				[
					SNew(STextBlock)
					.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
					.Text_Lambda([this]()
					{
						return NameText;
					})
				]
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SInstancedStructPicker, BindingDataHandle, InCustomizationUtils.GetPropertyUtilities())
			.OnStructPicked_Lambda([this](const UScriptStruct* InStruct)
			{
				OnBindingChanged();
			})
		]
	];

	Refresh();
}

void FVariableBindingPropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FVariableBindingPropertyCustomization::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged)
{
	OnBindingChanged();
}

void FVariableBindingPropertyCustomization::OnBindingChanged()
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);
	for(UObject* Object : OuterObjects)
	{
		if(UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(Object))
		{
			VariableEntry->BroadcastModified(EAnimNextEditorDataNotifType::VariableBindingChanged);
		}
	}
	
	RequestRefresh();
}

void FVariableBindingPropertyCustomization::RequestRefresh()
{
	if(!bRefreshRequested)
	{
		bRefreshRequested = true;
		ContainerWidget->RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double /*InCurrentTime*/, float /*InDeltaTime*/)
		{
			bRefreshRequested = false;
			Refresh();
			return EActiveTimerReturnType::Stop;
		}));
	}
}

void FVariableBindingPropertyCustomization::Refresh()
{
	bShowBindingSelector = false;

	if(!Type.IsValid())
	{
		NameText = LOCTEXT("MultipleTypes", "Multiple Types");
		TooltipText = LOCTEXT("MultipleTypes", "Multiple Types");
		bIsBindingValid = true;
		bShowBindingSelector = true;
		return;
	}

	FEdGraphPinType PinType  = UncookedOnly::FUtils::GetPinTypeFromParamType(Type);
	Icon = FBlueprintEditorUtils::GetIconFromPin(PinType, true);
	IconColor = GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);

	TOptional<const UScriptStruct*> CommonBindingStruct;
	TOptional<TInstancedStruct<FAnimNextVariableBindingData>> CommonBindingData;
	BindingDataHandle->EnumerateConstRawData([this, &CommonBindingData, &CommonBindingStruct](const void* RawData, const int32 DataIndex, const int32 NumDatas)
	{
		const TInstancedStruct<FAnimNextVariableBindingData>& BindingData = *static_cast<const TInstancedStruct<FAnimNextVariableBindingData>*>(RawData);
		if(!CommonBindingData.IsSet())
		{
			CommonBindingData = BindingData;
		}
		else if(CommonBindingData.GetValue() != BindingData)
		{
			// No common binding
			CommonBindingData = TInstancedStruct<FAnimNextVariableBindingData>();
		}

		if(!CommonBindingStruct.IsSet())
		{
			CommonBindingStruct = BindingData.GetScriptStruct();
		}
		else if(CommonBindingStruct.GetValue() != BindingData.GetScriptStruct())
		{
			// No common struct
			CommonBindingStruct = nullptr;
		}
		return true;
	});

	const UScriptStruct* BindingStruct = CommonBindingStruct.IsSet() ? CommonBindingStruct.GetValue() : nullptr;
	TInstancedStruct<FAnimNextVariableBindingData> BindingData = CommonBindingData.IsSet() ? CommonBindingData.GetValue() : TInstancedStruct<FAnimNextVariableBindingData>();
	if(BindingData.IsValid())
	{
		// Common values
		NameText = GetBindingDisplayNameText(BindingData);
		TooltipText = GetBindingTooltipText(BindingData);
		bIsBindingValid = ValidateBinding(BindingData);
		bShowBindingSelector = true;
	}
	else if(BindingStruct != nullptr)
	{
		// No common values, but common struct, allow selection
		NameText = LOCTEXT("MultipleValues", "Multiple Values");
		TooltipText = LOCTEXT("MultipleValues", "Multiple Values");
		bIsBindingValid = true;
		bShowBindingSelector = true;
	}
	else
	{
		// No common struct, dont allow binding selection 
		NameText = LOCTEXT("MultipleValues", "Multiple Values");
		TooltipText = LOCTEXT("MultipleValues", "Multiple Values");
		bIsBindingValid = true;
		bShowBindingSelector = false;
	}
}

FText FVariableBindingPropertyCustomization::GetBindingDisplayNameText(TConstStructView<FAnimNextVariableBindingData> InBindingData)
{
	if(InBindingData.IsValid())
	{
		UncookedOnly::IAnimNextUncookedOnlyModule& UncookedOnlyModule = FModuleManager::GetModuleChecked<UncookedOnly::IAnimNextUncookedOnlyModule>("UAFUncookedOnly");
		if(TSharedPtr<UncookedOnly::IVariableBindingType> BindingType = UncookedOnlyModule.FindVariableBindingType(InBindingData.GetScriptStruct()))
		{
			return BindingType->GetDisplayText(InBindingData);
		}
		else
		{
			return LOCTEXT("UnknownBindingLabel", "Unknown");
		}
	}

	return LOCTEXT("NoBindingLabel", "None");
}

FText FVariableBindingPropertyCustomization::GetBindingTooltipText(TConstStructView<FAnimNextVariableBindingData> InBindingData)
{
	FTextBuilder TextBuilder;

	if(InBindingData.IsValid())
	{
		UncookedOnly::IAnimNextUncookedOnlyModule& UncookedOnlyModule = FModuleManager::GetModuleChecked<UncookedOnly::IAnimNextUncookedOnlyModule>("UAFUncookedOnly");
		if(TSharedPtr<UncookedOnly::IVariableBindingType> BindingType = UncookedOnlyModule.FindVariableBindingType(InBindingData.GetScriptStruct()))
		{
			TextBuilder.AppendLine(BindingType->GetTooltipText(InBindingData));
		}
		else
		{
			TextBuilder.AppendLine(FText::Format(LOCTEXT("UnknownBindingTooltipFormat", "Unknown binding: {0}"), InBindingData.GetScriptStruct()->GetDisplayNameText()));
		}
	}
	else
	{
		TextBuilder.AppendLine(LOCTEXT("NoBindingTooltip", "No binding"));
	}
	
	return TextBuilder.ToText();
}

bool FVariableBindingPropertyCustomization::ValidateBinding(TConstStructView<FAnimNextVariableBindingData> InBindingData)
{
	if(InBindingData.IsValid())
	{
		UncookedOnly::IAnimNextUncookedOnlyModule& UncookedOnlyModule = FModuleManager::GetModuleChecked<UncookedOnly::IAnimNextUncookedOnlyModule>("UAFUncookedOnly");
		if(TSharedPtr<UncookedOnly::IVariableBindingType> BindingType = UncookedOnlyModule.FindVariableBindingType(InBindingData.GetScriptStruct()))
		{
			FText Error;
			return BindingType->Validate(InBindingData, Error);
		}
	}

	return true;
}

TSharedRef<SWidget> FVariableBindingPropertyCustomization::CreateBindingWidget() const
{
	if(!Type.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TOptional<const UScriptStruct*> CommonBindingStruct;
	BindingDataHandle->EnumerateConstRawData([&CommonBindingStruct](const void* RawData, const int32 DataIndex, const int32 NumDatas)
	{
		const TInstancedStruct<FAnimNextVariableBindingData>& BindingData = *static_cast<const TInstancedStruct<FAnimNextVariableBindingData>*>(RawData);
		if(!CommonBindingStruct.IsSet())
		{
			CommonBindingStruct = BindingData.GetScriptStruct();
		}
		else if(CommonBindingStruct.GetValue() != BindingData.GetScriptStruct())
		{
			// No common binding
			CommonBindingStruct = nullptr;
			return false;
		}
		return true;
	});

	const UScriptStruct* BindingStruct = CommonBindingStruct.IsSet() ? CommonBindingStruct.GetValue() : nullptr;

	UncookedOnly::IAnimNextUncookedOnlyModule& UncookedOnlyModule = FModuleManager::GetModuleChecked<UncookedOnly::IAnimNextUncookedOnlyModule>("UAFUncookedOnly");
	if(TSharedPtr<UncookedOnly::IVariableBindingType> BindingType = UncookedOnlyModule.FindVariableBindingType(BindingStruct))
	{
		return BindingType->CreateEditWidget(BindingDataHandle.ToSharedRef(), Type);
	}

	return SNullWidget::NullWidget;
}

void FVariableBindingPropertyCustomization::OnEditorDataModified(ERigVMGraphNotifType NotifType, URigVMGraph* Graph, UObject* Subject, TWeakObjectPtr<UAnimNextVariableEntry> WeakEntry)
{
	UAnimNextVariableEntry* Entry = WeakEntry.Get();
	if (Entry && Subject == Entry)
	{
		RequestRefresh();
	}
}

}

#undef LOCTEXT_NAMESPACE