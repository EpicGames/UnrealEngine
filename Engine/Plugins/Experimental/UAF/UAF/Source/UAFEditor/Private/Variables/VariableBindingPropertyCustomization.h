// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "Misc/NotifyHook.h"
#include "Param/ParamType.h"

class URigVMGraph;
enum class ERigVMGraphNotifType : uint8;
struct FAnimNextVariableBindingData;
class UAnimNextVariableEntry;

namespace UE::UAF::Editor
{

class FVariableBindingPropertyCustomization : public IPropertyTypeCustomization, public FNotifyHook
{
private:
	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	// FNotifyHook interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FEditPropertyChain* PropertyThatChanged) override;

	void OnBindingChanged();
	void Refresh(); 
	void OnEditorDataModified(ERigVMGraphNotifType NotifType, URigVMGraph* Graph, UObject* Subject, TWeakObjectPtr<UAnimNextVariableEntry> WeakEntry);

	void RequestRefresh();

	/** Gets a name to display for a variable binding in the editor */
	static FText GetBindingDisplayNameText(TConstStructView<FAnimNextVariableBindingData> InBindingData);

	/** Gets a name to display for a variable binding's tooltip in the editor */
	static FText GetBindingTooltipText(TConstStructView<FAnimNextVariableBindingData> InBindingData);

	/** Checks a binding for validity in the editor */
	static bool ValidateBinding(TConstStructView<FAnimNextVariableBindingData> InBindingData);

	/** Create a binding widget */
	TSharedRef<SWidget> CreateBindingWidget() const;

private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	TSharedPtr<IPropertyHandle> BindingDataHandle;
	FAnimNextParamType Type;
	FText NameText;
	FText TooltipText;
	const FSlateBrush* Icon = nullptr;
	FSlateColor IconColor = FLinearColor::Gray;
	TSharedPtr<SWidget> ValueWidget;
	TSharedPtr<SWidget> ContainerWidget;
	bool bIsBindingValid = false;
	bool bRefreshRequested = false;
	bool bShowBindingSelector = false;
};

}
