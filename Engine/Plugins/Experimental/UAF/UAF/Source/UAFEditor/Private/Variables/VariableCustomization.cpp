// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/VariableCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EdGraphSchema_K2.h"
#include "EditorUtils.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "UncookedOnlyUtils.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Module/AnimNextModule_EditorData.h"
#include "UObject/GarbageCollectionSchema.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "VariableCustomization"

namespace UE::UAF::Editor
{

FVariableCustomization::FVariableCustomization() : DefaultCategoryName(LOCTEXT("DefaultCategoryLabel", "Default").ToString())
{
}

void FVariableCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{	
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	if (Objects.IsEmpty())
	{
		return;
	}
		
	bool bVariablesArePartOfSameAsset = true;
	UUAFRigVMAsset* MainAsset = nullptr;
	
	for(TWeakObjectPtr<UObject> WeakObject : Objects)
	{
		UAnimNextVariableEntry* Variable = Cast<UAnimNextVariableEntry>(WeakObject.Get());
		if(Variable == nullptr)
		{
			continue;
		}

		Variables.Add(Variable);

		UUAFRigVMAsset* Asset = Variable->GetTypedOuter<UUAFRigVMAsset>();

		if (Asset && MainAsset == nullptr)
		{
			MainAsset = Asset;
		}
		else if (Asset == nullptr || Asset != MainAsset)
		{
			bVariablesArePartOfSameAsset = false;
		}

		// Disable access specifier switching specifically for shared variables
		if(ExactCast<UUAFSharedVariables>(Asset))
		{
			TSharedPtr<IPropertyHandle> AccessProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimNextVariableEntry, Access));
			AccessProperty->MarkHiddenByCustomization();
		}
	}
	
	TSharedPtr<IPropertyHandle> CategoryProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimNextVariableEntry, Category));
	CategoryProperty->MarkHiddenByCustomization();

	// Only allow changing categories if the variables are part of the same asset
	if (bVariablesArePartOfSameAsset)
	{
		OuterRigVMAsset = MainAsset;
		PopulateCategories();

		IDetailPropertyRow& CategoryRow = DetailBuilder.AddPropertyToCategory(CategoryProperty);	
		CategoryRow.CustomWidget()
		.NameContent()
		[
			CategoryProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		[
			SAssignNew(CategoryComboButton, SComboButton)
			.ContentPadding(FMargin(0.0f, 0.0f, 5.0f, 0.0f))
			.ButtonContent()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder") )
				.Padding(FMargin(0, 0, 5, 0))
				[
					SNew(SEditableTextBox)
						.Text(this, &FVariableCustomization::GetCategory)
						.OnTextCommitted(this, &FVariableCustomization::SetCategory)
						.OnVerifyTextChanged(this, &FVariableCustomization::OnVerifyCategoryName)
						//.ToolTip(CategoryProperty->GetToolTipText()) TODO 
						.SelectAllTextWhenFocused(true)
						.RevertTextOnEscape(true)
						.Font( DetailBuilder.GetDetailFont() )
				]
			]
			.MenuContent()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(400.0f)
				[
					SAssignNew(CategoryListView, SListView<TSharedPtr<FText>>)
						.ListItemsSource(&Categories)
						.OnGenerateRow_Lambda([](const TSharedPtr<FText>& Item, const TSharedRef<STableViewBase>& OwnerTable)
						{
							return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
							[
								SNew(STextBlock).Text(*Item.Get())
							];
						})
						.OnSelectionChanged(this, &FVariableCustomization::OnCategorySelectionChanged)
				]
			]
		];
	}
	
	// Register modified notify callback to force update the details customization (handle mismatch of property value widget vs new type) 
	for (const TWeakObjectPtr<UObject>& CustomizedObject : Objects)
	{
		if (CustomizedObject.Get())
		{
			if (UUAFRigVMAssetEditorData* OuterEditorData = CustomizedObject->GetTypedOuter<UUAFRigVMAssetEditorData>())
			{
				if (WeakLayoutBuilder.IsValid())
				{
					OuterEditorData->ModifiedDelegate.AddSPLambda(this, [LocalWeakLayoutBuilder = WeakLayoutBuilder, WeakObject = CustomizedObject](UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType, const UObject* InSubject)
					{
						if (InType == EAnimNextEditorDataNotifType::VariableTypeChanged && WeakObject.Get() == InSubject) 
						{
							if (auto SharedLayoutBuilder = LocalWeakLayoutBuilder.Pin())
							{
								SharedLayoutBuilder->ForceRefreshDetails();
							}
						}
					});
				}
			}
		}
	}

	// Don't customize default value if we have multi-selection
	if(Objects.Num() > 1)
	{
		return;
	}

	if (UAnimNextVariableEntry* Variable = Cast<UAnimNextVariableEntry>(Objects[0].Get()))
	{
		IDetailCategoryBuilder& DefaultValueCategory = DetailBuilder.EditCategory(TEXT("DefaultValue"), FText::GetEmpty(), ECategoryPriority::Default);

		TSharedRef< SWidget > ColumnWidget = SNullWidget::NullWidget;

		FAddPropertyParams AddPropertyParams;
		TArray<IDetailPropertyRow*> DetailPropertyRows;

		FInstancedPropertyBag& PropertyBag = Variable->GetMutablePropertyBag();
		const FPropertyBagPropertyDesc* PropertyDesc = PropertyBag.FindPropertyDescByName(IUAFRigVMVariableInterface::ValueName);
		if (PropertyDesc != nullptr)
		{
			IDetailPropertyRow* DetailPropertyRow = DefaultValueCategory.AddExternalStructureProperty(MakeShared<FInstancePropertyBagStructureDataProvider>(PropertyBag), IUAFRigVMVariableInterface::ValueName, EPropertyLocation::Default, AddPropertyParams);
			
			DetailPropertyRow->ShouldAutoExpand(true);
			
			if (TSharedPtr<IPropertyHandle> Handle = DetailPropertyRow->GetPropertyHandle(); Handle.IsValid())
			{
				Handle->SetPropertyDisplayName(FText::FromName(Variable->GetEntryName()));

				const auto OnPropertyValueChange = [WeakVariable = TWeakObjectPtr<UAnimNextVariableEntry>(Variable)](const FPropertyChangedEvent& InEvent)
				{
					if (UAnimNextVariableEntry* PinnedVariable = WeakVariable.Get())
					{
						PinnedVariable->MarkPackageDirty();
						PinnedVariable->BroadcastModified(EAnimNextEditorDataNotifType::PropertyChanged);
					}
				};

				Handle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda(OnPropertyValueChange));
				Handle->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda(OnPropertyValueChange));
			}
		}
		
		
		// Handle renaming of the parameter name and validating that name before commit 
		const TSharedPtr<IPropertyHandle> ParameterNameProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimNextVariableEntry, ParameterName));
		IDetailPropertyRow& ParameterNameRow = DetailBuilder.AddPropertyToCategory(ParameterNameProperty);	
		ParameterNameRow.CustomWidget()
		.NameContent()
		[
			ParameterNameProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.OnVerifyTextChanged(this, &FVariableCustomization::OnVerifyNameChanged)
			.Text(this, &FVariableCustomization::GetName)
			.OnTextCommitted(this, &FVariableCustomization::SetName)
		
		];
	}
}

void FVariableCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) 
{
	WeakLayoutBuilder = DetailBuilder;
	CustomizeDetails(*DetailBuilder);
}

FText FVariableCustomization::GetName() const
{
	// Only allow name interaction in single select case 
	if (Variables.Num() == 1 && Variables[0].IsValid())
	{
		return Variables[0]->GetDisplayName();
	}
	return FText();
}

void FVariableCustomization::SetName(const FText& InNewText, ETextCommit::Type InCommitType)
{
	// Only allow name interaction in single select case 
	if (Variables.Num() == 1 && Variables[0].IsValid())
	{
		UE::UAF::UncookedOnly::FUtils::RenameVariable(Variables[0].Get(), FName(InNewText.ToString()));
	}
}

bool FVariableCustomization::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage)
{
	// Only allow name interaction in single select case 
	if (Variables.Num() == 1 && Variables[0].IsValid())
	{
		return FUtils::IsValidParameterNameForEntry(Variables[0].Get(), InText, OutErrorMessage);
	}
	
	return false;
}

FText FVariableCustomization::GetCategory() const
{
	FString CommonCategory;

	for(int32 Index = 0; Index < Variables.Num(); Index++)
	{
		TWeakObjectPtr<UAnimNextVariableEntry> WeakVariable = Variables[Index];
		if (WeakVariable.IsValid())
		{
			const UAnimNextVariableEntry* PinnedVariable = WeakVariable.Get();
			check(PinnedVariable);
			if (Index == 0)
			{
				CommonCategory = PinnedVariable->GetVariableCategory(); 	
			}
			else
			{
				if (CommonCategory != PinnedVariable->GetVariableCategory())
				{
					return LOCTEXT("MultipleValuesLabel", "Multiple Values");
				}
			}
		}
	}

	return FText::FromString(CommonCategory);
}

void FVariableCustomization::SetCategory(const FText& InNewCategory, ETextCommit::Type InCommitType)
{
	if(const UUAFRigVMAsset* RigVMAsset = OuterRigVMAsset.Get())
	{
		UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(RigVMAsset);
		const FString CategoryString = InNewCategory.ToString();

		const FScopedTransaction Transaction(LOCTEXT("ChangeVariableCategory", "Change Variable Category"));

		if (!EditorData->VariableAndFunctionCategories.Contains(CategoryString))
		{
			EditorData->AddCategory(CategoryString);
		}
		
		SetCategory(CategoryString);
	}
}

bool FVariableCustomization::OnVerifyCategoryName(const FText& InText, FText& OutErrorMessage)
{
	if (InText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("CategoryEmpty", "Cannot add a category with an empty string.");
		return false;
	}
	else if (InText.ToString() == DefaultCategoryName)
	{
		OutErrorMessage = LOCTEXT("CategoryDefault", "Cannot add a category with name 'Default'.");
		return false;
	}
	return true;
}

void FVariableCustomization::OnCategorySelectionChanged(TSharedPtr<FText> ProposedSelection, ESelectInfo::Type)
{
	if (ProposedSelection.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeVariableCategory", "Change Variable Category"));
		
		const FString CategoryString = ProposedSelection->ToString();
		SetCategory(CategoryString);

		CategoryListView->ClearSelection();
		CategoryComboButton->SetIsOpen(false);
	}
}

void FVariableCustomization::PopulateCategories()
{
	Categories.Empty();
	if(const UUAFRigVMAsset* RigVMAsset = OuterRigVMAsset.Get())
	{
		UUAFRigVMAssetEditorData* EditorData = UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(RigVMAsset);
		Algo::Transform(EditorData->VariableAndFunctionCategories, Categories, [](const FString& InCategory)
		{
			return MakeShared<FText>(FText::FromString(InCategory));
		});
	}
	
	Categories.Add(MakeShared<FText>(FText::FromString(DefaultCategoryName)));
}

void FVariableCustomization::SetCategory(const FString& CategoryName)
{
	for(TWeakObjectPtr<UAnimNextVariableEntry> WeakVariable : Variables)
	{
		if(UAnimNextVariableEntry* PinnedVariable = Cast<UAnimNextVariableEntry>(WeakVariable.Get()))
		{
			const FString NewCategoryName = CategoryName == DefaultCategoryName ? TEXT("") : CategoryName;
			PinnedVariable->SetVariableCategory(NewCategoryName);
		}
	}
	
	PopulateCategories();
}
}

#undef LOCTEXT_NAMESPACE
