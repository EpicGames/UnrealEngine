// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"

class UUAFRigVMAsset;
class UAnimNextVariableEntry;

namespace UE::UAF::Editor
{

class FVariableCustomization : public IDetailCustomization
{
public:	
	FVariableCustomization();

private:
	/** Called when details should be customized */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	/**
	 * Called when details should be customized, this version allows the customization to store a weak ptr to the layout builder.
	 * This allows property changes to trigger a force refresh of the detail panel.
	 */
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& DetailBuilder) override;

	FText GetName() const;
	void SetName(const FText& InNewText, ETextCommit::Type InCommitType);
	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage);

	FText GetCategory() const;
	void SetCategory(const FText& InNewCategory, ETextCommit::Type InCommitType);	
	bool OnVerifyCategoryName(const FText& InText, FText& OutErrorMessage);
	void OnCategorySelectionChanged(TSharedPtr<FText> ProposedSelection, ESelectInfo::Type);
	void PopulateCategories();
	void SetCategory(const FString& CategoryName);

	TSharedPtr<SListView<TSharedPtr<FText>>> CategoryListView;
	TSharedPtr<SComboButton> CategoryComboButton;
	TArray<TSharedPtr<FText>> Categories;

	TArray<TWeakObjectPtr<UAnimNextVariableEntry>> Variables;
	TWeakObjectPtr<UUAFRigVMAsset> OuterRigVMAsset;

	TWeakPtr<IDetailLayoutBuilder> WeakLayoutBuilder;
	FString DefaultCategoryName;
};

}
