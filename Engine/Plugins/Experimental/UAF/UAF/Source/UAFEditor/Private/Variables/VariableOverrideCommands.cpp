// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariableOverrideCommands.h"

#define LOCTEXT_NAMESPACE "VariableOverrideCommands"

namespace UE::UAF
{

FVariableOverrideCommands::FVariableOverrideCommands()
	: TCommands<FVariableOverrideCommands>("UAF", LOCTEXT("UAFVariableOverrides", "Variable Overrides"), NAME_None, "AnimNextStyle")
{
}

void FVariableOverrideCommands::RegisterCommands()
{
	UI_COMMAND(OverrideVariable, "Override Variable", "Override this variable from its inherited value.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResetPropertyToDefault, "Reset to Default", "Revert this property to its inherited value, but retain its override.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ClearOverride, "Clear Override", "Revert this property to its inherited value, and clear its override.", EUserInterfaceActionType::Button, FInputChord());
}

}

#undef LOCTEXT_NAMESPACE
