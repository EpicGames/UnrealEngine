// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariableReferencePropertyCustomization.h"

#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "Variables/SVariablePickerCombo.h"
#include "UncookedOnlyUtils.h"
#include "RigVMCore/RigVMRegistry.h"
#include "Variables/AnimNextVariableReference.h"
#include "Variables/VariablePickerArgs.h"
#include "AnimNextRigVMAsset.h"
#include "ScopedTransaction.h"
#include "Component/AnimNextComponent.h"
#include "Factory/SystemFactory.h"
#include "Module/UAFSystemAssetData.h"
#include "UAF/UAFAssetFactory.h"
#include "Variables/AnimNextSoftVariableReference.h"

#define LOCTEXT_NAMESPACE "VariableReferencePropertyCustomization"

namespace UE::UAF::Editor
{

void FVariableReferencePropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	const FString AllowedType = PropertyHandle->GetMetaData("AllowedType");
	FilterType = FAnimNextParamType::FromString(AllowedType);

	bIsContextSensitive = PropertyHandle->GetMetaData("ShowAll") != TEXT("true");

	// Grab the objects from the details view itself as we will often be customizing traits, which have no object associated with their property handle
	TArray<TWeakObjectPtr<UObject>> Objects = InCustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();
	for (TWeakObjectPtr<UObject> WeakObject : Objects)
	{
		if (UObject* Object = WeakObject.Get())
		{
			UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(Object);
			if (Asset != nullptr)
			{
				FilterAssets.Add(Asset);
			}

			UUAFRigVMAsset* OuterAsset = Object->GetTypedOuter<UUAFRigVMAsset>();
			if (OuterAsset != nullptr)
			{
				FilterAssets.Add(OuterAsset);
			}
			
			UUAFComponent* Component = Cast<UUAFComponent>(Object);
			if (Component != nullptr)
			{
				const UUAFSystem* System = nullptr;
				
				if (Component->GetAssetData().IsValid())
				{
					if (Component->GetAssetData().GetScriptStruct()->IsChildOf<FUAFSystemFactoryAsset_System>())
					{
						System = Component->GetAssetData().Get<FUAFSystemFactoryAsset_System>().System.Get();
					}
					else
					{
						FUAFSystemFactoryParams FactoryParams = FSystemFactory::GetDefaultParamsForAsset(Component->GetAssetData());
						System = FSystemFactory::BuildSystem(FactoryParams.GetBuilder());
					}
				}
				
				if (System)
				{
					FilterAssets.Add(System);
				}
			}
		}
	}
	
	// Add any referenced assets
	TArray<TWeakObjectPtr<const UUAFRigVMAsset>> ReferencedAssets;
	for (TWeakObjectPtr<const UUAFRigVMAsset> FilterAsset : FilterAssets)
	{
		for (const UUAFRigVMAsset* ReferencedAsset : FilterAsset->GetReferencedVariableAssets())
		{
			ReferencedAssets.Add(ReferencedAsset);
		}
	}
	
	FilterAssets.Append(ReferencedAssets);

	FVariablePickerArgs Args;
	Args.OnVariablePicked = FOnVariablePicked::CreateLambda([this](const FAnimNextSoftVariableReference& InVariableReference, const FAnimNextParamType& InType)
	{
		FScopedTransaction Transaction(LOCTEXT("PickVariableReference", "Pick Variable Reference"));

		PropertyHandle->NotifyPreChange();

		PropertyHandle->EnumerateRawData([this, &InVariableReference](void* RawData, const int32 DataIndex, const int32 NumDatas)
		{
			SetValue(InVariableReference, RawData);
			return true;
		});

		PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		PropertyHandle->NotifyFinishedChangingProperties();

		UpdateCachedData();
	});

	if (FilterType.IsValid())
	{
		Args.CompatibleTypes = { FilterType };
	}

	Args.OnFilterVariableType = FOnFilterVariableType::CreateLambda([](const FAnimNextParamType& InParamType) -> EFilterVariableResult
	{
		if (InParamType.IsValid())
		{
			const FRigVMTemplateArgumentType RigVMType = InParamType.ToRigVMTemplateArgument();
			if (!RigVMType.IsValid() || FRigVMRegistry::Get().GetTypeIndex(RigVMType) == INDEX_NONE)
			{
				return EFilterVariableResult::Exclude;
			}
		}
		return EFilterVariableResult::Include;
	});

	if (FilterAssets.Num() > 0)
	{
		OnIsContextSensitiveDelegate = FOnIsContextSensitive::CreateLambda([this]()
		{
			return bIsContextSensitive;
		});

		Args.OnIsContextSensitive = &OnIsContextSensitiveDelegate;
		Args.OnContextSensitivityChanged = FOnContextSensitivityChanged::CreateLambda([this](bool bInIsContextSensitive)
		{
			bIsContextSensitive = bInIsContextSensitive;
		});
		
		Args.OnGetIncludeExcludeFilter = FOnGetIncludeExcludeFilter::CreateLambda([this](EAnimNextExportedVariableFlags& OutFlagInclusionFilter, EAnimNextExportedVariableFlags& OutFlagExclusionFilter)
		{
			// If we are context-sensitive and we have filter assets, we allow private variable references from those assets
			if (bIsContextSensitive && FilterAssets.Num() > 0)
			{
				OutFlagInclusionFilter = EAnimNextExportedVariableFlags::Declared;
				OutFlagExclusionFilter = EAnimNextExportedVariableFlags::Referenced;
			}
			else
			{
				OutFlagInclusionFilter = EAnimNextExportedVariableFlags::Declared | EAnimNextExportedVariableFlags::Public;
				OutFlagExclusionFilter = EAnimNextExportedVariableFlags::Referenced;
			}
		});
	}

	Args.OnFilterVariable = FOnFilterVariable::CreateLambda([this](const FAnimNextSoftVariableReference& InVariableReference)
	{
		if (FilterAssets.Num() == 0 || !bIsContextSensitive)
		{
			return EFilterVariableResult::Include;
		}

		for (TWeakObjectPtr<const UUAFRigVMAsset> WeakFilterAsset : FilterAssets)
		{
			if (const UUAFRigVMAsset* FilterAsset = WeakFilterAsset.Get())
			{
				FSoftObjectPath FilterAssetPath(FilterAsset);
				if (InVariableReference.GetSoftObjectPath() == FilterAssetPath)
				{
					return EFilterVariableResult::Include;
				}
			}
		}

		return EFilterVariableResult::Exclude;
	});

	InHeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SVariablePickerCombo)
		.PickerArgs(Args)
		.VariableName_Lambda([this]()
		{
			return bMultipleValues ? LOCTEXT("MultipleValues", "Multiple Values") : FText::FromName(CachedVariableReference.GetName());
		})
		.VariableTooltip_Lambda([this]()
		{
			return bMultipleValues ? LOCTEXT("MultipleValues", "Multiple Values") : FText::Format(LOCTEXT("VariableTooltipFormat", "{0}\n{1}"), FText::FromName(CachedVariableReference.GetName()), FText::FromString(CachedVariableReference.GetSoftObjectPath().ToString()));
		})
		.OnGetVariableReference_Lambda([this]()
		{
			return CachedVariableReference;
		})
		.OnGetVariableType_Lambda([this]()
		{
			return CachedType;
		})
	];

	UpdateCachedData();
}

void FVariableReferencePropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FVariableReferencePropertyCustomization::SetValue(const FAnimNextSoftVariableReference& InVariableReference, void* InValue) const
{
	FAnimNextVariableReference& VariableReference = *static_cast<FAnimNextVariableReference*>(InValue);
	VariableReference = FAnimNextVariableReference(InVariableReference);
}

FAnimNextSoftVariableReference FVariableReferencePropertyCustomization::GetValue(const void* InValue) const
{
	return FAnimNextSoftVariableReference(*static_cast<const FAnimNextVariableReference*>(InValue));
}

void FVariableReferencePropertyCustomization::UpdateCachedData()
{
	bMultipleValues = false;
	CachedVariableReference = FAnimNextSoftVariableReference();
	CachedType = FAnimNextParamType();

	TOptional<FAnimNextParamType> CommonType;
	TOptional<FAnimNextSoftVariableReference> CommonVariableReference;
	PropertyHandle->EnumerateConstRawData([this, &CommonType, &CommonVariableReference](const void* RawData, const int32 DataIndex, const int32 NumDatas)
	{
		FAnimNextSoftVariableReference VariableReference = GetValue(RawData);
		if(!CommonVariableReference.IsSet())
		{
			CommonVariableReference = FAnimNextSoftVariableReference(VariableReference);
		}
		else if(CommonVariableReference.GetValue() != VariableReference)
		{
			// No common variable reference
			CommonVariableReference = FAnimNextSoftVariableReference();
		}

		FAnimNextParamType Type = UncookedOnly::FUtils::FindVariableType(VariableReference);
		if(!CommonType.IsSet())
		{
			CommonType = Type;
		}
		else if(CommonType.GetValue() != Type)
		{
			// No common type
			CommonType = FAnimNextParamType();
		}
		return true;
	});

	if (CommonVariableReference.IsSet())
	{
		CachedVariableReference = CommonVariableReference.GetValue();
	}
	else
	{
		bMultipleValues = true;
	}

	if (CommonType.IsSet())
	{
		CachedType = CommonType.GetValue();
	}
}

}

#undef LOCTEXT_NAMESPACE
