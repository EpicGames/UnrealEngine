// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "Param/ParamType.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "Variables/AnimNextVariableReference.h"
#include "Variables/VariablePickerArgs.h"

class UUAFRigVMAsset;

namespace UE::UAF::Editor
{

class FVariableReferencePropertyCustomization : public IPropertyTypeCustomization
{
protected:
	virtual void SetValue(const FAnimNextSoftVariableReference& InVariableReference, void* InValue) const;
	virtual FAnimNextSoftVariableReference GetValue(const void* InValue) const;

private:
	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	void UpdateCachedData();

	TSharedPtr<IPropertyHandle> PropertyHandle;

	FAnimNextSoftVariableReference CachedVariableReference;

	FAnimNextParamType CachedType;

	FAnimNextParamType FilterType;

	TArray<TWeakObjectPtr<const UUAFRigVMAsset>> FilterAssets;

	FOnIsContextSensitive OnIsContextSensitiveDelegate;

	bool bIsContextSensitive = true;

	bool bMultipleValues = false;
};

}
