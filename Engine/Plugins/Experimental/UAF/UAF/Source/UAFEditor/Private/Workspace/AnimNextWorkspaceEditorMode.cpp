// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextWorkspaceEditorMode.h"

#include "AnimNextEditorModule.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "UAFCompilationScope.h"
#include "Component/AnimNextComponent.h"
#include "Common/UAFBrowserTabSummoner.h"
#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "HAL/IConsoleManager.h"
#include "IAssetCompilationHandler.h"
#include "InteractiveToolManager.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "Modules/ModuleManager.h"
#include "RigVMCommands.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Toolkits/BaseToolkit.h"
#include "ToolMenus.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "UAF/Workspace/RewindDebuggerExtension.h"
#include "UAF/Workspace/SDebugObjectSelector.h"
#include "UObject/UObjectIterator.h"
#include "UncookedOnlyUtils.h"
#include "WidgetDrawerConfig.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextWorkspaceEditorMode)

#define LOCTEXT_NAMESPACE "AnimNextWorkspaceEditorMode"

const FEditorModeID UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace("AnimNextWorkspace");

UAnimNextWorkspaceEditorMode::UAnimNextWorkspaceEditorMode()
{
	Info = FEditorModeInfo(UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace,
		LOCTEXT("AnimNextWorkspaceEditorModeName", "AnimNextWorkspaceEditorMode"),
		FSlateIcon(),
		false);
}

void UAnimNextWorkspaceEditorMode::Enter()
{
	Super::Enter();
	
	TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = GetWorkspaceEditor();
	if(!WorkspaceEditor.IsValid())
	{
		return;
	}
	
	WorkspaceEditor->OnFocusedDocumentChanged().AddUObject(this, &UAnimNextWorkspaceEditorMode::HandleFocusedDocumentChanged);

	TArray<UObject*> Assets;
	WorkspaceEditor->GetOpenedAssets<UUAFRigVMAsset>(Assets);
	for(UObject* Asset : Assets)
	{
		UUAFRigVMAsset* AnimNextRigVMAsset = static_cast<UUAFRigVMAsset*>(Asset);
		UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(AnimNextRigVMAsset);

		TWeakObjectPtr<UUAFRigVMAssetEditorData> WeakEditorData = EditorData;
		EditorData->RigVMCompiledEvent.RemoveAll(this);
		EditorData->RigVMCompiledEvent.AddUObject(this, &UAnimNextWorkspaceEditorMode::HandleRigVMCompiledEvent);
		EditorData->RigVMGraphModifiedEvent.RemoveAll(this);
		EditorData->RigVMGraphModifiedEvent.AddUObject(this, &UAnimNextWorkspaceEditorMode::HandleRigVMModifiedEvent, WeakEditorData);
		EditorData->RecompileRequiredChangedEvent.RemoveAll(this);
		EditorData->RecompileRequiredChangedEvent.AddUObject(this, &UAnimNextWorkspaceEditorMode::UpdateCompileStatus);

		WeakAssets.Add(Asset);
	}

	// Add UAF Browser Drawer
	IConsoleVariable* EnableUAFBrowserCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("a.UAF.EnableUAFBrowser"));
	if (EnableUAFBrowserCVar && EnableUAFBrowserCVar->GetBool())
	{
		using namespace UE::UAF::Editor;

		FWidgetDrawerConfig UAFDrawer(FUAFBrowserTabSummoner::StatusBarSystemDrawerID);
		TWeakPtr<UE::Workspace::IWorkspaceEditor> WeakEditor = WorkspaceEditor;

		if (!UAFBrowserStatusbarWidget)
		{
			FUAFBrowserTabSummoner UAFBrowserTabSummoner(WorkspaceEditor, ETabState::StatusbarTab, FUAFBrowserTabSummoner::DrawerTabID, LOCTEXT("UAFBrowserDrawerLabel", "UAF Browser (Drawer)"));
			FWorkflowTabSpawnInfo SpawnInfo;
			UAFBrowserStatusbarWidget = UAFBrowserTabSummoner.CreateTabBody(SpawnInfo);
		}

		TWeakPtr<SWidget> WeakUAFBrowserStatusbarWidget = UAFBrowserStatusbarWidget;
		UAFDrawer.GetDrawerContentDelegate.BindLambda([WeakEditor, WeakUAFBrowserStatusbarWidget]()
		{
			if (TSharedPtr<UE::Workspace::IWorkspaceEditor> EditorPtr = WeakEditor.Pin())
			{
				if (TSharedPtr<SWidget> UAFBrowserStatusbarWidgetPinned = WeakUAFBrowserStatusbarWidget.Pin())
				{
					return UAFBrowserStatusbarWidgetPinned.ToSharedRef();
				}
			}

			return SNullWidget::NullWidget;
		});
		UAFDrawer.OnDrawerOpenedDelegate.BindLambda([WeakEditor](FName StatusBarWithDrawerName)
		{
			// @TODO: DarenC - Find best focus targets, these have issues dismissing via hotkey, but without these we have issues dismissing via click
			//if (TSharedPtr<UE::Workspace::IWorkspaceEditor> EditorPtr = WeakEditor.Pin())
			//{
			//	FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), EditorPtr->GetExternalEditorWidget(FUAFBrowserTabSummoner::StatusBarSystemDrawerID));
			//}
		});
		UAFDrawer.OnDrawerDismissedDelegate.BindLambda([WeakEditor](const TSharedPtr<SWidget>& NewlyFocusedWidget)
		{
			// @TODO: DarenC - Find best focus targets, these have issues dismissing via hotkey, but without these we have issues dismissing via click
			//if (TSharedPtr<UE::Workspace::IWorkspaceEditor> EditorPtr = WeakEditor.Pin())
			//{
			//	FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), EditorPtr->GetDetailsView());
			//}
		});
		UAFDrawer.ButtonText = LOCTEXT("StatusBar_UAFBrowser", "UAF Browser");
		UAFDrawer.ToolTipText = LOCTEXT("StatusBar_UAFBrowserToolTip", "Opens UAF Browser (Ctrl+Shift+Space).");
				
		// @TODO: DarenC - temp, see if we have real icon
		UAFDrawer.Icon = FAppStyle::Get().GetBrush("ContentBrowser.TabIcon");
		WorkspaceEditor->RegisterDrawer(MoveTemp(UAFDrawer), 1);
	}

	UpdateCompileStatus();
	
	RewindDebuggerExtension = MakeShared<UE::UAF::Editor::FRewindDebuggerWorkspaceExtension>();
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, RewindDebuggerExtension.Get());
	
	if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(WorkspaceEditor->GetToolMenuToolbarName()))
	{
		FToolMenuSection& DebugSection = Menu->AddSection("Debug", FText::GetEmpty(), FToolMenuInsert("WorkspaceOperations", EToolMenuInsertType::After));

		TSharedRef<UE::UAF::Editor::SDebugObjectSelector> DebugObjectSelectorWidget = SNew(UE::UAF::Editor::SDebugObjectSelector)
			.GetDebugObjects_UObject(this, &UAnimNextWorkspaceEditorMode::GetDebugObjects);

		DebugObjectSelector = DebugObjectSelectorWidget;

		DebugSection.AddEntry(FToolMenuEntry::InitWidget("DebugObjectSelector", DebugObjectSelectorWidget, FText::GetEmpty(), true, false, true));
	}
}

void UAnimNextWorkspaceEditorMode::Exit()
{
	TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = GetWorkspaceEditor();
	if(WorkspaceEditor.IsValid())
	{
		const TSharedRef<FUICommandList>& CommandList = WorkspaceEditor->GetToolkitCommands();
		const UE::UAF::FRigVMCommands& RigVMCommands = UE::UAF::FRigVMCommands::Get();
		CommandList->UnmapAction(RigVMCommands.Compile);
		CommandList->UnmapAction(RigVMCommands.AutoCompile);

		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(WorkspaceEditor->GetToolMenuToolbarName()))
		{
			Menu->RemoveSection("Debug");
		}
	}

	Super::Exit();

	for (TPair<FObjectKey, TSharedRef<UE::UAF::Editor::IAssetCompilationHandler>>& AssetCompiler : AssetCompilers)
	{
		TSharedRef<UE::UAF::Editor::IAssetCompilationHandler> AssetCompilerRef = AssetCompiler.Value;
		AssetCompilerRef->OnCompileStatusChanged().Unbind();
	}

	AssetCompilers.Reset();

	for(TObjectKey<UObject> Asset : WeakAssets)
	{
		UUAFRigVMAsset* AnimNextRigVMAsset = Cast<UUAFRigVMAsset>(Asset.ResolveObjectPtr());
		if(AnimNextRigVMAsset == nullptr)
		{
			continue;;
		}
		
		UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(AnimNextRigVMAsset);

		EditorData->RigVMCompiledEvent.RemoveAll(this);
		EditorData->RigVMGraphModifiedEvent.RemoveAll(this);
		EditorData->RecompileRequiredChangedEvent.RemoveAll(this);
	}

	WeakAssets.Reset();

	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, RewindDebuggerExtension.Get());
}

void UAnimNextWorkspaceEditorMode::BindCommands()
{
	Super::BindCommands();

	TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = GetWorkspaceEditor();
	if(!WorkspaceEditor.IsValid())
	{
		return;
	}

	TSharedRef<FUICommandList> ToolkitCommands = WorkspaceEditor->GetToolkitCommands();

	const UE::UAF::FRigVMCommands& RigVMCommands = UE::UAF::FRigVMCommands::Get();
	ToolkitCommands->MapAction(RigVMCommands.Compile,
		FExecuteAction::CreateUObject(this, &UAnimNextWorkspaceEditorMode::HandleCompile),
		FCanExecuteAction::CreateUObject(this, &UAnimNextWorkspaceEditorMode::CanCompile));

	ToolkitCommands->MapAction(RigVMCommands.AutoCompile,
		FExecuteAction::CreateUObject(this, &UAnimNextWorkspaceEditorMode::HandleAutoCompile),
		FCanExecuteAction::CreateUObject(this, &UAnimNextWorkspaceEditorMode::CanCompile),
		FIsActionChecked::CreateUObject(this, &UAnimNextWorkspaceEditorMode::IsAutoCompileChecked));
}

void UAnimNextWorkspaceEditorMode::HandleCompile()
{
	TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = GetWorkspaceEditor();
	if(!WorkspaceEditor.IsValid())
	{
		return;
	}

	TArray<UUAFRigVMAsset*> AssetsToCompile;

	FText JobName = LOCTEXT("DefaultJobName", "Job");
	if(UObject* FocusedDocument = WorkspaceEditor->GetFocusedWorkspaceDocument().GetObject())
	{
		// Find the outer asset for this document
		UObject* FocussedObject = FocusedDocument;
		while(FocussedObject && (!FocussedObject->IsAsset() || FocussedObject->IsPackageExternal()))
		{
			FocussedObject = FocussedObject->GetOuter();
		}

		if(UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(FocussedObject))
		{
			JobName = FText::FromName(Asset->GetFName());
			AssetsToCompile.Add(Asset);
		}
	}

	if(AssetsToCompile.Num() == 0)
	{
		return;
	}

	// Open a batch compile scope
	UE::UAF::UncookedOnly::FCompilationScope CompileScope(JobName, AssetsToCompile);

	UE::UAF::Editor::FAnimNextEditorModule& AnimNextEditorModule = FModuleManager::LoadModuleChecked<UE::UAF::Editor::FAnimNextEditorModule>("UAFEditor");
	for(UObject* AssetToCompile : AssetsToCompile)
	{
		if(TSharedPtr<UE::UAF::Editor::IAssetCompilationHandler> FoundCompiler = GetAssetCompiler(AssetToCompile))
		{
			FoundCompiler->Compile(WorkspaceEditor.ToSharedRef(), AssetToCompile);
		}
	}
}

bool UAnimNextWorkspaceEditorMode::CanCompile() const
{
	TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = GetWorkspaceEditor();
	return WorkspaceEditor.IsValid();
}

void UAnimNextWorkspaceEditorMode::HandleAutoCompile()
{
	TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = GetWorkspaceEditor();
	if (WorkspaceEditor.IsValid())
	{
		if (UUAFRigVMAsset* FocusedAsset = Cast<UUAFRigVMAsset>(WorkspaceEditor->GetFocusedAsset()))
		{
			UUAFRigVMAssetEditorData* FocusedAssetEditorData = CastChecked<UUAFRigVMAssetEditorData>(FocusedAsset->EditorData);
			FocusedAssetEditorData->SetAutoVMRecompile(!FocusedAssetEditorData->GetAutoVMRecompile());
		}
	}
}

bool UAnimNextWorkspaceEditorMode::IsAutoCompileChecked() const
{
	TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = GetWorkspaceEditor();
	if (WorkspaceEditor.IsValid())
	{
		if (UUAFRigVMAsset* FocusedAsset = Cast<UUAFRigVMAsset>(WorkspaceEditor->GetFocusedAsset()))
		{
			UUAFRigVMAssetEditorData* FocusedAssetEditorData = CastChecked<UUAFRigVMAssetEditorData>(FocusedAsset->EditorData);
			return FocusedAssetEditorData->GetAutoVMRecompile();
		}
	}

	return false;
}

void UAnimNextWorkspaceEditorMode::HandleFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument)
{
	UObject* InObject = InDocument.GetObject();
	
	if(InObject == nullptr)
	{
		return;
	}

	UUAFRigVMAsset* AnimNextRigVMAsset = Cast<UUAFRigVMAsset>(InObject);
	if(AnimNextRigVMAsset == nullptr)
	{
		AnimNextRigVMAsset = InObject->GetTypedOuter<UUAFRigVMAsset>();
	}
	
	if(AnimNextRigVMAsset == nullptr)
	{
		return;
	}

	// Subscribe to asset compilation/modification
	UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(AnimNextRigVMAsset);
	TWeakObjectPtr<UUAFRigVMAssetEditorData> WeakEditorData = EditorData;
	EditorData->RigVMCompiledEvent.RemoveAll(this);
	EditorData->RigVMCompiledEvent.AddUObject(this, &UAnimNextWorkspaceEditorMode::HandleRigVMCompiledEvent);
	EditorData->RigVMGraphModifiedEvent.RemoveAll(this);
	EditorData->RigVMGraphModifiedEvent.AddUObject(this, &UAnimNextWorkspaceEditorMode::HandleRigVMModifiedEvent, WeakEditorData);
	EditorData->RecompileRequiredChangedEvent.RemoveAll(this);
	EditorData->RecompileRequiredChangedEvent.AddUObject(this, &UAnimNextWorkspaceEditorMode::UpdateCompileStatus);

	WeakAssets.Add(AnimNextRigVMAsset);

	UpdateCompileStatus();
}

void UAnimNextWorkspaceEditorMode::UpdateCompileStatus()
{
	TSharedPtr<UE::Workspace::IWorkspaceEditor> WorkspaceEditor = GetWorkspaceEditor();
	if(!WorkspaceEditor.IsValid())
	{
		return;
	}

	UUAFRigVMAsset* Asset = WorkspaceEditor->GetFocusedWorkspaceDocument().Export.GetFirstObjectOfType<UUAFRigVMAsset>();

	if (Asset)
	{
		if (TSharedPtr<UE::UAF::Editor::IAssetCompilationHandler> FoundCompiler = GetAssetCompiler(Asset))
		{
			UE::UAF::Editor::ECompileStatus AssetCompileStatus = FoundCompiler->GetCompileStatus(WorkspaceEditor.ToSharedRef(), Asset);
			CompileStatus = AssetCompileStatus;
		}
	}
	else
	{
		CompileStatus = UE::UAF::Editor::ECompileStatus::Unknown;
	}
}

void UAnimNextWorkspaceEditorMode::HandleRigVMCompiledEvent(UObject* InAsset, URigVM* InVM, FRigVMExtendedExecuteContext& InExtendedExecuteContext)
{
	UpdateCompileStatus();
}

void UAnimNextWorkspaceEditorMode::HandleRigVMModifiedEvent(ERigVMGraphNotifType InType, URigVMGraph* InGraph, UObject* InSubject, TWeakObjectPtr<UUAFRigVMAssetEditorData> WeakEditorData)
{
	if(InGraph == nullptr)
	{
		return;
	}

	UUAFRigVMAssetEditorData* EditorData = InGraph->GetTypedOuter<UUAFRigVMAssetEditorData>();

	// Some notifies will not produce a valid outer editor data e.g. deleting or changing a RigVM function - so try to retrieve it from the weak-object ptr (set when registering delegate)
	if (!EditorData)
	{
		EditorData = WeakEditorData.Get();
	}
	
	if (!ensureMsgf(EditorData, TEXT("Failed to find UAF asset for RigVM modification, asset may not update correctly. It also may fail to propagate dirty status to other assets.")))
	{
		return;
	}

	if(EditorData->IsDirtyForRecompilation())
	{
		CompileStatus = UE::UAF::Editor::ECompileStatus::Dirty;
	}
}

TSharedPtr<UE::Workspace::IWorkspaceEditor> UAnimNextWorkspaceEditorMode::GetWorkspaceEditor() const
{
	if (const UContextObjectStore* ContextObjectStore = GetModeManager()->GetInteractiveToolsContext()->ContextObjectStore)
	{
		if (UAssetEditorToolkitMenuContext* Context = ContextObjectStore->FindContext<UAssetEditorToolkitMenuContext>())
		{
			if (TSharedPtr<FAssetEditorToolkit> ToolkitShared = Context->Toolkit.Pin())
			{
				return StaticCastSharedPtr<UE::Workspace::IWorkspaceEditor>(ToolkitShared);
			}
		}
	}

	return nullptr;
}

TSharedPtr<UE::UAF::Editor::IAssetCompilationHandler> UAnimNextWorkspaceEditorMode::GetAssetCompiler(UObject* InAsset)
{
	if(TSharedRef<UE::UAF::Editor::IAssetCompilationHandler>* FoundCompiler = AssetCompilers.Find(InAsset))
	{
		return *FoundCompiler;
	}

	UE::UAF::Editor::FAnimNextEditorModule& AnimNextEditorModule = FModuleManager::LoadModuleChecked<UE::UAF::Editor::FAnimNextEditorModule>("UAFEditor");
	if(const UE::UAF::Editor::FAssetCompilationHandlerFactoryDelegate* FoundFactory = AnimNextEditorModule.FindAssetCompilationHandlerFactory(InAsset->GetClass()))
	{
		TSharedRef<UE::UAF::Editor::IAssetCompilationHandler> NewCompiler = AssetCompilers.Add(InAsset, FoundFactory->Execute(InAsset));
		NewCompiler->OnCompileStatusChanged().BindUObject(this, &UAnimNextWorkspaceEditorMode::UpdateCompileStatus);
		return NewCompiler;
	}

	return nullptr;
}

void UAnimNextWorkspaceEditorMode::GetDebugObjects(TArray<UObject*>& OutObjects)
{
	for (TObjectIterator<UUAFComponent> It; It; ++It)
	{
		UUAFComponent* Object = *It;

		UWorld* World = Object->GetWorld();
		if (!World || Object->HasAnyFlags(RF_ClassDefaultObject))
		{
			continue;
		}

		bool bPendingKill = false;
		UObject* Outer = Object;
		do
		{
			bPendingKill = !IsValid(Outer);
			Outer = Outer->GetOuter();
		}
		while (!bPendingKill && Outer != nullptr);

		if (bPendingKill)
		{
			continue;
		}

		if (World->WorldType != EWorldType::PIE)
		{
			continue;
		}
		
		OutObjects.Add(Object);
	}
}

TSharedPtr<UE::UAF::Editor::FRewindDebuggerWorkspaceExtension> UAnimNextWorkspaceEditorMode::GetRewindDebuggerExtension()
{
	return RewindDebuggerExtension;
}

TSharedPtr<UE::UAF::Editor::IDebugObjectSelector> UAnimNextWorkspaceEditorMode::GetDebugObjectSelector()
{
	return DebugObjectSelector;
}

#undef LOCTEXT_NAMESPACE
