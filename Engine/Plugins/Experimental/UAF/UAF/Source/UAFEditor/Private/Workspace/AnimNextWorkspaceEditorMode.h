// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextWorkspaceState.h"
#include "EdMode.h"
#include "IAssetCompilationHandler.h"

#include "AnimNextWorkspaceEditorMode.generated.h"

class UAnimNextWorkspaceSchema;
class UUAFRigVMAssetEditorData;
class URigVM;
class URigVMGraph;
struct FRigVMExtendedExecuteContext;
enum class ERigVMGraphNotifType : uint8;

namespace UE::Workspace
{
struct FWorkspaceDocument;
class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{
	class FAssetWizard;
	class FRewindDebuggerWorkspaceExtension;
	class IDebugObjectSelector;
}

UCLASS(Transient)
class UAnimNextWorkspaceEditorMode : public UEdMode
{
	GENERATED_BODY()

public:
	const static FEditorModeID EM_AnimNextWorkspace;

	UAnimNextWorkspaceEditorMode();

	// Get the current compilation status
	UE::UAF::Editor::ECompileStatus GetLatestCompileStatus() const { return CompileStatus; }

	// Get the currently stored workspace state
	const FAnimNextWorkspaceState& GetState() const { return State; } 

	TSharedPtr<UE::UAF::Editor::FRewindDebuggerWorkspaceExtension> GetRewindDebuggerExtension();

	TSharedPtr<UE::UAF::Editor::IDebugObjectSelector> GetDebugObjectSelector();

	bool IsAutoCompileChecked() const;

private:
	// UEdMode interface
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override { return true; }
	virtual void Enter() override;
	virtual void Exit() override;
	virtual void BindCommands() override;
	virtual bool UsesToolkits() const override { return false; }
	void HandleCompile();
	bool CanCompile() const;
	void HandleAutoCompile();

	// Updates the compile status. Scans all opened assets in the workspace.
	void UpdateCompileStatus();

	// Handle an asset in the workspace getting compiled 
	void HandleRigVMCompiledEvent(UObject* InAsset, URigVM* InVM, FRigVMExtendedExecuteContext& InExtendedExecuteContext);

	// Handle an asset getting modified
	void HandleRigVMModifiedEvent(ERigVMGraphNotifType InType, URigVMGraph* InGraph, UObject* InSubject, TWeakObjectPtr<UUAFRigVMAssetEditorData> WeakEditorData);

	// Handle the document focus changing
	void HandleFocusedDocumentChanged(const UE::Workspace::FWorkspaceDocument& InDocument); 

	TSharedPtr<UE::Workspace::IWorkspaceEditor> GetWorkspaceEditor() const;

	// Lazily construct an asset compiler for the supplied asset
	TSharedPtr<UE::UAF::Editor::IAssetCompilationHandler> GetAssetCompiler(UObject* InAsset);

	void GetDebugObjects(TArray<UObject*>& OutObjects);

	friend UAnimNextWorkspaceSchema;
	friend UE::UAF::Editor::FAssetWizard;

	FAnimNextWorkspaceState State;

	// Asset compilers for all current assets 
	TMap<FObjectKey, TSharedRef<UE::UAF::Editor::IAssetCompilationHandler>> AssetCompilers;

	// All assets that we are currently tracking for compilation status
	TSet<TObjectKey<UObject>> WeakAssets;

	// Current compilation status
	UE::UAF::Editor::ECompileStatus CompileStatus = UE::UAF::Editor::ECompileStatus::Unknown;

	TSharedPtr<UE::UAF::Editor::FRewindDebuggerWorkspaceExtension> RewindDebuggerExtension;

	TSharedPtr<UE::UAF::Editor::IDebugObjectSelector> DebugObjectSelector;

	// UAF browser status bar widget, used to store a custom drawer next to the content browser
	TSharedPtr<SWidget> UAFBrowserStatusbarWidget = nullptr;
};
