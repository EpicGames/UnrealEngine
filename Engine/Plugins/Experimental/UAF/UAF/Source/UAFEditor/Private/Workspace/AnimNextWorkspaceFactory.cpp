// Copyright Epic Games, Inc. All Rights Reserved.

#include "Workspace/AnimNextWorkspaceFactory.h"
#include "AnimNextWorkspaceSchema.h"
#include "AssetTypeCategories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextWorkspaceFactory)

#define LOCTEXT_NAMESPACE "AnimNextWorkspaceFactory"

UAnimNextWorkspaceFactory::UAnimNextWorkspaceFactory()
{
	SetSchemaClass(UAnimNextWorkspaceSchema::StaticClass());
}

bool UAnimNextWorkspaceFactory::ConfigureProperties()
{
	return true;
}

FText UAnimNextWorkspaceFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "UAF Workspace");
}

uint32 UAnimNextWorkspaceFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

const TArray<FText>& UAnimNextWorkspaceFactory::GetMenuCategorySubMenus() const
{
	static const TArray<FText> Categories = { LOCTEXT("UAFSubMenu", "Animation Framework") };
	return Categories;
}

bool UAnimNextWorkspaceFactory::ShouldShowInNewMenu() const
{
	return false;
}

#undef LOCTEXT_NAMESPACE
