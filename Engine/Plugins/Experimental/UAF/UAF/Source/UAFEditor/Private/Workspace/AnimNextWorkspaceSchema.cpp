// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextWorkspaceSchema.h"

#include "AnimNextEditorModule.h"
#include "AnimNextWorkspaceEditorMode.h"
#include "AssetEditorModeManager.h"
#include "IWorkspaceEditor.h"
#include "StructUtils/InstancedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextWorkspaceSchema)

#define LOCTEXT_NAMESPACE "AnimNextWorkspaceSchema"

FText UAnimNextWorkspaceSchema::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "UAF Workspace");
}

TConstArrayView<FTopLevelAssetPath> UAnimNextWorkspaceSchema::GetSupportedAssetClassPaths() const
{
	const UE::UAF::Editor::FAnimNextEditorModule& Module = FModuleManager::Get().LoadModuleChecked<UE::UAF::Editor::FAnimNextEditorModule>("UAFEditor");

	return Module.SupportedAssetClasses;
}

void UAnimNextWorkspaceSchema::OnSaveWorkspaceState(TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor, FInstancedStruct& OutWorkspaceState) const
{
	UAnimNextWorkspaceEditorMode* EditorMode = Cast<UAnimNextWorkspaceEditorMode>(InWorkspaceEditor->GetEditorModeManager().GetActiveScriptableMode(UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace));
	if(EditorMode == nullptr)
	{
		return;
	}

	OutWorkspaceState = FInstancedStruct::Make(EditorMode->State);
}

void UAnimNextWorkspaceSchema::OnLoadWorkspaceState(TSharedRef< UE::Workspace::IWorkspaceEditor> InWorkspaceEditor, const FInstancedStruct& InWorkspaceState) const
{
	// Activate and set up our mode
	InWorkspaceEditor->GetEditorModeManager().ActivateMode(UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace);

	UAnimNextWorkspaceEditorMode* EditorMode = Cast<UAnimNextWorkspaceEditorMode>(InWorkspaceEditor->GetEditorModeManager().GetActiveScriptableMode(UAnimNextWorkspaceEditorMode::EM_AnimNextWorkspace));
	if(EditorMode == nullptr)
	{
		return;
	}

	const FAnimNextWorkspaceState* State = InWorkspaceState.GetPtr<FAnimNextWorkspaceState>();
	if(State == nullptr)
	{
		return;
	}

	EditorMode->State = *State;
}

#undef LOCTEXT_NAMESPACE
