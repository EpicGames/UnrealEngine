// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkspaceSchema.h"
#include "UAF/Viewport/ViewportSceneDescription.h"

#include "AnimNextWorkspaceSchema.generated.h"

// Workspace schema allowing all asset types
UCLASS()
class UAnimNextWorkspaceSchema : public UWorkspaceSchema
{
	GENERATED_BODY()

	// UWorkspaceSchema interface
	virtual FText GetDisplayName() const override;
	virtual TConstArrayView<FTopLevelAssetPath> GetSupportedAssetClassPaths() const override;
	virtual void OnSaveWorkspaceState(TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor, FInstancedStruct& OutWorkspaceState) const override;
	virtual void OnLoadWorkspaceState(TSharedRef<UE::Workspace::IWorkspaceEditor> InWorkspaceEditor, const FInstancedStruct& InWorkspaceState) const override;
	virtual bool SupportsViewport() const { return true; };
	virtual TObjectPtr<UWorkspaceViewportSceneDescription> CreateSceneDescription() const { return NewObject<UUAFViewportSceneDescription>(); }
	virtual bool SupportsMultipleAssets() const { return false; }
	virtual bool HasPersistentRootAsset() const { return true; }
	virtual bool CanSaveWorkspace() const { return false; }
};