// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNextWorkspaceState.generated.h"

// Custom persistent workspace state 
USTRUCT()
struct FAnimNextWorkspaceState
{
	GENERATED_BODY()
};