// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAssetCompilationHandler.h"

#define UE_API UAFEDITOR_API

namespace UE::UAF::Editor
{

// Asset compiler that can compile assets based on UUAFRigVMAsset
class FAssetCompilationHandler : public IAssetCompilationHandler
{
public:
	UE_API explicit FAssetCompilationHandler(const UObject* InAsset);

protected:
	// IAssetCompilationHandler interface
	UE_API virtual void Compile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, UObject* InAsset) override;
	UE_API virtual void SetAutoCompile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, UObject* InAsset, bool bInAutoCompile) override;
	UE_API virtual bool GetAutoCompile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, const UObject* InAsset) const override;
	UE_API virtual ECompileStatus GetCompileStatus(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, const UObject* InAsset) const override;
};

}

#undef UE_API
