// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorViewportClient.h"
#include "Engine/StreamableManager.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Toolkits/AssetEditorToolkit.h"

#define UE_API UAFEDITOR_API

class FAssetEditorToolkit;
class FSceneViewport;
struct FStreamableHandle;

namespace UE::UAF::Editor
{
	class IUAFAssetPreview;
};

/**
 * Note: In this experimental state we frequently mention an 'IAssetPreviewCustomization' API. At a high level, imagine this as an in-progress future API.
 * Similar to detail customizations where you can register a widget for some particular type & it automatically gets used (Ex: In content browser previews).
 * Our goal is to make a MVP that fully explores the space of what we want before we integrate with this API.
 */
namespace UE::UAF::Editor
{


/**
 * Experimental. May be removed whenever we switch to the future generic content browser IAssetPreviewCustomization API.
 *
 * Base struct for factories that can create IUAFAssetPreview widgets.
 * 
 * @TODO: Reconsider the type constraint. It is okay but some factories may support multiple types. 
 * Should we allow for multiple factories for the same type based on a priority system?
 */
struct FUAFAssetPreviewFactory
{
public:

	/** Callback used to create the preview for the given asset data. */
	UE_API virtual TSharedRef<IUAFAssetPreview> CreateAssetPreviewWidget(TSharedPtr<FAssetEditorToolkit> InAssetEditorToolkit, const FAssetData& InAssetData) const = 0;

	/** Get the preview type this factory is used for. */
	UE_API virtual const UStruct* GetPreviewType() const = 0;

	virtual ~FUAFAssetPreviewFactory() = default;
};


/** 
 * Experimental. May be removed whenever we switch to the future generic content browser IAssetPreviewCustomization API. 
 * 
 * Base class for asset preview widgets. Currently previews are restricted to UAF, but may not be in the future.
 * 
 * @TODO: This class currently supports individual async loading, but users may want to async preload a set of assets for responsiveness.
 * @TODO: Once we use the future content browser API, consider removing the "UAF" aspect of this class. Maybe move to engine as IAsyncAssetPreview? Need editor buy in first.
 */
class IUAFAssetPreview : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(IUAFAssetPreview) {}
	SLATE_END_ARGS()

	UE_API virtual void Construct(const FArguments& InArgs, TSharedPtr<FAssetEditorToolkit> InAssetEditorToolkit, const FAssetData& InAssetData);

public:

	/** 
	 * Callback for when another asset of the same type is selected. Will load the asset async before updating preview.
	 * When different assets are selected this preview should be replaced. 
	 */
	UE_API virtual void OnSameTypeAssetSelectedAsync(const FAssetData& InAssetData);

	/** Utility method to get the current preview type this widget is associated with. */
	UE_API virtual const UStruct* GetAssetPreviewType() const = 0;

	/** Utility method to load an asset async with function callback */
	UE_API virtual void RequestAsyncLoad(const FSoftObjectPath& SoftObject, TFunction<void()>&& Callback);

	/** Utility method to load an asset async with a delegate callback */
	UE_API virtual void RequestAsyncLoad(const FSoftObjectPath& SoftObject, FStreamableDelegate DelegateToCall);

	/** Utility method to cancel async load of asset to preview */
	UE_API virtual void CancelAsyncLoad();

protected:

	/** 
	 * Callback for when another asset of the same type is selected & loaded. 
	 * 
	 * @return True if we were able to set up a valid preview for the given asset
	 */
	UE_API virtual bool OnSameTypeAssetSelected(const FAssetData& InAssetData) = 0;

	/** Called to show a loading widget when the asset is loading, and a preview widget when the asset is done loading. */
	UE_API virtual void OnConstructAsyncPreviewWidget(const FAssetData& InAssetData);

	/** Callback used to construct preview widget when async loading is occuring. Results should be stored in CachedLoadingAssetPreviewWidget. */
	UE_API virtual TSharedRef<SWidget> OnConstructLoadingAssetPreviewWidget();

	/** 
	 * Callback used to construct preview widget whenever async loading is done or an asset is already loaded. 
	 * Implementers are encouraged to cache their results on first create to avoid widget recreation overhead. 
	 */
	UE_API virtual TSharedRef<SWidget> OnConstructAssetPreviewWidget();

protected:

	/** Pointer back to the editor that owns us */
	TWeakPtr<FAssetEditorToolkit> AssetEditorToolkitPtr;

	/** Last asset data previewed by this widget */
	FAssetData CachedCurrentAssetData;

	/** Handle for currently streaming asset, we load assets async for previews by default. */
	TSharedPtr<FStreamableHandle> StreamingHandle;

	/** Path for currently streaming asset */
	FSoftObjectPath StreamingObjectPath;

	/** Cached loading widget. Saved so we can communicate with it and don't need to re-create it uncessarily. */
	TSharedPtr<SWidget> CachedLoadingAssetPreviewWidget;
};


} // namespace UE::UAF::Editor

#undef UE_API
