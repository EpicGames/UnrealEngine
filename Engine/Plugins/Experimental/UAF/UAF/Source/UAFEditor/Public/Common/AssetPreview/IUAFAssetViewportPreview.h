// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/AssetPreview/IUAFAssetPreview.h"
#include "AdvancedPreviewScene.h"
#include "SWorkspaceViewport.h"

#define UE_API UAFEDITOR_API

class FEditorViewportClient;
class FSceneViewport;

/**
 * Note: In this experimental state we frequently mention an 'IAssetPreviewCustomization' API. At a high level, imagine this as an in-progress future API.
 * Similar to detail customizations where you can register a widget for some particular type & it automatically gets used (Ex: In content browser previews).
 * Our goal is to make a MVP that fully explores the space of what we want before we integrate with this API.
 */
namespace UE::UAF::Editor
{


/** 
 * Experimental. May be removed / relocated whenever we switch to the future generic content browser IAssetPreviewCustomization API. 
 * 
 * Base class for asset previews that want to use an viewport and don't want to roll their own viewport implementation
 * @TODO: Same consideration as above. Later on consider removing "UAF" aspect from this.
 */
class IUAFAssetViewportPreview : public IUAFAssetPreview
{

protected:

	//~ Begin IUAFAssetPreview Interface
	UE_API virtual TSharedRef<SWidget> OnConstructAssetPreviewWidget() override;
	//~ End IUAFAssetPreview Interface

protected:

	/** 
	 * Require child classes to customize preview scene / viewport. These are passed together as viewport customizations
	 * may depend on the preview scene.
	 * 
	 * Note: Params are passed for convience. They are technically not needed since child classes have access.
	 * 
	 * @param InPreviewScene - Same as our local preview scene. Implementation should set this up to some meaningful asset preview.
	 * @param InEditorViewportClient - Viewport client for camera setup and other controls. 
	 */
	UE_API virtual void OnCustomizePreviewScene(FAdvancedPreviewScene& InPreviewScene, FEditorViewportClient& InEditorViewportClient) = 0;

protected:

	/** The actual viewport widget. Used as cached result for OnConstructAssetPreviewWidget. */
	TSharedPtr<UE::Workspace::SWorkspaceViewport> ViewportWidget;

	/** The scene viewport data */
	TSharedPtr<FSceneViewport> SceneViewport;

	/** The scene to show in the asset previews */
	TUniquePtr<FAdvancedPreviewScene> PreviewScene;

protected:

	/** Callback to tick the preview world */
	UE_API virtual EActiveTimerReturnType UpdatePreviewWorld(double InCurrentTime, float InDeltaTime);
};

} // namespace UE::UAF::Editor

#undef UE_API
