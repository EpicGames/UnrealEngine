// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Common/AssetPreview/IUAFAssetPreview.h"

#include "UAFAssetPreviewFactorySubsystem.generated.h"

#define UE_API UAFEDITOR_API

namespace UE::UAF::Editor
{
	struct FUAFAssetPreviewFactory;
};

/**
 * Experimental & Intended for removal. 
 * 
 * This class associates asset types to preview widgets for the UAF browser.
 * However it is intended to be replaced with a future API within the editor / content browser that does the same.
 * 
 * See: IUAFAssetPreview.h for more future design considerations.
 */
UCLASS(MinimalAPI, Experimental)
class UUAFAssetPreviewFactorySubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	/** 
	 * Method to associate a preview factory with some type. For now only supports one factory per type.
	 * 
	 * @return True if the factory was added for the type, false if we already have a type for it
	 */
	UE_API bool AddAssetPreviewFactory(TSharedPtr<UE::UAF::Editor::FUAFAssetPreviewFactory> InFactory);

	/** Get a Factory for the given type, or nullptr if no factory exists. */
	UE_API const TSharedPtr<UE::UAF::Editor::FUAFAssetPreviewFactory> GetAssetPreviewFactory(const UStruct* InPreviewType) const;

private:

	/** Map of classes to preview factories */
	TMap<const UStruct*, TSharedPtr<UE::UAF::Editor::FUAFAssetPreviewFactory>> AssetTypeFactoryMap;
};

#undef UE_API
