// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FindInGraph.h"

class FAssetEditorToolkit;
class SSearchBox;

namespace UE::UAF::Editor
{

/** Item that matched the search results */
class FFindInAnimNextRigVMAssetResult : public FFindInGraphResult
{
public:

	FFindInAnimNextRigVMAssetResult(const FFindInGraphResult::FCreateParams& InCreateParams);

	//~ Begin FFindInGraphResult Interface
	virtual void JumpToNode(TWeakPtr<FAssetEditorToolkit> AssetEditorToolkit, const UEdGraphNode* InNode) const override;
	//~ End FFindInGraphResult Interface
};


/** Widget for searching for items that are part of a AnimNextRigVMAsset Graph */
class SFindInAnimNextRigVMAsset : public SFindInGraph
{

protected:

	//~ Begin SFindInGraph Interface
	virtual TSharedPtr<FFindInGraphResult> MakeSearchResult(const FFindInGraphResult::FCreateParams& InParams) override;
	virtual void MatchTokens(const TArray<FString>& Tokens) override;
	virtual bool MatchTokensInNode(const UEdGraphNode* Node, const TArray<FString>& Tokens) override;
	//~ End SFindInGraph Interface
};

} // namespace UE::UAF::Editor