// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

class SWidget;
class IMessageLogListing;
class UEdGraphNode;
struct FTopLevelAssetPath;
struct FGraphContextMenuBuilder;

namespace UE::Workspace
{
	class IWorkspaceEditor;
	struct FWorkspaceEditorContext;
}

typedef TSet<class UObject*> FGraphPanelSelectionSet;

namespace UE::UAF::Editor
{

const FLazyName CompilerResultsTabName("CompilerResultsTab");
const FLazyName LogListingName("AnimNextCompilerResults");
const FLazyName FindTabName("FindTab");
const FLazyName FindAndReplaceTabName("FindAndReplaceTab");
const FLazyName UAFBrowserTabName("UAFBrowserTab");

struct FVariablePickerArgs;
struct FActionMenuContextData;
class IAssetCompilationHandler;

// A factory function used to make an asset compilation handler for an asset
using FAssetCompilationHandlerFactoryDelegate = TDelegate<TSharedRef<IAssetCompilationHandler>(UObject*)>;

class IAnimNextEditorModule : public IModuleInterface
{
public:
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FCollectGraphMenuActionsMulticast, const TWeakPtr<UE::Workspace::IWorkspaceEditor>& /*IWorkspaceEditor*/, FGraphContextMenuBuilder& /*InContextMenuBuilder*/, const FActionMenuContextData& /*ActionMenuContextData*/);
	typedef FCollectGraphMenuActionsMulticast::FDelegate FOnCollectGraphMenuActionsDelegate;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FNodeDblClickNotificationMulticast, const UE::Workspace::FWorkspaceEditorContext& /*InWorkspaceEditorContext*/, const UEdGraphNode* /*InNode*/);
	typedef FNodeDblClickNotificationMulticast::FDelegate FNodeDblClickNotificationDelegate;

	// Register a valid fragment type name to be used with parameter UOLs
	// @param InLocatorFragmentEditorName The name of the locator fragment editor
	virtual void RegisterLocatorFragmentEditorType(FName InLocatorFragmentEditorName) = 0;

	// Unregister a valid fragment type name to be used with parameter UOLs
	// @param InLocatorFragmentEditorName The name of the locator fragment editor
	virtual void UnregisterLocatorFragmentEditorType(FName InLocatorFragmentEditorName) = 0;

	// Add a UClass path to the set of classes which can be opened within an AnimNext Workspace
	// @param InClassAssetPath Asset path for to-be-registered Class 
	virtual void AddWorkspaceSupportedAssetClass(const FTopLevelAssetPath& InClassAssetPath) = 0;
	
	// Remove a UClass path to the set of classes which can be opened within an AnimNext Workspace
	// @param InClassAssetPath Asset path for to-be-unregistered Class 
	virtual void RemoveWorkspaceSupportedAssetClass(const FTopLevelAssetPath& InClassAssetPath) = 0;

	// Register a graph context menu actions provider
	// @param InCollectDelegate Delegate to add
	// @return FDelegateHandle
	virtual FDelegateHandle RegisterGraphMenuActionsProvider(const FOnCollectGraphMenuActionsDelegate& InCollectDelegate) = 0;

	// Unregister a graph context menu actions provider
	// @param InDelegateHandle Handle of the delegate to remove
	virtual void UnregisterGraphMenuActionsProvider(const FDelegateHandle& InDelegateHandle) = 0;

	// Register an asset compilation handler
	// @param InClassPath The path of the asset's class
	// @param InAssetCompilerFactory A factory function used to make an asset compiler for an asset
	virtual void RegisterAssetCompilationHandler(const FTopLevelAssetPath& InClassPath, FAssetCompilationHandlerFactoryDelegate InAssetCompilationHandlerFactory) = 0;

	// Unregister an asset compilation handler
	// @param InClassPath The path of the asset's class
	virtual void UnregisterAssetCompilationHandler(const FTopLevelAssetPath& InClassPath) = 0;

	// Register a node double click notifications handler
	// @param InGraphNodeActionsHandler Handler interface to register
	virtual FDelegateHandle RegisterNodeDblClickHandler(const FNodeDblClickNotificationDelegate& InNodeDblClickNotificationDelegate) = 0;

	// Unregister a node double click notifications handler
	// @param InDelegateHandle Handle of the delegate to remove
	virtual void UnregisterNodeDblClickHandler(const FDelegateHandle& InDelegateHandle) = 0;

	// Adds the UAF asset wizard to the content browser asset menu
	virtual void EnableAssetWizard() = 0;

	// Removes the UAF asset wizard from the content browser asset menu
	virtual void DisableAssetWizard() = 0;
};

}
