// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::Workspace
{
class IWorkspaceEditor;
}

namespace UE::UAF::Editor
{

// Status of compilation. Order matters - status is compared by magnitude to indicate severity
enum class ECompileStatus : uint8
{
	Unknown,
	UpToDate,
	Dirty,
	Warning,
	Error,
};

class IAssetCompilationHandler : public TSharedFromThis<IAssetCompilationHandler>
{
public:
	virtual ~IAssetCompilationHandler() = default;

	// Called to compile an asset from the editor
	virtual void Compile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, UObject* InAsset) = 0;

	// Called to set the auto compile mode of the asset
	virtual void SetAutoCompile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, UObject* InAsset, bool bInAutoCompile) {}

	// Called to get the auto compile mode of the asset
	virtual bool GetAutoCompile(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, const UObject* InAsset) const { return false; }

	// Called to get the compile status of the asset
	virtual ECompileStatus GetCompileStatus(TSharedRef<Workspace::IWorkspaceEditor> InWorkspaceEditor, const UObject* InAsset) const { return ECompileStatus::Unknown; }

	// Called to notify that the compile status of the asset may have changed
	FSimpleDelegate& OnCompileStatusChanged() { return CompileStatusChangedDelegate; }

private:
	FSimpleDelegate CompileStatusChangedDelegate;
};

}
