// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkspaceViewportSceneDescription.h"

#include "ViewportSceneDescription.generated.h"

class FAdvancedPreviewScene;
class USkeletalMesh;
class UPreviewMeshCollection;

namespace UE::Workspace
{
	class IWorkspaceEditor;
}

UCLASS(MinimalAPI)
class UUAFViewportSceneDescription : public UWorkspaceViewportSceneDescription
{
	GENERATED_BODY()
	
public:
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	UPROPERTY(EditAnywhere, Category = Default)
	TObjectPtr<USkeletalMesh> SkeletalMesh;

	UPROPERTY(EditAnywhere, Category = Default)
	TObjectPtr<UPreviewMeshCollection> AdditionalMeshes;
};