// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Variables/VariablePickerArgs.h"
#include "EdGraph/EdGraphPin.h"
#include "Param/ParamType.h"
#include "Variables/AnimNextSoftVariableReference.h"

struct FAnimNextParamInstanceIdentifier;

namespace UE::UAF::Editor
{
	class SVariablePicker;
	class SGraphPinVariableReference;
}

namespace UE::UAF::Editor
{

/** Retrieves the variable reference to display */
using FOnGetVariableReference = TDelegate<FAnimNextSoftVariableReference()>;

/** Retrieves the variable type to display */
using FOnGetVariableType = TDelegate<FAnimNextParamType()>;

class UAFEDITOR_API SVariablePickerCombo : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SVariablePickerCombo) {}

	/** Arguments for configuring the picker in the dropdown menu */
	SLATE_ARGUMENT(FVariablePickerArgs, PickerArgs)

	/** Retrieves the variable name to display */
	SLATE_EVENT(FOnGetVariableReference, OnGetVariableReference)

	/** Retrieves the variable type to display */
	SLATE_EVENT(FOnGetVariableType, OnGetVariableType)

	/** The variable name text to display. This will be used if valid, otherwise the text of the variable reference will be used. */
	SLATE_ATTRIBUTE(FText, VariableName)

	/** The variable tooltip text to display. This will be used if valid, otherwise the tooltip of the variable reference will be used. */
	SLATE_ATTRIBUTE(FText, VariableTooltip)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	void RequestRefresh();

	// Retrieves the variable reference to display
	FOnGetVariableReference OnGetVariableReferenceDelegate;

	// Retrieves the variable type to display
	FOnGetVariableType OnGetVariableTypeDelegate;

	// Cached pin type
	FEdGraphPinType PinType;

	// Cached variable reference
	FAnimNextSoftVariableReference CachedVariableReference;

	// Attribute for variable name
	TAttribute<FText> VariableName;
	
	// Attribute for variable tooltip
	TAttribute<FText> VariableTooltip;

	// Cached display name
	FText CachedVariableNameText;

	// Cached tooltip
	FText CachedVariableNameTooltipText;

	// Cached variable type
	FAnimNextParamType VariableType;

	// Cached icon
	const FSlateBrush* Icon = nullptr;

	// Cached color
	FSlateColor IconColor = FLinearColor::Gray;

	// Picker widget used to focus after the popup is displayed
	TWeakPtr<SVariablePicker> PickerWidget;

	// Arguments for the picker popup
	FVariablePickerArgs PickerArgs;

	bool bRefreshRequested = false;

	friend class SGraphPinVariableReference;
};

}