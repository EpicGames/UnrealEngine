// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFEditor : ModuleRules
	{
		public UAFEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ApplicationCore",
					"Engine",
					"AssetTools",
					"UnrealEd",
					"UAF",
					"UAFUncookedOnly",
					"UnrealEd",
					"SlateCore",
					"AnimationCore",
					"AnimGraph",
					"Slate",
					"InputCore",
					"PropertyEditor",
					"RigVM",
					"RigVMEditor",
					"RigVMDeveloper",
					"GraphEditor",
					"ToolWidgets",
					"ToolMenus",
					"AssetDefinition",
					"SourceControl", 
					"KismetWidgets",
					"StructUtilsEditor",
					"BlueprintGraph",	// For K2 Schema
					"DesktopWidgets",
					"ClassViewer",
					"ContentBrowserAssetDataSource",
					"ContentBrowserFileDataSource",
					"SubobjectEditor",
					"Settings",
					"StatusBar",
					"EditorWidgets",
					"WorkspaceEditor",
					"ContentBrowser",
					"ContentBrowserData",
					"DataHierarchyEditor",
					"UniversalObjectLocator",
					"UniversalObjectLocatorEditor",
					"UserAssetTagsEditor",
					"Kismet",
					"AdvancedWidgets", 
					"SceneOutliner",
					"EditorFramework",
					"EditorSubsystem",
					"InteractiveToolsFramework",
					"EditorInteractiveToolsFramework",
					"TraceAnalysis",
					"TraceLog",
					"TraceServices",
					"TraceInsights",
					"RewindDebuggerInterface",
					"GameplayInsights",
					"Projects",
					"Persona",
					"DeveloperSettings",
					"AdvancedPreviewScene",
					"HierarchyTableRuntime",
					"HierarchyTableEditor",
					"HierarchyTableAnimationRuntime",
					"PropertyBindingUtils",
				}
			);
			
			bAllowUETypesInNamespaces = true; 
		}
	}
}