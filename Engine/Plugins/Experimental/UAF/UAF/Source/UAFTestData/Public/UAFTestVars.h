// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UAFTestVars.generated.h"

UENUM()
enum class EUAFTestEnum : uint8
{
	ValueA = 0,
	ValueB = 1,
	ValueC = 2,
};

/** Old-style enum for TEnumAsByte / FByteProperty testing. */
UENUM()
namespace EUAFTestByteEnum
{
	enum Type : uint8
	{
		ByteA = 0,
		ByteB = 1,
		ByteC = 2,
	};
}

/** Struct with adjacent small fields for testing SubProperty overread safety. */
USTRUCT()
struct FUAFPackedByteStruct
{
	GENERATED_BODY()

	UPROPERTY()
	uint8 Val1 = 0;

	UPROPERTY()
	uint8 Val2 = 0;

	UPROPERTY()
	EUAFTestEnum EnumVal = EUAFTestEnum::ValueA;

	UPROPERTY()
	float FloatVal = 0.0f;
};

/** Nested struct for FBindableStruct SubProperty binding tests. */
USTRUCT()
struct FUAFNestedTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FVector Vec = FVector::ZeroVector;

	UPROPERTY()
	FQuat Quat = FQuat::Identity;

	UPROPERTY()
	FTransform Transform = FTransform::Identity;
};

/**
 * Minimal USTRUCT used only in UAF LLT unit tests.
 * Provides a set of typed properties that cover the common resolved types
 * (bool, float, double, int32, FVector, FQuat) so that FBindableXxx runtime resolution
 * tests can exercise variable and sub-property binding against a live FUAFAssetInstance.
 */
USTRUCT()
struct FUAFTestVars
{
	GENERATED_BODY()

	UPROPERTY()
	bool bBool = false;

	UPROPERTY()
	float FloatVal = 0.0f;

	UPROPERTY()
	double DoubleVal = 0.0;

	UPROPERTY()
	int32 IntVal = 0;

	UPROPERTY()
	FVector VectorVar = FVector::ZeroVector;

	UPROPERTY()
	FQuat QuatVar = FQuat::Identity;

	UPROPERTY()
	FTransform TransformVar = FTransform::Identity;

	UPROPERTY()
	FUAFNestedTestStruct NestedVar;

	UPROPERTY()
	int64 Int64Val = 0;

	UPROPERTY()
	uint8 ByteVal = 0;

	UPROPERTY()
	FName NameVar;

	UPROPERTY()
	EUAFTestEnum EnumVar = EUAFTestEnum::ValueA;

	UPROPERTY()
	TEnumAsByte<EUAFTestByteEnum::Type> EnumAsByteVar = EUAFTestByteEnum::ByteA;

	UPROPERTY()
	TObjectPtr<UObject> ObjectVar = nullptr;

	UPROPERTY()
	FUAFPackedByteStruct PackedByteVar;
};

/** 10-property struct used by bulk-resolution performance benchmarks. */
USTRUCT()
struct FUAFPerfVars10
{
	GENERATED_BODY()

	// Float targets/sources for variable-binding benchmark
	UPROPERTY()
	float f0 = 0.f;

	UPROPERTY()
	float f1 = 0.f;

	UPROPERTY()
	float f2 = 0.f;

	UPROPERTY()
	float f3 = 0.f;

	UPROPERTY()
	float f4 = 0.f;

	UPROPERTY()
	float f5 = 0.f;

	UPROPERTY()
	float f6 = 0.f;

	UPROPERTY()
	float f7 = 0.f;

	UPROPERTY()
	float f8 = 0.f;

	UPROPERTY()
	float f9 = 0.f;

	// Vector sources for subproperty-binding benchmark (bind .X → f0..f9)
	UPROPERTY()
	FVector v0 = FVector::ZeroVector;

	UPROPERTY()
	FVector v1 = FVector::ZeroVector;

	UPROPERTY()
	FVector v2 = FVector::ZeroVector;

	UPROPERTY()
	FVector v3 = FVector::ZeroVector;

	UPROPERTY()
	FVector v4 = FVector::ZeroVector;

	UPROPERTY()
	FVector v5 = FVector::ZeroVector;

	UPROPERTY()
	FVector v6 = FVector::ZeroVector;

	UPROPERTY()
	FVector v7 = FVector::ZeroVector;

	UPROPERTY()
	FVector v8 = FVector::ZeroVector;

	UPROPERTY()
	FVector v9 = FVector::ZeroVector;

	// Bool sources for FBindableBool direct binding benchmark
	UPROPERTY()
	bool bBool0 = false;

	UPROPERTY()
	bool bBool1 = false;

	UPROPERTY()
	bool bBool2 = false;

	UPROPERTY()
	bool bBool3 = false;

	UPROPERTY()
	bool bBool4 = false;

	UPROPERTY()
	bool bBool5 = false;

	UPROPERTY()
	bool bBool6 = false;

	UPROPERTY()
	bool bBool7 = false;

	UPROPERTY()
	bool bBool8 = false;

	UPROPERTY()
	bool bBool9 = false;

	// Double sources for FBindableDouble direct binding benchmark
	UPROPERTY()
	double d0 = 0.0;

	UPROPERTY()
	double d1 = 0.0;

	UPROPERTY()
	double d2 = 0.0;

	UPROPERTY()
	double d3 = 0.0;

	UPROPERTY()
	double d4 = 0.0;

	UPROPERTY()
	double d5 = 0.0;

	UPROPERTY()
	double d6 = 0.0;

	UPROPERTY()
	double d7 = 0.0;

	UPROPERTY()
	double d8 = 0.0;

	UPROPERTY()
	double d9 = 0.0;

	// Int32 sources for FBindableInt32 direct binding benchmark
	UPROPERTY()
	int32 i0 = 0;

	UPROPERTY()
	int32 i1 = 0;

	UPROPERTY()
	int32 i2 = 0;

	UPROPERTY()
	int32 i3 = 0;

	UPROPERTY()
	int32 i4 = 0;

	UPROPERTY()
	int32 i5 = 0;

	UPROPERTY()
	int32 i6 = 0;

	UPROPERTY()
	int32 i7 = 0;

	UPROPERTY()
	int32 i8 = 0;

	UPROPERTY()
	int32 i9 = 0;

	// Int64 sources for FBindableInt64 direct binding benchmark
	UPROPERTY()
	int64 l0 = 0;

	UPROPERTY()
	int64 l1 = 0;

	UPROPERTY()
	int64 l2 = 0;

	UPROPERTY()
	int64 l3 = 0;

	UPROPERTY()
	int64 l4 = 0;

	UPROPERTY()
	int64 l5 = 0;

	UPROPERTY()
	int64 l6 = 0;

	UPROPERTY()
	int64 l7 = 0;

	UPROPERTY()
	int64 l8 = 0;

	UPROPERTY()
	int64 l9 = 0;

	// Byte sources for FBindableByte direct binding benchmark
	UPROPERTY()
	uint8 u0 = 0;

	UPROPERTY()
	uint8 u1 = 0;

	UPROPERTY()
	uint8 u2 = 0;

	UPROPERTY()
	uint8 u3 = 0;

	UPROPERTY()
	uint8 u4 = 0;

	UPROPERTY()
	uint8 u5 = 0;

	UPROPERTY()
	uint8 u6 = 0;

	UPROPERTY()
	uint8 u7 = 0;

	UPROPERTY()
	uint8 u8 = 0;

	UPROPERTY()
	uint8 u9 = 0;

	// FName sources for FBindableName direct binding benchmark
	UPROPERTY()
	FName nm0;

	UPROPERTY()
	FName nm1;

	UPROPERTY()
	FName nm2;

	UPROPERTY()
	FName nm3;

	UPROPERTY()
	FName nm4;

	UPROPERTY()
	FName nm5;

	UPROPERTY()
	FName nm6;

	UPROPERTY()
	FName nm7;

	UPROPERTY()
	FName nm8;

	UPROPERTY()
	FName nm9;

	// Enum sources for FBindableEnum direct binding benchmark
	UPROPERTY()
	EUAFTestEnum en0 = EUAFTestEnum::ValueA;

	UPROPERTY()
	EUAFTestEnum en1 = EUAFTestEnum::ValueA;

	UPROPERTY()
	EUAFTestEnum en2 = EUAFTestEnum::ValueA;

	UPROPERTY()
	EUAFTestEnum en3 = EUAFTestEnum::ValueA;

	UPROPERTY()
	EUAFTestEnum en4 = EUAFTestEnum::ValueA;

	UPROPERTY()
	EUAFTestEnum en5 = EUAFTestEnum::ValueA;

	UPROPERTY()
	EUAFTestEnum en6 = EUAFTestEnum::ValueA;

	UPROPERTY()
	EUAFTestEnum en7 = EUAFTestEnum::ValueA;

	UPROPERTY()
	EUAFTestEnum en8 = EUAFTestEnum::ValueA;

	UPROPERTY()
	EUAFTestEnum en9 = EUAFTestEnum::ValueA;

	// Quat sources for FBindableQuat direct binding benchmark
	UPROPERTY()
	FQuat q0 = FQuat::Identity;

	UPROPERTY()
	FQuat q1 = FQuat::Identity;

	UPROPERTY()
	FQuat q2 = FQuat::Identity;

	UPROPERTY()
	FQuat q3 = FQuat::Identity;

	UPROPERTY()
	FQuat q4 = FQuat::Identity;

	UPROPERTY()
	FQuat q5 = FQuat::Identity;

	UPROPERTY()
	FQuat q6 = FQuat::Identity;

	UPROPERTY()
	FQuat q7 = FQuat::Identity;

	UPROPERTY()
	FQuat q8 = FQuat::Identity;

	UPROPERTY()
	FQuat q9 = FQuat::Identity;

	// FUAFNestedTestStruct sources for FBindableVector sub-property benchmark (bind .Vec)
	UPROPERTY()
	FUAFNestedTestStruct ns0;

	UPROPERTY()
	FUAFNestedTestStruct ns1;

	UPROPERTY()
	FUAFNestedTestStruct ns2;

	UPROPERTY()
	FUAFNestedTestStruct ns3;

	UPROPERTY()
	FUAFNestedTestStruct ns4;

	UPROPERTY()
	FUAFNestedTestStruct ns5;

	UPROPERTY()
	FUAFNestedTestStruct ns6;

	UPROPERTY()
	FUAFNestedTestStruct ns7;

	UPROPERTY()
	FUAFNestedTestStruct ns8;

	UPROPERTY()
	FUAFNestedTestStruct ns9;
};
