// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UAFTestData : ModuleRules
{
	public UAFTestData(ReadOnlyTargetRules Target) : base(Target)
	{
		bAllowUETypesInNamespaces = true;

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"UAF",
		});
	}
}
