// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Entries/AnimNextVariableEntry.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "Param/ParamType.h"
#include "IAnimNextRigVMGraphInterface.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "Module/AnimNextModule.h"
#include "Variables/AnimNextSharedVariables.h"

#include "AnimNextAssetWorkspaceAssetUserData.generated.h"

class UUAFRigVMAssetEntry;
class URigVMEdGraphNode;

// Base struct used to identify asset entries
USTRUCT()
struct FAnimNextRigVMAssetOutlinerData : public FWorkspaceOutlinerItemData
{
	GENERATED_BODY()

	FAnimNextRigVMAssetOutlinerData() = default;
	
	UUAFRigVMAsset* GetAsset() const
	{
		return SoftAssetPtr.LoadSynchronous();
	}

	UPROPERTY(VisibleAnywhere, Category=AnimNext)
	TSoftObjectPtr<UUAFRigVMAsset> SoftAssetPtr;
};

USTRUCT()
struct FAnimNextModuleOutlinerData : public FAnimNextRigVMAssetOutlinerData
{
	GENERATED_BODY()

	FAnimNextModuleOutlinerData() = default;

	UUAFSystem* GetModule() const
	{
		return Cast<UUAFSystem>(GetAsset());
	}
};

USTRUCT()
struct FAnimNextSharedVariablesOutlinerData : public FAnimNextRigVMAssetOutlinerData
{
	GENERATED_BODY()

	FAnimNextSharedVariablesOutlinerData() = default;

	UUAFSharedVariables* GetSharedVariables() const
	{
		return Cast<UUAFSharedVariables>(GetAsset());
	}
};

// Base struct used to identify asset sub-entries
USTRUCT()
struct FAnimNextAssetEntryOutlinerData : public FWorkspaceOutlinerItemData
{
	GENERATED_BODY()
	
	FAnimNextAssetEntryOutlinerData() = default;
	
	UUAFRigVMAssetEntry* GetEntry() const
	{
		return SoftEntryPtr.LoadSynchronous();
	}

	UPROPERTY(VisibleAnywhere, Category=AnimNext)
	TSoftObjectPtr<UUAFRigVMAssetEntry> SoftEntryPtr;
};

USTRUCT()
struct FAnimNextVariableOutlinerData : public FAnimNextAssetEntryOutlinerData
{
	GENERATED_BODY()
	
	FAnimNextVariableOutlinerData() = default;

	UPROPERTY(VisibleAnywhere, Category=AnimNext)
	FAnimNextParamType Type;
};

USTRUCT()
struct FAnimNextCollapseGraphsOutlinerDataBase : public FWorkspaceOutlinerItemData
{
	GENERATED_BODY()
	
	FAnimNextCollapseGraphsOutlinerDataBase() = default;

	UPROPERTY(VisibleAnywhere, Category=AnimNext)
	TSoftObjectPtr<URigVMEdGraph> SoftEditorObject;

	static bool IsCollapsedGraphBase(const FWorkspaceOutlinerItemExport& InExport)
	{
		return InExport.HasData() && InExport.GetData().GetScriptStruct()->IsChildOf(FAnimNextCollapseGraphsOutlinerDataBase::StaticStruct());
	}
};

USTRUCT()
struct FAnimNextCollapseGraphOutlinerData : public FAnimNextCollapseGraphsOutlinerDataBase
{
	GENERATED_BODY()
	
	FAnimNextCollapseGraphOutlinerData() = default;
};

USTRUCT()
struct FAnimNextGraphFunctionOutlinerData : public FAnimNextCollapseGraphsOutlinerDataBase
{
	GENERATED_BODY()
	
	FAnimNextGraphFunctionOutlinerData() = default;

	UPROPERTY(VisibleAnywhere, Category=AnimNext)
	TSoftObjectPtr<URigVMEdGraphNode> SoftEdGraphNode;
};

USTRUCT()
struct FAnimNextGraphOutlinerData : public FAnimNextAssetEntryOutlinerData
{
	GENERATED_BODY()
	
	FAnimNextGraphOutlinerData() = default;
	
	IUAFRigVMGraphInterface* GetGraphInterface() const
	{
		if (UUAFRigVMAssetEntry* Entry = GetEntry())
		{
			if (IUAFRigVMGraphInterface* GraphInterface = CastChecked<IUAFRigVMGraphInterface>(Entry))
			{
				return GraphInterface;
			}
		}

		return nullptr;
	}
};

UCLASS(MinimalAPI)
class UAnimNextAssetWorkspaceAssetUserData : public UAssetUserData
{
public:
	virtual bool IsEditorOnly() const override { return true; }

private:
	GENERATED_BODY()

	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	mutable FWorkspaceOutlinerItemExports CachedExports;
};
