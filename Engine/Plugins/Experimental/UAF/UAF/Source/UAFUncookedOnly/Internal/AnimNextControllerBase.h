// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMModel/RigVMController.h"
#include "AnimNextControllerBase.generated.h"

class UUAFSharedVariables;
class UUAFSharedVariableNode;
class UUAFRigVMAssetEditorData;

/**
  * Implements AnimNext RigVM controller extensions
  */
UCLASS(MinimalAPI)
class UAnimNextControllerBase : public URigVMController
{
	GENERATED_BODY()

public:
	// Add a shared variable from the specified asset
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UAFUNCOOKEDONLY_API UUAFSharedVariableNode* AddAssetSharedVariableNode(const UUAFSharedVariables* InAsset, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Add a shared variable from the specified struct
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UAFUNCOOKEDONLY_API UUAFSharedVariableNode* AddStructSharedVariableNode(const UScriptStruct* InStruct, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Refreshes a UUAFSharedVariableNode instance with provided data (similar to URigVMController::RefreshVariableNode)
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UAFUNCOOKEDONLY_API void RefreshSharedVariableNode(const FName& InNodeName, const FString& InSourceObjectPath, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bSetupOrphanPins = true, bool bPrintPythonCommand = false);

	// Add a shared variable from the specified source object
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UAFUNCOOKEDONLY_API UUAFSharedVariableNode* AddSharedVariableNode(const FString& InSourceObjectPath, const FName& InVariableName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand);
	
	// Adds a Function / Struct Node to the edited Graph.
	// UnitNode represent a RIGVM_METHOD declaration on a USTRUCT.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = "UAF|Controller")
	UAFUNCOOKEDONLY_API URigVMUnitNode* AddUnitNodeOfClass(const FString& InScriptStructPath, const FString& InUnitNodeClassPath, const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	
	UUAFSharedVariableNode* ReplaceVariableNodeWithSharedVariableNode(URigVMVariableNode* InVariableNode, FName InNewVariableName, const UObject* InAssetOrStruct, bool bSetupUndoRedo, bool bPrintPythonCommand);
	URigVMVariableNode* ReplaceSharedVariableNodeWithVariableNode(UUAFSharedVariableNode* InVariableNode, FName InNewVariableName, bool bSetupUndoRedo, bool bPrintPythonCommand);
};
