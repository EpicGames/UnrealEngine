// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/RigVMEdGraph.h"
#include "AnimNextEdGraph.generated.h"

class UUAFRigVMAssetEditorData;

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

/**
  * Wraps UEdGraph which represents the node graph
  */
UCLASS(MinimalAPI)
class UAnimNextEdGraph : public URigVMEdGraph
{
	GENERATED_BODY()

	friend class UUAFSystem_EditorData;
	friend class UUAFRigVMAssetEditorData;

	// UObject interface
	virtual void PostLoad() override;

	// URigVMEdGraph interface
	virtual FRigVMClient* GetRigVMClient() const override;
	
	void Initialize(UUAFRigVMAssetEditorData* InEditorData);
};