// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/RigVMEdGraphNode.h"

#include "AnimNextEdGraphNode.generated.h"

// EdGraphNode representation for AnimNext nodes.
// No longer has a specific use, but keeping this class around just in case.
UCLASS(MinimalAPI)
class UAnimNextEdGraphNode : public URigVMEdGraphNode
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Customization", Transient)
	FLinearColor NodeColor;
};
