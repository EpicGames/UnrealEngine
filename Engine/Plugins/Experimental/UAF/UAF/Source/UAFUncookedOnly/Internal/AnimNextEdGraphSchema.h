// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "AnimNextEdGraphNode.h"
#include "AnimNextEdGraphSchema.generated.h"

class IAssetReferenceFilter;

#define UE_API UAFUNCOOKEDONLY_API

UCLASS(MinimalAPI)
class UAnimNextEdGraphSchema : public URigVMEdGraphSchema
{
	GENERATED_BODY()

	// URigVMEdGraphSchema interface
	virtual TSubclassOf<URigVMEdGraphNode> GetGraphNodeClass(const URigVMEdGraph* InGraph) const override { return UAnimNextEdGraphNode::StaticClass(); }
	UE_API virtual bool ShouldHidePinOnNode(const URigVMPin* InModelPin) const override;
	UE_API virtual bool IsStructEditable(UStruct* InStruct) const override;

	// UEdGraphSchema interface
	UE_API virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;

protected:
	UE_API static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UEdGraph* Graph);
};

#undef UE_API