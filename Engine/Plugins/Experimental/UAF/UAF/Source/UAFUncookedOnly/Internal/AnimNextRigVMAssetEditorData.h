// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextControllerBase.h"
#include "IAnimNextRigVMExportInterface.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMCore/RigVMGraphFunctionHost.h"
#include "EdGraph/RigVMEdGraph.h"
#include "UncookedOnlyUtils.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "Variables/AnimNextProgrammaticVariable.h"
#include "Variables/AnimNextVariableReference.h"
#include "AnimNextRigVMAssetEditorData.generated.h"

#define UE_API UAFUNCOOKEDONLY_API

enum class ERigVMGraphNotifType : uint8;
class UUAFRigVMAssetEditorData;
class UAnimNextEdGraph;
class UUAFSharedVariablesEntry;
class UAnimNextWorkspaceEditorMode;
class UUAFTestBlueprintLibrary;
class UUAFSharedVariableNode;
struct FAnimNextRigVMAssetCompileContext;
struct FAnimNextGetFunctionHeaderCompileContext;
struct FAnimNextGetVariableCompileContext;
struct FAnimNextGetGraphCompileContext;
struct FAnimNextProcessGraphCompileContext;

namespace UE::UAF::UncookedOnly
{
	struct FPublicVariablesImpl;
	struct FUtils;
	struct FUtilsPrivate;
	class FCompilationScope;
	struct Compilation;
	class FModule;
}

namespace UE::UAF::Editor
{
	struct FUtils;
	class SRigVMAssetView;
	class SParameterPicker;
	class SRigVMAssetViewRow;
	class FVariableCustomization;
	class FFindInAnimNextRigVMAssetResult;
	class SFindInAnimNextRigVMAsset;
	class FAnimNextEditorModule;
	class FWorkspaceEditor;
	class FAnimNextAssetItemDetails;
	class FAnimNextGraphItemDetails;
	class FAnimNextFunctionItemDetails;
	struct FVariablesOutlinerEntryItem;
	class FVariablesOutlinerMode;
	class FVariablesOutlinerHierarchy;
	class FOutlinerHierarchy;
	class SVariablesOutlinerValue;
	class SFunctionsView;
	class SVariablesView;
	class FFunctionsOutlinerHierarchy;
	class SAddVariablesDialog;
	class FVariableProxyCustomization;
	class FAnimNextAnimGraphEditorModule;
	class FCallFunctionSharedDataDetails;
	class FCallFunctionInfoDetails;
	class FAssetCompilationHandler;
	class FVariableBindingPropertyCustomization;
	struct FFunctionsOutlinerEntryItem;
	struct FVariablesOutlinerCategoryItem;
	struct FFunctionsOutlinerCategoryItem;
	struct FFunctionsOutlinerCollapsedGraphItem;
	class FFunctionsOutlinerMode;
	class SCommonOutliner;
	class FOutOfDateAssetCompilation;
}

namespace UE::UAF::Tests
{
	class FEditor_Graphs;
	class FEditor_AnimGraph_Graphs;
	class FEditor_Variables;
	class FEditor_AnimGraph_Variables;
	class FVariables;
	class FVariables_UOLBindings;
	class FDataInterfaceCompile;
	struct FUAFSystemTests;
}

enum class EAnimNextEditorDataNotifType : uint8
{
	PropertyChanged,	// An property was changed (Subject == UObject)
	EntryAdded,		// An entry has been added (Subject == UUAFVMAssetEntry)
	EntryRemoved,	// An entry has been removed (Subject == UUAFRigVMAssetEditorData)
	EntryRenamed,	// An entry has been renamed (Subject == UUAFRigVMAssetEntry)
	EntryAccessSpecifierChanged,	// An entry access specifier has been changed (Subject == UUAFRigVMAssetEntry)
	VariableTypeChanged,	// A variable entry type changed (Subject == UAnimNextVariableEntry)
	UndoRedo,		// Transaction was performed (Subject == UObject)
	VariableDefaultValueChanged,	// A variable entry default value changed (Subject == UAnimNextVariableEntry)
	VariableBindingChanged,	// A variable entry binding changed (Subject == UAnimNextVariableEntry)
	VariableCategoryChanged, // A variable entry its category changed (Subject == UAnimNextVariableEntry)
	CategoryAdded, // A category has been added (Subject == UUAFRigVMAssetEditorData)
	CategoryChanged, // A category has been changed (Subject == UUAFRigVMAssetEditorData)
	CategoryRemoved, // A category has been removed (Subject == UUAFRigVMAssetEditorData)
	VariablesReordered,  // Asset entries have been reordered (Subject == UUAFRigVMAssetEditorData)
};

namespace UE::UAF::UncookedOnly
{
	// A delegate for subscribing / reacting to editor data modifications.
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnEditorDataModified, UUAFRigVMAssetEditorData* /* InEditorData */, EAnimNextEditorDataNotifType /* InType */, UObject* /* InSubject */);

	// An interaction bracket count reached 0
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnInteractionBracketFinished, UUAFRigVMAssetEditorData* /* InEditorData */);
}

// Script-callable editor API hoisted onto UUAFRigVMAsset
UCLASS()
class UUAFRigVMAssetLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Finds an entry in an AnimNext asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API UUAFRigVMAssetEntry* FindEntry(UUAFRigVMAsset* InAsset, FName InName);

	/** Removes an entry from an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API bool RemoveEntry(UUAFRigVMAsset* InAsset, UUAFRigVMAssetEntry* InEntry, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Removes multiple entries from an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API bool RemoveEntries(UUAFRigVMAsset* InAsset, const TArray<UUAFRigVMAssetEntry*>& InEntries, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Removes all entries from an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API bool RemoveAllEntries(UUAFRigVMAsset* InAsset, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds a parameter to an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API UAnimNextVariableEntry* AddVariable(UUAFRigVMAsset* InAsset, FName InName, EPropertyBagPropertyType InValueType, EPropertyBagContainerType InContainerType = EPropertyBagContainerType::None, const UObject* InValueTypeObject = nullptr, const FString& InDefaultValue = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds an event graph to an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API UAnimNextEventGraphEntry* AddEventGraph(UUAFRigVMAsset* InAsset, FName InName, UScriptStruct* InEventStruct, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds shared variables to an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API UUAFSharedVariablesEntry* AddSharedVariables(UUAFRigVMAsset* InAsset, UUAFSharedVariables* InSharedVariables, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds shared variables struct to an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API UUAFSharedVariablesEntry* AddSharedVariablesStruct(UUAFRigVMAsset* InAsset, UScriptStruct* InStruct, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);
	
	/** Adds a function to an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API URigVMLibraryNode* AddFunction(UUAFRigVMAsset* InAsset, FName InFunctionName, bool bInMutable, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds a category to an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API bool AddCategory(UUAFRigVMAsset* InAsset, const FString& CategoryName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Renames a category within an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API bool RenameCategory(UUAFRigVMAsset* InAsset, const FString& CategoryName, const FString& NewCategoryName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);
	
	/** Removes a category from within an UAF asset */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API bool RemoveCategory(UUAFRigVMAsset* InAsset, const FString& CategoryName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Renames a variable in an UAF asset, updating all references across the project */
	UFUNCTION(BlueprintCallable, Category = "UAF|Entries", meta=(ScriptMethod))
	static UAFUNCOOKEDONLY_API bool RenameVariable(UUAFRigVMAsset* InAsset, FName InVariableName, FName InNewName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);
};

/* Base class for all AnimNext editor data objects that use RigVM */
UCLASS(MinimalAPI, Abstract)
class UUAFRigVMAssetEditorData : public UObject, public IRigVMClientHost, public IRigVMGraphFunctionHost, public IRigVMClientExternalModelHost
{
	GENERATED_BODY()

public:
	/** Adds a parameter to this asset */
	UE_API UAnimNextVariableEntry* AddVariable(FName InName, FAnimNextParamType InType, const FString& InDefaultValue = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds an event graph to this asset */
	UE_API UAnimNextEventGraphEntry* AddEventGraph(FName InName, UScriptStruct* InEventStruct, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds shared variables to this asset */
	UE_API UUAFSharedVariablesEntry* AddSharedVariables(const UUAFSharedVariables* InSharedVariables, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds shared variables struct to this asset */
	UE_API UUAFSharedVariablesEntry* AddSharedVariablesStruct(const UScriptStruct* InStruct, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds shared variables from IRigVMRuntimeAssetInterfaces to this asset */
	UE_API UUAFSharedVariablesEntry* AddSharedVariablesRigVMAsset(const IRigVMRuntimeAssetInterface* InRigVMAsset, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Adds a function to this asset */
	UE_API URigVMLibraryNode* AddFunction(FName InFunctionName, bool bInMutable, bool bInSetupUndoRedo = true, bool bPrintPythonCommand = true);

	// Find an entry by name
	UE_API UUAFRigVMAssetEntry* FindEntry(FName InName) const;

	/** Adds a category for variables/functions to this asset */
	UE_API bool AddCategory(const FString& CategoryName, bool bInSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Renames a category using provided new name */
	UE_API bool RenameCategory(const FString& OldName, const FString& NewName, bool bInSetupUndoRedo = true, bool bPrintPythonCommand = true);
	
	/** Removes category with provided name */
	UE_API bool RemoveCategory(const FString CategoryName, bool bInSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Reorders category entry to appear before provided BeforeCategoryName */
	UE_API void ReorderCategory(const FString& CategoryName, const FString& BeforeCategoryName, bool bInSetupUndoRedo = true, bool bPrintPythonCommand = true);

	/** Reorders variable entry to appear before provided BeforeVariableEntry */
	UE_API void ReorderVariable(UAnimNextVariableEntry* VariableEntry, const UAnimNextVariableEntry* BeforeVariableEntry, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	// Report an error to the user, typically used for scripting APIs
	static UE_API void ReportError(const TCHAR* InMessage);

protected:
	friend class UE::UAF::Editor::SRigVMAssetView;
	friend class UE::UAF::Editor::SRigVMAssetViewRow;
	friend struct UE::UAF::Editor::FUtils;
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend class UE::UAF::Editor::FAnimNextEditorModule;
	friend class UE::UAF::Editor::FWorkspaceEditor;
	friend class UUAFRigVMAssetEntry;
	friend class UUAFRigVMAssetLibrary;
	friend class UAnimNextEdGraph;
	friend class UE::UAF::Tests::FEditor_Graphs;
	friend class UE::UAF::Tests::FEditor_Variables;
	friend class UE::UAF::Tests::FEditor_AnimGraph_Graphs;
	friend class UE::UAF::Tests::FEditor_AnimGraph_Variables;
	friend class UE::UAF::Editor::FFindInAnimNextRigVMAssetResult;
	friend class UE::UAF::Editor::SFindInAnimNextRigVMAsset;
	friend class UE::UAF::Tests::FVariables;
	friend class UE::UAF::Tests::FVariables_UOLBindings;
	friend class UE::UAF::Editor::FVariableCustomization;
	friend class UE::UAF::Editor::FAnimNextAssetItemDetails;
	friend class UE::UAF::Editor::FAnimNextGraphItemDetails;
	friend class UE::UAF::Editor::FAnimNextFunctionItemDetails;
	friend struct UE::UAF::Editor::FVariablesOutlinerEntryItem;
	friend struct UE::UAF::Editor::FVariablesOutlinerCategoryItem;
	friend struct UE::UAF::Editor::FFunctionsOutlinerCategoryItem;
	friend class UE::UAF::Editor::FVariablesOutlinerMode;
	friend class UE::UAF::Editor::FVariablesOutlinerHierarchy;
	friend class UE::UAF::Editor::FFunctionsOutlinerHierarchy;	
	friend class UE::UAF::Editor::FOutlinerHierarchy;
	friend class UE::UAF::Editor::SVariablesOutlinerValue;
	friend class UE::UAF::Editor::FFunctionsOutlinerMode;
	friend UE::UAF::Tests::FUAFSystemTests;
	friend class UAnimNextModuleWorkspaceAssetUserData;
	friend class UE::UAF::Editor::SAddVariablesDialog;
	friend class UE::UAF::Editor::FVariableProxyCustomization;
	friend class UUAFSharedVariablesEntry;
	friend class UUAFTestBlueprintLibrary;
	friend struct UE::UAF::UncookedOnly::FPublicVariablesImpl;
	friend UE::UAF::Editor::FAnimNextAnimGraphEditorModule;
	friend class UE::UAF::Editor::FCallFunctionSharedDataDetails;
	friend class UE::UAF::Editor::FCallFunctionInfoDetails;
	friend UAnimNextWorkspaceEditorMode;
	friend UE::UAF::UncookedOnly::FCompilationScope;
	friend UE::UAF::UncookedOnly::Compilation;
	friend UE::UAF::Editor::FAssetCompilationHandler;
	friend class UAnimNextStateTreeTreeEditorData;
	friend class UE::UAF::Editor::FVariableBindingPropertyCustomization;
	friend class UAnimNextAssetFindReplaceVariables;
	friend struct FAnimNextSchemaAction_Variable;
	friend struct FAnimNextSchemaAction_Function;
	friend struct FAnimNextModuleTests;
	friend struct FAnimNextAnimationGraphTests;
	friend FAnimNextRigVMAssetCompileContext;
	friend UE::UAF::Editor::FFunctionsOutlinerEntryItem;
	friend UE::UAF::Editor::FFunctionsOutlinerCollapsedGraphItem;
	friend UE::UAF::Editor::SFunctionsView;
	friend UE::UAF::Editor::SCommonOutliner;
	friend UE::UAF::Editor::SVariablesView;
	friend UUAFSharedVariableNode;
	friend class UE::UAF::UncookedOnly::FModule;
	friend class UE::UAF::Editor::FOutOfDateAssetCompilation;

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	virtual bool IsEditorOnly() const override { return true; }
	UE_API virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;	
	UE_API virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;

	UE_API void HandlePackageDone(const FEndLoadPackageContext& Context);
	UE_API void HandlePackageDone();

	UE_API virtual void GetAnimNextAssetRegistryTags(FAssetRegistryTagsContext& Context, FAnimNextAssetRegistryExports& OutExports) const;

	// IRigVMClientHost interface
	virtual FString GetAssetName() const override { return GetName(); }
	UE_API virtual UClass* GetRigVMSchemaClass() const override;
	UE_API virtual UScriptStruct* GetRigVMExecuteContextStruct() const override;
	UE_API virtual UClass* GetRigVMEdGraphClass() const override;
	UE_API virtual UClass* GetRigVMEdGraphNodeClass() const override;
	UE_API virtual UClass* GetRigVMEdGraphSchemaClass() const override;
	UE_API virtual UClass* GetRigVMEditorSettingsClass() const override;
	UE_API virtual FRigVMClient* GetRigVMClient() override;
	UE_API virtual const FRigVMClient* GetRigVMClient() const override;
	UE_API virtual TScriptInterface<IRigVMGraphFunctionHost> GetRigVMGraphFunctionHost() override;
	UE_API virtual TScriptInterface<const IRigVMGraphFunctionHost> GetRigVMGraphFunctionHost() const override;
	UE_API virtual void HandleRigVMGraphAdded(const FRigVMClient* InClient, const FString& InNodePath) override;
	UE_API virtual void HandleRigVMGraphRemoved(const FRigVMClient* InClient, const FString& InNodePath) override;
	UE_API virtual void HandleRigVMGraphRenamed(const FRigVMClient* InClient, const FString& InOldNodePath, const FString& InNewNodePath) override;
	UE_API virtual void HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure) override;
	UE_API virtual UObject* GetEditorObjectForRigVMGraph(const URigVMGraph* InVMGraph) const override;
	UE_API virtual URigVMGraph* GetRigVMGraphForEditorObject(UObject* InObject) const override;
	UE_API virtual void RecompileVM() override;
	UE_API virtual void RecompileVMIfRequired() override;
	UE_API virtual void RequestAutoVMRecompilation() override;
	UE_API virtual void SetAutoVMRecompile(bool bAutoRecompile) override;
	UE_API virtual bool GetAutoVMRecompile() const override;
	UE_API virtual void IncrementVMRecompileBracket() override;
	UE_API virtual void DecrementVMRecompileBracket() override;
	UE_API virtual void RefreshAllModels(ERigVMLoadType InLoadType) override;
	UE_API virtual void OnRigVMRegistryChanged() override;
	UE_API virtual void RequestRigVMInit() override;
	UE_API virtual URigVMGraph* GetModel(const UEdGraph* InEdGraph = nullptr) const override;
	UE_API virtual URigVMGraph* GetModel(const FString& InNodePath) const override;
	UE_API virtual URigVMGraph* GetDefaultModel() const override;
	UE_API virtual TArray<URigVMGraph*> GetAllModels() const override;
	UE_API virtual URigVMFunctionLibrary* GetLocalFunctionLibrary() const override;
	UE_API virtual URigVMFunctionLibrary* GetOrCreateLocalFunctionLibrary(bool bSetupUndoRedo =  true) override;
	UE_API virtual URigVMGraph* AddModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true) override;
	UE_API virtual bool RemoveModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true) override;
	UE_API virtual FRigVMGetFocusedGraph& OnGetFocusedGraph() override;
	UE_API virtual const FRigVMGetFocusedGraph& OnGetFocusedGraph() const override;
	UE_API virtual URigVMGraph* GetFocusedModel() const override;
	UE_API virtual URigVMController* GetController(const URigVMGraph* InGraph = nullptr) const override;
	UE_API virtual URigVMController* GetControllerByName(const FString InGraphName = TEXT("")) const override;
	UE_API virtual URigVMController* GetOrCreateController(URigVMGraph* InGraph = nullptr) override;
	UE_API virtual URigVMController* GetController(const UEdGraph* InEdGraph) const override;
	UE_API virtual URigVMController* GetOrCreateController(const UEdGraph* InGraph) override;
	UE_API virtual TArray<FString> GeneratePythonCommands() override;
	UE_DEPRECATED(5.8, "Use TArray<FString> GeneratePythonCommands() instead.")
	virtual TArray<FString> GeneratePythonCommands(const FString InNewBlueprintName) override { return TArray<FString>(); }
	UE_API virtual void SetupPinRedirectorsForBackwardsCompatibility() override;
	UE_API virtual FRigVMGraphModifiedEvent& OnModified() override;
	UE_API virtual bool IsFunctionPublic(const FName& InFunctionName) const override;
	UE_API virtual void MarkFunctionPublic(const FName& InFunctionName, bool bIsPublic = true) override;
	UE_API virtual void RenameGraph(const FString& InNodePath, const FName& InNewName) override;


	// IRigVMGraphFunctionHost interface
	UE_API virtual FRigVMGraphFunctionStore* GetRigVMGraphFunctionStore() override;
	UE_API virtual const FRigVMGraphFunctionStore* GetRigVMGraphFunctionStore() const override;

	// IRigVMClientExternalModelHost interface
	virtual const TArray<TObjectPtr<URigVMGraph>>& GetExternalModels() const override { return GraphModels; }
	UE_API virtual TObjectPtr<URigVMGraph> CreateContainedGraphModel(URigVMCollapseNode* CollapseNode, const FName& Name) override;

	// Override called during initialization to determine what RigVM controller class is used
	virtual TSubclassOf<URigVMController> GetControllerClass() const { return UAnimNextControllerBase::StaticClass(); }

	// Override called during initialization to determine what RigVM execute struct is used
	virtual UScriptStruct* GetExecuteContextStruct() const PURE_VIRTUAL(UUAFRigVMAssetEditorData::GetExecuteContextStruct, return nullptr;)

	// Create and store a UEdGraph that corresponds to a URigVMGraph
	UE_API virtual UEdGraph* CreateEdGraph(URigVMGraph* InRigVMGraph, bool bForce);

	// Create and store a UEdGraph that corresponds to a URigVMCollapseNode
	UE_API virtual void CreateEdGraphForCollapseNode(URigVMCollapseNode* InNode, bool bForce);

	// Destroy a UEdGraph that corresponds to a URigVMCollapseNode
	UE_API virtual bool RemoveEdGraphForCollapseNode(URigVMCollapseNode* InNode, bool bNotify);

	// Remove the UEdGraph that corresponds to a URigVMGraph
	UE_API virtual bool RemoveEdGraph(URigVMGraph* InModel);

	// Initialize the asset for use
	UE_API virtual void Initialize(bool bRecompileVM);

	// Handle RigVM modification events
	UE_API virtual void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);

	// Class to use when instantiating AssetUserData for the EditorData instance
	UE_API virtual TSubclassOf<UAssetUserData> GetAssetUserDataClass() const;

	// Override point called during initialization (PostLoad/PostDuplicate) used for setting up asset user data.
	// By default this instantiates any asset user data that is missing according to GetAssetUserDataClass().
	UE_API virtual void InitializeAssetUserData();

	// Get all the kinds of entry for this asset
	virtual TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> GetEntryClasses() const PURE_VIRTUAL(UUAFRigVMAssetEditorData::GetEntryClasses, return {};)

	// Override to allow assets to prevent certain entries being created
	virtual bool CanAddNewEntry(TSubclassOf<UUAFRigVMAssetEntry> InClass) const { return true; }

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// Begin Compilation overrides, in order of operation

	// Called before a batch of compile jobs for dependent asset starts
	virtual void OnCompileJobStarted() {}

	// Compilation phase 1: Called before RigVM compilation to setup compiler settings and clean our outer asset of compiler-generated data
	virtual void OnPreCompileAsset(FRigVMCompileSettings& InSettings) {}

	// Compilation phase 2: Called before RigVM compilation to allow this asset to specify function headers that require generation, along with function generation metadata.
	// While users may manually generate graphs using function headers, for convience we provide an autogeneration process for function headers requested here.
	virtual void OnPreCompileGetProgrammaticFunctionHeaders(const FRigVMCompileSettings& InSettings, FAnimNextGetFunctionHeaderCompileContext& OutCompileContext) {}

	// Compilation phase 3: Called before RigVM compilation to allow this asset to generate variables to be injected, separate method to allow programmatic graphs to use these vars
	// These variables will be regenerated each compile, and are not saved between compiles
	virtual void OnPreCompileGetProgrammaticVariables(const FRigVMCompileSettings& InSettings, FAnimNextGetVariableCompileContext& OutCompileContext) {}
	
	// Compilation phase 3.1: Called after variable compilation, before RigVM compilation to allow this asset to generate variables to be injected, separate method to allow programmatic graphs to use these vars
	// These variables will be regenerated each compile, and are not saved between compiles
	virtual void OnPostCompileVariables(const FRigVMCompileSettings& InSettings, const FAnimNextGetVariableCompileContext& InCompileContext) {}

	// Compilation phase 4: Called before RigVM compilation to allow this asset to generate graphs to be injected
	virtual void OnPreCompileGetProgrammaticGraphs(const FRigVMCompileSettings& InSettings, FAnimNextGetGraphCompileContext& OutCompileContext) {}

	// Compilation phase 5: Called before RigVM compilation to allow this asset to process, transform or replace the graphs that will be compiled
	virtual void OnPreCompileProcessGraphs(const FRigVMCompileSettings& InSettings, FAnimNextProcessGraphCompileContext& OutCompileContext) {}

	// Compilation phase 6: Called after RigVM compilation to clean up/finish the compilation process
	virtual void OnPostCompileCleanup(const FRigVMCompileSettings& InSettings) {}

	// Called after a batch of compile jobs for dependent asset finishes and re-allocation can occur
	virtual void OnCompileJobFinished() {}

	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// End Compilation overrides

	// Return a function header compile context. Used when an external asset needs compile contexts for this asset.
	UE_API FAnimNextGetFunctionHeaderCompileContext GetFunctionHeaderContext(const FRigVMCompileSettings& InSettings, FAnimNextRigVMAssetCompileContext& InCompileContext) const;

	// Return a variable compile context. Used when an external asset needs compile contexts for this asset.
	UE_API FAnimNextGetVariableCompileContext GetVariableCompileContext(const FRigVMCompileSettings& InSettings, FAnimNextRigVMAssetCompileContext& InCompileContext) const;

	// Customization point for derived types to transform new asset entries
	virtual void CustomizeNewAssetEntry(UUAFRigVMAssetEntry* InNewEntry) const {}

	// Helper for creating new sub-entries. Sets package flags and outers appropriately 
	static UE_API UObject* CreateNewSubEntry(UUAFRigVMAssetEditorData* InEditorData, TSubclassOf<UObject> InClass);

	// Helper for creating new sub-entries. Sets package flags and outers appropriately
	template<typename EntryClassType>
	static EntryClassType* CreateNewSubEntry(UUAFRigVMAssetEditorData* InEditorData)
	{
		return CastChecked<EntryClassType>(CreateNewSubEntry(InEditorData, EntryClassType::StaticClass()));
	}

	// Get all the entries for this asset
	TConstArrayView<TObjectPtr<UUAFRigVMAssetEntry>> GetAllEntries() const { return Entries; } 

	// Access all the UEdGraphs in this asset
	UE_API TArray<UEdGraph*> GetAllEdGraphs() const;

	// Called during compilation to gather instance components from nodes etc.
	void GatherDefaultComponents(UUAFRigVMAsset* InAsset);

	// Iterate over all entries of the specified type
	// If predicate returns false, iteration is stopped
	template<typename EntryType, typename PredicateType>
	void ForEachEntryOfType(PredicateType InPredicate) const
	{
		for(UUAFRigVMAssetEntry* Entry : Entries)
		{
			if(EntryType* TypedEntry = Cast<EntryType>(Entry))
			{
				if(!InPredicate(TypedEntry))
				{
					return;
				}
			}
		}
	}

	// Find the first entry of the specified type
	template<typename EntryType>
	EntryType* FindFirstEntryOfType() const
	{
		EntryType* FirstEntry = nullptr;
		ForEachEntryOfType<EntryType>([&FirstEntry](EntryType* InEntry)
		{
			FirstEntry = InEntry;
			return false;
		});
		return FirstEntry;
	}

	// Returns all nodes in all graphs of the specified class
	template<class T>
	void GetAllNodesOfClass(TArray<T*>& OutNodes) const
	{
		ForEachEntryOfType<IUAFRigVMGraphInterface>([&OutNodes](IUAFRigVMGraphInterface* InGraphInterface)
		{
			URigVMEdGraph* RigVMEdGraph = InGraphInterface->GetEdGraph();
			check(RigVMEdGraph)

			TArray<T*> GraphNodes;
			RigVMEdGraph->GetNodesOfClass<T>(GraphNodes);

			TArray<UEdGraph*> SubGraphs;
			RigVMEdGraph->GetAllChildrenGraphs(SubGraphs);
			for (const UEdGraph* SubGraph : SubGraphs)
			{
				if (SubGraph)
				{
					SubGraph->GetNodesOfClass<T>(GraphNodes);
				}
			}

			OutNodes.Append(GraphNodes);

			return true;
		});

		for (URigVMEdGraph* RigVMEdGraph : FunctionEdGraphs)
		{
			if (RigVMEdGraph)
			{
				RigVMEdGraph->GetNodesOfClass<T>(OutNodes);

				TArray<UEdGraph*> SubGraphs;
				RigVMEdGraph->GetAllChildrenGraphs(SubGraphs);
				for (const UEdGraph* SubGraph : SubGraphs)
				{
					if (SubGraph)
					{
						SubGraph->GetNodesOfClass<T>(OutNodes);
					}
				}
			}
		}
	}

	// Remove an entry from the asset
	// @return true if the item was removed
	UE_API bool RemoveEntry(UUAFRigVMAssetEntry* InEntry, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	// Remove a number of entries from the asset
	// @return true if any items were removed
	UE_API bool RemoveEntries(TConstArrayView<UUAFRigVMAssetEntry*> InEntries, bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	// Remove all entries from the asset
	// @return true if any items were removed
	UE_API bool RemoveAllEntries(bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	UE_API void BroadcastModified(EAnimNextEditorDataNotifType InType, UObject* InSubject);

	UE_API void ReconstructAllNodes();
	
	UE_API void RequestVMRecompilation();

	// Find an entry that corresponds to the specified RigVMGraph. This uses the name of the graph to match the entry 
	UE_API UUAFRigVMAssetEntry* FindEntryForRigVMGraph(const URigVMGraph* InRigVMGraph) const;

	// Find an entry that corresponds to the specified RigVMGraph. This uses the name of the graph to match the entry 
	UE_API UUAFRigVMAssetEntry* FindEntryForRigVMEdGraph(const URigVMEdGraph* InRigVMEdGraph) const;

	// Checks all entries to see if any are public variables
	UE_API bool HasPublicVariables() const;

	// Information about a variable returned from GetAllVariables
	struct FVariableInfo
	{
		FName Name;
		FAnimNextParamType Type;
		const UObject* SourceObject = nullptr;
		EAnimNextExportAccessSpecifier Access = EAnimNextExportAccessSpecifier::Private;
		const FProperty* Property = nullptr;
		TConstArrayView<uint8> DefaultValue;
		FGuid StableGuid;

		friend uint32 GetTypeHash(const FVariableInfo& Info)
		{
			return HashCombine(HashCombine(GetTypeHash(Info.Name), GetTypeHash(Info.Type)), GetTypeHash(Info.SourceObject), GetTypeHash(Info.StableGuid));
		}

		bool operator==(const FVariableInfo& Other) const
		{
			return Name == Other.Name
				&& Type == Other.Type
				&& SourceObject == Other.SourceObject
				&& Access == Other.Access
				&& StableGuid == Other.StableGuid;
		}
	};

	// How to recurse when getting variable info
	enum class EVariableRecursion
	{
		SelfOnly,
		IncludeShared,
	};

	// How to filter variables by access level when getting variable info
	enum class EVariableAccessFilter
	{
		PublicOnly,
		All,
	};
	
	// Gets any variables that this asset has according to the pass-in params. Variables have no specified order.
	// May recurse into shared variables, so variables returned by this function may not be directly owned by this asset. In this case the
	// SourceObject will indicate where the variable came from.
	UE_API void GetAllVariables(TArray<FVariableInfo>& OutVariables, EVariableRecursion InRecursion, EVariableAccessFilter InAccess) const;

	// Refresh the 'external' models for the RigVM client to reference
	UE_API void RefreshExternalModels();

	// Update referenced function impl graph nodes to avoid compile issues. Doesn't recompile, only updates graph nodes.
	UE_API void RefreshFunctionImplementationGraphs();

	// Clear the error info for all EdGraphNodes
	UE_API void ClearErrorInfoForAllEdGraphs();

	// Handle compiler reporting
	UE_API void HandleReportFromCompiler(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage) const;
	UE_API void HandleMessageFromCompiler(TSharedRef<FTokenizedMessage> InMessage, bool bAddMessageToLog = true) const;

	// Support extra references in GC
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	// Add a new entry to this asset, taking into account external packaging status
	UE_API void AddEntryInternal(UUAFRigVMAssetEntry* InEntry);

	// Inserts a new entry to this asset, at the provided index, while taking into account external packaging status
	void InsertEntryInternal(UUAFRigVMAssetEntry* InEntry, int32 InsertionIndex);

	// Remove an entry to this asset, taking into account external packaging status
	UE_API void RemoveEntryInternal(UUAFRigVMAssetEntry* InEntry);

	// Remove any programmatic graphs generated during compilation and consign them to the transient package
	UE_API void RemoveProgrammaticGraphs(TArrayView<URigVMGraph*> InGraphs);

	// Remove any transient graphs in the passed-in array (e.g. generated during compilation) and consign them to the transient package
	UE_API void RemoveTransientGraphs(TArrayView<URigVMGraph*> InGraphs);

	// Handle removing a notify
	static UE_API void HandleRemoveNotify(UObject* InAsset, const FString& InFindString, bool bFindWholeWord, ESearchCase::Type InSearchCase);

	// Handle replacing a notify
	static UE_API void HandleReplaceNotify(UObject* InAsset, const FString& InFindString, const FString& InReplaceString, bool bFindWholeWord, ESearchCase::Type InSearchCase);

	// Check whether this asset should be recompiled
	UE_API bool IsDirtyForRecompilation() const;

	// Called pre programmatic function header population to allow population of compile context in a const manner. May be used by external assets querying our VM. 
	UE_API virtual void BuildFunctionHeadersContext(const FRigVMCompileSettings& InSettings, FAnimNextGetFunctionHeaderCompileContext& OutCompileContext) const;

	// Called pre programmatic variable population to allow population of compile context in a const manner. May be used by external assets querying our VM.
	virtual void BuildProgrammaticVariablesContext(const FRigVMCompileSettings& InSettings, FAnimNextGetVariableCompileContext& OutCompileContext) const {}

	// Called during compilation to generate internal variables used for natively-callable RigVM functions
	UE_API void BuildFunctionWrapperEventVariables(FAnimNextRigVMAssetCompileContext& InContext) const;

	// Called during compilation to generate wrapper events for natively-callable RigVM functions
	UE_API void BuildFunctionWrapperEvents(FAnimNextRigVMAssetCompileContext& InContext, const FRigVMCompileSettings& InSettings);

	// Generates a combined property bag consisting of all variables both internal and shared
	UE_API FInstancedPropertyBag GenerateCombinedPropertyBag(const FRigVMCompileSettings& InSettings, const FAnimNextGetVariableCompileContext& InCompileContext) const;

	// Serialization/post-load fixup for old data
	void UpgradeDataInterfaces();

	// Conditionally calls PatchFunctionReferencesOnLoad and PatchFunctionsOnLoad on the RigVMClient	
	void ConditionalPatchFunctionsOnLoad();

	void SetVMRecompilationRequired(bool bInIsRequired);

	/** All entries in this asset */
	UPROPERTY()
	TArray<TObjectPtr<UUAFRigVMAssetEntry>> Entries;

	UE_DEPRECATED(5.7, "This property is deprecated. Please use Entries instead")
	UPROPERTY()
	TArray<TObjectPtr<UUAFRigVMAssetEntry>> InternalEntries_DEPRECATED;

	UPROPERTY()
	FRigVMClient RigVMClient;

	UPROPERTY()
	FRigVMGraphFunctionStore GraphFunctionStore;

	// The native C++ struct that is used to communicate with this asset
	UE_DEPRECATED(5.6, "This property is deprecated. Please use NativeInterfaces instead")
	UPROPERTY()
	TObjectPtr<const UScriptStruct> NativeInterface_DEPRECATED;

	// The list of native C++ structs that are used to communicate with this asset
	UE_DEPRECATED(5.6, "This property is deprecated. Please use NativeInterfaces instead")
	UPROPERTY()
	TArray<TObjectPtr<const UScriptStruct>> NativeInterfaces_DEPRECATED;

	UPROPERTY(EditAnywhere, Category = "User Interface")
	FRigVMEdGraphDisplaySettings RigGraphDisplaySettings;

	UPROPERTY(EditAnywhere, Category = "VM")
	FRigVMRuntimeSettings VMRuntimeSettings;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VM", meta = (AllowPrivateAccess = "true"))
	FRigVMCompileSettings VMCompileSettings;

	UPROPERTY(transient, DuplicateTransient)
	TMap<FString, FRigVMOperand> PinToOperandMap;

	UPROPERTY()
	TArray<FEditedDocumentInfo> LastEditedDocuments;

	// Additional components that should be allocated for instances of this asset
	UPROPERTY(EditAnywhere, Category = Components)
	TArray<TInstancedStruct<FUAFAssetInstanceComponent>> AdditionalComponents;

	// Components that will be allocated for instances of this asset
	// This is a combination of required components based on the asset and additional components added by the user 
	UPROPERTY(VisibleAnywhere, Category=Components, AdvancedDisplay, meta = (DisplayName = "Required Components"))
	TArray<TInstancedStruct<FUAFAssetInstanceComponent>> Components;

	int32 VMRecompilationBracket = 0;
	bool bVMRecompilationRequired = false;
	bool bIsCompiling = false;

	UE_DEPRECATED(5.7, "Storing asset entries as external packages has been deprecated")
	UPROPERTY()
	bool bUsesExternalPackages_DEPRECATED = false;

	UPROPERTY()
	FName DefaultInjectionSite_DEPRECATED;

	// The injection site (AnimNext Graph) that will be used by default
	UPROPERTY(EditAnywhere, DisplayName = "Default Injection Site", Category=Injection, meta=(AllowedType = FAnimNextAnimGraph))
	FAnimNextVariableReference DefaultInjectionSiteReference;

	// Set of category names, assignable to variables/functions by user
	UPROPERTY()
	TArray<FString> VariableAndFunctionCategories;
	
	FOnRigVMCompiledEvent RigVMCompiledEvent;

	FRigVMGraphModifiedEvent RigVMGraphModifiedEvent;

	// Compile status depends upon bVMRecompilationRequired so it useful to know when this value is changed
	FSimpleMulticastDelegate RecompileRequiredChangedEvent;

	// Delegate to subscribe to modifications to this editor data
	UE::UAF::UncookedOnly::FOnEditorDataModified ModifiedDelegate;

	// Delegate to get notified when an interaction bracket reaches 0
	UE::UAF::UncookedOnly::FOnInteractionBracketFinished InteractionBracketFinished;

	// Cached exports, generated lazily or on compilation
	mutable TOptional<FAnimNextAssetRegistryExports> CachedExports;
	
	// Collection of models gleaned from graphs
	TArray<TObjectPtr<URigVMGraph>> GraphModels;

	// Set of functions implemented for this graph
	UPROPERTY()
	TArray<TObjectPtr<URigVMEdGraph>> FunctionEdGraphs;

	// Default FunctionLibrary EdGraph
	UPROPERTY()
	TObjectPtr<UAnimNextEdGraph> FunctionLibraryEdGraph;

	UPROPERTY()
	bool bAutoRecompileVM = true;
	
	/** Whether to show a warning when attempting to start in PIE and there is a compiler error on this asset */
	UPROPERTY(transient)
	bool bDisplayCompilePIEWarning;

	bool bVMRecompilationRequested = false;
	mutable bool bErrorsDuringCompilation = false;
	mutable bool bWarningsDuringCompilation = false;
	bool bSuspendModelNotificationsForSelf = false;
	bool bSuspendAllNotifications = false;
	bool bCompileInDebugMode = false;
	bool bSuspendPythonMessagesForRigVMClient = true;
	bool bSuspendEditorDataNotifications = false;
	bool bSuspendCompilationNotifications = false;
	bool bSuspendCompilerReports = false;
	bool bUpgradeDataInterfacesOnLoad = false;
	bool bFunctionsPatched = false;
	bool bHasCompiledVariables = false;
};

#undef UE_API
