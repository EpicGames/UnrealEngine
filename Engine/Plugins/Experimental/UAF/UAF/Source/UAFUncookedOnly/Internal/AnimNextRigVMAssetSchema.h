// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextEdGraphSchema.h"
#include "RigVMModel/RigVMSchema.h"
#include "AnimNextRigVMAssetSchema.generated.h"

#define UE_API UAFUNCOOKEDONLY_API

UCLASS(MinimalAPI)
class UUAFRigVMAssetSchema : public URigVMSchema
{
	GENERATED_BODY()

	virtual UClass* GetEdGraphSchemaClass() const override
	{
		return UAnimNextEdGraphSchema::StaticClass();
	}

	virtual bool SupportsNodeLayouts(const URigVMGraph* InGraph) const override
	{
		return true;
	}

protected:
	UE_API UUAFRigVMAssetSchema();
	
	virtual bool ShouldPatchGraphSchema() const override { return false; }
};

#undef UE_API
