// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Entries/AnimNextSharedVariablesEntry.h"
#include "RigVMModel/Nodes/RigVMVariableNode.h"
#include "AnimNextSharedVariableNode.generated.h"

class UUAFSharedVariables;
class UAnimNextControllerBase;

namespace UE::UAF::UncookedOnly
{
struct FUtils;
}

namespace UE::UAF::Editor
{
class FAnimNextEditorModule;
}

UCLASS()
class UUAFSharedVariableNode : public URigVMVariableNode
{
	GENERATED_BODY()

	// URigVMVariableNode interface
	virtual FString GetNodeSubTitle() const override;
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;

private:
	void UpdateCachedVariableGUID();

private:
	friend UAnimNextControllerBase;
	friend UE::UAF::UncookedOnly::FUtils;
	friend UE::UAF::Editor::FAnimNextEditorModule;

	UPROPERTY()
	TObjectPtr<const UUAFSharedVariables> Asset;

	UPROPERTY()
	TObjectPtr<const UScriptStruct> Struct;

	UPROPERTY()
	TScriptInterface<const IRigVMRuntimeAssetInterface> RigVMAsset;
	
	UPROPERTY()
	EAnimNextSharedVariablesType Type;
	
	UPROPERTY()
	FGuid CachedGuid;
};
