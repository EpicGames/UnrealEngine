// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"

#include "AnimNextUnitNode.generated.h"

/**
  * Implements AnimNext RigVM unit node extensions
  */
UCLASS(MinimalAPI)
class UAnimNextUnitNode : public URigVMUnitNode
{
	GENERATED_BODY()

public:
	// Override node functions
	virtual bool HasNonNativePins() const
	{
		return true;
	}
};
