// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/AnimNextRigVMAssetCompileContext.h"

#define UE_API UAFUNCOOKEDONLY_API

class UUAFRigVMAssetEditorData;

/**
 * Struct holding temporary compilation info during function header population 
 */
struct FAnimNextGetFunctionHeaderCompileContext
{
	UE_API FAnimNextGetFunctionHeaderCompileContext(FAnimNextRigVMAssetCompileContext& InCompilerContext);

public:
	
	UE_API const TArray<FAnimNextProgrammaticFunctionHeader>& GetFunctionHeaders() const;
	UE_API void AddUniqueFunctionHeader(const FAnimNextProgrammaticFunctionHeader& InFunctionHeader);
	UE_API const UUAFRigVMAssetEditorData* GetOwningAssetEditorData() const;

protected:

	FAnimNextRigVMAssetCompileContext& CompilerContext;
};

#undef UE_API
