// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/AnimNextRigVMAssetCompileContext.h"

#define UE_API UAFUNCOOKEDONLY_API

class UUAFRigVMAssetEditorData;

/**
 * Struct holding temporary compilation info during function header population
 */
struct FAnimNextGetGraphCompileContext
{
	UE_API FAnimNextGetGraphCompileContext(FAnimNextRigVMAssetCompileContext& InCompilerContext);

public:

	UE_API const TArray<FAnimNextProgrammaticFunctionHeader>& GetFunctionHeaders() const;

	UE_API const TArray<URigVMGraph*>& GetProgrammaticGraphs() const;
	UE_API TArray<URigVMGraph*>& GetMutableProgrammaticGraphs();
	UE_API const UUAFRigVMAssetEditorData* GetOwningAssetEditorData() const;

protected:

	FAnimNextRigVMAssetCompileContext& CompilerContext;
};

#undef UE_API
