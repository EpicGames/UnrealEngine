// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/AnimNextRigVMAssetCompileContext.h"

#define UE_API UAFUNCOOKEDONLY_API

class UUAFRigVMAssetEditorData;

/**
 * Struct holding temporary compilation info during function header population
 */
struct FAnimNextGetVariableCompileContext
{
	UE_API FAnimNextGetVariableCompileContext(FAnimNextRigVMAssetCompileContext& InCompilerContext);

public:

	UE_API const TArray<FAnimNextProgrammaticFunctionHeader>& GetFunctionHeaders() const;

	UE_API const TArray<FAnimNextProgrammaticVariable>& GetProgrammaticVariables() const;
	UE_API TArray<FAnimNextProgrammaticVariable>& GetMutableProgrammaticVariables();
	UE_API const UUAFRigVMAssetEditorData* GetOwningAssetEditorData() const;
	
	UE_API const FAnimNextRigVMAssetCompileContext& GetAssetCompileContext() const;

protected:

	FAnimNextRigVMAssetCompileContext& CompilerContext;
};

#undef UE_API
