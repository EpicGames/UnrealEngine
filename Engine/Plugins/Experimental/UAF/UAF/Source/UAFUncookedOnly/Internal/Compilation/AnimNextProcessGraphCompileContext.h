// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/AnimNextRigVMAssetCompileContext.h"

#define UE_API UAFUNCOOKEDONLY_API

class UUAFRigVMAssetEditorData;

/**
 * Struct holding temporary compilation info during function header population
 */
struct FAnimNextProcessGraphCompileContext
{
	UE_API FAnimNextProcessGraphCompileContext(FAnimNextRigVMAssetCompileContext& InCompilerContext);

public:

	UE_API const TArray<URigVMGraph*>& GetAllGraphs() const;
	UE_API TArray<URigVMGraph*>& GetMutableAllGraphs();
	UE_API const UUAFRigVMAssetEditorData* GetOwningAssetEditorData() const;

protected:

	FAnimNextRigVMAssetCompileContext& CompilerContext;
};

#undef UE_API
