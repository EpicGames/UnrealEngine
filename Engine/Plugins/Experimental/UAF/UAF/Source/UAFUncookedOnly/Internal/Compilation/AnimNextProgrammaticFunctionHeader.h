// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Param/ParamType.h"
#include "StructUtils/PropertyBag.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"

/**
 * Struct wrapping a RigVM Function header. Also contains some metadata used to drive compilation.
 */
struct FAnimNextProgrammaticFunctionHeader
{
	/** RigVM Function Header we are wrapping */
	FRigVMGraphFunctionHeader FunctionHeader;

	bool operator==(const FAnimNextProgrammaticFunctionHeader& Other) const = default;
};

