// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Param/ParamType.h"
#include "StructUtils/PropertyBag.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "Compilation/AnimNextProgrammaticFunctionHeader.h"
#include "Variables/AnimNextProgrammaticVariable.h"

#define UE_API UAFUNCOOKEDONLY_API

class URigVMGraph;
class UUAFRigVMAssetEditorData;

/**
 * Struct holding temporary compilation info for a single RigVM asset
 * 
 * Note: Sub compile contexts give limited access. But feel free to expand
 * the access of any particular sub context as needed.
 */
struct FAnimNextRigVMAssetCompileContext
{
public:

	FAnimNextRigVMAssetCompileContext(const UUAFRigVMAssetEditorData* InOwningAssetEditorData)
		: OwningAssetEditorData(InOwningAssetEditorData)
	{
	}

#if WITH_EDITOR
	template<typename... ArgTypes>
	void Error(UObject* Object, FTextFormat Format, ArgTypes... Args) const
	{
		Message(EMessageSeverity::Error, Object, FText::Format(Format, Args...));
	}

	template<typename... ArgTypes>
	void Error(FTextFormat Format, ArgTypes... Args) const
	{
		Message(EMessageSeverity::Error, OwningAssetEditorData, FText::Format(Format, Args...));
	}

	template<typename... ArgTypes>
	void Warning(UObject* Object, FTextFormat Format, ArgTypes... Args) const
	{
		Message(EMessageSeverity::Warning, Object, FText::Format(Format, Args...));
	}

	template<typename... ArgTypes>
	void Warning(FTextFormat Format, ArgTypes... Args) const
	{
		Message(EMessageSeverity::Warning, OwningAssetEditorData, FText::Format(Format, Args...));
	}

	template<typename... ArgTypes>
	void Info(UObject* Object, FTextFormat Format, ArgTypes... Args) const
	{
		Message(EMessageSeverity::Info, Object, FText::Format(Format, Args...));
	}

	template<typename... ArgTypes>
	void Info(FTextFormat Format, ArgTypes... Args) const
	{
		Message(EMessageSeverity::Info, OwningAssetEditorData, FText::Format(Format, Args...));
	}
	
	UE_API void Message(TSharedRef<FTokenizedMessage> InMessage) const;

private:
	UE_API void Message(EMessageSeverity::Type Severity, UObject* Object, const FText& Message) const;
#endif // WITH_EDITOR

private:
	friend class UUAFRigVMAssetEditorData;
	friend struct FAnimNextGetFunctionHeaderCompileContext;
	friend struct FAnimNextGetVariableCompileContext;
	friend struct FAnimNextGetGraphCompileContext;
	friend struct FAnimNextProcessGraphCompileContext;

protected:

	/** Function Headers we should generate variables / graphs for */
	TArray<FAnimNextProgrammaticFunctionHeader> FunctionHeaders;

	/** Programmtic variables generated during this compile */
	TArray<FAnimNextProgrammaticVariable> ProgrammaticVariables;

	/** Programmtic graphs generated during this compile */
	TArray<URigVMGraph*> ProgrammaticGraphs;

	/** All graphs compiled by this RigVM Asset */
	TArray<URigVMGraph*> AllGraphs;

	/** RigVM Asset Editor Data this compile context is designed for. Used to ensure we don't cross contexts. */
	const UUAFRigVMAssetEditorData* OwningAssetEditorData;
};

#undef UE_API // UAFUNCOOKEDONLY_API