// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextRigVMAssetEntry.h"

#include "AnimNextCategoryEntry.generated.h"

namespace UE::UAF::Editor
{
struct FVariablesOutlinerCategoryItem;
class FVariablesOutlinerHierarchy;
}

UCLASS(MinimalAPI, DisplayName = "Category")
class UAnimNextCategoryEntry : public UUAFRigVMAssetEntry
{
	GENERATED_BODY()

	friend class UUAFRigVMAssetEditorData;
	friend UE::UAF::Editor::FVariablesOutlinerCategoryItem;
	friend UE::UAF::Editor::FVariablesOutlinerHierarchy;

	// UUAFRigVMAssetEntry interface
	virtual void Initialize(UUAFRigVMAssetEditorData* InEditorData) override;	
	virtual FName GetEntryName() const override { return CategoryName; }
	virtual void SetEntryName(FName InName, bool bSetupUndoRedo = true) override;

protected:
	UPROPERTY(VisibleAnywhere, Category = "Category")
	FName CategoryName;
};