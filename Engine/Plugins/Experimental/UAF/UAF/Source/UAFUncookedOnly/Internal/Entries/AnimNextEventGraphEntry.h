// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextRigVMAssetEntry.h"
#include "IAnimNextRigVMExportInterface.h"
#include "IAnimNextRigVMGraphInterface.h"
#include "Variables/IAnimNextRigVMVariableInterface.h"
#include "AnimNextEventGraphEntry.generated.h"

class UUAFRigVMAssetEditorData;
class UUAFSystem_EditorData;
class UAnimNextEdGraph;

namespace UE::UAF::Tests
{
	class FEditor_Variables;
	class FVariables_UOLBindings;
	class FEditor_Graphs;
	class FEditor_AnimGraph_Variables;
	class FEditor_AnimGraph_Graphs;
	class FVariables;
}

namespace UE::UAF::Editor
{
	class FAnimNextEditorModule;
}

UCLASS(MinimalAPI, Category = "Event Graphs", DisplayName = "Event Graph")
class UAnimNextEventGraphEntry : public UUAFRigVMAssetEntry, public IUAFRigVMGraphInterface
{
	GENERATED_BODY()

	friend class UUAFRigVMAssetEditorData;
	friend class UUAFSystem_EditorData;
	friend class UUAFSystemFactory;
	friend class UE::UAF::Tests::FEditor_Variables;
	friend class UE::UAF::Tests::FVariables_UOLBindings;
	friend class UE::UAF::Tests::FEditor_AnimGraph_Variables;
	friend class UE::UAF::Tests::FEditor_Graphs;
	friend class UE::UAF::Tests::FEditor_AnimGraph_Graphs;
	friend class UE::UAF::Tests::FVariables;
	friend class UE::UAF::Editor::FAnimNextEditorModule;

	// UUAFRigVMAssetEntry interface
	virtual FName GetEntryName() const override { return GraphName; }
	virtual void SetEntryName(FName InName, bool bSetupUndoRedo = true) override;
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayNameTooltip() const override;

	// IAnimNextRigVMGraphInterface interface
	virtual const FName& GetGraphName() const override;
	virtual URigVMGraph* GetRigVMGraph() const override;
	virtual URigVMEdGraph* GetEdGraph() const override;
	virtual void SetRigVMGraph(URigVMGraph* InGraph) override;
	virtual void SetEdGraph(URigVMEdGraph* InGraph) override;

	/** The name of the graph */
	UPROPERTY(VisibleAnywhere, Category = "Event Graph")
	FName GraphName;

	/** Graph */
	UPROPERTY()
	TObjectPtr<URigVMGraph> Graph;

	/** Graph */
	UPROPERTY()
	TObjectPtr<UAnimNextEdGraph> EdGraph;
};
