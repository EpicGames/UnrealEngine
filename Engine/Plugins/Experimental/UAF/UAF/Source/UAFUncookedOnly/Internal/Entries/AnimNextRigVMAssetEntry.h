// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "AnimNextRigVMAssetEntry.generated.h"

#define UE_API UAFUNCOOKEDONLY_API

class UUAFRigVMAssetEditorData;
enum class ERigVMGraphNotifType : uint8;
enum class EAnimNextEditorDataNotifType : uint8;
class URigVMGraph;

namespace UE::UAF::Editor
{
	class SRigVMAssetView;
	class SRigVMAssetViewRow;
	struct FRigVMAssetViewEntry;
	class FVariableCustomization;
}

/** Base class that defines an entry in a module, e.g. a parameter or a graph */
UCLASS(MinimalAPI, BlueprintType, Abstract)
class UUAFRigVMAssetEntry : public UObject
{
	GENERATED_BODY()
public:
	// Binds delegates to owning editor data
	UE_API virtual void Initialize(UUAFRigVMAssetEditorData* InEditorData);

	// Allow entries the opportunity to handle RigVM modification events
	virtual void HandleRigVMGraphModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject) {}

	// Get this entry's name
	virtual FName GetEntryName() const PURE_VIRTUAL(UUAFRigVMAssetEntry::GetEntryName, return NAME_None;)

	// Set this entry's name
	virtual void SetEntryName(FName InName, bool bSetupUndoRedo = true) {}

	// Get the name to be displayed in the UI for this entry
	virtual FText GetDisplayName() const { return FText::FromName(GetEntryName()); }

	// Get the tooltip to be displayed for the name in the UI for this entry
	virtual FText GetDisplayNameTooltip() const { return FText::FromName(GetEntryName()); }

	// Set whether this entry is hidden in the outliner
	void SetHiddenInOutliner(bool bInHide) { bHiddenInOutliner = bInHide; }

	// Get whether this entry is hidden in the outliner
	bool IsHiddenInOutliner() const { return bHiddenInOutliner; }

	// UObject interface
	UE_API virtual bool IsAsset() const override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	UE_API virtual void PostEditUndo() override;
#endif

protected:
	UE_API void BroadcastModified(EAnimNextEditorDataNotifType InType);

	friend class UE::UAF::Editor::FVariableCustomization;
	friend class UE::UAF::Editor::SRigVMAssetView;

	UPROPERTY()
	uint32 bHiddenInOutliner : 1 = false;
};

#undef UE_API
