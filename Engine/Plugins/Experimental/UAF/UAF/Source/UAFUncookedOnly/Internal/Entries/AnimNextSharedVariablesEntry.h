// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextRigVMAssetEntry.h"
#include "Param/ParamType.h"
#include "AnimNextSharedVariablesEntry.generated.h"

class IRigVMRuntimeAssetInterface;
class UUAFSharedVariables;
class UUAFRigVMAssetEditorData;
class UAssetDefinition_AnimNextSharedVariablesEntry;

namespace UE::UAF::Editor
{
	class FVariablesOutlinerHierarchy;
	struct FVariablesOutlinerStructSharedVariablesItem;
	class SVariablesOutlinerStructSharedVariablesLabel;
	class SVariablesOutlinerValue;
	class FVariableProxyCustomization;
	class SAddVariablesDialog;
	class SVariableOverride;
	class FVariablesOutlinerMode;
	class FFunctionsOutlinerMode;
	class FOutlinerHierarchy;
	class SCommonOutliner;
}

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}


UENUM()
enum class EAnimNextSharedVariablesType : uint8
{
	Asset,		// reference to shared variable defined in UUAFSharedVariables
	Struct,		// Reference to a property of a native struct
	RigVMAsset	// Reference to an external variable defined in a RigVMAsset
};

UCLASS(MinimalAPI, Category = "Shared Variables", DisplayName = "Shared Variables")
class UUAFSharedVariablesEntry : public UUAFRigVMAssetEntry
{
	GENERATED_BODY()

	friend class UUAFRigVMAssetEditorData;
	friend class UAssetDefinition_AnimNextSharedVariablesEntry;
	friend class UE::UAF::Editor::FVariablesOutlinerHierarchy;
	friend class UE::UAF::Editor::FOutlinerHierarchy;
	friend struct UE::UAF::Editor::FVariablesOutlinerStructSharedVariablesItem;
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend class UE::UAF::Editor::SVariablesOutlinerStructSharedVariablesLabel;
	friend class UE::UAF::Editor::SVariablesOutlinerValue;
	friend class UE::UAF::Editor::FVariableProxyCustomization;
	friend class UE::UAF::Editor::SAddVariablesDialog;
	friend class UE::UAF::Editor::SVariableOverride;
	friend UE::UAF::Editor::FVariablesOutlinerMode;
	friend UE::UAF::Editor::SCommonOutliner;
	friend UE::UAF::Editor::FFunctionsOutlinerMode;
	friend class UAnimNextAssetFindReplaceVariables;

	// UObject interface
	UAFUNCOOKEDONLY_API virtual void Serialize(FArchive& Ar) override;
	UAFUNCOOKEDONLY_API virtual void PostLoad() override;

	// UUAFRigVMAssetEntry interface
	virtual void Initialize(UUAFRigVMAssetEditorData* InEditorData) override;
	virtual FName GetEntryName() const override;
	virtual void SetEntryName(FName InName, bool bSetupUndoRedo = true) override {}
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayNameTooltip() const override;

	// Get the type of this shared variables entry
	EAnimNextSharedVariablesType GetType() const
	{
		return Type;
	}

	// Set this entry to use an asset (rather than a struct)
	UAFUNCOOKEDONLY_API void SetAsset(const UUAFSharedVariables* InAsset, bool bSetupUndoRedo = true);

	// Set this entry to use an asset (rather than a struct)
	UAFUNCOOKEDONLY_API void SetRigVMAsset(const IRigVMRuntimeAssetInterface* InRigVMAsset, bool bSetupUndoRedo = true);

	// Get the asset that this entry represents. If this entry uses a struct, this call will return nullptr
	UAFUNCOOKEDONLY_API const UUAFSharedVariables* GetAsset() const;

	// Get the path to the asset/struct path that this entry represents, if any
	UAFUNCOOKEDONLY_API FSoftObjectPath GetObjectPath() const;

	// Set this entry to use a struct (rather than an asset)
	UAFUNCOOKEDONLY_API void SetStruct(const UScriptStruct* InStruct, bool bSetupUndoRedo = true);

	// Get the struct that this entry represents. If this entry uses an asset, this call will return nullptr
	UAFUNCOOKEDONLY_API const UScriptStruct* GetStruct() const;

	UAFUNCOOKEDONLY_API const TScriptInterface<const IRigVMRuntimeAssetInterface>& GetRigVMAsset() const { return RigVMAsset; }

	// Recompiles this asset when the linked asset is modified
	void HandleAssetModified(UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType, UObject* InSubject);
	
	// Marks this asset as requiring compilation when the linked asset is marked as such
	void HandleAssetRequiresCompilation(TWeakObjectPtr<UUAFRigVMAssetEditorData> WeakEditorData) const;

#if WITH_LIVE_CODING
	// Recompiles this asset when live coding is run (potentially modifying the struct we reference)
	void HandlePatchComplete();
#endif

	/** The asset whose variables we share */
	UPROPERTY(VisibleAnywhere, Category = "Asset")
	TObjectPtr<const UUAFSharedVariables> Asset;

	/** The asset whose variables we share */
	UPROPERTY(VisibleAnywhere, Category = "Asset")
	TScriptInterface<const IRigVMRuntimeAssetInterface> RigVMAsset;

	/** Soft reference to the asset/struct for error reporting */
	UPROPERTY()
	FSoftObjectPath ObjectPath;

	/** The struct whose variables we use */
	UPROPERTY(VisibleAnywhere, Category = "Asset")
	TObjectPtr<const UScriptStruct> Struct;

	UPROPERTY()
	EAnimNextSharedVariablesType Type;
};
