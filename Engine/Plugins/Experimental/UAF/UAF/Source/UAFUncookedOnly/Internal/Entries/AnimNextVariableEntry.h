// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextRigVMAssetEntry.h"
#include "IAnimNextRigVMExportInterface.h"
#include "Variables/IAnimNextRigVMVariableInterface.h"
#include "Param/ParamType.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Logging/StructuredLog.h"
#include "Variables/AnimNextVariableBinding.h"
#include "AnimNextVariableEntry.generated.h"

class UUAFRigVMAssetEditorData;
class UUAFSystem_EditorData;
class UUAFSharedVariables_EditorData;
class UAnimNextVariableBinding;
class UUAFSharedVariablesEntry;
class FAnimationAnimNextRuntimeTest_Variables;

namespace UE::UAF::Editor
{
	class FVariableCustomization;
	class FVariableBindingPropertyCustomization;
	class FUniversalObjectLocatorBindingCustomization;
	class FVariablesOutlinerMode;
	class SAddVariablesDialog;
	class FVariablesOutlinerHierarchy;
	class SVariablesOutlinerValue;
	class FVariableProxyCustomization;
	class FAnimNextEditorModule;
	class SVariablesOutlinerEntryLabel;
	class SVariablesOutlinerAssetLabel;
	class SVariablesOutlinerCategoryLabel;
	struct FVariablesOutlinerCategoryItem;
	struct FVariablesOutlinerEntryItem;
	class FVariablesOutlinerMode;
}

namespace UE::UAF::Tests
{
	class FEditor_Variables;
	class FVariables_UOLBindings;
	class FEditor_AnimGraph_Variables;
	class FDataInterfaceCompile;
	class FVariableReferenceTest;
}

namespace UE::UAF::UncookedOnly
{
	struct FPublicVariablesImpl;
	struct FUtils;
	class FModule;
}

UCLASS(MinimalAPI, Category = "Variables", DisplayName = "Variable")
class UAnimNextVariableEntry : public UUAFRigVMAssetEntry, public IUAFRigVMVariableInterface, public IUAFRigVMExportInterface
{
	GENERATED_BODY()

	friend class UUAFRigVMAssetEditorData;
	friend class UUAFSystem_EditorData;
	friend class UUAFSharedVariables_EditorData;
	friend class UUAFSharedVariablesEntry;
	friend class UAnimNextStateTreeTreeEditorData;
	friend class UE::UAF::Tests::FEditor_Variables;
	friend class UE::UAF::Tests::FEditor_AnimGraph_Variables;
	friend class UE::UAF::Tests::FVariables_UOLBindings;
	friend class FAnimationAnimNextRuntimeTest_Variables;
	friend class UE::UAF::Editor::FVariableCustomization;
	friend class UE::UAF::Editor::FVariableBindingPropertyCustomization;
	friend class UE::UAF::Editor::FUniversalObjectLocatorBindingCustomization;
	friend class UE::UAF::Editor::FVariablesOutlinerMode;
	friend class UE::UAF::Editor::SAddVariablesDialog;
	friend class UE::UAF::Editor::SVariablesOutlinerValue;
	friend class UE::UAF::Editor::FVariablesOutlinerHierarchy;
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend class UE::UAF::Editor::FVariableProxyCustomization;
	friend struct UE::UAF::UncookedOnly::FPublicVariablesImpl;
	friend class UE::UAF::Editor::FAnimNextEditorModule;
	friend struct FAnimNextSchemaAction_PromoteToVariable;
	friend class UE::UAF::Editor::SVariablesOutlinerEntryLabel;
	friend class UE::UAF::Editor::SVariablesOutlinerAssetLabel;
	friend class UE::UAF::Editor::SVariablesOutlinerCategoryLabel;
	friend class UAnimNextAssetFindReplaceVariables;
	friend struct UE::UAF::Editor::FVariablesOutlinerCategoryItem;
	friend struct UE::UAF::Editor::FVariablesOutlinerEntryItem;
	friend class UE::UAF::Editor::FVariablesOutlinerMode;
	friend class UE::UAF::UncookedOnly::FModule;
	friend class UE::UAF::Tests::FVariableReferenceTest;
	friend struct FAssetVariableTests;


	UAnimNextVariableEntry();

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	// IAnimNextRigVMExportInterface interface
	virtual FAnimNextParamType GetExportType() const override;
	virtual FName GetExportName() const override;
	virtual EAnimNextExportAccessSpecifier GetExportAccessSpecifier() const override;
	virtual void SetExportAccessSpecifier(EAnimNextExportAccessSpecifier InAccessSpecifier, bool bSetupUndoRedo = true) override;

	// UUAFRigVMAssetEntry interface
	virtual FName GetEntryName() const override;
	virtual void SetEntryName(FName InName, bool bSetupUndoRedo = true) override;
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayNameTooltip() const override;

	// IAnimNextRigVMVariableInterface interface
	virtual FAnimNextParamType GetType() const override;
	virtual bool SetType(const FAnimNextParamType& InType, bool bSetupUndoRedo = true) override;
	virtual FName GetVariableName() const override;
	virtual void SetVariableName(FName InName, bool bSetupUndoRedo = true) override;
	virtual bool SetDefaultValue(TConstArrayView<uint8> InValue, bool bSetupUndoRedo = true) override;
	virtual bool SetDefaultValueFromString(const FString& InDefaultValue, bool bSetupUndoRedo = true) override;
	virtual const FInstancedPropertyBag& GetPropertyBag() const override;
	virtual bool GetDefaultValue(const FProperty*& OutProperty, TConstArrayView<uint8>& OutValue) const override;
	virtual bool GetDefaultValueString(FString& OutValueString) const override;
	virtual void SetBindingType(UScriptStruct* InBindingTypeStruct, bool bSetupUndoRedo = true) override;
	virtual void SetBinding(TInstancedStruct<FAnimNextVariableBindingData>&& InBinding, bool bSetupUndoRedo = true) override;
	virtual TConstStructView<FAnimNextVariableBindingData> GetBinding() const override;
	virtual FStringView GetVariableCategory() const override;
	virtual void SetVariableCategory(FStringView InCategoryName, bool bSetupUndoRedo = true) override;
	virtual const FGuid& GetGuid() const override { return Guid; }

	// Set the default value
	template<typename ValueType>
	bool SetDefaultValue(const ValueType& InValue, bool bSetupUndoRedo = true)
	{
		const FPropertyBagPropertyDesc* Desc = DefaultValue.FindPropertyDescByName(IUAFRigVMVariableInterface::ValueName);
		if(Desc == nullptr)
		{
			UE_LOGFMT(LogAnimation, Error, "UAnimNextVariableEntry::SetDefaultValue: Could not find default value in property bag");
			return false;
		}

		FAnimNextParamType SuppliedType = FAnimNextParamType::GetType<ValueType>();
		FAnimNextParamType InternalType = FAnimNextParamType(Desc->ValueType, Desc->ContainerTypes.GetFirstContainerType(), Desc->ValueTypeObject);
		if(SuppliedType != InternalType)
		{
			UE_LOGFMT(LogAnimation, Error, "UAnimNextVariableEntry::SetDefaultValue: Mismatched types");
			return false;
		}

		return SetDefaultValue(TConstArrayView<uint8>((uint8*)&InValue, sizeof(ValueType)), bSetupUndoRedo);
	}

	/** Access specifier - whether the variable is visible external to this asset */
	UPROPERTY(EditAnywhere, Category = "Variable")
	EAnimNextExportAccessSpecifier Access = EAnimNextExportAccessSpecifier::Private;

	/** Parameter name we reference */
	UPROPERTY(VisibleAnywhere, Category = "Variable")
	FName ParameterName;

	/** The variable's type */
	UPROPERTY(EditAnywhere, Category = "Variable")
	FAnimNextParamType Type = FAnimNextParamType::GetType<bool>();

	/** Binding data */
	UPROPERTY(EditAnywhere, Category = "Variable")
	FAnimNextVariableBinding Binding;

	/** Comment to display in editor */
	UPROPERTY(EditAnywhere, Category = "Variable", meta=(MultiLine))
	FString Comment;

	/** Category this variable should be assigned to */
	UPROPERTY(EditAnywhere, Category = "Variable")
	FString Category;

	/** Property bag holding the default value of the variable */
	UPROPERTY()
	FInstancedPropertyBag DefaultValue;
	
	/** Stable (unique) Guid used to reference variable without requiring ParameterName */
	UPROPERTY()
	FGuid Guid;
};
