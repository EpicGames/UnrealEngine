// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAnimNextRigVMExportInterface.generated.h"

struct FAnimNextParamType;

UENUM()
enum class EAnimNextExportAccessSpecifier : int32
{
	// Export can only be used/referenced in the graph it is declared in
	Private,

	// Export can be used/referenced in other graphs/contexts (e.g. BP, Verse)
	Public,
};

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UUAFRigVMExportInterface : public UInterface
{
	GENERATED_BODY()
};

class IUAFRigVMExportInterface
{
	GENERATED_BODY()

public:
	// Get the export type
	virtual FAnimNextParamType GetExportType() const = 0;

	// Get the export name (e.g. asset path + entry name)
	virtual FName GetExportName() const = 0;

	// Get the export access specifier
	virtual EAnimNextExportAccessSpecifier GetExportAccessSpecifier() const = 0;

	// Set the export access specifier
	virtual void SetExportAccessSpecifier(EAnimNextExportAccessSpecifier InAccessSpecifier, bool bSetupUndoRedo = true) = 0;
};
