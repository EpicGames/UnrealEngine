// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAnimNextRigVMGraphInterface.generated.h"

class URigVMGraph;
class URigVMEdGraph;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UUAFRigVMGraphInterface : public UInterface
{
	GENERATED_BODY()
};

// Interface for entries that contain a RigVM graph
class IUAFRigVMGraphInterface
{
	GENERATED_BODY()

public:
	// Get the AnimNext graph name
	virtual const FName& GetGraphName() const = 0;

	// Get the RigVM graph
	virtual URigVMGraph* GetRigVMGraph() const = 0;

	// Get the Editor graph
	virtual URigVMEdGraph* GetEdGraph() const = 0;

private:
	friend class UUAFRigVMAssetEditorData;

	// Set the RigVM graph
	virtual void SetRigVMGraph(URigVMGraph* InGraph) = 0;

	// Get the Editor graph
	virtual void SetEdGraph(URigVMEdGraph* InGraph) = 0;
};
