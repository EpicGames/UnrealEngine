// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextRigVMAssetSchema.h"
#include "AnimNextEventGraphSchema.generated.h"

UCLASS(MinimalAPI)
class UAnimNextEventGraphSchema : public UUAFRigVMAssetSchema
{
	GENERATED_BODY()

	// URigVMSchema interface
	virtual bool SupportsUnitFunction_NoLock(URigVMController* InController, const FRigVMFunction* InUnitFunction, FRigVMRegistryHandle& InRegistry) const override;
};

