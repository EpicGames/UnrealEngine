// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMModel/RigVMGraph.h"
#include "AnimNextEdGraph.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"
#include "AnimNextExecuteContext.h"
#include "Module/RigVMTrait_ModuleEventDependency.h"
#include "AnimNextModule_EditorData.generated.h"

class UUAFSystem;
class UAnimNextModule_FunctionGraph;
enum class ERigVMGraphNotifType : uint8;
class FAnimationAnimNextRuntimeTest_GraphAddTrait;
class FAnimationAnimNextRuntimeTest_GraphExecute;
class FAnimationAnimNextRuntimeTest_GraphExecuteLatent;

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

namespace UE::UAF::Editor
{
	class FModuleEditor;
	class SAnimNextGraphView;
	struct FUtils;
}

/** Editor data for UAF systems */
UCLASS(MinimalAPI)
class UUAFSystem_EditorData : public UUAFSharedVariables_EditorData
{
	GENERATED_BODY()

	friend class UUAFSystemFactory;
	friend class UAnimNextEdGraph;
	friend struct UE::UAF::UncookedOnly::FUtils;
	friend struct UE::UAF::Editor::FUtils;
	friend class UE::UAF::Editor::FModuleEditor;
	friend class UE::UAF::Editor::SAnimNextGraphView;
	friend struct FAnimNextGraphSchemaAction_RigUnit;
	friend struct FAnimNextGraphSchemaAction_DispatchFactory;
	friend class FAnimationAnimNextEditorTest_GraphAddTrait;
	friend class FAnimationAnimNextEditorTest_GraphTraitOperations;
	friend class FAnimationAnimNextRuntimeTest_GraphExecute;
	friend class FAnimationAnimNextRuntimeTest_GraphExecuteLatent;
	friend class FAnimationAnimNextEditorTest_GraphManifest;

public:
	static inline FName DefaultEventGraphName = TEXT("EventGraph");

private:
	// UUAFRigVMAssetEditorData interface
	virtual UScriptStruct* GetExecuteContextStruct() const override { return FAnimNextExecuteContext::StaticStruct(); }
	virtual TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> GetEntryClasses() const override;
	virtual void OnPreCompileAsset(FRigVMCompileSettings& InSettings) override;
	virtual void OnPreCompileGetProgrammaticGraphs(const FRigVMCompileSettings& InSettings, FAnimNextGetGraphCompileContext& OutCompileContext) override;
	virtual void OnPreCompileProcessGraphs(const FRigVMCompileSettings& InSettings, FAnimNextProcessGraphCompileContext& OutCompileContext) override;
	virtual void CustomizeNewAssetEntry(UUAFRigVMAssetEntry* InNewEntry) const override;

private:
	UPROPERTY()
	TArray<TObjectPtr<UAnimNextEdGraph>> Graphs_DEPRECATED;

	// All dependencies that should be set up when the module initializes
	UPROPERTY(EditAnywhere, Category = "Dependencies", NoClear, meta=(ExcludeBaseStruct))
	TArray<TInstancedStruct<FRigVMTrait_ModuleEventDependency>> Dependencies;
};
