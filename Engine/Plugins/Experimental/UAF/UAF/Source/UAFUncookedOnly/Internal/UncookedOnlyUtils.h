// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "IAnimNextRigVMGraphInterface.h"
#include "AssetRegistry/AssetData.h"
#include "Param/ParamType.h"
#include "RigVMCore/RigVMTemplate.h"
#include "StructUtils/InstancedStruct.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "UObject/SoftObjectPath.h"
#include "RigVMModel/RigVMGraph.h"
#include "AnimNextExports.h"
#include "AssetRegistry/IAssetRegistry.h"

#define UE_API UAFUNCOOKEDONLY_API

class UUAFSharedVariables;
struct FAnimNextSoftVariableReference;
struct FAnimNextVariableReference;
struct FAnimNextGetVariableCompileContext;
struct FAnimNextVariableBindingData;
struct FEdGraphPinType;
struct FWorkspaceOutlinerItemExports;
struct FWorkspaceOutlinerItemExport;
struct FRigVMGraphFunctionData;
struct FRigVMCompileSettings;
struct FRigVMGraphFunctionHeaderArray;
struct FGraphContextMenuBuilder;
struct FRigVMFunction;
class UEdGraphPin;
class UUAFSystem;
class UUAFSystem_EditorData;
class UAnimNextEdGraph;
class URigVMController;
class URigVMGraph;
class UAnimNextEdGraph;
class UUAFRigVMAsset;
class UUAFRigVMAssetEditorData;
class UUAFRigVMAssetEntry;
class URigVMUnitNode;
class UAnimNextControllerBase;
class URigVMCollapseNode;

namespace UE::UAF
{
	static const FLazyName AnimNextPublicGraphFunctionsExportsRegistryTag = TEXT("AnimNextPublicGraphFunctions");
	static const FLazyName ControlRigAssetPublicGraphFunctionsExportsRegistryTag = TEXT("PublicGraphFunctions");
}

namespace UE::UAF::UncookedOnly
{

extern UAFUNCOOKEDONLY_API TAutoConsoleVariable<bool> CVarDumpProgrammaticGraphs;

struct FUtils
{
	static UE_API void CompileVariables(const FRigVMCompileSettings& InSettings, UUAFRigVMAsset* InAsset, FAnimNextGetVariableCompileContext& OutCompileContext);

	static UE_API FInstancedPropertyBag MakePropertyBagForEditorData(const UUAFRigVMAssetEditorData* InEditorData, const FAnimNextGetVariableCompileContext& InCompileContext);
	
	static UE_API void CompileVariableBindings(const FRigVMCompileSettings& InSettings, UUAFRigVMAsset* InAsset, TArray<URigVMGraph*>& OutGraphs);

	static UE_API void RecreateVM(UUAFRigVMAsset* InAsset);

	// Get the corresponding asset from an asset's editor data (casts the outer appropriately)
	static UE_API UUAFRigVMAsset* GetAsset(UUAFRigVMAssetEditorData* InEditorData);

	template<typename AssetType, typename EditorDataType>
	static AssetType* GetAsset(EditorDataType* InEditorData)
	{
		using NonConstEditorDataType = std::remove_const_t<EditorDataType>;
		return CastChecked<AssetType>(GetAsset(const_cast<NonConstEditorDataType*>(InEditorData)));
	}

	// Get the corresponding editor data from an asset (casts the editor data appropriately)
	static UE_API UUAFRigVMAssetEditorData* GetEditorData(UUAFRigVMAsset* InAsset);
	static UE_API const UUAFRigVMAssetEditorData* GetEditorData(const UUAFRigVMAsset* InAsset);

	template<typename EditorDataType, typename AssetType>
	static EditorDataType* GetEditorData(AssetType* InAsset)
	{
		using NonConstAssetType = std::remove_const_t<AssetType>;
		return CastChecked<EditorDataType>(GetEditorData(const_cast<NonConstAssetType*>(InAsset)));
	}

	/**
	 * Get an AnimNext parameter type from an FEdGraphPinType.
	 * Note that the returned handle may not be valid, so should be checked using IsValid() before use.
	 **/
	static UE_API FAnimNextParamType GetParamTypeFromPinType(const FEdGraphPinType& InPinType);

	/**
	 * Get an FEdGraphPinType from an AnimNext parameter type/handle.
	 * Note that the returned pin type may not be valid.
	 **/
	static UE_API FEdGraphPinType GetPinTypeFromParamType(const FAnimNextParamType& InParamType);

	/**
	 * Get an FRigVMTemplateArgumentType from an AnimNext parameter type/handle.
	 * Note that the returned pin type may not be valid.
	 **/
	static UE_API FRigVMTemplateArgumentType GetRigVMArgTypeFromParamType(const FAnimNextParamType& InParamType);

	/** Set up a simple event graph */
	static UE_API void SetupEventGraph(URigVMController* InController, UScriptStruct* InEventStruct, FName InEventName = NAME_None, bool bPrintPythonCommand = false);

	// Gets the variables that are exported to the asset registry for an asset
	static UE_API bool GetExportedVariablesForAsset(const FAssetData& InAsset, FAnimNextAssetRegistryExports& OutExports);

	// Gets all the variables that are exported to the asset registry
	static UE_API bool GetExportedVariablesFromAssetRegistry(TMap<FAssetData, FAnimNextAssetRegistryExports>& OutExports);

	template<typename ExportDataType>
	static bool GetExportsOfTypeForAsset(const FAssetData& InAsset, FAnimNextAssetRegistryExports& OutExports)
	{
		const int32 StartNum = OutExports.Exports.Num();
		FAnimNextAssetRegistryExports AssetExports;
		const FString TagValue = InAsset.GetTagValueRef<FString>(UE::UAF::ExportsAnimNextAssetRegistryTag);
		if (FAnimNextAssetRegistryExports::StaticStruct()->ImportText(*TagValue, &AssetExports, nullptr, PPF_None, nullptr, FAnimNextAssetRegistryExports::StaticStruct()->GetName()) != nullptr)
		{
			Algo::TransformIf(AssetExports.Exports, OutExports.Exports,
				[](const FAnimNextExport& Export)
				{
					return Export.Data.GetScriptStruct() == ExportDataType::StaticStruct();
				},
				[](const FAnimNextExport& Export)
				{
					return Export;
				});
		}

		return OutExports.Exports.Num() > StartNum;
	}

	template<typename ExportDataType>
	static bool GetExportsOfTypeFromAssetRegistry(TMap<FAssetData, FAnimNextAssetRegistryExports>& OutExports)
	{
		TArray<FAssetData> AssetData;
		IAssetRegistry::GetChecked().GetAssetsByTags({UE::UAF::ExportsAnimNextAssetRegistryTag}, AssetData);

		for (const FAssetData& Asset : AssetData)
		{
			const FString TagValue = Asset.GetTagValueRef<FString>(UE::UAF::ExportsAnimNextAssetRegistryTag);
			FAnimNextAssetRegistryExports AssetExports;
			if (FAnimNextAssetRegistryExports::StaticStruct()->ImportText(*TagValue, &AssetExports, nullptr, PPF_None, nullptr, FAnimNextAssetRegistryExports::StaticStruct()->GetName()) != nullptr)
			{
				FAnimNextAssetRegistryExports& Export = OutExports.Add(Asset);
				Algo::TransformIf(AssetExports.Exports, Export.Exports,
					[](const FAnimNextExport& Export)
					{
						return Export.Data.GetScriptStruct() == ExportDataType::StaticStruct();
					},
					[](const FAnimNextExport& Export)
					{
						return Export;
					});
			}
		}

		return OutExports.Num() > 0;
	}

	static void GetAssetVariableExports(const UUAFRigVMAssetEditorData* EditorData, FAnimNextAssetRegistryExports& OutExports, FAssetRegistryTagsContext Context);

	// Gets all the functions that are exported to the asset registry for the specified Tag
	static UE_API bool GetExportedFunctionsFromAssetRegistry(FName Tag, TMap<FAssetData, FRigVMGraphFunctionHeaderArray>& OutExports);

	// Gets the exported public functions that are used by a RigVM asset
	static void GetAssetFunctions(const UUAFRigVMAssetEditorData* InEditorData, FRigVMGraphFunctionHeaderArray& OutExports);

	// Gets the non-exported private functions that are used by a RigVM asset
	static UE_API void GetAssetPrivateFunctions(const UUAFRigVMAssetEditorData* InEditorData, FRigVMGraphFunctionHeaderArray& OutExports);

	// Gets the asset-registry information needed for representing the contained data into the Workspace Outliner
	// Note: We pass parent as index to avoid the ref from being invalidated due to realloc as the export array grows recursively
	static UE_API void GetAssetWorkspaceExports(const UUAFRigVMAssetEditorData* EditorData, FWorkspaceOutlinerItemExports& OutExports, FAssetRegistryTagsContext Context);

	// Attempts to determine the type from a variable name & asset/struct path
	// If the name cannot be found, the returned type will be invalid
	// Note that this is expensive and can query the asset registry
	static UE_API FAnimNextParamType FindVariableType(const FAnimNextVariableReference& InVariableReference);
	static UE_API FAnimNextParamType FindVariableType(const FAnimNextSoftVariableReference& InSoftVariableReference);

	// Returns an user friendly name for the Function Library
	static UE_API const FText& GetFunctionLibraryDisplayName();

#if WITH_EDITOR
	static UE_API void OpenProgrammaticGraphs(UUAFRigVMAssetEditorData* InEditorData, const TArray<URigVMGraph*>& ProgrammaticGraphs);
#endif // WITH_EDITOR

	// Make a variable name that we use as a wrapper for a function param or return
	static UE_API FString MakeFunctionWrapperVariableName(FName InFunctionName, FName InVariableName);

	// Make an event name that we use as a wrapper to call RigVM functions 
	static UE_API FString MakeFunctionWrapperEventName(FName InFunctionName);

	// Returns all unique variable names found within provided editor data, optionally recursing into SharedVariable entries
	static UE_API void GetVariableNames(UUAFRigVMAssetEditorData* InEditorData, TArray<FName>& OutVariableNames, bool bRecursive = true);

	// Return valid variable FName within the context of the provided EditorData (recurses into SharedVariable entries as well) 
	static UE_API FName GetValidVariableName(UUAFRigVMAssetEditorData* InEditorData, FName InBaseName);

	// Return valid variable FName within the context of the provided set of existing variable names
	static UE_API FName GetValidVariableName(FName InBaseName, const TArrayView<FName> ExistingNames);

	// Deletes the provided variable entry from its editor data, removing any references across the project its assets 
	static UE_API void DeleteVariable(UAnimNextVariableEntry* VariableEntry, bool bSetupUndoRedo = true, bool bPrintPythonCommands = true);

	// Deletes the provided variable entry from its editor data, renaming any references across the project its assets to reference it by the NewName
	static UE_API void RenameVariable(UAnimNextVariableEntry* VariableEntry, const FName NewName, bool bSetupUndoRedo = true, bool bPrintPythonCommands = true);

	// Moves the provided variable entry from its editor data to the NewOuter, replacing any references across the project its assets to reference it by its new outer asset
	static UE_API UAnimNextVariableEntry* MoveVariableToAsset(UAnimNextVariableEntry* VariableEntry, UUAFRigVMAsset* NewOuter, bool bSetupUndoRedo = true, bool bPrintPythonCommands = true);

	// Changes the provided variable entry its data type, replacing any references across the project to reference using its new data type
	static UE_API void SetVariableType(UAnimNextVariableEntry* VariableEntry, const FAnimNextParamType NewType, bool bSetupUndoRedo = true, bool bPrintPythonCommands = true);

	// Replaces the provided variable reference with ReferenceToReplaceWith within the context of InEditorData
	static UE_API void ReplaceVariableReferences(UUAFRigVMAssetEditorData* InEditorData, const FAnimNextSoftVariableReference& ReferenceToFind, const FAnimNextSoftVariableReference& ReferenceToReplaceWith, bool bForceReplace = false, bool bSetupUndoRedo = true, bool bPrintPythonCommands = true);
	
	// Replaces the provided variable reference with ReferenceToReplaceWith within the context of the project
	static UE_API void ReplaceVariableReferencesAcrossProject(const FAnimNextSoftVariableReference& ReferenceToFind, const FAnimNextSoftVariableReference& ReferenceToReplaceWith, bool bForceLoadAssets = false, bool bForceReplace = false, bool bSetupUndoRedo = true, bool bPrintPythonCommands = true);

	// Generates a deterministic GUID provided a UStruct FProperty entry
	static UE_API FGuid GenerateScriptStructPropertyGUID(const FProperty* Property);

	// Returns whether or not it is valid to add InSharedVariables to InEditorData, and if not OutErrorMessage will contain the reasoning
	static UE_API bool CanAddSharedVariablesReference(UUAFRigVMAssetEditorData* InEditorData, const UUAFSharedVariables* InSharedVariables, FString* OutErrorMessage = nullptr);
	
	// Removes a SharedVariable reference from the provided EditorData, and ensures that any references to variables within the SharedVariables are replaced
	static UE_API bool RemoveSharedVariablesReference(UUAFRigVMAssetEditorData* EditorData, const FSoftObjectPath& SharedVariablesPath);
	
	// Generate a (function) workspace export for the provided URigVMLibraryNode
	static UE_API FWorkspaceOutlinerItemExport MakeFunctionExport(const URigVMLibraryNode* FunctionNode, const FWorkspaceOutlinerItemExport& ParentExport);
	
	// Generate a (collapse graph) workspace export for the provided URigVMCollapseNode
	static UE_API FWorkspaceOutlinerItemExport MakeCollapsedGraphExport(const URigVMCollapseNode* CollapseNode, const FWorkspaceOutlinerItemExport& ParentExport);

	// Generate a (asset reference) workspace export for the provided asset path
	static UE_API FWorkspaceOutlinerItemExport MakeAssetReferenceExport(const FSoftObjectPath& ReferencedAssetPath, const FWorkspaceOutlinerItemExport& ParentExport, bool bShouldExpand = true);

	static UE_API void GetExternalVariables(const UUAFRigVMAssetEditorData* EditorData, TArray<FRigVMExternalVariable>& OutVariables);

	/** Parse a FAnimNextVariableReference from a pin's default value. Returns the parsed reference (may be None/invalid). */
	static UE_API FAnimNextVariableReference ParseVariableReferenceFromPin(const UEdGraphPin* InPin);

	/** Preload assets referenced by a variable reference pin. Call from PreloadRequiredAssets. */
	static UE_API void PreloadVariableReferenceAssets(const UEdGraphPin* InVariablePin);

private:
	static void GetInternalVariables(const UUAFRigVMAssetEditorData* EditorData, TArray<FRigVMExternalVariable>& OutVariables);

	static UE_API void CompileVariableBindingsInternal(const FRigVMCompileSettings& InSettings, UUAFRigVMAsset* InAsset, TArray<URigVMGraph*>& OutGraphs, bool bInThreadSafe);
	
	static void GetSubGraphWorkspaceExportsRecursive(const UUAFRigVMAssetEditorData* EditorData, FWorkspaceOutlinerItemExports& OutExports, const FWorkspaceOutlinerItemExport& RootExport, int32 ParentExportIndex, URigVMEdGraph* RigVMEdGraph, FAssetRegistryTagsContext Context);

	// Gets the exported variables that are used by a RigVM asset	
	static void GetAssetVariableExports(const UUAFRigVMAssetEditorData* EditorData, TSet<FAnimNextExport>& OutExports, FAssetRegistryTagsContext Context, EAnimNextExportedVariableFlags InFlags = EAnimNextExportedVariableFlags::NoFlags);

	static void GetStructVariableExports(const UScriptStruct* Struct, TSet<FAnimNextExport>& OutExports);
	static void GetSubGraphVariableExportsRecursive(const URigVMEdGraph* RigVMEdGraph, TSet<FAnimNextExport>& OutExports);
	static void ProcessPinAssetReferences(const URigVMPin* InPin, FWorkspaceOutlinerItemExports& OutExports, const FWorkspaceOutlinerItemExport& RootExport, int32 ParentExportIndex);
	static void RetrieveReferencedSharedVariableSources(const UUAFRigVMAsset* InAsset, TSet<const UUAFRigVMAsset*>& ReferencedVariableAssets, TSet<const UScriptStruct*>& ReferencedVariableStructs, TSet<TScriptInterface<const IRigVMRuntimeAssetInterface>>& ReferencedVariableRigVMAssets);
};

}

#undef UE_API
