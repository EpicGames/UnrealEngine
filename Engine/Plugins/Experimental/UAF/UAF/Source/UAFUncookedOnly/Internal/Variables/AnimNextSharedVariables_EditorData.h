// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextExecuteContext.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "AnimNextSharedVariables_EditorData.generated.h"

#define UE_API UAFUNCOOKEDONLY_API

class UUAFSharedVariablesFactory;

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

/** Editor data for UAF shared variables */
UCLASS(MinimalAPI)
class UUAFSharedVariables_EditorData : public UUAFRigVMAssetEditorData
{
	GENERATED_BODY()

protected:
	// UUAFRigVMAssetEditorData interface
	virtual UScriptStruct* GetExecuteContextStruct() const override { return FAnimNextExecuteContext::StaticStruct(); }
	UE_API virtual TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> GetEntryClasses() const override;
	UE_API virtual void CustomizeNewAssetEntry(UUAFRigVMAssetEntry* InNewEntry) const override;

	friend class UUAFSharedVariablesFactory;
	friend UE::UAF::UncookedOnly::FUtils;
};

#undef UE_API
