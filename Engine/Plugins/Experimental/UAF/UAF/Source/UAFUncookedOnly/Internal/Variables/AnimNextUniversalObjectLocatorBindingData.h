// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorStringParams.h"
#include "Variables/AnimNextVariableBindingData.h"
#include "AnimNextUniversalObjectLocatorBindingData.generated.h"

// Type of binding
UENUM()
enum class FAnimNextUniversalObjectLocatorBindingType : uint8
{
	// Binding resolves via the result of the UOL (only valid for object type bindings)
	UOL,
	// Binding resolves via resolving a property on the UOL's resolved object
	Property,
	// Binding resolves via calling a function on the UOL's resolved object
	Function,
	// Binding resolves via calling a hoisted function on a BP function library with the UOL's resolved object
	HoistedFunction,
};

// Allows binding of module variables to gameplay data via Universal Object Locators
USTRUCT(DisplayName = "Universal Object Locator")
struct FAnimNextUniversalObjectLocatorBindingData : public FAnimNextVariableBindingData
{
	GENERATED_BODY()

	// Property to use (if a property)
	UPROPERTY(EditAnywhere, Category = "Binding")
	TFieldPath<FProperty> Property;

	// Function to use (if a function)
	UPROPERTY(EditAnywhere, Category = "Binding")
	TSoftObjectPtr<UFunction> Function;

	// Object locator
	UPROPERTY(EditAnywhere, Category = "Binding", meta = (LocatorContext="UAFContext"))
	FUniversalObjectLocator Locator;

	UPROPERTY(EditAnywhere, Category = "Binding")
	FAnimNextUniversalObjectLocatorBindingType Type = FAnimNextUniversalObjectLocatorBindingType::Property;

	// FAnimNextVariableBindingData interface
	virtual bool IsValid() const override
	{
		return !Locator.IsEmpty();
	}

#if WITH_EDITORONLY_DATA
	// Check if this binding data is safe to execute on a worker thread
	virtual bool IsThreadSafe() const override;
#endif
};
