// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_CallFunction.h"

#include "K2Node_UAFComponentGetVariable.generated.h"

UCLASS(MinimalAPI)
class UK2Node_UAFComponentGetVariable : public UK2Node_CallFunction
{
	GENERATED_BODY()

public:
	UAFUNCOOKEDONLY_API UK2Node_UAFComponentGetVariable();

private:
	// //~ Begin UEdGraphNode Interface.
	UAFUNCOOKEDONLY_API virtual void ReconstructNode() override;
	UAFUNCOOKEDONLY_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	// //~ End UEdGraphNode Interface.

	//~ Begin K2Node Interface
	UAFUNCOOKEDONLY_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const override;
	UAFUNCOOKEDONLY_API virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	UAFUNCOOKEDONLY_API virtual void PreloadRequiredAssets() override;
	//~ End K2Node Interface

	void CachePinType();
	void PropagateCachedTypeToPin();

	// Cached type of the pin used for regenerating after wildcard is reconstructed
	UPROPERTY()
	FEdGraphPinType CachedPinType;
};