// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_CallFunction.h"
#include "K2Node_UAFComponentSetInputBinding.generated.h"

/** A custom node that supports setting dynamic input bindings on UAF components, restricted to FValueBundle variables */
UCLASS(MinimalAPI)
class UK2Node_UAFComponentSetInputBinding : public UK2Node_CallFunction
{
	GENERATED_BODY()

public:
	UAFUNCOOKEDONLY_API UK2Node_UAFComponentSetInputBinding();

private:
	//~ Begin UEdGraphNode Interface
	UAFUNCOOKEDONLY_API virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	UAFUNCOOKEDONLY_API virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	UAFUNCOOKEDONLY_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const override;
	UAFUNCOOKEDONLY_API virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	UAFUNCOOKEDONLY_API virtual void PreloadRequiredAssets() override;
	//~ End K2Node Interface
	
	static const FName VariablePinName;
	static const FName ComponentPinName;
	static const TArray<const UClass*> AllowedComponentClasses;
};
