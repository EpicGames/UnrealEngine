// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAssetWorkspaceAssetUserData.h"

#include "AnimNextExports.h"
#include "UncookedOnlyUtils.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModule_EditorData.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextAssetWorkspaceAssetUserData)

void UAnimNextAssetWorkspaceAssetUserData::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	// Updated cached export data outside of saving call-path, and simply return cached data when actually saving 
	if (!Context.IsSaving())
	{
		CachedExports.Exports.Reset();
		{
			UUAFRigVMAsset* Asset = CastChecked<UUAFRigVMAsset>(GetOuter());
			if (const UUAFRigVMAssetEditorData* GraphEditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset))
			{
				FWorkspaceOutlinerItemExport& RootAssetExport = CachedExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(Asset->GetFName(), Asset));

				if(Asset->IsA<UUAFSystem>())
				{
					RootAssetExport.GetData().InitializeAsScriptStruct(FAnimNextModuleOutlinerData::StaticStruct());
				}
				else if(Asset->IsA<UUAFSharedVariables>())
				{
					RootAssetExport.GetData().InitializeAsScriptStruct(FAnimNextSharedVariablesOutlinerData::StaticStruct());
				}
				else
				{
					RootAssetExport.GetData().InitializeAsScriptStruct(FAnimNextRigVMAssetOutlinerData::StaticStruct());
				}
				FAnimNextRigVMAssetOutlinerData& Data = RootAssetExport.GetData().GetMutable<FAnimNextRigVMAssetOutlinerData>();
				Data.SoftAssetPtr = Asset;

				UE::UAF::UncookedOnly::FUtils::GetAssetWorkspaceExports(GraphEditorData, CachedExports, Context);
			}
		}
	}
	
	FString TagValue;
	FWorkspaceOutlinerItemExports::StaticStruct()->ExportText(TagValue, &CachedExports, nullptr, nullptr, PPF_None, nullptr);
	Context.AddTag(FAssetRegistryTag(UE::Workspace::ExportsWorkspaceItemsRegistryTag, TagValue, FAssetRegistryTag::TT_Hidden));
}

