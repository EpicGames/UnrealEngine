// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextControllerBase.h"
#include "AnimNextSharedVariableNode.h"
#include "RigVMPythonUtils.h"
#include "UncookedOnlyUtils.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextControllerBase)

UUAFSharedVariableNode* UAnimNextControllerBase::AddAssetSharedVariableNode(const UUAFSharedVariables* InAsset, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (InAsset == nullptr)
	{
		ReportError(TEXT("Invalid asset supplied to AddAssetSharedVariableNode."));
		return nullptr;
	}

	return AddSharedVariableNode(InAsset->GetPathName(), InVariableName, InCPPType, InCPPTypeObject ? InCPPTypeObject->GetPathName() : TEXT(""), bIsGetter, InDefaultValue, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

UUAFSharedVariableNode* UAnimNextControllerBase::AddStructSharedVariableNode(const UScriptStruct* InStruct, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (InStruct == nullptr)
	{
		ReportError(TEXT("Invalid struct supplied to AddStructSharedVariableNode."));
		return nullptr;
	}

	return AddSharedVariableNode(InStruct->GetPathName(), InVariableName, InCPPType, InCPPTypeObject ? InCPPTypeObject->GetPathName() : TEXT(""), bIsGetter, InDefaultValue, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

UUAFSharedVariableNode* UAnimNextControllerBase::AddSharedVariableNode(const FString& InSourceObjectPath, const FName& InVariableName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	UObject* SourceObject = nullptr;
	if (!InSourceObjectPath.IsEmpty())
	{
		SourceObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InSourceObjectPath);
		if (SourceObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InSourceObjectPath);
			return nullptr;
		}
	}

	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsEmpty())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath);
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath);
			return nullptr;
		}
	}

	UUAFSharedVariableNode* VariableNode = CastChecked<UUAFSharedVariableNode>(AddVariableNode(InVariableName, UUAFSharedVariableNode::StaticClass(), InCPPType, CPPTypeObject, bIsGetter, InDefaultValue, InPosition, InNodeName, bSetupUndoRedo, false), ECastCheckedType::Type::NullAllowed);
	if (VariableNode == nullptr)
	{
		return nullptr;
	}

	if (const UUAFSharedVariables* Asset = Cast<UUAFSharedVariables>(SourceObject))
	{
		VariableNode->Type = EAnimNextSharedVariablesType::Asset;
		VariableNode->Asset = Asset;
		
		VariableNode->UpdateCachedVariableGUID();
	}
	else if (const UScriptStruct* Struct = Cast<UScriptStruct>(SourceObject))
	{
		VariableNode->Type = EAnimNextSharedVariablesType::Struct;
		VariableNode->Struct = Struct;
		
		VariableNode->UpdateCachedVariableGUID();
	}
	else if (const IRigVMRuntimeAssetInterface* RigVMAsset = Cast<IRigVMRuntimeAssetInterface>(SourceObject))
	{
		VariableNode->Type = EAnimNextSharedVariablesType::RigVMAsset;
		VariableNode->RigVMAsset = SourceObject;
	}
	else
	{
		return nullptr;
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_shared_variable_node('%s', '%s', '%s', '%s', '%s', '%s', %s, '%s')"),
				*GraphName,
				*InSourceObjectPath,
				*InVariableName.ToString(),
				*InCPPType,
				*InCPPTypeObjectPath,
				bIsGetter ? TEXT("True") : TEXT("False"),
				*InDefaultValue,
				*RigVMPythonUtils::Vector2DToPythonString(InPosition),
				*InNodeName
			)
		);
	}

	return VariableNode;
}

URigVMUnitNode* UAnimNextControllerBase::AddUnitNodeOfClass(const FString& InScriptStructPath, const FString& InUnitNodeClassPath, const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	UScriptStruct* ScriptStruct = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UScriptStruct>(InScriptStructPath);
	if (ScriptStruct == nullptr)
	{
		ReportErrorf(TEXT("Cannot find struct for path '%s'."), *InScriptStructPath);
		return nullptr;
	}
	
	UClass* UnitNodeClass = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UClass>(InUnitNodeClassPath);
	if (UnitNodeClass == nullptr)
	{
		ReportErrorf(TEXT("Cannot find class for path '%s'."), *InUnitNodeClassPath);
		return nullptr;
	}
	
	if (!UnitNodeClass->IsChildOf(URigVMUnitNode::StaticClass()))
	{
		ReportErrorf(TEXT("Provided UnitNodeClass (%s) is not a subclass of URigVMUnitNode."), *UnitNodeClass->GetName());
		return nullptr;
	}

	URigVMUnitNode* UnitNode = AddUnitNode(ScriptStruct, UnitNodeClass, InMethodName, InPosition, InNodeName, bSetupUndoRedo, false);
	
	if (!UnitNode)
	{
		ReportErrorf(TEXT("Failed to AddUnitNodeOfClass (%s)"), *UnitNodeClass->GetName());
		return nullptr;
	}
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), FString::Printf(TEXT("asset.get_controller_by_name('%s').add_unit_node_of_class('%s', '%s', '%s', %s, '%s')"),
					*GraphName,
					*InScriptStructPath,
					*InUnitNodeClassPath,
					*InMethodName.ToString(),
					*RigVMPythonUtils::Vector2DToPythonString(UnitNode->GetPosition()),
					*UnitNode->GetName()));
	}
	
	return UnitNode;
}

void UAnimNextControllerBase::RefreshSharedVariableNode(const FName& InNodeName, const FString& InSourceObjectPath, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bSetupOrphanPins, bool bPrintPythonCommand)
{
	UObject* SourceObject = nullptr;
	if (!InSourceObjectPath.IsEmpty())
	{
		SourceObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InSourceObjectPath);
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);
	if (UUAFSharedVariableNode* VariableNode = Cast<UUAFSharedVariableNode>(Graph->FindNodeByName(InNodeName)))
	{
		if (bSetupUndoRedo)
		{
			VariableNode->Modify();
		}
		
		RefreshVariableNode(InNodeName, InVariableName, InCPPType, InCPPTypeObject, bSetupUndoRedo, bSetupOrphanPins);

		if (SourceObject)
		{
			if (const UUAFSharedVariables* Asset = Cast<UUAFSharedVariables>(SourceObject))
			{
				VariableNode->Type = EAnimNextSharedVariablesType::Asset;
				VariableNode->Asset = Asset;
			}
			else if (const UScriptStruct* Struct = Cast<UScriptStruct>(SourceObject))
			{
				VariableNode->Type = EAnimNextSharedVariablesType::Struct;
				VariableNode->Struct = Struct;
			}
		}
		else
		{
			VariableNode->Type = EAnimNextSharedVariablesType::Asset;
			VariableNode->Asset = nullptr;
		}
		
		VariableNode->UpdateCachedVariableGUID();

		Notify(ERigVMGraphNotifType::NodeTitleChanged, VariableNode);

		if (bPrintPythonCommand)
		{
			const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
				FString::Printf(TEXT("asset.get_controller_by_name('%s').refresh_shared_variable_node('%s', '%s', '%s', '%s', '%s', '%s', '%s', 'True')"),
					*GraphName,
					*InNodeName.ToString(),
					*InSourceObjectPath,
					*InVariableName.ToString(),
					*InCPPType,
					*RigVMTypeUtils::CPPTypeFromObject(InCPPTypeObject),
					bSetupUndoRedo ? TEXT("True") : TEXT("False"),
					bSetupOrphanPins ? TEXT("True") : TEXT("False")
				)
			);
		}
	}
}

UUAFSharedVariableNode* UAnimNextControllerBase::ReplaceVariableNodeWithSharedVariableNode(URigVMVariableNode* InVariableNode, FName InNewVariableName, const UObject* InAssetOrStruct, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	check(InVariableNode);
	check(InAssetOrStruct);
	check(InNewVariableName != NAME_None);

	UUAFSharedVariableNode* SharedVariableNode = AddSharedVariableNode(InAssetOrStruct->GetPathName(), InNewVariableName, InVariableNode->GetCPPType(), InVariableNode->GetCPPTypeObject() ? InVariableNode->GetCPPTypeObject()->GetPathName() : TEXT(""), InVariableNode->IsGetter(), InVariableNode->GetDefaultValue(), InVariableNode->GetPosition(), FString(), bSetupUndoRedo, bPrintPythonCommand);
	if (SharedVariableNode)
	{
		if (!InVariableNode->IsGetter())
		{
			RewireLinks(InVariableNode->GetValuePin(), SharedVariableNode->GetValuePin(), true, bSetupUndoRedo);
			RewireLinks(InVariableNode->FindExecutePin(), SharedVariableNode->FindExecutePin(), true, bSetupUndoRedo);
			RewireLinks(InVariableNode->FindExecutePin(), SharedVariableNode->FindExecutePin(), false, bSetupUndoRedo);
		}
		else
		{
			RewireLinks(InVariableNode->GetValuePin(), SharedVariableNode->GetValuePin(), false, bSetupUndoRedo);
		}
		RemoveNode(InVariableNode, bSetupUndoRedo, bPrintPythonCommand);
	}

	return SharedVariableNode;
}

URigVMVariableNode* UAnimNextControllerBase::ReplaceSharedVariableNodeWithVariableNode(UUAFSharedVariableNode* InVariableNode, FName InNewVariableName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	check(InVariableNode);
	check(InNewVariableName != NAME_None);


	URigVMVariableNode* VariableNode = AddVariableNode(InNewVariableName, InVariableNode->GetCPPType(), InVariableNode->GetCPPTypeObject(), InVariableNode->IsGetter(), InVariableNode->GetDefaultValue(), InVariableNode->GetPosition(), FString(), bSetupUndoRedo, bPrintPythonCommand);
	if (VariableNode)
	{
		if (!InVariableNode->IsGetter())
		{
			RewireLinks(InVariableNode->GetValuePin(), VariableNode->GetValuePin(), true, bSetupUndoRedo);
			RewireLinks(InVariableNode->FindExecutePin(), VariableNode->FindExecutePin(), true, bSetupUndoRedo);
			RewireLinks(InVariableNode->FindExecutePin(), VariableNode->FindExecutePin(), false, bSetupUndoRedo);
		}
		else
		{
			RewireLinks(InVariableNode->GetValuePin(), VariableNode->GetValuePin(), false, bSetupUndoRedo);
		}
		RemoveNode(InVariableNode, bSetupUndoRedo, bPrintPythonCommand);
	}

	return VariableNode;
}
