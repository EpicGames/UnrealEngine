// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextEdGraph.h"
#include "Module/AnimNextModule_EditorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextEdGraph)

void UAnimNextEdGraph::PostLoad()
{
	Super::PostLoad();

	UUAFRigVMAssetEditorData* EditorData = GetTypedOuter<UUAFRigVMAssetEditorData>();
	check(EditorData);
	Initialize(EditorData);
}

void UAnimNextEdGraph::Initialize(UUAFRigVMAssetEditorData* InEditorData)
{
	Schema = InEditorData->GetRigVMEdGraphSchemaClass();

	InEditorData->RigVMGraphModifiedEvent.RemoveAll(this);
	InEditorData->RigVMGraphModifiedEvent.AddUObject(this, &UAnimNextEdGraph::HandleModifiedEvent);
	InEditorData->RigVMCompiledEvent.RemoveAll(this);
	InEditorData->RigVMCompiledEvent.AddUObject(this, &UAnimNextEdGraph::HandleVMCompiledEvent);
}

FRigVMClient* UAnimNextEdGraph::GetRigVMClient() const
{
	if (const UUAFRigVMAssetEditorData* EditorData = GetTypedOuter<UUAFRigVMAssetEditorData>())
	{
		return const_cast<FRigVMClient*>(EditorData->GetRigVMClient());
	}
	return nullptr;
}
