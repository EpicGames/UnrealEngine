// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextEdGraphSchema.h"

#include "AnimNextRigVMAsset.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Entries/AnimNextRigVMAssetEntry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextEdGraphSchema)

#define LOCTEXT_NAMESPACE "AnimNextEdGraphSchema"

bool UAnimNextEdGraphSchema::ShouldHidePinOnNode(const URigVMPin* InModelPin) const
{
	if (InModelPin)
	{
		if (const URigVMPin* RootPin = InModelPin->GetRootPin())
		{
			if (RootPin->IsDefinedAsInputVariable() && RootPin->GetTypedOuter<UUAFRigVMAsset>())
			{
				return true;
			}
		}
	}
	return Super::ShouldHidePinOnNode(InModelPin);
}

void UAnimNextEdGraphSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	Super::GetGraphDisplayInformation(Graph, DisplayInfo);
	
	if(UUAFRigVMAssetEntry* AssetEntry = Cast<UUAFRigVMAssetEntry>(Graph.GetOuter()))
	{
		DisplayInfo.DisplayName = FText::Format(LOCTEXT("GraphTabTitleFormat", "{0}: {1}"), FText::FromName(AssetEntry->GetEntryName()), FText::FromName(AssetEntry->GetTypedOuter<UUAFRigVMAsset>()->GetFName()));
		DisplayInfo.Tooltip = FText::Format(LOCTEXT("GraphTabTooltipFormat", "{0} in:\n{1}"), FText::FromName(AssetEntry->GetEntryName()), FText::FromString(AssetEntry->GetTypedOuter<UUAFRigVMAsset>()->GetPathName()));
	}
}

TSharedPtr<IAssetReferenceFilter> UAnimNextEdGraphSchema::MakeAssetReferenceFilter(const UEdGraph* Graph)
{
	if (Graph)
	{
		if (UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(Graph->GetTypedOuter<UUAFRigVMAsset>()))
		{
			if (GEditor)
			{
				FAssetReferenceFilterContext AssetReferenceFilterContext;
				AssetReferenceFilterContext.AddReferencingAsset(Asset);
				return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
			}
		}
	}

	return {};
}


bool UAnimNextEdGraphSchema::IsStructEditable(UStruct* InStruct) const
{
	if (InStruct == FAnimNextVariableReference::StaticStruct())
	{
		return true;
	}
	return Super::IsStructEditable(InStruct);
}

#undef LOCTEXT_NAMESPACE
