// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextRigVMAssetEditorData.h"
#include "AnimNextEdGraph.h"
#include "AnimNextEdGraphSchema.h"
#include "AnimNextRigVMAsset.h"
#include "Compilation/AnimNextRigVMAssetCompileContext.h"
#include "Compilation/AnimNextGetFunctionHeaderCompileContext.h"
#include "Compilation/AnimNextGetVariableCompileContext.h"
#include "Compilation/AnimNextGetGraphCompileContext.h"
#include "Compilation/AnimNextProcessGraphCompileContext.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "AnimNextRigVMAssetSchema.h"
#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "AnimNextControllerBase.h"
#include "UAFCompilationScope.h"
#include "RigVMPythonUtils.h"
#include "ExternalPackageHelper.h"
#include "IAnimNextRigVMGraphInterface.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "ObjectTools.h"
#include "UncookedOnlyUtils.h"
#include "Animation/Skeleton.h"
#include "AnimNextRigVMFunctionData.h"
#include "Variables/StructDataCache.h"
#include "DataInterface/AnimNextNativeDataInterface.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Entries/AnimNextEventGraphEntry.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Graph/RigUnit_AnimNextBeginExecution.h"
#include "Misc/TransactionObjectEvent.h"
#include "Module/AnimNextEventGraphSchema.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "RigVMModel/RigVMNotifications.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "RigVMFunctions/Execution/RigVMFunction_UserDefinedEvent.h"
#include "Misc/UObjectToken.h"
#include "UObject/SavePackage.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Script/UAFRigVMComponent.h"
#include "String/ParseTokens.h"

#if WITH_EDITOR
#include "FileHelpers.h"
#include "PackageSourceControlHelper.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextRigVMAssetEditorData)

#define LOCTEXT_NAMESPACE "AnimNextRigVMAssetEditorData"

void UUAFRigVMAssetEditorData::BroadcastModified(EAnimNextEditorDataNotifType InType, UObject* InSubject)
{
	using namespace UE::UAF::UncookedOnly;

	if (!IsValid(this))
	{
		return;
	}

	// Modifications here can trigger compilation, so add a scope to catch compile batches
	UUAFRigVMAsset* Asset = FUtils::GetAsset<UUAFRigVMAsset>(this);
	FCompilationScope CompileScope(LOCTEXT("ModifiedAssetsJobName", "Modified Assets"), Asset);

	if(!bSuspendEditorDataNotifications)
	{
		ModifiedDelegate.Broadcast(this, InType, InSubject);
	}

	RequestAutoVMRecompilation();
}

void UUAFRigVMAssetEditorData::ReportError(const TCHAR* InMessage)
{
	FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, InMessage, TEXT(""));
}

void UUAFRigVMAssetEditorData::ReconstructAllNodes()
{
	// Avoid refreshing EdGraph nodes during cook
	if (GIsCookerLoadingPackage)
	{
		return;
	}
	
	if (GetRigVMClient()->GetDefaultModel() == nullptr)
	{
		return;
	}

	TArray<URigVMEdGraphNode*> AllNodes;
	GetAllNodesOfClass(AllNodes);

	for (URigVMEdGraphNode* Node : AllNodes)
	{
		Node->SetFlags(RF_Transient);
	}

	for(URigVMEdGraphNode* Node : AllNodes)
	{
		Node->ReconstructNode();
	}

	for (URigVMEdGraphNode* Node : AllNodes)
	{
		Node->ClearFlags(RF_Transient);
	}
}

void UUAFRigVMAssetEditorData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	// Prevent doing this during transactions, as NewObject<UUAFRigVMAssetEditorData> with RF_Transactional will mark _this_ as garbage which breaks assumptions around SetOuterClientHost (reliant on TWeakObjPtr which thus returns nullptr)
	if (!Ar.IsTransacting())
	{
		RigVMClient.SetDefaultSchemaClass(UUAFRigVMAssetSchema::StaticClass());
		RigVMClient.SetOuterClientHost(this, GET_MEMBER_NAME_CHECKED(UUAFRigVMAssetEditorData, RigVMClient));
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextVariableReferences)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DefaultInjectionSiteReference = FAnimNextVariableReference(DefaultInjectionSite_DEPRECATED);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (NativeInterface_DEPRECATED)
		{
			NativeInterfaces_DEPRECATED.Add(NativeInterface_DEPRECATED);
			NativeInterface_DEPRECATED = nullptr;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		bUpgradeDataInterfacesOnLoad = true;
	}
	
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::DeprecateUAFExternalPackages)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (bUsesExternalPackages_DEPRECATED)
		{
			UE_LOGF(LogAnimation, Error, "External packages support for UAF has been deprecated in 5.7, please resave content as non-external packages in 5.6 before upgrading.");
       	}
		// Copy any internal entries to the main entries array and clear out deprecated array
		Entries.Append(InternalEntries_DEPRECATED);
		InternalEntries_DEPRECATED.Empty();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	for (UUAFRigVMAssetEntry* Entry : Entries)
	{
		Entry->ConditionalPreload();
	}
}

void UUAFRigVMAssetEditorData::Initialize(bool bRecompileVM)
{
	RigVMClient.bDefaultModelCanBeRemoved = true;
	RigVMClient.SetDefaultSchemaClass(UUAFRigVMAssetSchema::StaticClass());
	RigVMClient.SetControllerClass(GetControllerClass());
	RigVMClient.SetOuterClientHost(this, GET_MEMBER_NAME_CHECKED(UUAFRigVMAssetEditorData, RigVMClient));
	RigVMClient.SetExternalModelHost(this);

	URigVMFunctionLibrary* RigVMFunctionLibrary = nullptr;
	{
		TGuardValue<bool> DisableClientNotifs(RigVMClient.bSuspendNotifications, true);
		RigVMFunctionLibrary = RigVMClient.GetOrCreateFunctionLibrary(false);
	}

	ensure(RigVMFunctionLibrary->GetFunctionHostObjectPathDelegate.IsBound());

	if (RigVMClient.GetController(0) == nullptr)
	{
		if(RigVMClient.GetDefaultModel())
		{
			RigVMClient.GetOrCreateController(RigVMClient.GetDefaultModel());
		}

		check(RigVMFunctionLibrary);
		RigVMClient.GetOrCreateController(RigVMFunctionLibrary);

		if (!FunctionLibraryEdGraph)
		{
			FunctionLibraryEdGraph = NewObject<UAnimNextEdGraph>(CastChecked<UObject>(this), NAME_None, RF_Transactional);

			FunctionLibraryEdGraph->Schema = UAnimNextEdGraphSchema::StaticClass();
			FunctionLibraryEdGraph->bAllowRenaming = 0;
			FunctionLibraryEdGraph->bEditable = 0;
			FunctionLibraryEdGraph->bAllowDeletion = 0;
			FunctionLibraryEdGraph->bIsFunctionDefinition = false;
			FunctionLibraryEdGraph->ModelNodePath = RigVMClient.GetFunctionLibrary()->GetNodePath();
			FunctionLibraryEdGraph->Initialize(this);
		}

		// Init function library controllers
		for(URigVMLibraryNode* LibraryNode : RigVMClient.GetFunctionLibrary()->GetFunctions())
		{
			RigVMClient.GetOrCreateController(LibraryNode->GetContainedGraph());
		}
		
		if(bRecompileVM)
		{
			RecompileVM();
		}
	}

	for(UUAFRigVMAssetEntry* Entry : Entries)
	{
		Entry->Initialize(this);
	}

	InitializeAssetUserData();
}

void UUAFRigVMAssetEditorData::InitializeAssetUserData()
{
	if (IInterface_AssetUserData* OuterUserData = Cast<IInterface_AssetUserData>(GetOuter()))
	{
		if(!OuterUserData->HasAssetUserDataOfClass(GetAssetUserDataClass()))
		{
			OuterUserData->AddAssetUserDataOfClass(GetAssetUserDataClass());
		}
	}
}

void UUAFRigVMAssetEditorData::PostLoad()
{
	Super::PostLoad();

	GraphModels.Reset();

	// Postload our entries, as we need them for RefreshExternalModels
	for (UUAFRigVMAssetEntry* Entry : Entries)
	{
		Entry->ConditionalPostLoad();
	}

	RefreshExternalModels();

	Initialize(/*bRecompileVM*/false);

	// Mark this as being dirty so that we recompile when needed
	SetVMRecompilationRequired(true);

	// Queue compilation once the package has been fully loaded
	// This is necessary in case we have external packages that haven't post-loaded yet
	// However, if we are duplicating the asset OnEndLoadPackage won't be called
	FCoreUObjectDelegates::OnEndLoadPackage.AddUObject(this, &UUAFRigVMAssetEditorData::HandlePackageDone);
}

void UUAFRigVMAssetEditorData::RequestVMRecompilation()
{
	if (VMRecompilationBracket == 0)
	{
		RecompileVM();
	}
    else
    {
		SetVMRecompilationRequired(true);
		bVMRecompilationRequested = true;
	}
}

void UUAFRigVMAssetEditorData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	BroadcastModified(EAnimNextEditorDataNotifType::PropertyChanged, this);
}

void UUAFRigVMAssetEditorData::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		BroadcastModified(EAnimNextEditorDataNotifType::UndoRedo, this);
	}
}

void UUAFRigVMAssetEditorData::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	{
		// We may not have compiled yet, so cache exports if we havent already
		if (!CachedExports.IsSet())
		{
			CachedExports = FAnimNextAssetRegistryExports();
			FAnimNextAssetRegistryExports& OutExports = CachedExports.GetValue();
			GetAnimNextAssetRegistryTags(Context, OutExports);
		}

		FString TagValue;
		FAnimNextAssetRegistryExports::StaticStruct()->ExportText(TagValue, &CachedExports.GetValue(), nullptr, nullptr, PPF_None, nullptr);
		Context.AddTag(FAssetRegistryTag(UE::UAF::ExportsAnimNextAssetRegistryTag, TagValue, FAssetRegistryTag::TT_Hidden));
	}

	{
		FRigVMGraphFunctionHeaderArray FunctionExports;
		UE::UAF::UncookedOnly::FUtils::GetAssetFunctions(this, FunctionExports);

		FString TagValue;
		const FArrayProperty* HeadersProperty = CastField<FArrayProperty>(FRigVMGraphFunctionHeaderArray::StaticStruct()->FindPropertyByName(TEXT("Headers")));
		HeadersProperty->ExportText_Direct(TagValue, &(FunctionExports.Headers), &(FunctionExports.Headers), nullptr, PPF_None, nullptr);
		Context.AddTag(FAssetRegistryTag(UE::UAF::AnimNextPublicGraphFunctionsExportsRegistryTag, TagValue, FAssetRegistryTag::TT_Hidden));
	}

	{
		// Export user defined events as notifies
		FString NotifyList = USkeleton::AnimNotifyTagDelimiter;
		for(FName EventName : RigVMClient.GetEntryNames(FRigVMFunction_UserDefinedEvent::StaticStruct()))
		{
			NotifyList += FString::Printf(TEXT("%s%s"), *EventName.ToString(), *USkeleton::AnimNotifyTagDelimiter);
		}
		Context.AddTag(FAssetRegistryTag(USkeleton::AnimNotifyTag, NotifyList, FAssetRegistryTag::TT_Hidden));
	}
}

bool UUAFRigVMAssetEditorData::Rename(const TCHAR* NewName, UObject* NewOuter, ERenameFlags Flags)
{
	FExternalPackageHelper::FRenameExternalObjectsHelperContext Context(this, Flags);
	return Super::Rename(NewName, NewOuter, Flags);
}

void UUAFRigVMAssetEditorData::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	UObject::PreDuplicate(DupParams);
	FExternalPackageHelper::DuplicateExternalPackages(this, DupParams);
}

void UUAFRigVMAssetEditorData::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	HandlePackageDone();
}

void UUAFRigVMAssetEditorData::HandlePackageDone(const FEndLoadPackageContext& Context)
{
	if (!Context.LoadedPackages.Contains(GetPackage()))
	{
		return;
	}
	HandlePackageDone();
}

void UUAFRigVMAssetEditorData::HandlePackageDone()
{
	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);

	{
		TGuardValue<bool> ReentrantGuardSelf(bSuspendModelNotificationsForSelf, true);
		GetRigVMClient()->PatchModelsOnLoad();
		GetRigVMClient()->RefreshAllModels(ERigVMLoadType::PostLoad, false, bIsCompiling);
	}

	ConditionalPatchFunctionsOnLoad();

	// Register function references at RigVMBuildData
	if (URigVMBuildData* BuildData = URigVMBuildData::Get())
	{
		TArray<FRigVMReferenceNodeData> ReferenceNodeDatas;
		const TArray<URigVMGraph*> AllModels = GetAllModels();
		for (URigVMGraph* ModelToVisit : AllModels)
		{
			for (URigVMNode* Node : ModelToVisit->GetNodes())
			{
				if (URigVMFunctionReferenceNode* ReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
				{
					ReferenceNodeDatas.Add(ReferenceNode->GetReferenceNodeData());
				}
			}
		}

		// update the build data from the current function references
		for (const FRigVMReferenceNodeData& ReferenceNodeData : ReferenceNodeDatas)
		{
			BuildData->RegisterFunctionReference(ReferenceNodeData);
		}

		BuildData->ClearInvalidReferences();
	}

	ReconstructAllNodes(); // If this is not executed on a node for whatever reason, it will appear transparent in the editor

	TGuardValue<bool> DisableCompilationNotifications(bSuspendCompilationNotifications, true);

	if (bUpgradeDataInterfacesOnLoad)
	{
		UpgradeDataInterfaces();
	}

	RecompileVM();
}

void UUAFRigVMAssetEditorData::GetAnimNextAssetRegistryTags(FAssetRegistryTagsContext& Context, FAnimNextAssetRegistryExports& OutExports) const
{	
	UE::UAF::UncookedOnly::FUtils::GetAssetVariableExports(this, OutExports, Context);
}

void UUAFRigVMAssetEditorData::RefreshAllModels(ERigVMLoadType InLoadType)
{
}

void UUAFRigVMAssetEditorData::OnRigVMRegistryChanged()
{
	GetRigVMClient()->RefreshAllModels(ERigVMLoadType::PostLoad, false, bIsCompiling);
	//RebuildGraphFromModel(); // TODO zzz : Move from blueprint to client
}

void UUAFRigVMAssetEditorData::RequestRigVMInit()
{
	// TODO zzz : How we do this on AnimNext ?
}

URigVMGraph* UUAFRigVMAssetEditorData::GetModel(const UEdGraph* InEdGraph) const
{
	return RigVMClient.GetModel(InEdGraph);
}

URigVMGraph* UUAFRigVMAssetEditorData::GetModel(const FString& InNodePath) const
{
	return RigVMClient.GetModel(InNodePath);
}

URigVMGraph* UUAFRigVMAssetEditorData::GetDefaultModel() const 
{
	return RigVMClient.GetDefaultModel();
}

TArray<URigVMGraph*> UUAFRigVMAssetEditorData::GetAllModels() const
{
	return RigVMClient.GetAllModels(true, true);
}

URigVMFunctionLibrary* UUAFRigVMAssetEditorData::GetLocalFunctionLibrary() const
{
	return RigVMClient.GetFunctionLibrary();
}

URigVMFunctionLibrary* UUAFRigVMAssetEditorData::GetOrCreateLocalFunctionLibrary(bool bSetupUndoRedo)
{
	return RigVMClient.GetOrCreateFunctionLibrary(bSetupUndoRedo);
}

URigVMGraph* UUAFRigVMAssetEditorData::AddModel(FString InName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	TGuardValue<bool> EnablePythonPrint(bSuspendPythonMessagesForRigVMClient, !bPrintPythonCommand);
	return RigVMClient.AddModel(InName, bSetupUndoRedo, bPrintPythonCommand);
}

bool UUAFRigVMAssetEditorData::RemoveModel(FString InName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	TGuardValue<bool> EnablePythonPrint(bSuspendPythonMessagesForRigVMClient, !bPrintPythonCommand);
	return RigVMClient.RemoveModel(InName, bSetupUndoRedo, bPrintPythonCommand);
}

FRigVMGetFocusedGraph& UUAFRigVMAssetEditorData::OnGetFocusedGraph()
{
	return RigVMClient.OnGetFocusedGraph();
}

const FRigVMGetFocusedGraph& UUAFRigVMAssetEditorData::OnGetFocusedGraph() const
{
	return RigVMClient.OnGetFocusedGraph();
}

URigVMGraph* UUAFRigVMAssetEditorData::GetFocusedModel() const
{
	return RigVMClient.GetFocusedModel();
}

URigVMController* UUAFRigVMAssetEditorData::GetController(const URigVMGraph* InGraph) const
{
	return RigVMClient.GetController(InGraph);
};

URigVMController* UUAFRigVMAssetEditorData::GetControllerByName(const FString InGraphName) const
{
	return RigVMClient.GetControllerByName(InGraphName);
};

URigVMController* UUAFRigVMAssetEditorData::GetOrCreateController(URigVMGraph* InGraph)
{
	return RigVMClient.GetOrCreateController(InGraph);
};

URigVMController* UUAFRigVMAssetEditorData::GetController(const UEdGraph* InEdGraph) const
{
	return RigVMClient.GetController(InEdGraph);
};

URigVMController* UUAFRigVMAssetEditorData::GetOrCreateController(const UEdGraph* InEdGraph)
{
	return RigVMClient.GetOrCreateController(InEdGraph);
};

TArray<FString> UUAFRigVMAssetEditorData::GeneratePythonCommands()
{
	return TArray<FString>();
}

void UUAFRigVMAssetEditorData::SetupPinRedirectorsForBackwardsCompatibility()
{
}

FRigVMGraphModifiedEvent& UUAFRigVMAssetEditorData::OnModified()
{
	return RigVMGraphModifiedEvent;
}

bool UUAFRigVMAssetEditorData::IsFunctionPublic(const FName& InFunctionName) const
{
	return GetLocalFunctionLibrary()->IsFunctionPublic(InFunctionName);
}

void UUAFRigVMAssetEditorData::MarkFunctionPublic(const FName& InFunctionName, bool bIsPublic)
{
	if (IsFunctionPublic(InFunctionName) == bIsPublic)
	{
		return;
	}

	URigVMController* Controller = RigVMClient.GetOrCreateController(GetLocalFunctionLibrary());
	Controller->MarkFunctionAsPublic(InFunctionName, bIsPublic);
}

void UUAFRigVMAssetEditorData::RenameGraph(const FString& InNodePath, const FName& InNewName)
{
	if (URigVMGraph* ModelForNodePath = GetModel(InNodePath))
	{
		if (UEdGraph* EdGraph = Cast<UEdGraph>(GetEditorObjectForRigVMGraph(ModelForNodePath)))
		{
			RigVMClient.RenameModel(InNodePath, InNewName, true);
		}
	}
}

UClass* UUAFRigVMAssetEditorData::GetRigVMSchemaClass() const
{
	return UUAFRigVMAssetSchema::StaticClass();
}

UScriptStruct* UUAFRigVMAssetEditorData::GetRigVMExecuteContextStruct() const 
{
	return FAnimNextExecuteContext::StaticStruct();
}

UClass* UUAFRigVMAssetEditorData::GetRigVMEdGraphClass() const 
{
	return UAnimNextEdGraph::StaticClass();
}

UClass* UUAFRigVMAssetEditorData::GetRigVMEdGraphNodeClass() const
{
	return UAnimNextEdGraphNode::StaticClass();
}

UClass* UUAFRigVMAssetEditorData::GetRigVMEdGraphSchemaClass() const
{
	return UAnimNextEdGraphSchema::StaticClass();
}

UClass* UUAFRigVMAssetEditorData::GetRigVMEditorSettingsClass() const
{
	return URigVMEditorSettings::StaticClass();
}

FRigVMClient* UUAFRigVMAssetEditorData::GetRigVMClient()
{
	return &RigVMClient;
}

const FRigVMClient* UUAFRigVMAssetEditorData::GetRigVMClient() const
{
	return &RigVMClient;
}

TScriptInterface<IRigVMGraphFunctionHost> UUAFRigVMAssetEditorData::GetRigVMGraphFunctionHost()
{
	return this;
}

TScriptInterface<const IRigVMGraphFunctionHost> UUAFRigVMAssetEditorData::GetRigVMGraphFunctionHost() const
{
	return this;
}

void UUAFRigVMAssetEditorData::HandleRigVMGraphAdded(const FRigVMClient* InClient, const FString& InNodePath)
{
	if(URigVMGraph* RigVMGraph = InClient->GetModel(InNodePath))
	{
		RigVMGraph->SetExecuteContextStruct(GetExecuteContextStruct());

		if(!HasAnyFlags(RF_ClassDefaultObject | RF_NeedInitialization | RF_NeedLoad | RF_NeedPostLoad) &&
			GetOuter() != GetTransientPackage())
		{
			CreateEdGraph(RigVMGraph, true);
			RequestAutoVMRecompilation();
		}
		
#if WITH_EDITOR
		if(!bSuspendPythonMessagesForRigVMClient)
		{
			const FString AssetName = RigVMGraph->GetSchema()->GetSanitizedName(GetName(), true, false);
			RigVMPythonUtils::Print(AssetName, FString::Printf(TEXT("asset.add_graph('%s')"), *RigVMGraph->GetName()));
		}
#endif
	}
}

void UUAFRigVMAssetEditorData::HandleRigVMGraphRemoved(const FRigVMClient* InClient, const FString& InNodePath)
{
	if(URigVMGraph* RigVMGraph = InClient->GetModel(InNodePath))
	{
		if (UUAFRigVMAssetEntry* Entry = FindEntryForRigVMGraph(RigVMGraph))
		{
			if (IUAFRigVMGraphInterface* GraphInterface = Cast<IUAFRigVMGraphInterface>(Entry))
			{
				GraphInterface->SetRigVMGraph(nullptr);
			}
		}
		GraphModels.Remove(RigVMGraph);

		RemoveEdGraph(RigVMGraph);
		RequestAutoVMRecompilation();

#if WITH_EDITOR
		if(!bSuspendPythonMessagesForRigVMClient)
		{
			const FString AssetName = RigVMGraph->GetSchema()->GetSanitizedName(GetName(), true, false);
			RigVMPythonUtils::Print(AssetName, FString::Printf(TEXT("asset.remove_graph('%s')"), *RigVMGraph->GetName()));
		}
#endif
	}
}

void UUAFRigVMAssetEditorData::HandleRigVMGraphRenamed(const FRigVMClient* InClient, const FString& InOldNodePath, const FString& InNewNodePath)
{
	if (InClient->GetModel(InNewNodePath))
	{
		TArray<UEdGraph*> EdGraphs = GetAllEdGraphs();
		for (UEdGraph* EdGraph : EdGraphs)
		{
			if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(EdGraph))
			{
				RigGraph->HandleRigVMGraphRenamed(InOldNodePath, InNewNodePath);
			}
		}
	}
}


void UUAFRigVMAssetEditorData::HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure)
{
	InControllerToConfigure->OnModified().AddUObject(this, &UUAFRigVMAssetEditorData::HandleModifiedEvent);

	TWeakObjectPtr<UUAFRigVMAssetEditorData> WeakThis(this);

	InControllerToConfigure->GetExternalVariablesDelegate.BindLambda([](URigVMGraph* InGraph) -> TArray<FRigVMExternalVariable> {
		if (InGraph)
		{
			if(URigVMHost* RigVMHost = InGraph->GetTypedOuter<URigVMHost>())
			{
				return RigVMHost->GetExternalVariables();
			}
		}
		return TArray<FRigVMExternalVariable>();
	});
	
	// this delegate is used by the controller
	// to retrieve the current bytecode of the VM
	InControllerToConfigure->GetCurrentByteCodeDelegate.BindLambda([WeakThis]() -> const FRigVMByteCode*
	{
		if (WeakThis.IsValid())
		{
			if(UUAFRigVMAsset* Asset = WeakThis->GetTypedOuter<UUAFRigVMAsset>())
			{
				if (Asset->VM)
				{
					return &Asset->VM->GetByteCode();
				}
			}
		}
		return nullptr;

	});

#if WITH_EDITOR
	InControllerToConfigure->SetupDefaultUnitNodeDelegates(TDelegate<FName(FRigVMExternalVariable, FString)>::CreateLambda(
		[](FRigVMExternalVariable InVariableToCreate, FString InDefaultValue) -> FName
		{
			return NAME_None;
		}
	));
#endif
}

UObject* UUAFRigVMAssetEditorData::GetEditorObjectForRigVMGraph(const URigVMGraph* InVMGraph) const
{
	if(InVMGraph)
	{
		if (InVMGraph->IsA<URigVMFunctionLibrary>())
		{
			return Cast<UObject>(FunctionLibraryEdGraph.Get());
		}

		const auto FindSubgraph = ([](const FString SearchGraphNodePath, URigVMEdGraph* EdGraph) -> URigVMEdGraph*
		{
			TArray<UEdGraph*> SubGraphs;
			EdGraph->GetAllChildrenGraphs(SubGraphs);
			for (UEdGraph* SubGraph : SubGraphs)
			{
				if (URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(SubGraph))
				{
					if (RigVMEdGraph->GetRigVMNodePath() == SearchGraphNodePath)
					{
						return RigVMEdGraph;
					}
				}
			}
			return nullptr;
		});

		const FString GraphNodePath = InVMGraph->GetNodePath();
		for(UUAFRigVMAssetEntry* Entry : Entries)
		{
			if(IUAFRigVMGraphInterface* GraphInterface = Cast<IUAFRigVMGraphInterface>(Entry))
			{
				URigVMEdGraph* EdGraph = GraphInterface->GetEdGraph();

				if (const URigVMGraph* RigVMGraph = GraphInterface->GetRigVMGraph())
				{
					if (RigVMGraph == InVMGraph)
					{
						return EdGraph;
					}
				}

				if (URigVMEdGraph* RigVMEdGraph = FindSubgraph(GraphNodePath, EdGraph))
				{
					return RigVMEdGraph;
				}
			}
		}

		for (const TObjectPtr<URigVMEdGraph>& FunctionEdGraph : FunctionEdGraphs)
		{
			if (FunctionEdGraph->ModelNodePath == GraphNodePath)
			{
				return FunctionEdGraph;
			}

			if (URigVMEdGraph* RigVMEdGraph = FindSubgraph(GraphNodePath, FunctionEdGraph))
			{
				return RigVMEdGraph;
			}
		}
	}
	return nullptr;
}

URigVMGraph* UUAFRigVMAssetEditorData::GetRigVMGraphForEditorObject(UObject* InObject) const
{
	if(const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(InObject))
	{
		if (Graph->bIsFunctionDefinition)
		{
			if (URigVMLibraryNode* LibraryNode = RigVMClient.GetFunctionLibrary()->FindFunction(*Graph->ModelNodePath))
			{
				return LibraryNode->GetContainedGraph();
			}
		}
		else
		{
			return RigVMClient.GetModel(Graph->ModelNodePath);
		}
	}

	return nullptr;
}

FRigVMGraphFunctionStore* UUAFRigVMAssetEditorData::GetRigVMGraphFunctionStore()
{
	ConditionalPatchFunctionsOnLoad();
	return &GraphFunctionStore;
}

const FRigVMGraphFunctionStore* UUAFRigVMAssetEditorData::GetRigVMGraphFunctionStore() const
{
	return &GraphFunctionStore;
}

TObjectPtr<URigVMGraph> UUAFRigVMAssetEditorData::CreateContainedGraphModel(URigVMCollapseNode* CollapseNode, const FName& Name)
{
	check(CollapseNode);

	TObjectPtr<URigVMGraph> Model = NewObject<URigVMGraph>(CollapseNode, Name);

	check(CollapseNode->GetGraph());
	if (CollapseNode->GetGraph()->GetSchema() != nullptr)
	{
		Model->SetSchemaClass(CollapseNode->GetGraph()->GetSchema()->GetClass());
	}
	else
	{
		Model->SetSchemaClass(RigVMClient.GetDefaultSchemaClass());
	}

	URigVMGraph* CollapseNodeModelRootGraph = CollapseNode->GetRootGraph();
	check(CollapseNodeModelRootGraph);

	return Model;
}

void UUAFRigVMAssetEditorData::BuildFunctionHeadersContext(const FRigVMCompileSettings& InSettings, FAnimNextGetFunctionHeaderCompileContext& OutCompileContext) const
{
	// By default gather & generate event stubs for all public functions so they can be called externally.
	for (const FRigVMGraphFunctionData& FunctionData : GraphFunctionStore.PublicFunctions)
	{
		if (FunctionData.Header.IsValid())
		{
			FAnimNextProgrammaticFunctionHeader AnimNextFunctionHeader = {};
			AnimNextFunctionHeader.FunctionHeader = FunctionData.Header;
			OutCompileContext.AddUniqueFunctionHeader(AnimNextFunctionHeader);
		}
	}
}

void UUAFRigVMAssetEditorData::BuildFunctionWrapperEventVariables(FAnimNextRigVMAssetCompileContext& InContext) const
{
	using namespace UE::UAF::UncookedOnly;
	
	for (const FAnimNextProgrammaticFunctionHeader& ProgrammaticFunctionHeader : InContext.FunctionHeaders)
	{
		const FRigVMGraphFunctionHeader& FunctionHeader = ProgrammaticFunctionHeader.FunctionHeader;

		// Don't generate internal variables for functions we don't own. we will get those via shared variables
		if (FunctionHeader.GetFunctionHostObject() != this)
		{
			continue;
		}

		for (const FRigVMGraphFunctionArgument& Argument : FunctionHeader.Arguments)
		{
			// Don't create internal variables for execution IO, visible only, or hidden pins. Those types do not need variable nodes in the graph.
			if (Argument.Direction == ERigVMPinDirection::Input || Argument.Direction == ERigVMPinDirection::Output)
			{
				if (Argument.IsValid())
				{
					if (Argument.bIsInputVariable)
					{
						continue;
					}
				
					FRigVMGraphFunctionArgument InternallyNamedArgument = Argument;
					FName InternalArgName = FName(FUtils::MakeFunctionWrapperVariableName(FunctionHeader.Name, Argument.Name));
					FName SanitizedInternalArgName = FRigVMPropertyDescription::SanitizeName(InternalArgName);
					InternallyNamedArgument.Name = SanitizedInternalArgName;
					InContext.ProgrammaticVariables.Add(FAnimNextProgrammaticVariable::FromRigVMGraphFunctionArgument(InternallyNamedArgument));
				}
			}
		}
	}
}

void UUAFRigVMAssetEditorData::BuildFunctionWrapperEvents(FAnimNextRigVMAssetCompileContext& InContext, const FRigVMCompileSettings& InSettings)
{
	using namespace UE::UAF::UncookedOnly;

	UUAFRigVMAsset* Asset = FUtils::GetAsset<UUAFRigVMAsset>(this);
	Asset->FunctionData.Reset();

	FRigVMClient* VMClient = GetRigVMClient();

	// Create all shim events for our traits to call
	const bool bSetupUndoRedo = false;
	URigVMGraph* WrapperGraph = NewObject<URigVMGraph>(this, NAME_None, RF_Transient);
	URigVMController* Controller = VMClient->GetOrCreateController(WrapperGraph);
	FRigVMControllerNotifGuard NotifGuard(Controller);
	FRigVMControllerASTLinkCheckGuard LinkCheckGuard(Controller);
	bool bAddedWrapperEvent = true;
	TArray<FRigVMExternalVariable> ExternalVariables = Asset->GetExternalVariables();

	for(const FAnimNextProgrammaticFunctionHeader& AnimNextFunctionHeader : InContext.FunctionHeaders)
	{
		// Controller needs to notify the AST of variable changes to make new links
		constexpr bool bSuspendNotificationForInternalVariables = false;
		FRigVMControllerNotifGuard VarNotifGuard(Controller, bSuspendNotificationForInternalVariables);

		const FRigVMGraphFunctionHeader& FunctionHeader = AnimNextFunctionHeader.FunctionHeader;

		URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(FunctionHeader.LibraryPointer.GetNodeSoftPath().TryLoad());
		if(LibraryNode == nullptr)
		{
			InSettings.ReportError(FString::Printf(TEXT("Could not find function '%s'"), *FunctionHeader.Name.ToString()));
			continue;
		}

		// Create user-defined entry point
		FString WrapperEventName = FUtils::MakeFunctionWrapperEventName(FunctionHeader.Name);
		URigVMUnitNode* EventNode = Controller->AddUnitNode(FRigVMFunction_UserDefinedEvent::StaticStruct(), TEXT("Execute"), FVector2D::ZeroVector, FunctionHeader.Name.ToString(), bSetupUndoRedo);
		if(EventNode == nullptr)
		{
			InSettings.ReportError(FString::Printf(TEXT("Could not spawn event node for function '%s'"), *FunctionHeader.Name.ToString()));
			continue;
		}
		URigVMPin* EventNamePin = EventNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigVMFunction_UserDefinedEvent, EventName));
		if(EventNamePin == nullptr)
		{
			InSettings.ReportError(FString::Printf(TEXT("Could not find custom event name pin")));
			continue;
		}
		Controller->SetPinDefaultValue(EventNamePin->GetPinPath(), WrapperEventName, true, bSetupUndoRedo);

		// Call function
		URigVMFunctionReferenceNode* FunctionNode = Controller->AddFunctionReferenceNode(LibraryNode, FVector2D::ZeroVector, FunctionHeader.Name.ToString(), bSetupUndoRedo);
		if(FunctionNode == nullptr)
		{
			InSettings.ReportError(FString::Printf(TEXT("Could not spawn function node for function '%s'"), *FunctionHeader.Name.ToString()));
			continue;
		}

		if (!ExternalVariables.IsEmpty())
		{
			for (const URigVMPin* Pin : FunctionNode->GetPins())
			{
				if (!Pin->IsDefinedAsInputVariable())
				{
					continue;
				}
				Controller->BindPinToVariable(Pin->GetPinPath(), Pin->GetName());
			}
		}

		// Link up Execute nodes if needed, function may be pure & lack an input pin
		URigVMPin* CurrentExecuteOutputPin = EventNode->FindPin(FRigVMStruct::ExecuteContextName.ToString());
		URigVMPin* ExecuteInputPin = FunctionNode->FindPin(FRigVMStruct::ExecuteContextName.ToString());
		if (ExecuteInputPin && !Controller->AddLink(CurrentExecuteOutputPin, ExecuteInputPin, bSetupUndoRedo))
		{
			InSettings.ReportError(FString::Printf(TEXT("Could not link execute pins for function '%s'"), *FunctionHeader.Name.ToString()));
			continue;
		}

		// Update current execute pin, RigVM doesn't have a concept of input / output execute pins, just one execute content pin used for both
		CurrentExecuteOutputPin = ExecuteInputPin ? ExecuteInputPin : CurrentExecuteOutputPin;

		// Generate & link input arguments, also generate result variable node but link later
		TArray<int32> ArgIndices;
		for (const FRigVMGraphFunctionArgument& Argument : FunctionHeader.Arguments)
		{
			// Execution context is captured as arg pins, skip those for internal variable gen
			if (Argument.Direction == ERigVMPinDirection::IO || !Argument.IsValid() || Argument.bIsInputVariable)
			{
				continue;
			}

			bool bIsGetter = Argument.Direction == ERigVMPinDirection::Input;
			FName InternalArgName = FName(FUtils::MakeFunctionWrapperVariableName(FunctionHeader.Name, Argument.Name));
			FName SanitizedInternalArgName = FRigVMPropertyDescription::SanitizeName(InternalArgName);
			int32 VariableIndex = ExternalVariables.IndexOfByPredicate([&SanitizedInternalArgName](const FRigVMExternalVariable& InVariable)
			{
				return InVariable.GetName() == SanitizedInternalArgName;
			});

			if (VariableIndex == INDEX_NONE)
			{
				if (FunctionHeader.GetFunctionHostObject() != this)
				{
					InSettings.ReportError(FString::Printf(TEXT("Missing Shared Variables Reference to %s required for function %s"), FunctionHeader.GetFunctionHostObject() ? *FUtils::GetAsset(Cast<UUAFRigVMAssetEditorData>(FunctionHeader.GetFunctionHostObject().GetObject()))->GetFName().ToString() : TEXT("None"), *FunctionHeader.Name.ToString()));
				}
				else
				{
					InSettings.ReportError(FString::Printf(TEXT("Failed to link internal variable param node to function: %s -> %s"), *Argument.Name.ToString(), *FunctionHeader.Name.ToString()));
				}
				
				continue;
			}
			
			ArgIndices.Add(VariableIndex);

			if (bIsGetter)
			{
				URigVMVariableNode* FunctionParamVariableNode = Controller->AddVariableNode(SanitizedInternalArgName
					, Argument.CPPType.ToString()
					, Argument.CPPTypeObject.Get()
					, bIsGetter
					, Argument.DefaultValue
					, FVector2D::ZeroVector
					, SanitizedInternalArgName.ToString()
					, bSetupUndoRedo);

				if (!FunctionParamVariableNode)
				{
					InSettings.ReportError(FString::Printf(TEXT("Failed to add internal variable node for param: %s, var: %s"), *FunctionHeader.Name.ToString(), *InternalArgName.ToString()));
					return;
				}

				// Link Param Pins
				URigVMPin* ParamValuePin = FunctionParamVariableNode->GetValuePin();
				URigVMPin* FunctionArgumentPin = FunctionNode->FindPin(Argument.Name.ToString());
				if (!Controller->AddLink(ParamValuePin, FunctionArgumentPin, bSetupUndoRedo))
				{
					InSettings.ReportError(FString::Printf(TEXT("Failed to link internal variable param node to function: %s -> %s"), *GetNameSafe(ParamValuePin), *GetNameSafe(FunctionArgumentPin)));
					return;
				}
			}

			if (!bIsGetter)
			{
				
				URigVMVariableNode* FunctionResultVariableNode = Controller->AddVariableNode(SanitizedInternalArgName
					, Argument.CPPType.ToString()
					, Argument.CPPTypeObject.Get()
					, bIsGetter
					, Argument.DefaultValue
					, FVector2D::ZeroVector
					, SanitizedInternalArgName.ToString()
					, bSetupUndoRedo);

				if (!FunctionResultVariableNode)
				{
					InSettings.ReportError(FString::Printf(TEXT("Failed to add internal variable node for result: %s, var: %s"), *FunctionHeader.Name.ToString(), *InternalArgName.ToString()));
					return;
				}

				// Link Result pins
				URigVMPin* FunctionResultPin = FunctionNode->FindPin(Argument.Name.ToString());
				URigVMPin* ResultValuePin = FunctionResultVariableNode->GetValuePin();
				if (!Controller->AddLink(FunctionResultPin, ResultValuePin, bSetupUndoRedo))
				{
					InSettings.ReportError(FString::Printf(TEXT("Failed to link internal variable result node to function: %s -> %s"), *GetNameSafe(FunctionResultPin), *GetNameSafe(ResultValuePin)));
					return;
				}

				// Link Result Execute pins
				URigVMPin* ResultExecuteInputPin = FunctionResultVariableNode->FindPin(FRigVMStruct::ExecuteContextName.ToString());
				if (!Controller->AddLink(CurrentExecuteOutputPin, ResultExecuteInputPin, bSetupUndoRedo))
				{
					InSettings.ReportError(FString::Printf(TEXT("Failed to link execute pins for variable result node: %s -> %s"), *GetNameSafe(CurrentExecuteOutputPin), *GetNameSafe(ResultExecuteInputPin)));
					return;
				}

				// Update current execute pin, RigVM doesn't have a concept of input / output execute pins, just one execute content pin used for both
				CurrentExecuteOutputPin = ResultExecuteInputPin;
			}
		}

		bAddedWrapperEvent = true;

		FAnimNextRigVMFunctionData FunctionData;
		FunctionData.FunctionGuid = FunctionHeader.Variant.Guid;
		FunctionData.Name = FunctionHeader.Name;
		FunctionData.EventName = FName(*WrapperEventName);
		FunctionData.ArgIndices = MoveTemp(ArgIndices);
		Asset->FunctionData.Add(MoveTemp(FunctionData));
	}

	if(bAddedWrapperEvent)
	{
		InContext.ProgrammaticGraphs.Add(WrapperGraph);
	}
}

void UUAFRigVMAssetEditorData::RecompileVM()
{
	using namespace UE::UAF::UncookedOnly;

	if (bIsCompiling)
	{
		return;
	}

	TGuardValue<bool> CompilingGuard(bIsCompiling, true);

	TRACE_CPUPROFILER_EVENT_SCOPE(UUAFRigVMAssetEditorData::RecompileVM);

	UUAFRigVMAsset* Asset = FUtils::GetAsset<UUAFRigVMAsset>(this);
	FCompilationScope CompileScope(Asset);

	VMCompileSettings.SetExecuteContextStruct(FAnimNextExecuteContext::StaticStruct());
	FRigVMCompileSettings Settings = (bCompileInDebugMode) ? FRigVMCompileSettings::Fast(VMCompileSettings.GetExecuteContextStruct()) : VMCompileSettings;
	Settings.SurpressInfoMessages = false;
	Settings.bWarnAboutDuplicateEvents = true;
	Settings.ASTSettings.ReportDelegate.BindUObject(this, &UUAFRigVMAssetEditorData::HandleReportFromCompiler);

	Asset->VMRuntimeSettings = VMRuntimeSettings;
	Asset->Components.Empty();
	
	bDisplayCompilePIEWarning = true;

	OnPreCompileAsset(Settings);

	CachedExports.Reset();  // asset variables and other tags will be updated at the end by AssetRegistry->AssetUpdateTags

	bWarningsDuringCompilation = false;
	bErrorsDuringCompilation = false;
	bHasCompiledVariables = false;

	RigGraphDisplaySettings.MinMicroSeconds = RigGraphDisplaySettings.LastMinMicroSeconds = DBL_MAX;
	RigGraphDisplaySettings.MaxMicroSeconds = RigGraphDisplaySettings.LastMaxMicroSeconds = (double)INDEX_NONE;

	FAnimNextRigVMAssetCompileContext CompileContext = { this };
	{
		TGuardValue<bool> ReentrantGuardSelf(bSuspendModelNotificationsForSelf, true);
		TGuardValue<bool> ReentrantGuardOthers(RigVMClient.bSuspendModelNotificationsForOthers, true);

		FUtils::RecreateVM(Asset);

		GatherDefaultComponents(Asset);

		{
			FAnimNextGetFunctionHeaderCompileContext GetFunctionHeaderCompileContext(CompileContext);
			BuildFunctionHeadersContext(Settings, GetFunctionHeaderCompileContext);
			OnPreCompileGetProgrammaticFunctionHeaders(Settings, GetFunctionHeaderCompileContext);
		}

		BuildFunctionWrapperEventVariables(CompileContext);

		{
			FAnimNextGetVariableCompileContext GetVariableCompileContext(CompileContext);
			BuildProgrammaticVariablesContext(Settings, GetVariableCompileContext);
			FUtils::CompileVariables(Settings, Asset, GetVariableCompileContext);
			OnPostCompileVariables(Settings, GetVariableCompileContext);
		}

		BuildFunctionWrapperEvents(CompileContext, Settings);

		{
			FAnimNextGetGraphCompileContext GetGraphCompileContext(CompileContext);
			OnPreCompileGetProgrammaticGraphs(Settings, GetGraphCompileContext);
		}

		for(URigVMGraph* ProgrammaticGraph : CompileContext.ProgrammaticGraphs)
		{
			check(ProgrammaticGraph != nullptr);
		}

		FRigVMClient* VMClient = GetRigVMClient();

		CompileContext.AllGraphs = VMClient->GetAllModels(false, false);
		CompileContext.AllGraphs.Append(CompileContext.ProgrammaticGraphs);

		{
			FAnimNextProcessGraphCompileContext ProcessGraphCompileContext(CompileContext);
			OnPreCompileProcessGraphs(Settings, ProcessGraphCompileContext);
		}

		if(CompileContext.AllGraphs.Num() > 0)
		{
			URigVMController* Controller = VMClient->GetOrCreateController(CompileContext.AllGraphs[0]);

			URigVMCompiler* Compiler = URigVMCompiler::StaticClass()->GetDefaultObject<URigVMCompiler>();
			Compiler->Compile(Settings, CompileContext.AllGraphs, Controller, Asset->VM, Asset->ExtendedExecuteContext, Asset->GetExternalVariables(), &PinToOperandMap);
		}

		// Initialize right away, in packaged builds we initialize during PostLoad
		Asset->VM->Initialize(Asset->ExtendedExecuteContext);
		Asset->GenerateUserDefinedDependenciesData(Asset->ExtendedExecuteContext);
		Asset->GenerateRequiredPluginsData(Asset->ExtendedExecuteContext);

		// Notable difference with vanilla RigVM host behavior - we init the VM here at the moment as we only have one 'instance'
		Asset->InitializeVM(FRigUnit_AnimNextBeginExecution::EventName);

		if (bErrorsDuringCompilation)
		{
			if(Settings.SurpressErrors)
			{
				Settings.Reportf(EMessageSeverity::Info, Asset, TEXT("Compilation Errors may be suppressed for AnimNext asset: %s. See VM Compile Settings for more Details"), *Asset->GetName());
			}
		}

		SetVMRecompilationRequired(false);
		bVMRecompilationRequested = false;

		if(Asset->VM)
		{
			RigVMCompiledEvent.Broadcast(Asset, Asset->VM, Asset->ExtendedExecuteContext);
		}

#if WITH_EDITOR
		// Display programmatic graphs
		if(CVarDumpProgrammaticGraphs.GetValueOnGameThread())
		{
			FUtils::OpenProgrammaticGraphs(this, CompileContext.ProgrammaticGraphs);
		}
		else
#endif
		{
			RemoveProgrammaticGraphs(CompileContext.ProgrammaticGraphs);
		}

		RemoveTransientGraphs(CompileContext.AllGraphs);

		OnPostCompileCleanup(Settings);

#if WITH_EDITOR
		//	RefreshBreakpoints(EditorData);
#endif

		// Refresh CachedExports
		if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
		{
			AssetRegistry->AssetUpdateTags(Asset, EAssetRegistryTagsCaller::Fast);
		}
	}

	checkf(bHasCompiledVariables, TEXT("Asset variables are expected to be compiled as part of compilation process"));
}

void UUAFRigVMAssetEditorData::GatherDefaultComponents(UUAFRigVMAsset* InAsset)
{
	TArray<TInstancedStruct<FUAFAssetInstanceComponent>> DefaultComponents;

	// Ensure we add our script component
	DefaultComponents.Add(TInstancedStruct<FUAFRigVMComponent>::Make());

	// Gather any required components from node metadata
	FRigVMClient* VMClient = GetRigVMClient();
	for(const URigVMGraph* Graph : VMClient->GetAllModels(false, false))
	{
		for(URigVMNode* Node : Graph->GetNodes())
		{
			URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(Node);
			if(TemplateNode == nullptr)
			{
				continue;
			}

			UScriptStruct* Struct = TemplateNode->GetScriptStruct();
			if(Struct == nullptr)
			{
				continue;
			}

			static FName Metadata_RequiredComponents("RequiredComponents");
			FString ComponentsString = Struct->GetMetaData(Metadata_RequiredComponents);
			if(ComponentsString.Len() == 0)
			{
				continue;
			}

			UE::String::ParseTokens(ComponentsString, TEXT(','), [this, &DefaultComponents](FStringView InToken)
			{
				FString StructName(InToken);
				UScriptStruct* ComponentStruct = FindFirstObject<UScriptStruct>(*StructName);
				if(ComponentStruct && !DefaultComponents.ContainsByPredicate([ComponentStruct](const TInstancedStruct<FUAFAssetInstanceComponent>& InComponent){ return InComponent.GetScriptStruct() == ComponentStruct; }))
				{
					TInstancedStruct<FUAFAssetInstanceComponent> NewComponent;
					NewComponent.InitializeAsScriptStruct(ComponentStruct);
					DefaultComponents.Add(MoveTemp(NewComponent));
				}
			});
		}
	}

	// Filter components we no longer need
	Components.RemoveAll([&DefaultComponents](const TInstancedStruct<FUAFAssetInstanceComponent>& InComponent)
	{
		return !DefaultComponents.ContainsByPredicate([&InComponent](const TInstancedStruct<FUAFAssetInstanceComponent>& InDefaultComponent)
			{
				return InDefaultComponent.GetScriptStruct() == InComponent.GetScriptStruct();
			});
	});

	// Add any new components
	for (TInstancedStruct<FUAFAssetInstanceComponent>& DefaultComponent : DefaultComponents)
	{
		if(!Components.ContainsByPredicate([&DefaultComponent](const TInstancedStruct<FUAFAssetInstanceComponent>& InComponent)
			{
				return DefaultComponent.GetScriptStruct() == InComponent.GetScriptStruct();
			}))
		{
			Components.Add(MoveTemp(DefaultComponent));
		}
	}

	// Add additional components added by the user
	for (const TInstancedStruct<FUAFAssetInstanceComponent>& AdditionalComponent : AdditionalComponents)
	{
		if (AdditionalComponent.IsValid() && !Components.ContainsByPredicate([&AdditionalComponent](const TInstancedStruct<FUAFAssetInstanceComponent>& InComponent)
			{
				return AdditionalComponent.GetScriptStruct() == InComponent.GetScriptStruct();
			}))
		{
			Components.Add(AdditionalComponent);
		}
	}

	// Copy to the asset
	InAsset->Components = Components;
}

void UUAFRigVMAssetEditorData::RemoveProgrammaticGraphs(TArrayView<URigVMGraph*> InGraphs)
{
	FRigVMClient* VMClient = GetRigVMClient();
	
	for(URigVMGraph* Graph : InGraphs)
	{
		VMClient->RemoveController(Graph);
		Graph->Rename(nullptr, GetTransientPackage(), REN_AllowPackageLinkerMismatch | REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
	}
}

void UUAFRigVMAssetEditorData::RemoveTransientGraphs(TArrayView<URigVMGraph*> InGraphs)
{
	FRigVMClient* VMClient = GetRigVMClient();
	
	for(URigVMGraph* Graph : InGraphs)
	{
		if(Graph->HasAnyFlags(RF_Transient))
		{
			VMClient->RemoveController(Graph);
			Graph->Rename(nullptr, GetTransientPackage(), REN_AllowPackageLinkerMismatch | REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		}
	}
}

void UUAFRigVMAssetEditorData::HandleRemoveNotify(UObject* InAsset, const FString& InFindString, bool bFindWholeWord, ESearchCase::Type InSearchCase)
{
	UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(InAsset);
	if(Asset == nullptr)
	{
		return;
	}

	UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
	if(EditorData == nullptr)
	{
		return;
	}

	URigVMController* Controller = EditorData->GetController();
	Controller->OpenUndoBracket(LOCTEXT("RemoveNotifyEvents", "Remove Notify Events").ToString());

	for(TObjectPtr<URigVMGraph> Model : EditorData->RigVMClient.GetModels())
	{
		for(URigVMNode* Node : Model->GetNodes())
		{
			if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
			{
				if(UnitNode->GetScriptStruct()->IsChildOf(FRigVMFunction_UserDefinedEvent::StaticStruct()))
				{
					URigVMPin* Pin = UnitNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigVMFunction_UserDefinedEvent, EventName));
					FString EventNameString = Pin->GetDefaultValue();
					if( (bFindWholeWord && EventNameString.Equals(InFindString, InSearchCase)) ||
						(!bFindWholeWord && EventNameString.Contains(InFindString, InSearchCase)))
					{
						Controller->RemoveNode(Node, true, true);
					}
				}
			}
		}
	}

	Controller->CloseUndoBracket();
}

void UUAFRigVMAssetEditorData::HandleReplaceNotify(UObject* InAsset, const FString& InFindString, const FString& InReplaceString, bool bFindWholeWord, ESearchCase::Type InSearchCase)
{
	UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(InAsset);
	if(Asset == nullptr)
	{
		return;
	}

	UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
	if(EditorData == nullptr)
	{
		return;
	}

	URigVMController* Controller = EditorData->GetController();
	Controller->OpenUndoBracket(LOCTEXT("ReplaceNotifyEvents", "Replace Notify Events").ToString());

	for(TObjectPtr<URigVMGraph> Model : EditorData->RigVMClient.GetModels())
	{
		for(URigVMNode* Node : Model->GetNodes())
		{
			if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
			{
				if(UnitNode->GetScriptStruct()->IsChildOf(FRigVMFunction_UserDefinedEvent::StaticStruct()))
				{
					URigVMPin* Pin = UnitNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigVMFunction_UserDefinedEvent, EventName));
					FString EventNameString = Pin->GetDefaultValue();
					if( (bFindWholeWord && EventNameString.Equals(InFindString, InSearchCase)) ||
						(!bFindWholeWord && EventNameString.Contains(InFindString, InSearchCase)))
					{
						const FString NewName = EventNameString.Replace(*InFindString, *InReplaceString, InSearchCase);
						Controller->SetPinDefaultValue(Pin->GetPinPath(), NewName, true, true, false, true);
					}
				}
			}
		}
	}

	Controller->CloseUndoBracket();
}

bool UUAFRigVMAssetEditorData::IsDirtyForRecompilation() const
{
	if(bVMRecompilationRequired)
	{
		return true;
	}

	bool bDependencyDirty = false;
	ForEachEntryOfType<UUAFSharedVariablesEntry>([&bDependencyDirty](UUAFSharedVariablesEntry* InEntry)
	{
		if(InEntry->Asset)
		{
			UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(InEntry->Asset.Get());
			if(EditorData->IsDirtyForRecompilation())
			{
				bDependencyDirty = true;
				return false;
			}
		}
		return true;
	});

	return bDependencyDirty;
}

void UUAFRigVMAssetEditorData::RecompileVMIfRequired()
{
	if (bVMRecompilationRequired)
	{
		RecompileVM();
	}
}

void UUAFRigVMAssetEditorData::RequestAutoVMRecompilation()
{
	SetVMRecompilationRequired(true);
	if (bAutoRecompileVM && VMRecompilationBracket == 0)
	{
		RecompileVMIfRequired();
	}
}

void UUAFRigVMAssetEditorData::SetVMRecompilationRequired(bool bInIsRequired)
{
	if (bInIsRequired != bVMRecompilationRequired)
	{
		bVMRecompilationRequired = bInIsRequired;
		RecompileRequiredChangedEvent.Broadcast();
	}
}

void UUAFRigVMAssetEditorData::SetAutoVMRecompile(bool bAutoRecompile)
{
	if (bAutoRecompileVM != bAutoRecompile)
	{
		bAutoRecompileVM = bAutoRecompile;
		MarkPackageDirty();
	}
}

bool UUAFRigVMAssetEditorData::GetAutoVMRecompile() const
{
	return bAutoRecompileVM;
}

void UUAFRigVMAssetEditorData::IncrementVMRecompileBracket()
{
	VMRecompilationBracket++;
}

void UUAFRigVMAssetEditorData::DecrementVMRecompileBracket()
{
	if (VMRecompilationBracket == 1)
	{
		// We should only recompile when the asset is dirty and we have auto-compile enabled
		// or when a recompilation has been explicitly requested (e.g. clicking compile when auto-compile is disabled)
		if ((bAutoRecompileVM && bVMRecompilationRequired) || bVMRecompilationRequested)
		{
			RecompileVM();
		}

		VMRecompilationBracket = 0;

		if (InteractionBracketFinished.IsBound())
		{
			InteractionBracketFinished.Broadcast(this);
		}
	}
	else if (VMRecompilationBracket > 0)
	{
		VMRecompilationBracket--;
	}
}

void UUAFRigVMAssetEditorData::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	// Skip any notifications we get while compiling (they can come from programmatic graph generation)
	if(bIsCompiling)
	{
		return;
	}
	
	bool bNotifForOthersPending = true;

	if (!bSuspendModelNotificationsForSelf)
	{
		switch(InNotifType)
		{
		case ERigVMGraphNotifType::InteractionBracketOpened:
			{
				IncrementVMRecompileBracket();
				break;
			}
		case ERigVMGraphNotifType::InteractionBracketClosed:
		case ERigVMGraphNotifType::InteractionBracketCanceled:
			{
				DecrementVMRecompileBracket();
				break;
			}
		case ERigVMGraphNotifType::NodeAdded:
			{
				if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
				{
					CreateEdGraphForCollapseNode(CollapseNode, false);
					break;
				}
				RequestAutoVMRecompilation();
				break;
			}
		case ERigVMGraphNotifType::NodeRemoved:
			{
				if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
				{
					bNotifForOthersPending = !RemoveEdGraphForCollapseNode(CollapseNode, true);
					break;
				}
				RequestAutoVMRecompilation();
				break;
			}
		case ERigVMGraphNotifType::NodeRenamed:
			{
				if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
				{
					FString NewNodePath = CollapseNode->GetNodePath(true /* recursive */);
					FString Left, Right = NewNodePath;
					URigVMNode::SplitNodePathAtEnd(NewNodePath, Left, Right);
					FString OldNodePath = CollapseNode->GetPreviousFName().ToString();
					if (!Left.IsEmpty())
					{
						OldNodePath = URigVMNode::JoinNodePath(Left, OldNodePath);
					}

					HandleRigVMGraphRenamed(GetRigVMClient(), OldNodePath, NewNodePath);

					if (UEdGraph* ContainedEdGraph = Cast<UEdGraph>(GetEditorObjectForRigVMGraph(CollapseNode->GetContainedGraph())))
					{
						UObject* Outer = FindEntryForRigVMGraph(CollapseNode->GetRootGraph());
						if (Outer == nullptr)
						{
							Outer = this; // function library graph has no entry
						}
						
						const FName SubGraphName = RigVMClient.GetUniqueName(Outer, *CollapseNode->GetEditorSubGraphName());
						ContainedEdGraph->Rename(*SubGraphName.ToString(), nullptr);
					}
				}
				break;
			}
		case ERigVMGraphNotifType::LinkAdded:
		case ERigVMGraphNotifType::LinkRemoved:
		case ERigVMGraphNotifType::PinArraySizeChanged:
		case ERigVMGraphNotifType::PinDirectionChanged:
			{
				RequestAutoVMRecompilation();
				break;
			}

		case ERigVMGraphNotifType::PinDefaultValueChanged:
			{
				if (InGraph->GetRuntimeAST().IsValid())
				{
					URigVMPin* RootPin = CastChecked<URigVMPin>(InSubject)->GetRootPin();
					FRigVMASTProxy RootPinProxy = FRigVMASTProxy::MakeFromUObject(RootPin);
					const FRigVMExprAST* Expression = InGraph->GetRuntimeAST()->GetExprForSubject(RootPinProxy);
					if (Expression == nullptr)
					{
						InGraph->ClearAST();
					}
					else if (Expression->NumParents() > 1)
					{
						InGraph->ClearAST();
					}
				}

				RequestAutoVMRecompilation();	// We need to rebuild our metadata when a default value changes
				break;
			}
		case ERigVMGraphNotifType::PinAdded:
			{
				// Rebind function variable pins. Needed when a function impl has new referenced vars as it's new var arg pin will be unbound.
				RefreshFunctionImplementationGraphs();

				if (URigVMPin* Pin = Cast<URigVMPin>(InSubject))
				{
					if (Pin->IsTraitPin())
					{
						RequestAutoVMRecompilation();
					}
				}
				break;
			}
		case ERigVMGraphNotifType::PinRemoved:
			{
				RequestAutoVMRecompilation(); // can not check if it is a trait pin, as it has been already removed
				break;
			}
		case ERigVMGraphNotifType::PinCategoryChanged:
		case ERigVMGraphNotifType::PinCategoriesChanged:
			{
				RequestAutoVMRecompilation();
				break;
			}
		}
	}
	
	// if the notification still has to be sent...
	if (bNotifForOthersPending && !RigVMClient.bSuspendModelNotificationsForOthers)
	{
		if (RigVMGraphModifiedEvent.IsBound())
		{
			RigVMGraphModifiedEvent.Broadcast(InNotifType, InGraph, InSubject);
		}
	}
}

TSubclassOf<UAssetUserData> UUAFRigVMAssetEditorData::GetAssetUserDataClass() const
{
	return UAnimNextAssetWorkspaceAssetUserData::StaticClass();
}

TArray<UEdGraph*> UUAFRigVMAssetEditorData::GetAllEdGraphs() const
{
	TArray<UEdGraph*> Graphs;
	for(UUAFRigVMAssetEntry* Entry : Entries)
	{
		if(IUAFRigVMGraphInterface* GraphInterface = Cast<IUAFRigVMGraphInterface>(Entry))
		{
			UEdGraph* EdGraph = GraphInterface->GetEdGraph();
			Graphs.Add(EdGraph);
			EdGraph->GetAllChildrenGraphs(Graphs);
		}
	}
	for (URigVMEdGraph* RigVMEdGraph : FunctionEdGraphs)
	{
		Graphs.Add(RigVMEdGraph);
		RigVMEdGraph->GetAllChildrenGraphs(Graphs);
	}

	return Graphs;
}

UUAFRigVMAssetEntry* UUAFRigVMAssetLibrary::FindEntry(UUAFRigVMAsset* InAsset, FName InName)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->FindEntry(InName);
}

UUAFRigVMAssetEntry* UUAFRigVMAssetEditorData::FindEntry(FName InName) const
{
	if(InName == NAME_None)
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::FindEntry: Invalid name supplied."));
		return nullptr;
	}

	const TObjectPtr<UUAFRigVMAssetEntry>* FoundEntry = Entries.FindByPredicate([InName](const UUAFRigVMAssetEntry* InEntry)
	{
		if (!InEntry)
		{
			return false;
		}

		return InEntry->GetEntryName() == InName;
	});

	return FoundEntry != nullptr ? *FoundEntry : nullptr;
}

bool UUAFRigVMAssetEditorData::AddCategory(const FString& CategoryName, bool bInSetupUndoRedo, bool bPrintPythonCommand)
{
	if (CategoryName.IsEmpty())
    {
		ReportError(*FString::Printf(TEXT("UUAFRigVMAssetEditorData::AddCategory: Invalid empty category name provided.")));
		return false;
    }

	const int32 CategoryIndex = VariableAndFunctionCategories.IndexOfByKey(CategoryName);
	if(CategoryIndex != INDEX_NONE)
	{
		ReportError(*FString::Printf(TEXT("UUAFRigVMAssetEditorData::AddCategory: Already contains category with name %s."), *CategoryName));
		return false;
	}

	if (bInSetupUndoRedo)
	{
		Modify();
	}

	VariableAndFunctionCategories.Add(CategoryName);
	BroadcastModified(EAnimNextEditorDataNotifType::CategoryAdded, this);

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
				FString::Printf(TEXT("asset.add_category('%s')"),
				 *CategoryName));
	}

	return true;
}

bool UUAFRigVMAssetEditorData::RenameCategory(const FString& OldName, const FString& NewName, bool bInSetupUndoRedo, bool bPrintPythonCommand)
{
	const int32 CategoryIndex = VariableAndFunctionCategories.IndexOfByKey(OldName);
	if (CategoryIndex == INDEX_NONE)
	{
		ReportError(*FString::Printf(TEXT("UUAFRigVMAssetEditorDataUUAFRigVMAssetEditorData::RenameCategory: Invalid existing category name provided (not found).")));
		return false;
	}
	
	if (bInSetupUndoRedo)
	{
		Modify();
	}

	VariableAndFunctionCategories[CategoryIndex] = NewName;

	for(UUAFRigVMAssetEntry* Entry : Entries)
	{
		if(UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(Entry))
		{
			if (VariableEntry->GetVariableCategory() == OldName)
			{
				VariableEntry->SetVariableCategory(NewName, bInSetupUndoRedo);
			}
		}
	}
	
	BroadcastModified(EAnimNextEditorDataNotifType::CategoryChanged, this);

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
			FString::Printf(TEXT("asset.rename_category('%s', '%s')"),
			 *OldName,
			 *NewName));
	}

	return true;
}

bool UUAFRigVMAssetEditorData::RemoveCategory(const FString CategoryName, bool bInSetupUndoRedo, bool bPrintPythonCommand)
{
	if (CategoryName.IsEmpty())
	{
		ReportError(*FString::Printf(TEXT("UUAFRigVMAssetEditorData::RemoveCategory: Invalid empty category name provided.")));
		return false;
	}

	const int32 CategoryIndex = VariableAndFunctionCategories.IndexOfByKey(CategoryName);
	if(CategoryIndex == INDEX_NONE)
	{
		ReportError(*FString::Printf(TEXT("UUAFRigVMAssetEditorData::RemoveCategory: Could not find category with name %s."), *CategoryName));
		return false;
	}

	if (bInSetupUndoRedo)
	{
		Modify();
	}

	VariableAndFunctionCategories.Remove(CategoryName);
	BroadcastModified(EAnimNextEditorDataNotifType::CategoryRemoved, this);

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
				FString::Printf(TEXT("asset.remove_category('%s')"),
				 *CategoryName));
	}

	return true;
}

void UUAFRigVMAssetEditorData::ReorderCategory(const FString& CategoryName, const FString& BeforeCategoryName, bool bInSetupUndoRedo, bool bPrintPythonCommand)
{
	const int32 CategoryIndex = VariableAndFunctionCategories.IndexOfByKey(CategoryName);
	if (CategoryIndex == INDEX_NONE)
	{
		return;
	}
	
	const int32 BeforeCategoryIndex = VariableAndFunctionCategories.IndexOfByKey(BeforeCategoryName);
	if (BeforeCategoryIndex == INDEX_NONE)
	{
		return;
	}

	if (BeforeCategoryIndex == CategoryIndex)
	{
		return;
	}

	if (bInSetupUndoRedo)
	{
		Modify();
	}

	VariableAndFunctionCategories.RemoveAt(CategoryIndex);
	
	int32 NewCategoryIndex = 0;

	// Moving up
	if (CategoryIndex > BeforeCategoryIndex)
	{
		// RemoveAt didn't have any impact
		NewCategoryIndex = FMath::Max(BeforeCategoryIndex, 0);
	}
	else
	{
		// RemoveAt means insertion index has to be offset by 1
		NewCategoryIndex = FMath::Max(BeforeCategoryIndex - 1, 0);
	}
	
	VariableAndFunctionCategories.Insert(CategoryName, NewCategoryIndex);

	BroadcastModified(EAnimNextEditorDataNotifType::CategoryChanged, this);
}

void UUAFRigVMAssetEditorData::ReorderVariable(UAnimNextVariableEntry* VariableEntry, const UAnimNextVariableEntry* BeforeVariableEntry, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	const int32 VariableEntryIndex = Entries.IndexOfByKey(VariableEntry);
	if (VariableEntryIndex == INDEX_NONE)
	{
		return;
	}
	
	const int32 BeforeVariableEntryIndex = Entries.IndexOfByKey(BeforeVariableEntry);
	if (BeforeVariableEntryIndex == INDEX_NONE)
	{
		return;
	}

	if (BeforeVariableEntryIndex == VariableEntryIndex)
	{
		return;
	}

	if (bSetupUndoRedo)
	{
		Modify();
	}

	RemoveEntryInternal(VariableEntry);

	int32 NewVariableEntryIndex = 0;

	// Moving up
	if (VariableEntryIndex > BeforeVariableEntryIndex)
	{
		// RemoveAt didn't have any impact
		NewVariableEntryIndex = FMath::Max(BeforeVariableEntryIndex, 0);
	}
	else
	{
		// RemoveAt means insertion index has to be offset by 1
		NewVariableEntryIndex = FMath::Max(BeforeVariableEntryIndex - 1, 0);
	}
	
	InsertEntryInternal(VariableEntry, NewVariableEntryIndex);

	BroadcastModified(EAnimNextEditorDataNotifType::EntryAdded, VariableEntry);
}

bool UUAFRigVMAssetLibrary::RemoveEntry(UUAFRigVMAsset* InAsset, UUAFRigVMAssetEntry* InEntry,  bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->RemoveEntry(InEntry, bSetupUndoRedo, bPrintPythonCommand);
}

bool UUAFRigVMAssetEditorData::RemoveEntry(UUAFRigVMAssetEntry* InEntry, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(InEntry == nullptr)
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::RemoveEntry: Invalid entry supplied."));
		return false;
	}
	
	TObjectPtr<UUAFRigVMAssetEntry>* EntryToRemovePtr = Entries.FindByKey(InEntry);
	if(EntryToRemovePtr == nullptr)
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::RemoveEntry: Asset does not contain the supplied entry."));
		return false;
	}

	if(bSetupUndoRedo)
	{
		Modify();
	}

	// Remove from internal array
	UUAFRigVMAssetEntry* EntryToRemove = *EntryToRemovePtr;

	bool bResult = true;
	if(const IUAFRigVMGraphInterface* GraphInterface = Cast<IUAFRigVMGraphInterface>(EntryToRemove))
	{
		// Remove any graphs
		if(URigVMGraph* RigVMGraph = GraphInterface->GetRigVMGraph())
		{
			TGuardValue<bool> EnablePythonPrint(bSuspendPythonMessagesForRigVMClient, !bPrintPythonCommand);
			TGuardValue<bool> DisableAutoCompile(bAutoRecompileVM, false);
			bResult = RigVMClient.RemoveModel(RigVMGraph->GetNodePath(), bSetupUndoRedo);
		}
	}

	if (bSetupUndoRedo)
	{
		EntryToRemove->Modify();
	}
	RemoveEntryInternal(EntryToRemove);
	RefreshExternalModels();

	// This will cause any external package to be removed when saved
	EntryToRemove->MarkAsGarbage();

	BroadcastModified(EAnimNextEditorDataNotifType::EntryRemoved, this);

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
				FString::Printf(TEXT("asset.remove_entry(asset.find_entry('%s'))"),
				*InEntry->GetEntryName().ToString()));
	}

	return bResult;
}

bool UUAFRigVMAssetLibrary::RemoveEntries(UUAFRigVMAsset* InAsset, const TArray<UUAFRigVMAssetEntry*>& InEntries,  bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->RemoveEntries(InEntries, bSetupUndoRedo, bPrintPythonCommand);
}

bool UUAFRigVMAssetEditorData::RemoveEntries(TConstArrayView<UUAFRigVMAssetEntry*> InEntries, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	bool bResult = false;
	{
		TGuardValue<bool> DisableEditorDataNotifications(bSuspendEditorDataNotifications, true);
		TGuardValue<bool> DisableAutoCompile(bAutoRecompileVM, false);
		for(UUAFRigVMAssetEntry* Entry : InEntries)
		{
			bResult |= RemoveEntry(Entry, bSetupUndoRedo, false);
		}
	}

	BroadcastModified(EAnimNextEditorDataNotifType::EntryRemoved, this);

	if (bPrintPythonCommand)
	{

		FString ArrayStr = TEXT("[");
		for (int32 Index = 0; Index < InEntries.Num(); ++Index)
		{
			ArrayStr += TEXT("asset.find_entry('") + InEntries[Index]->GetEntryName().ToString() + TEXT("')");
			if (Index < InEntries.Num() - 1)
			{
				ArrayStr += TEXT(", ");
			}
		}
		ArrayStr += TEXT("]");


		RigVMPythonUtils::Print(GetName(), 
							FString::Printf(TEXT("asset.remove_entries(%s)"),
											*ArrayStr));
	}

	return bResult;
}

bool UUAFRigVMAssetLibrary::RemoveAllEntries(UUAFRigVMAsset* InAsset, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->RemoveAllEntries(bSetupUndoRedo, bPrintPythonCommand);
}

bool UUAFRigVMAssetEditorData::RemoveAllEntries(bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	bool bResult = false;
	{
		TGuardValue<bool> DisableEditorDataNotifications(bSuspendEditorDataNotifications, true);
		TGuardValue<bool> DisableAutoCompile(bAutoRecompileVM, false);
		TArray<UUAFRigVMAssetEntry*> EntriesCopy = Entries; 
		for(UUAFRigVMAssetEntry* Entry : EntriesCopy)
		{
			bResult |= RemoveEntry(Entry, bSetupUndoRedo, false);
		}
	}

	BroadcastModified(EAnimNextEditorDataNotifType::EntryRemoved, this);


	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(), 
							FString::Printf(TEXT("asset.remove_all_entries()")));
	}

	return bResult;
}

FAnimNextGetFunctionHeaderCompileContext UUAFRigVMAssetEditorData::GetFunctionHeaderContext(const FRigVMCompileSettings& InSettings, FAnimNextRigVMAssetCompileContext& InCompileContext) const
{
	// Since the compile context is a reference it must be set in advance by the caller. If we created it we would return a ref to a temp.
	ensureMsgf(this == InCompileContext.OwningAssetEditorData, TEXT("Crossing RigVM asset with a different asset's compile context. Expect incorrect memory layout / variable generation"));

	FAnimNextGetFunctionHeaderCompileContext GetFunctionHeaderCompileContext(InCompileContext);
	BuildFunctionHeadersContext(InSettings, GetFunctionHeaderCompileContext);
	return GetFunctionHeaderCompileContext;
}

FAnimNextGetVariableCompileContext UUAFRigVMAssetEditorData::GetVariableCompileContext(const FRigVMCompileSettings& InSettings, FAnimNextRigVMAssetCompileContext& InCompileContext) const
{
	// Since the compile context is a reference it must be set in advance by the caller. If we created it we would return a ref to a temp.
	ensureMsgf(this == InCompileContext.OwningAssetEditorData, TEXT("Crossing RigVM asset with a different asset's compile context. Expect incorrect memory layout / variable generation"));

	{
		FAnimNextGetFunctionHeaderCompileContext GetFunctionHeaderCompileContext(InCompileContext);
		BuildFunctionHeadersContext(InSettings, GetFunctionHeaderCompileContext);
	}

	BuildFunctionWrapperEventVariables(InCompileContext);

	FAnimNextGetVariableCompileContext GetVariableCompileContext(InCompileContext);
	BuildProgrammaticVariablesContext(InSettings, GetVariableCompileContext);
	return GetVariableCompileContext;
}

UObject* UUAFRigVMAssetEditorData::CreateNewSubEntry(UUAFRigVMAssetEditorData* InEditorData, TSubclassOf<UObject> InClass)
{
	UObject* NewEntry = NewObject<UObject>(InEditorData, InClass.Get(), NAME_None, RF_Transactional);
	// If we are a transient asset, dont use external packages
	UUAFRigVMAsset* Asset = UE::UAF::UncookedOnly::FUtils::GetAsset(InEditorData);
	check(Asset);

	return NewEntry;
}

UUAFRigVMAssetEntry* UUAFRigVMAssetEditorData::FindEntryForRigVMGraph(const URigVMGraph* InRigVMGraph) const
{
	for(UUAFRigVMAssetEntry* Entry : Entries)
	{
		if(IUAFRigVMGraphInterface* GraphInterface = Cast<IUAFRigVMGraphInterface>(Entry))
		{
			if (const URigVMGraph* RigVMGraph = GraphInterface->GetRigVMGraph())
			{
				if(RigVMGraph == InRigVMGraph)
				{
					return Entry;
				}
			}
		}
	}

	return nullptr;
}

UUAFRigVMAssetEntry* UUAFRigVMAssetEditorData::FindEntryForRigVMEdGraph(const URigVMEdGraph* InRigVMEdGraph) const
{
	for (UUAFRigVMAssetEntry* Entry : Entries)
	{
		if (IUAFRigVMGraphInterface* GraphInterface = Cast<IUAFRigVMGraphInterface>(Entry))
		{
			if (GraphInterface->GetEdGraph() == InRigVMEdGraph)
			{
				return Entry;
			}
		}
	}

	return nullptr;
}

void UUAFRigVMAssetEditorData::CreateEdGraphForCollapseNode(URigVMCollapseNode* InNode, bool bForce)
{
	check(InNode);
	URigVMGraph* CollapseNodeGraph = InNode->GetGraph();
	check(CollapseNodeGraph);

	if (bForce)
	{
		RemoveEdGraphForCollapseNode(InNode, false);
	}

	// For Function node
	if (InNode->GetGraph()->IsA<URigVMFunctionLibrary>())
	{
		if (URigVMGraph* ContainedGraph = InNode->GetContainedGraph())
		{
			bool bFunctionGraphExists = false;
			for (UEdGraph* FunctionGraph : FunctionEdGraphs)
			{
				if (URigVMEdGraph* RigFunctionGraph = Cast<URigVMEdGraph>(FunctionGraph))
				{
					if (RigFunctionGraph->ModelNodePath == ContainedGraph->GetNodePath())
					{
						bFunctionGraphExists = true;
						break;
					}
				}
			}

			if (!bFunctionGraphExists)
			{
				const FName SubGraphName = RigVMClient.GetUniqueName(this, *InNode->GetName());
				// create a sub graph
				UAnimNextEdGraph* RigFunctionGraph = NewObject<UAnimNextEdGraph>(this, SubGraphName, RF_Transactional);
				RigFunctionGraph->Schema = UAnimNextEdGraphSchema::StaticClass();
				RigFunctionGraph->bAllowRenaming = true;
				RigFunctionGraph->bEditable = true;
				RigFunctionGraph->bAllowDeletion = true;
				RigFunctionGraph->ModelNodePath = ContainedGraph->GetNodePath();
				RigFunctionGraph->bIsFunctionDefinition = true;

				RigFunctionGraph->Initialize(this);

				FunctionEdGraphs.Add(RigFunctionGraph);

				RigVMClient.GetOrCreateController(ContainedGraph)->ResendAllNotifications();
			}
		}
	}
	// --- For Collapse nodes ---
	else if (URigVMEdGraph* RigEdGraph = Cast<URigVMEdGraph>(GetEditorObjectForRigVMGraph(InNode->GetGraph())))
	{
		if (URigVMGraph* ContainedGraph = InNode->GetContainedGraph())
		{
			bool bSubGraphExists = false;

			const FString ContainedGraphNodePath = ContainedGraph->GetNodePath();
			for (UEdGraph* SubGraph : RigEdGraph->SubGraphs)
			{
				if (UAnimNextEdGraph* SubRigGraph = Cast<UAnimNextEdGraph>(SubGraph))
				{
					if (SubRigGraph->ModelNodePath == ContainedGraphNodePath)
					{
						bSubGraphExists = true;
						break;
					}
				}
			}

			if (!bSubGraphExists)
			{
				bool bEditable = true;
				if (InNode->IsA<URigVMAggregateNode>())
				{
					bEditable = false;
				}

				UObject* Outer = FindEntryForRigVMGraph(CollapseNodeGraph->GetRootGraph());
				if (Outer == nullptr)
				{
					Outer = this; // function library graph has no entry
				}

				const FName SubGraphName = RigVMClient.GetUniqueName(Outer, *InNode->GetEditorSubGraphName());
				// create a sub graph, no need to set external package if outer is an Entry
				UAnimNextEdGraph* SubRigGraph = NewObject<UAnimNextEdGraph>(Outer, SubGraphName, RF_Transactional);
				SubRigGraph->Schema = UAnimNextEdGraphSchema::StaticClass();
				SubRigGraph->bAllowRenaming = 1;
				SubRigGraph->bEditable = bEditable;
				SubRigGraph->bAllowDeletion = 1;
				SubRigGraph->ModelNodePath = ContainedGraphNodePath;
				SubRigGraph->bIsFunctionDefinition = false;

				RigEdGraph->SubGraphs.Add(SubRigGraph);

				SubRigGraph->Initialize(this);

				GetOrCreateController(ContainedGraph)->ResendAllNotifications();
			}
		}
	}
}

bool UUAFRigVMAssetEditorData::RemoveEdGraphForCollapseNode(URigVMCollapseNode* InNode, bool bNotify)
{
	check(InNode);

	if (InNode->GetGraph()->IsA<URigVMFunctionLibrary>())
	{
		if (URigVMGraph* ContainedGraph = InNode->GetContainedGraph())
		{
			for (UEdGraph* FunctionGraph : FunctionEdGraphs)
			{
				if (URigVMEdGraph* RigFunctionGraph = Cast<URigVMEdGraph>(FunctionGraph))
				{
					if (RigFunctionGraph->ModelNodePath == ContainedGraph->GetNodePath())
					{
						if (URigVMController* SubController = GetController(ContainedGraph))
						{
							SubController->OnModified().RemoveAll(RigFunctionGraph);
						}

						if (RigVMGraphModifiedEvent.IsBound() && bNotify)
						{
							RigVMGraphModifiedEvent.Broadcast(ERigVMGraphNotifType::NodeRemoved, InNode->GetGraph(), InNode);
						}

						FunctionEdGraphs.Remove(RigFunctionGraph);
						RigFunctionGraph->Rename(nullptr, GetTransientPackage(), REN_AllowPackageLinkerMismatch | REN_DontCreateRedirectors);
						RigFunctionGraph->MarkAsGarbage();
						return bNotify;
					}
				}
			}
		}
	}
	else if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GetEditorObjectForRigVMGraph(InNode->GetGraph())))
	{
		if (URigVMGraph* ContainedGraph = InNode->GetContainedGraph())
		{
			for (UEdGraph* SubGraph : RigGraph->SubGraphs)
			{
				if (URigVMEdGraph* SubRigGraph = Cast<URigVMEdGraph>(SubGraph))
				{
					if (SubRigGraph->ModelNodePath == ContainedGraph->GetNodePath())
					{
						if (URigVMController* SubController = GetController(ContainedGraph))
						{
							SubController->OnModified().RemoveAll(SubRigGraph);
						}

						if (RigVMGraphModifiedEvent.IsBound() && bNotify)
						{
							RigVMGraphModifiedEvent.Broadcast(ERigVMGraphNotifType::NodeRemoved, InNode->GetGraph(), InNode);
						}

						RigGraph->SubGraphs.Remove(SubRigGraph);
						SubRigGraph->Rename(nullptr, GetTransientPackage(), REN_AllowPackageLinkerMismatch | REN_DontCreateRedirectors);
						SubRigGraph->MarkAsGarbage();
						return bNotify;
					}
				}
			}
		}
	}

	return false;
}

UEdGraph* UUAFRigVMAssetEditorData::CreateEdGraph(URigVMGraph* InRigVMGraph, bool bForce)
{
	check(InRigVMGraph);

	if(InRigVMGraph->IsA<URigVMFunctionLibrary>())
	{
		return nullptr;
	}

	const bool bIsTransient = InRigVMGraph->HasAnyFlags(RF_Transient);
	IUAFRigVMGraphInterface* Entry = Cast<IUAFRigVMGraphInterface>(FindEntryForRigVMGraph(InRigVMGraph));
	if(Entry == nullptr && !bIsTransient)
	{
		// Not found, we could be adding a new entry, in which case the graph wont be assigned yet
		check(Entries.Num() > 0);
		check(Cast<IUAFRigVMGraphInterface>(Entries.Last()) != nullptr);
		check(Cast<IUAFRigVMGraphInterface>(Entries.Last())->GetRigVMGraph() == nullptr);
		Entry = Cast<IUAFRigVMGraphInterface>(FindEntryForRigVMGraph(nullptr));
	}

	if(Entry == nullptr && !bIsTransient)
	{
		return nullptr;
	}
	
	if(bForce)
	{
		RemoveEdGraph(InRigVMGraph);
	}

	UObject* Outer = nullptr;
	EObjectFlags Flags = RF_NoFlags;
	if(!bIsTransient)
	{
		Outer = CastChecked<UObject>(Entry);
		Flags = RF_Transactional;
	}
	else
	{
		// This outer is to allow URigVMEdGraph::GetModel to retrieve the graph in 'preview' scenarios 
		Outer = InRigVMGraph;
		Flags = RF_Transient;
	}

	const FName GraphName = Entry != nullptr ? RigVMClient.GetUniqueName(Outer, Entry->GetGraphName()) : NAME_None;
	UAnimNextEdGraph* RigFunctionGraph = NewObject<UAnimNextEdGraph>(Outer, GraphName, Flags);
	RigFunctionGraph->Schema = UAnimNextEdGraphSchema::StaticClass();
	RigFunctionGraph->bAllowDeletion = true;
	RigFunctionGraph->bIsFunctionDefinition = false;
	RigFunctionGraph->ModelNodePath = InRigVMGraph->GetNodePath();
	RigFunctionGraph->Initialize(this);

	if(!bIsTransient)
	{
		Entry->SetEdGraph(RigFunctionGraph);
		if(Entry->GetRigVMGraph() == nullptr)
		{
			Entry->SetRigVMGraph(InRigVMGraph);
		}
		else
		{
			check(Entry->GetRigVMGraph() == InRigVMGraph);
		}
	}

	return RigFunctionGraph;
}

bool UUAFRigVMAssetEditorData::RemoveEdGraph(URigVMGraph* InModel)
{
	if(IUAFRigVMGraphInterface* Entry = Cast<IUAFRigVMGraphInterface>(FindEntryForRigVMGraph(InModel)))
	{
		RigVMClient.DestroyObject(Entry->GetEdGraph());
		Entry->SetEdGraph(nullptr);
		return true;
	}
	return false;
}

UAnimNextVariableEntry* UUAFRigVMAssetLibrary::AddVariable(UUAFRigVMAsset* InAsset, FName InName, EPropertyBagPropertyType InValueType,
	EPropertyBagContainerType InContainerType, const UObject* InValueTypeObject, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->AddVariable(InName, FAnimNextParamType(InValueType, InContainerType, InValueTypeObject), InDefaultValue, bSetupUndoRedo, bPrintPythonCommand);
}

UAnimNextVariableEntry* UUAFRigVMAssetEditorData::AddVariable(FName InName, FAnimNextParamType InType, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	using namespace UE::UAF::UncookedOnly;

	if(InName == NAME_None)
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddVariable: Invalid variable name supplied."));
		return nullptr;
	}

	if(!GetEntryClasses().Contains(UAnimNextVariableEntry::StaticClass()) || !CanAddNewEntry(UAnimNextVariableEntry::StaticClass()))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddVariable: Cannot add a variable to this asset - entry is not allowed."));
		return nullptr;
	}
	
	if (!InType.IsValid())
	{
		ReportError(*FString::Printf(TEXT("UUAFRigVMAssetEditorData::AddVariable: Cannot add a variable to this asset - type (%s) is not valid."), *InType.ToString()));
		return nullptr;
	}
	
	if (InType.ToRigVMTemplateArgument().IsUnknownType() || InType.ToRigVMTemplateArgument().IsWildCard())
	{
		ReportError(*FString::Printf(TEXT("UUAFRigVMAssetEditorData::AddVariable: Cannot add a variable to this asset - type (%s) is not supported by RigVM."), *InType.ToString()));
		return nullptr;
	}

	// Check for duplicate name
	FName NewParameterName = InName;
	auto DuplicateNamePredicate = [&NewParameterName](const UUAFRigVMAssetEntry* InEntry)
	{
		if (!InEntry)
		{
			return false;
		}

		return InEntry->GetEntryName() == NewParameterName;
	};

	// We are about to perform multiple changes, add a scope now to avoid multiple compile broadcasts per change
	UUAFRigVMAsset* Asset = FUtils::GetAsset<UUAFRigVMAsset>(this);
	FCompilationScope CompileScope(LOCTEXT("ModifiedAssetsAddVariable", "Add Variable"), Asset);

	bool bAlreadyExists = Entries.ContainsByPredicate(DuplicateNamePredicate);
	int32 NameNumber = InName.GetNumber() + 1;
	while(bAlreadyExists)
	{
		NewParameterName = FName(InName, NameNumber++);
		bAlreadyExists = Entries.ContainsByPredicate(DuplicateNamePredicate);
	}

	UAnimNextVariableEntry* NewEntry = CreateNewSubEntry<UAnimNextVariableEntry>(this);
	{
		TGuardValue<bool> DisableEditorDataNotifications(bSuspendEditorDataNotifications, true);
		TGuardValue<bool> DisableAutoCompile(bAutoRecompileVM, false);

		NewEntry->SetVariableName(NewParameterName, false);
		NewEntry->SetType(InType, false);
		NewEntry->SetExportAccessSpecifier(EAnimNextExportAccessSpecifier::Public, false);
		if(InDefaultValue.Len() > 0)
		{
			NewEntry->SetDefaultValueFromString(InDefaultValue, false);
		}

		NewEntry->Initialize(this);
	}

	if(bSetupUndoRedo)
	{
		NewEntry->Modify();
		Modify();
	}

	AddEntryInternal(NewEntry);

	CustomizeNewAssetEntry(NewEntry);

	BroadcastModified(EAnimNextEditorDataNotifType::EntryAdded, NewEntry);

	if (bPrintPythonCommand)
	{
		const FString ValueTypeString = InType.GetValueTypeObject() ?
				FString::Printf(TEXT("unreal.%s.static_%s()"), *InType.GetValueTypeObject()->GetName(), InType.GetValueTypeObject()->IsA<UScriptStruct>() ? TEXT("struct") : TEXT("class"))
				: TEXT("None");
		RigVMPythonUtils::Print(GetName(), 
							FString::Printf(TEXT("asset.add_variable('%s', %s, %s, %s, '%s')"),
											*InName.ToString(),
											*RigVMPythonUtils::EnumValueToPythonString<EPropertyBagPropertyType>(static_cast<int64>(InType.GetValueType())),
											*RigVMPythonUtils::EnumValueToPythonString<EPropertyBagContainerType>(static_cast<int64>(InType.GetContainerType())),
											*ValueTypeString,
											*InDefaultValue));
	}

	return NewEntry;
}

UAnimNextEventGraphEntry* UUAFRigVMAssetLibrary::AddEventGraph(UUAFRigVMAsset* InAsset, FName InName, UScriptStruct* InEventStruct, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->AddEventGraph(InName, InEventStruct, bSetupUndoRedo, bPrintPythonCommand);
}

UAnimNextEventGraphEntry* UUAFRigVMAssetEditorData::AddEventGraph(FName InName, UScriptStruct* InEventStruct, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	using namespace UE::UAF::UncookedOnly;

	if(InName == NAME_None)
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddEventGraph: Invalid graph name supplied."));
		return nullptr;
	}

	if(InEventStruct == nullptr || !InEventStruct->IsChildOf(FRigVMStruct::StaticStruct()))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddEventGraph: Invalid event struct name supplied."));
		return nullptr;
	}

	if(!GetEntryClasses().Contains(UAnimNextEventGraphEntry::StaticClass()) || !CanAddNewEntry(UAnimNextEventGraphEntry::StaticClass()))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddEventGraph: Cannot add an event graph to this asset - entry is not allowed."));
		return nullptr;
	}

	// Check for duplicate name
	FName NewGraphName = InName;
	auto DuplicateNamePredicate = [&NewGraphName](const UUAFRigVMAssetEntry* InEntry)
	{
		return InEntry->GetEntryName() == NewGraphName;
	};

	bool bAlreadyExists = Entries.ContainsByPredicate(DuplicateNamePredicate);
	int32 NameNumber = InName.GetNumber() + 1;
	while(bAlreadyExists)
	{
		NewGraphName = FName(InName, NameNumber++);
		bAlreadyExists =  Entries.ContainsByPredicate(DuplicateNamePredicate);
	}

	// We are about to perform multiple changes, add a scope now to avoid multiple compile broadcasts per change
	UUAFRigVMAsset* Asset = FUtils::GetAsset<UUAFRigVMAsset>(this);
	FCompilationScope CompileScope(LOCTEXT("ModifiedAssetsAddEventGraph", "Add Event Graph"), Asset);

	UAnimNextEventGraphEntry* NewEntry = CreateNewSubEntry<UAnimNextEventGraphEntry>(this);
	NewEntry->GraphName = NewGraphName;
	NewEntry->Initialize(this);

	if(bSetupUndoRedo)
	{
		NewEntry->Modify();
		Modify();
	}

	AddEntryInternal(NewEntry);

	// Add new graph
	{
		TGuardValue<bool> EnablePythonPrint(bSuspendPythonMessagesForRigVMClient, !bPrintPythonCommand);
		TGuardValue<bool> DisableAutoCompile(bAutoRecompileVM, false);
		// Editor data has to be the graph outer, or RigVM unique name generator will not work
		URigVMGraph* NewRigVMGraphModel = RigVMClient.CreateModel(URigVMGraph::StaticClass()->GetFName(), UAnimNextEventGraphSchema::StaticClass(), bSetupUndoRedo, this);
		if (ensure(NewRigVMGraphModel))
		{
			// Then, to avoid the graph losing ref due to external package, set the same package as the Entry
			if (!NewRigVMGraphModel->HasAnyFlags(RF_Transient))
			{
				NewRigVMGraphModel->SetExternalPackage(CastChecked<UObject>(NewEntry)->GetExternalPackage());
			}
			ensure(NewRigVMGraphModel);
			NewEntry->Graph = NewRigVMGraphModel;

			RefreshExternalModels();
			RigVMClient.AddModel(NewRigVMGraphModel, true);
			URigVMController* Controller = RigVMClient.GetController(NewRigVMGraphModel);
			UE::UAF::UncookedOnly::FUtils::SetupEventGraph(Controller, InEventStruct, NewGraphName);
		}
	}

	CustomizeNewAssetEntry(NewEntry);

	BroadcastModified(EAnimNextEditorDataNotifType::EntryAdded, NewEntry);

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
				FString::Printf(TEXT("asset.add_event_graph('%s', unreal.%s)"),
				*InName.ToString(), *InEventStruct->GetName()));
	}

	return NewEntry;
}

UUAFSharedVariablesEntry* UUAFRigVMAssetLibrary::AddSharedVariables(UUAFRigVMAsset* InAsset, UUAFSharedVariables* InSharedVariables, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->AddSharedVariables(InSharedVariables, bSetupUndoRedo, bPrintPythonCommand);
}

UUAFSharedVariablesEntry* UUAFRigVMAssetEditorData::AddSharedVariables(const UUAFSharedVariables* InSharedVariables, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	using namespace UE::UAF::UncookedOnly;

	if(InSharedVariables == nullptr)
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariables: Invalid asset supplied."));
		return nullptr;
	}

	if(!GetEntryClasses().Contains(UUAFSharedVariablesEntry::StaticClass()) || !CanAddNewEntry(UUAFSharedVariablesEntry::StaticClass()))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariables: Cannot add a shared variables to this asset - entry is not allowed."));
		return nullptr;
	}
	
	// Check if interface has any public members or if any of its parent interfaces do
	UUAFSharedVariables_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFSharedVariables_EditorData>(InSharedVariables);
	if(EditorData == nullptr)
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariables: Invalid asset supplied - asset has no editor data."));
		return nullptr;
	}

	// Check for circularity
	if(!FUtils::CanAddSharedVariablesReference(this, InSharedVariables))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariables: Circular reference detected."));
		return nullptr;
	}

	// Check for duplicate entry
	auto DuplicatePredicate = [InSharedVariables](const UUAFRigVMAssetEntry* InEntry)
	{
		if(const UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(InEntry))
		{
			return SharedVariablesEntry->GetAsset() == InSharedVariables;
		}
		return false;
	};

	if(Entries.ContainsByPredicate(DuplicatePredicate))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariables: Shared variables already referenced."));
		return nullptr;
	}

	// We are about to perform multiple changes, add a scope now to avoid multiple compile broadcasts per change
	UUAFRigVMAsset* Asset = FUtils::GetAsset<UUAFRigVMAsset>(this);
	FCompilationScope CompileScope(LOCTEXT("ModifiedAssetsAddSharedVariables", "Add Shared Variables"), Asset);

	UUAFSharedVariablesEntry* NewEntry = CreateNewSubEntry<UUAFSharedVariablesEntry>(this);
	NewEntry->SetAsset(InSharedVariables);
	NewEntry->Initialize(this);

	if(bSetupUndoRedo)
	{
		NewEntry->Modify();
		Modify();
	}

	AddEntryInternal(NewEntry);

	CustomizeNewAssetEntry(NewEntry);

	BroadcastModified(EAnimNextEditorDataNotifType::EntryAdded, NewEntry);

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
				FString::Printf(TEXT("asset.add_shared_variables(unreal.find_object(outer=None, name='%s'))"),
				 *InSharedVariables->GetPathName()));
	}

	return NewEntry;
}


UUAFSharedVariablesEntry* UUAFRigVMAssetLibrary::AddSharedVariablesStruct(UUAFRigVMAsset* InAsset, UScriptStruct* InStruct, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->AddSharedVariablesStruct(InStruct, bSetupUndoRedo, bPrintPythonCommand);
}

UUAFSharedVariablesEntry* UUAFRigVMAssetEditorData::AddSharedVariablesStruct(const UScriptStruct* InStruct, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	using namespace UE::UAF::UncookedOnly;

	if(InStruct == nullptr)
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariablesStruct: Invalid struct supplied."));
		return nullptr;
	}

	if(!GetEntryClasses().Contains(UUAFSharedVariablesEntry::StaticClass()) || !CanAddNewEntry(UUAFSharedVariablesEntry::StaticClass()))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariablesStruct: Cannot add a shared variables struct to this asset - entry is not allowed."));
		return nullptr;
	}

	// Check for duplicate entry
	auto DuplicatePredicate = [InStruct](const UUAFRigVMAssetEntry* InEntry)
	{
		if(const UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(InEntry))
		{
			return SharedVariablesEntry->GetStruct() == InStruct;
		}
		return false;
	};

	if(Entries.ContainsByPredicate(DuplicatePredicate))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariables: Shared variables struct already referenced."));
		return nullptr;
	}

	// We are about to perform multiple changes, add a scope now to avoid multiple compile broadcasts per change
	UUAFRigVMAsset* Asset = FUtils::GetAsset<UUAFRigVMAsset>(this);
	FCompilationScope CompileScope(LOCTEXT("ModifiedAssetsAddSharedVariablesStruct", "Add Shared Variables Struct"),  Asset);

	UUAFSharedVariablesEntry* NewEntry = CreateNewSubEntry<UUAFSharedVariablesEntry>(this);
	NewEntry->SetStruct(InStruct);
	NewEntry->Initialize(this);

	if(bSetupUndoRedo)
	{
		NewEntry->Modify();
		Modify();
	}

	AddEntryInternal(NewEntry);

	CustomizeNewAssetEntry(NewEntry);

	BroadcastModified(EAnimNextEditorDataNotifType::EntryAdded, NewEntry);

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
				FString::Printf(TEXT("asset.add_shared_variables_struct(unreal.find_object(outer=None, name='%s'))"),
				 *InStruct->GetPathName()));
	}

	return NewEntry;
}

UUAFSharedVariablesEntry* UUAFRigVMAssetEditorData::AddSharedVariablesRigVMAsset(const IRigVMRuntimeAssetInterface* InRigVMAsset, bool bSetupUndoRedo,
	bool bPrintPythonCommand)
{
		using namespace UE::UAF::UncookedOnly;

	if(InRigVMAsset == nullptr)
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariablesRigVMAsset: Invalid asset supplied."));
		return nullptr;
	}

	if(!GetEntryClasses().Contains(UUAFSharedVariablesEntry::StaticClass()) || !CanAddNewEntry(UUAFSharedVariablesEntry::StaticClass()))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariables: Cannot add a shared variables to this asset - entry is not allowed."));
		return nullptr;
	}

	// Check for duplicate entry
	auto DuplicatePredicate = [InRigVMAsset](const UUAFRigVMAssetEntry* InEntry)
	{
		if(const UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(InEntry))
		{
			return SharedVariablesEntry->GetRigVMAsset() == InRigVMAsset;
		}
		return false;
	};

	if(Entries.ContainsByPredicate(DuplicatePredicate))
	{
		ReportError(TEXT("UUAFRigVMAssetEditorData::AddSharedVariables: Shared variables already referenced."));
		return nullptr;
	}

	// We are about to perform multiple changes, add a scope now to avoid multiple compile broadcasts per change
	UUAFRigVMAsset* Asset = FUtils::GetAsset<UUAFRigVMAsset>(this);
	FCompilationScope CompileScope(LOCTEXT("ModifiedAssetsAddSharedVariables", "Add Shared Variables"), Asset);

	UUAFSharedVariablesEntry* NewEntry = CreateNewSubEntry<UUAFSharedVariablesEntry>(this);
	NewEntry->SetRigVMAsset(InRigVMAsset);
	NewEntry->Initialize(this);

	if(bSetupUndoRedo)
	{
		NewEntry->Modify();
		Modify();
	}

	AddEntryInternal(NewEntry);

	CustomizeNewAssetEntry(NewEntry);

	BroadcastModified(EAnimNextEditorDataNotifType::EntryAdded, NewEntry);

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
				FString::Printf(TEXT("asset.add_shared_variables(unreal.find_object(outer=None, name='%s'))"),
				 *(Cast<UObject>(InRigVMAsset)->GetPathName())));
	}

	return NewEntry;
}

URigVMLibraryNode* UUAFRigVMAssetLibrary::AddFunction(UUAFRigVMAsset* InAsset, FName InFunctionName, bool bInMutable, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->AddFunction(InFunctionName, bInMutable, bSetupUndoRedo, bPrintPythonCommand);
}

bool UUAFRigVMAssetLibrary::AddCategory(UUAFRigVMAsset* InAsset, const FString& CategoryName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->AddCategory(CategoryName, bSetupUndoRedo, bPrintPythonCommand);
}

bool UUAFRigVMAssetLibrary::RenameCategory(UUAFRigVMAsset* InAsset, const FString& CategoryName, const FString& NewCategoryName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->RenameCategory(CategoryName, NewCategoryName, bSetupUndoRedo, bPrintPythonCommand);
}

bool UUAFRigVMAssetLibrary::RemoveCategory(UUAFRigVMAsset* InAsset, const FString& CategoryName, bool bSetupUndoRedo,
	bool bPrintPythonCommand)
{
	return UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset)->RemoveCategory(CategoryName, bSetupUndoRedo, bPrintPythonCommand);
}

bool UUAFRigVMAssetLibrary::RenameVariable(UUAFRigVMAsset* InAsset, FName InVariableName, FName InNewName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	UUAFRigVMAssetEditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData(InAsset);
	if (UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(EditorData->FindEntry(InVariableName)))
	{
		UE::UAF::UncookedOnly::FUtils::RenameVariable(VariableEntry, InNewName, bSetupUndoRedo, bPrintPythonCommand);
		return true;
	}

	UUAFRigVMAssetEditorData::ReportError(*FString::Printf(TEXT("UUAFRigVMAssetLibrary::RenameVariable: Variable '%s' not found."), *InVariableName.ToString()));
	return false;
}

URigVMLibraryNode* UUAFRigVMAssetEditorData::AddFunction(FName InFunctionName, bool bInMutable, bool bInSetupUndoRedo, bool bPrintPythonCommand)
{
	URigVMController* Controller = RigVMClient.GetOrCreateController(GetLocalFunctionLibrary());
	URigVMLibraryNode* Node = Controller->AddFunctionToLibrary(InFunctionName, bInMutable, FVector2D::ZeroVector, bInSetupUndoRedo, false);
	
	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetName(),
				FString::Printf(TEXT("asset.add_function('%s', %s)"),
				 *InFunctionName.ToString(),
				 bInMutable ? TEXT("True") : TEXT("False")));
	}

	return Node;
}

bool UUAFRigVMAssetEditorData::HasPublicVariables() const
{
	for(UUAFRigVMAssetEntry* Entry : Entries)
	{
		if(UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(Entry))
		{
			if(VariableEntry->GetExportAccessSpecifier() == EAnimNextExportAccessSpecifier::Public)
			{
				return true;
			}
		}
		else if(UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(Entry))
		{
			if(SharedVariablesEntry->GetAsset())
			{
				UUAFSharedVariables_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFSharedVariables_EditorData>(SharedVariablesEntry->GetAsset());
				return EditorData->HasPublicVariables();
			}
		}
	}
	return false;
}

void UUAFRigVMAssetEditorData::GetAllVariables(TArray<FVariableInfo>& OutVariables, EVariableRecursion InRecursion, EVariableAccessFilter InAccess) const
{
	for(UUAFRigVMAssetEntry* Entry : Entries)
	{
		if(UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(Entry))
		{
			if (InAccess == EVariableAccessFilter::All ||
				VariableEntry->GetExportAccessSpecifier() == EAnimNextExportAccessSpecifier::Public)
			{
				const UUAFRigVMAsset* Asset = UE::UAF::UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(this);

				const FProperty* Property;
				TConstArrayView<uint8> Value;
				VariableEntry->GetDefaultValue(Property, Value);
				
				OutVariables.Add(
					{
						VariableEntry->GetExportName(),
						VariableEntry->GetType(),
						Asset,
						VariableEntry->GetExportAccessSpecifier(),
						Property,
						Value,
						VariableEntry->GetGuid()
					});
			}
		}
		else if(UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(Entry))
		{
			if (InRecursion == EVariableRecursion::SelfOnly)
			{
				continue;
			}

			switch (SharedVariablesEntry->GetType())
			{
			case EAnimNextSharedVariablesType::Asset:
				if(SharedVariablesEntry->GetAsset())
				{
					UUAFSharedVariables_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFSharedVariables_EditorData>(SharedVariablesEntry->GetAsset());
					EditorData->GetAllVariables(OutVariables, InRecursion, EVariableAccessFilter::PublicOnly);
				}
				break;
			case EAnimNextSharedVariablesType::Struct:
				if(SharedVariablesEntry->GetStruct())
				{
					for (TFieldIterator<FProperty> It(SharedVariablesEntry->GetStruct()); It; ++It)
					{
						const FProperty* Property = *It;
						if (InAccess == EVariableAccessFilter::All || Property->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPublic))
						{
							OutVariables.Add(
								{
									Property->GetFName(),
									FAnimNextParamType::FromProperty(Property),
									SharedVariablesEntry->GetStruct(),
									EAnimNextExportAccessSpecifier::Public,
									Property,
									TConstArrayView<uint8>(),
									UE::UAF::UncookedOnly::FUtils::GenerateScriptStructPropertyGUID(Property)
								});
						}
					}
				}
				break;
			case EAnimNextSharedVariablesType::RigVMAsset:
				{
					if(const IRigVMRuntimeAssetInterface* RigVMAsset = SharedVariablesEntry->GetRigVMAsset().GetInterface())
					{
						TArray<FRigVMExternalVariable> RigVMVariables = const_cast<IRigVMRuntimeAssetInterface*>(RigVMAsset)->GetExternalVariables();
						for (const FRigVMExternalVariable& Variable : RigVMVariables)
						{
							const FProperty* Property = Variable.GetProperty();
							if (InAccess == EVariableAccessFilter::All || Property->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPublic))
							{
								FGuid Guid = FGuid::NewDeterministicGuid(Property->GetPathName());
								OutVariables.Add(
									{
										Property->GetFName(),
										FAnimNextParamType::FromProperty(Property),
										SharedVariablesEntry->GetRigVMAsset().GetObject(),
										EAnimNextExportAccessSpecifier::Public,
										Property,
										TConstArrayView<uint8>(),
										Guid
									});
							}
						}
					}
				}
				break;
			default:
				checkNoEntry();
			}
		}
	}
}

void UUAFRigVMAssetEditorData::HandleReportFromCompiler(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage) const
{
	if (bSuspendCompilerReports)
	{
		return;
	}

	UUAFRigVMAsset* Asset = UE::UAF::UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(this);
	FCompilerResultsLog& Log = UE::UAF::UncookedOnly::FCompilationScope::GetLogForObject(Asset);

	UObject* SubjectForMessage = InSubject;
	if(URigVMNode* ModelNode = Cast<URigVMNode>(SubjectForMessage))
	{
		if(IRigVMClientHost* RigVMClientHost = ModelNode->GetImplementingOuter<IRigVMClientHost>())
		{
			if(URigVMNode* OriginalModelNode = Cast<URigVMNode>(Log.FindSourceObject(ModelNode)))
			{
				if(URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(RigVMClientHost->GetEditorObjectForRigVMGraph(OriginalModelNode->GetGraph())))
				{
					if(UEdGraphNode* EdNode = EdGraph->FindNodeForModelNodeName(OriginalModelNode->GetFName()))
					{
						SubjectForMessage = EdNode;
					}
				}
			}
		}
	}

	TSharedPtr<FTokenizedMessage> Message;
	if (InSeverity == EMessageSeverity::Error)
	{
		// see UnitTest "ControlRig.Basics.OrphanedPins" to learn why errors are suppressed this way
		if (VMCompileSettings.SurpressErrors)
		{
			Log.bSilentMode = true;
		}

		if(InMessage.Contains(TEXT("@@")))
		{
			Message = Log.Error(*InMessage, SubjectForMessage);
		}
		else
		{
			Message = Log.Error(*InMessage);
		}

	}
	else if (InSeverity == EMessageSeverity::Warning)
	{
		if(InMessage.Contains(TEXT("@@")))
		{
			Message = Log.Warning(*InMessage, SubjectForMessage);
		}
		else
		{
			Message = Log.Warning(*InMessage);
		}
	}
	else
	{
		if(InMessage.Contains(TEXT("@@")))
		{
			Message = Log.Note(*InMessage, SubjectForMessage);
		}
		else
		{
			Message = Log.Note(*InMessage);
		}
	}

	if (Message.IsValid())
	{
		HandleMessageFromCompiler(Message.ToSharedRef(), false);
	}
}

void UUAFRigVMAssetEditorData::HandleMessageFromCompiler(TSharedRef<FTokenizedMessage> InMessage, bool bAddMessageToLog) const
{
	if (bSuspendCompilerReports)
	{
		return;
	}

	UUAFRigVMAsset* Asset = UE::UAF::UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(this);
	FCompilerResultsLog& Log = UE::UAF::UncookedOnly::FCompilationScope::GetLogForObject(Asset);
	
	TSharedPtr<FTokenizedMessage> Message;
	if (InMessage->GetSeverity() == EMessageSeverity::Error)
	{
		// see UnitTest "ControlRig.Basics.OrphanedPins" to learn why errors are suppressed this way
		if (VMCompileSettings.SurpressErrors)
		{
			Log.bSilentMode = true;
		}
		
		// see UnitTest "ControlRig.Basics.OrphanedPins" to learn why errors are suppressed this way
		if (!VMCompileSettings.SurpressErrors)
		{ 
			FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, *InMessage->ToText().ToString(), *FString());
		}

		bErrorsDuringCompilation = true;
	}
	else if (InMessage->GetSeverity() == EMessageSeverity::Warning)
	{
		FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Warning, *InMessage->ToText().ToString(), *FString());

		bWarningsDuringCompilation = true;
	}
	else
	{
		UE_LOGF(LogAnimation, Display, "%ls", *InMessage->ToText().ToString());
	}

	if (bAddMessageToLog)
	{		
		Log.AddTokenizedMessage(InMessage);
	}
}

void UUAFRigVMAssetEditorData::ClearErrorInfoForAllEdGraphs()
{
	for (UEdGraph* Graph : GetAllEdGraphs())
	{
		URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(Graph);
		if (RigGraph == nullptr)
		{
			continue;
		}

		for (UEdGraphNode* GraphNode : Graph->Nodes)
		{
			if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
			{
				RigVMEdGraphNode->ClearErrorInfo();
			}
		}
	}
}

void UUAFRigVMAssetEditorData::RefreshExternalModels()
{
	GraphModels.Reset();

	for (UUAFRigVMAssetEntry* Entry : Entries)
	{
		if (IUAFRigVMGraphInterface* GraphInterface = Cast<IUAFRigVMGraphInterface>(Entry))
		{
			if(URigVMGraph* Model = GraphInterface->GetRigVMGraph())
			{
				GraphModels.Add(Model);
			}
		}
	}
}


void UUAFRigVMAssetEditorData::RefreshFunctionImplementationGraphs()
{
	FRigVMClient* VMClient = GetRigVMClient();
	TArray<URigVMGraph*> AllModels = VMClient->GetAllModels(false, false);

	for (URigVMGraph* Model : AllModels)
	{
		URigVMController* Controller = RigVMClient.GetOrCreateController(Model);
		if (!Controller)
		{
			continue;
		}

		const bool bSetupUndoRedo = false;

		// Capture current nodes in separate array as binding can change node count.
		// No new function nodes should be introduced, so this is safe
		TArray<URigVMNode*> Nodes = Model->GetNodes();
		for (URigVMNode* Node : Nodes)
		{
			if (URigVMFunctionReferenceNode* ReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
			{
				// Remove outdated pins & related var nodes
				{
					// Pins will change as we unbind, cache.
					TArray<URigVMPin*> OrphanPins = ReferenceNode->GetOrphanedPins();
					TArray<URigVMNode*> OrphanVars;
					OrphanVars.Reserve(OrphanPins.Num());

					for (URigVMPin* OrphanPin : OrphanPins)
					{
						if (!OrphanPin->IsDefinedAsInputVariable())
						{
							continue;
						}
						OrphanVars.Add(OrphanPin->GetBoundVariableNode());
						Controller->UnbindPinFromVariable(OrphanPin->GetPinPath(), bSetupUndoRedo);
					}
					Controller->RemoveNodes(OrphanVars, bSetupUndoRedo);
					Controller->RemoveUnusedOrphanedPins(ReferenceNode);
				}

				// Bind new pins to variable nodes
				{
					for (URigVMPin* Pin : ReferenceNode->GetPins())
					{
						if (!Pin->IsDefinedAsInputVariable())
						{
							continue;
						}
						Controller->BindPinToVariable(Pin->GetPinPath(), Pin->GetName(), bSetupUndoRedo);
					}
				}
			}
		}
	}
}

void UUAFRigVMAssetEditorData::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UUAFRigVMAssetEditorData* This = CastChecked<UUAFRigVMAssetEditorData>(InThis);

	if (This->CachedExports.IsSet())
	{
		// Cached exports may hold references to objects, so make GC aware
		Collector.AddPropertyReferences(FAnimNextAssetRegistryExports::StaticStruct(), &This->CachedExports.GetValue(), InThis);
	}
}

void UUAFRigVMAssetEditorData::AddEntryInternal(UUAFRigVMAssetEntry* InEntry)
{
	InsertEntryInternal(InEntry, Entries.Num());	
}

void UUAFRigVMAssetEditorData::InsertEntryInternal(UUAFRigVMAssetEntry* InEntry, int32 InsertionIndex)
{
	Entries.Insert(InEntry, InsertionIndex);
}

void UUAFRigVMAssetEditorData::RemoveEntryInternal(UUAFRigVMAssetEntry* InEntry)
{
	Entries.Remove(InEntry);
}

FInstancedPropertyBag UUAFRigVMAssetEditorData::GenerateCombinedPropertyBag(const FRigVMCompileSettings& InSettings, const FAnimNextGetVariableCompileContext& InCompileContext) const
{
	// TODO:
	// It would be nice to remove the requirement that variable names must be unique across all referenced/owned variables inside an asset.
	// This stems from:
	//   1. RigVM external variables needing to have unique names.
	//   2. State tree needing to combine properties into a single property bag and property bag enforcing unique property names.
	// A solution where variable names are qualified using their owned asset as a prefix should be investigated to remove friction when authoring assets with variable dependencies.

	UUAFRigVMAsset* Asset = UE::UAF::UncookedOnly::FUtils::GetAsset<UUAFRigVMAsset>(this);

	TArray<FPropertyBagPropertyDesc> PropertyDescriptions;
	TArray<TConstArrayView<uint8>> PropertyDefaultValues;
	// Required to keep default values from being freed
	TArray<FInstancedPropertyBag> GeneratedPropertyBags;

	// Flag set in case any properties to be combined overlap, if true will prevent combined property bag from being populated
	bool bHasOverlappingProperties = false;	

	// Checks whether or not the provided property name has already been added to PropertyDescriptions, in which case there is an overlap that cannot be resolved
	auto HasOverlappingPropertyName = [&PropertyDescriptions, &InCompileContext, &bHasOverlappingProperties, Asset](const FName& PropertyNameToAdd, const UObject* SourceObject) -> bool
	{
		const bool bPropertyWithSameNameAlreadyExists = PropertyDescriptions.ContainsByPredicate([&PropertyNameToAdd](const FPropertyBagPropertyDesc& ExistingDesc)
		{
			return ExistingDesc.Name == PropertyNameToAdd;
		});
		
		if (bPropertyWithSameNameAlreadyExists)
		{
			bHasOverlappingProperties = true;
			InCompileContext.GetAssetCompileContext().Error(Asset, LOCTEXT("DuplicateVariableEntries", "Variable(s) with overlapping Name {0} found in {1}"), FText::FromName(PropertyNameToAdd), FText::FromString(SourceObject->GetPathName()));
		}
		
		return bPropertyWithSameNameAlreadyExists;
	};

	auto AddInstancedPropertyBag = [this, &PropertyDescriptions, &PropertyDefaultValues, &InCompileContext, Asset, &HasOverlappingPropertyName](const FInstancedPropertyBag& InPropertyBag, const UObject* SourceAsset, bool bValidateProperties = false)
	{
		if(const UPropertyBag* PropertyBag = InPropertyBag.GetPropertyBagStruct())
		{
			TConstArrayView<FPropertyBagPropertyDesc> VariableDescs = PropertyBag->GetPropertyDescs();
			if(VariableDescs.Num() != 0)
			{
				TArray<TConstArrayView<uint8>> Values;
				
				for (const FPropertyBagPropertyDesc& Desc : VariableDescs)
				{
					if (!bValidateProperties || !HasOverlappingPropertyName(Desc.Name, SourceAsset))
					{
						PropertyDescriptions.Add(Desc);
						PropertyDefaultValues.Add(TConstArrayView<uint8>(InPropertyBag.GetValue().GetMemory() + Desc.CachedProperty->GetOffset_ForInternal(), Desc.CachedProperty->GetSize()));
					}
				}
			}
		}
	};

	// NOTE: Order of external variables is important here!
	// It must match that of the variables FUAFInstanceVariableData::Initialize

	// First add internal variables
	FInstancedPropertyBag& AssetBag = GeneratedPropertyBags.Add_GetRef(UE::UAF::UncookedOnly::FUtils::MakePropertyBagForEditorData(this, InCompileContext));
	AddInstancedPropertyBag(AssetBag, Asset, false);

	// Next add shared variables
	for (const UUAFRigVMAsset* ReferencedVariableAsset : Asset->ReferencedVariableAssets)
	{
		if (ReferencedVariableAsset == nullptr)
		{
			continue;
		}

		const UUAFRigVMAssetEditorData* ReferencedEditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFRigVMAssetEditorData>(ReferencedVariableAsset);

		FAnimNextRigVMAssetCompileContext CompileContext = { ReferencedEditorData };
		FInstancedPropertyBag& ReferencedVariableAssetBag = GeneratedPropertyBags.Add_GetRef(UE::UAF::UncookedOnly::FUtils::MakePropertyBagForEditorData(ReferencedEditorData, ReferencedEditorData->GetVariableCompileContext(InSettings, CompileContext)));
		AddInstancedPropertyBag(ReferencedVariableAssetBag, ReferencedVariableAsset, true);
	}

	// add shared variables from Rig VM assets
	for (TScriptInterface ReferencedRigVMAsset : Asset->ReferencedVariableRigVMAssets)
	{
		if (ReferencedRigVMAsset == nullptr)
		{
			continue;
		}

		TArray<FRigVMExternalVariable> ExternalVariables = const_cast<IRigVMRuntimeAssetInterface*>(ReferencedRigVMAsset.GetInterface())->GetExternalVariables();
		for (FRigVMExternalVariable& ExternalVariable : ExternalVariables)
		{
			FPropertyBagPropertyDesc PropertyDesc(ExternalVariable.GetName(), ExternalVariable.GetProperty());

			if (!HasOverlappingPropertyName(ExternalVariable.GetName(), ReferencedRigVMAsset.GetObject()))
			{
				PropertyDescriptions.Add(PropertyDesc);
				TConstArrayView<uint8> DefaultValueMemory(ExternalVariable.GetMemory(), ExternalVariable.GetProperty()->GetSize());
				PropertyDefaultValues.Add(DefaultValueMemory);	
			}
		}
	}

	// Required to keep default struct property values from being freed
	TArray<FInstancedStruct> InstancedReferenceStructs;
	// Next add native structs
	for (const UScriptStruct* ReferencedStruct : Asset->ReferencedVariableStructs)
	{
		// Allocate default values for populating property bag entries with
		FInstancedStruct& StructInstance = InstancedReferenceStructs.AddDefaulted_GetRef();
		StructInstance.InitializeAs(ReferencedStruct);

		TSharedRef<UE::UAF::FStructDataCache> StructData = UE::UAF::FStructDataCache::GetStructInfo(ReferencedStruct);
		for (const UE::UAF::FStructDataCache::FPropertyInfo& PropertyInfo : StructData->GetProperties())
		{
			const FAnimNextParamType ParamType =  FAnimNextParamType::FromProperty(PropertyInfo.Property);
			const FRigVMTemplateArgumentType RigVMArgumentType = ParamType.ToRigVMTemplateArgument();
			if (!RigVMArgumentType.IsValid() || RigVMArgumentType.IsUnknownType())
			{
				InCompileContext.GetAssetCompileContext().Error(Asset, LOCTEXT("InvalidNativeVariableType", "@@ Variable '{0}' from '{1}' has unsupported variable type '{2}'"), FText::FromName(PropertyInfo.Property->GetFName()),  FText::FromName(ReferencedStruct->GetFName()), FText::FromString(ParamType.ToString()));
				continue;
			}

			if (!HasOverlappingPropertyName(PropertyInfo.Property->GetFName(), ReferencedStruct))
			{
				PropertyDescriptions.Add(FPropertyBagPropertyDesc(PropertyInfo.Property->GetFName(), PropertyInfo.Property));
				PropertyDefaultValues.Add(TConstArrayView<uint8>(StructInstance.GetMemory() + PropertyInfo.Property->GetOffset_ForInternal(), PropertyInfo.Property->GetSize()));
			}
		}
	}

	FInstancedPropertyBag CombinedPropertyBag;
	if (!bHasOverlappingProperties)
	{
		const EPropertyBagResult Result = CombinedPropertyBag.ReplaceAllPropertiesAndValues(PropertyDescriptions, PropertyDefaultValues);
		if (Result != EPropertyBagResult::Success)
		{
			InCompileContext.GetAssetCompileContext().Error(
				Asset,
				LOCTEXT("FailedToGenerateCombinedPropertyBag", "Failed to generate combined property bag for {0}"), FText::FromName(Asset->GetFName())
			);
		}
	 }

	return CombinedPropertyBag;
}

void UUAFRigVMAssetEditorData::UpgradeDataInterfaces()
{
	TGuardValue<bool> DisableEditorDataNotifications(bSuspendEditorDataNotifications, true);
	TGuardValue<bool> DisableAutoCompile(bAutoRecompileVM, false);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (NativeInterfaces_DEPRECATED.Num())
	{
		// Iterate over all the native interfaces and add shared variables for them
		for (const UScriptStruct* Struct : NativeInterfaces_DEPRECATED)
		{
			// Translate 'native interface' into the trait data it has become
			FInstancedStruct InstancedStruct(Struct);
			const UScriptStruct* UpgradeStruct = InstancedStruct.Get<FAnimNextNativeDataInterface>().GetUpgradeTraitStruct();
			if(UpgradeStruct)
			{
				AddSharedVariablesStruct(UpgradeStruct, false, false);

				// Remove all old variables
				for (TFieldIterator<FProperty> It(UpgradeStruct); It; ++It)
				{
					const FName VariableName = It->GetFName();

					// Try to find a variable both with and without the 'b' prefix 
					if (It->IsA<FBoolProperty>())
					{
						FString BoolName = VariableName.ToString();
						if (BoolName.RemoveFromStart(TEXT("b")))
						{
							if (UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(FindEntry(*BoolName)))
							{
								RemoveEntry(VariableEntry, false, false);
								continue;
							}
						}
					}

					if (UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(FindEntry(VariableName)))
					{
						RemoveEntry(VariableEntry, false, false);
						continue;
					}
				}
			}
		}

		NativeInterfaces_DEPRECATED.Empty();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	{
		// Recompile to incorporate any new native interface variables (will create errors, so suppress them)
		TGuardValue<bool> DisableCompilerReports(bSuspendCompilerReports, true);
		RecompileVM();
	}

	// Now replace variable nodes with scoped ones in all graphs, if possible
	for (URigVMGraph* Graph : GetRigVMClient()->GetAllModels(true, true))
	{
		UAnimNextControllerBase* Controller = CastChecked<UAnimNextControllerBase>(GetController(Graph));
		TArray<URigVMNode*> Nodes = Graph->GetNodes();
		for (URigVMNode* Node : Nodes)
		{
			if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
			{
				const FName OriginalVariableName = VariableNode->GetVariableName();
				FName NewVariableName = OriginalVariableName;
				TArray<FName> VariableNames;
				VariableNames.Add(OriginalVariableName);
				if (VariableNode->GetCPPType() == TEXT("bool"))
				{
					TStringBuilder<128> StringBuilder;
					StringBuilder.Append(TEXT("b"));
					OriginalVariableName.AppendString(StringBuilder);
					VariableNames.Add(StringBuilder.ToString());
				}

				// See if the variable exists in our shared variable entries
				for(UUAFRigVMAssetEntry* Entry : Entries)
				{
					const UObject* AssetOrStruct = nullptr;
					if (UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(Entry))
					{
						for (FName VariableName : VariableNames)
						{
							switch (SharedVariablesEntry->GetType())
							{
							case EAnimNextSharedVariablesType::Asset:
								{
									if (const UUAFRigVMAsset* Asset = SharedVariablesEntry->GetAsset())
									{
										const FInstancedPropertyBag& Variables = Asset->GetVariableDefaults();
										if (Variables.FindPropertyDescByName(VariableName) != nullptr)
										{
											AssetOrStruct = Asset;
											NewVariableName = VariableName;
										}
									}
									break;
								}
							case EAnimNextSharedVariablesType::Struct:
								{
									if (const UScriptStruct* Struct = SharedVariablesEntry->GetStruct())
									{
										if (Struct->FindPropertyByName(VariableName) != nullptr)
										{
											AssetOrStruct = Struct;
											NewVariableName = VariableName;
										}
									}
									break;
								}
							}

							if (AssetOrStruct)
							{
								break;
							}
						}
					}

					if (AssetOrStruct)
					{
						Controller->ReplaceVariableNodeWithSharedVariableNode(VariableNode, NewVariableName, AssetOrStruct, false, false);
						break;
					}
				}
			}
		}
	}
}

void UUAFRigVMAssetEditorData::ConditionalPatchFunctionsOnLoad()
{
	if (!bFunctionsPatched)
	{
		ensure(!HasAnyFlags(RF_NeedPostLoad));
		bFunctionsPatched = true;
		GetRigVMClient()->PatchFunctionReferencesOnLoad();
		TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader> OldHeaders;
		TArray<FName> BackwardsCompatiblePublicFunctions;
		GetRigVMClient()->PatchFunctionsOnLoad(this, BackwardsCompatiblePublicFunctions, OldHeaders);

		RefreshFunctionImplementationGraphs();
	}
}


#undef LOCTEXT_NAMESPACE
