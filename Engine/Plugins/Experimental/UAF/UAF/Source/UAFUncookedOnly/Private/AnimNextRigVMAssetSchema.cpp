// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextRigVMAssetSchema.h"
#include "AnimNextExecuteContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextRigVMAssetSchema)

UUAFRigVMAssetSchema::UUAFRigVMAssetSchema()
{
	SetExecuteContextStruct(FAnimNextExecuteContext::StaticStruct());
}
