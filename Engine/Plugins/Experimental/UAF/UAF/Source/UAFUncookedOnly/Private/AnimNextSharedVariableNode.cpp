// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextSharedVariableNode.h"
#include "UncookedOnlyUtils.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextSharedVariableNode)

void UUAFSharedVariableNode::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::UAFVariablesGuid)
	{
		UpdateCachedVariableGUID();
	}
	else if (!CachedGuid.IsValid())
	{
		UE_LOGF(LogAnimation, Warning, "%ls had an invalid variable cached GUID value for %ls in %ls",
			*GetPathName(), 
			*GetVariableName().ToString(),
			Type == EAnimNextSharedVariablesType::Asset ? (Asset ? *Asset->GetPathName() : TEXT("None"))
			: (Struct ? *Struct->GetPathName() : TEXT("None"))
		);
		
		UpdateCachedVariableGUID();
	}
	else
	{
		const IRigVMClientHost* OuterHost = GetImplementingOuter<IRigVMClientHost>();
		const URigVMGraph* OuterGraph = GetTypedOuter<URigVMGraph>();

		if (OuterHost && OuterGraph)
		{
			const FName ExpectedVariableName = GetVariableName();

			switch (Type)
			{
			case EAnimNextSharedVariablesType::Asset:
				{
					if (const UUAFSharedVariables* SharedVariablesAsset = Asset.Get())
					{
						const UUAFSharedVariables_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFSharedVariables_EditorData>(SharedVariablesAsset);
						const FRigVMTemplateArgumentType ExpectedVariableType(FName(*GetCPPType()), GetCPPTypeObject());

						TArray<UUAFRigVMAssetEditorData::FVariableInfo> AssetVariables;
						EditorData->GetAllVariables(AssetVariables, UUAFRigVMAssetEditorData::EVariableRecursion::SelfOnly, UUAFRigVMAssetEditorData::EVariableAccessFilter::PublicOnly);

						UUAFRigVMAssetEditorData::FVariableInfo* MatchingVariable = nullptr;
						MatchingVariable = AssetVariables.FindByPredicate([SearchGuid = CachedGuid](const UUAFRigVMAssetEditorData::FVariableInfo& VariableInfo) { return VariableInfo.StableGuid == SearchGuid; });
						if (MatchingVariable)
						{
							// Variable has matching Guid but different name
							if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
							{
								const FRigVMTemplateArgumentType VariableType = MatchingVariable->Type.ToRigVMTemplateArgument();
								if (VariableType != ExpectedVariableType || MatchingVariable->Name != GetVariableName())
								{
									// Update with new variable name - and type if required, populating orphan pins of non-compatible data types
									AnimNextController->RefreshSharedVariableNode(GetFName(), SharedVariablesAsset->GetPathName(), MatchingVariable->Name, VariableType.CPPType.ToString(), VariableType.CPPTypeObject.Get(), false, true, false );
								}
							}
						}
						else
						{
							// Look for variable with matching (node stored) variable name 
							MatchingVariable = AssetVariables.FindByPredicate([ExpectedVariableName](const UUAFRigVMAssetEditorData::FVariableInfo& VariableInfo) { return VariableInfo.Name == ExpectedVariableName; });
							if (MatchingVariable)
							{
								// Variable has matching name but different GUID 
								checkf(CachedGuid != MatchingVariable->StableGuid, TEXT("Expected GUID to be different, otherwise previous search using GUID should have been successful"));
								UpdateCachedVariableGUID();

								if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
								{
									const FRigVMTemplateArgumentType VariableType = MatchingVariable->Type.ToRigVMTemplateArgument();
									if (VariableType != ExpectedVariableType)
									{
										// Update data type, populating orphan pins of non-compatible data types
										AnimNextController->RefreshSharedVariableNode(GetFName(), SharedVariablesAsset->GetPathName(), ExpectedVariableName, VariableType.CPPType.ToString(), VariableType.CPPTypeObject.Get(), false, true, false );
									}
								}
							}
							else
							{
								// node cannot be resolved
								UE_LOGF(LogAnimation, Warning, "Unable to find public variable with expected name: %ls, type: %ls and GUID %ls in %ls, SharedVariableNode will be invalidated", *ExpectedVariableName.ToString(), *FAnimNextParamType::FromRigVMTemplateArgument(ExpectedVariableType).ToString(), *CachedGuid.ToString(EGuidFormats::Short), *SharedVariablesAsset->GetPathName());

								if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
								{
									AnimNextController->RefreshSharedVariableNode(GetFName(), SharedVariablesAsset->GetPathName(), ExpectedVariableName, *RigVMTypeUtils::GetWildCardCPPTypeName().ToString(), RigVMTypeUtils::GetWildCardCPPTypeObject(), false, true, false );
								}
							}
						}
					}
					else
					{
						UE_LOGF(LogAnimation, Warning, "Unable to load Asset for %ls when trying to resolve SharedVariable node source", *GetPathName());

						if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
						{
							AnimNextController->RefreshSharedVariableNode(GetFName(), TEXT(""), ExpectedVariableName, *RigVMTypeUtils::GetWildCardCPPTypeName().ToString(), RigVMTypeUtils::GetWildCardCPPTypeObject(), false, true, false );
						}
					}
					break;
				}

			case EAnimNextSharedVariablesType::Struct:
				{
					if (const UScriptStruct* SharedVariablesStruct = Struct.Get())
					{
						const FAnimNextParamType ExpectedVariableType = FAnimNextParamType::FromRigVMTemplateArgument(FRigVMTemplateArgumentType(FName(*GetCPPType()), GetCPPTypeObject()));	
						FName ExpectedStructVariableName = ExpectedVariableName;
						const FProperty* Property = SharedVariablesStruct->FindPropertyByName(ExpectedStructVariableName);

						// Try property redirector
						if (!Property)
						{
							const FName RedirectedName = FProperty::FindRedirectedPropertyName(SharedVariablesStruct, ExpectedStructVariableName);
							if (RedirectedName != NAME_None && RedirectedName != ExpectedStructVariableName)
							{
								Property = SharedVariablesStruct->FindPropertyByName(RedirectedName);
								ExpectedStructVariableName = RedirectedName;
							}
						}

						if (Property)
						{
							const FAnimNextParamType PropertyParamType = FAnimNextParamType::FromProperty(Property);
							if (ExpectedVariableType != PropertyParamType || ExpectedStructVariableName != ExpectedVariableName)
							{
								if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
								{
									// Update with new variable name - and type if required, populating orphan pins of non-compatible data types
									AnimNextController->RefreshSharedVariableNode(GetFName(), SharedVariablesStruct->GetPathName(), ExpectedStructVariableName, *Property->GetCPPType(), const_cast<UObject*>(PropertyParamType.GetValueTypeObject()), false, true, false );
								}
							}
						}
						else
						{
							// node cannot be resolved
							UE_LOGF(LogAnimation, Warning, "Unable to find struct FProperty with expected name: %ls, type: %ls in %ls, SharedVariableNode will be invalidated", *ExpectedVariableName.ToString(), *ExpectedVariableType.ToString(), *SharedVariablesStruct->GetPathName());

							if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
							{
								AnimNextController->RefreshSharedVariableNode(GetFName(), SharedVariablesStruct->GetPathName(), ExpectedStructVariableName, *RigVMTypeUtils::GetWildCardCPPTypeName().ToString(), RigVMTypeUtils::GetWildCardCPPTypeObject(), false, true, false );
							}
						}
					}
					else
					{
						UE_LOGF(LogAnimation, Warning, "Unable to find Struct for %ls when trying to resolve SharedVariable node source", *GetPathName());
						
						if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
						{
							AnimNextController->RefreshSharedVariableNode(GetFName(), TEXT(""), ExpectedVariableName, *RigVMTypeUtils::GetWildCardCPPTypeName().ToString(), RigVMTypeUtils::GetWildCardCPPTypeObject(), false, true, false );
						}
					}
					break;
				}
			case EAnimNextSharedVariablesType::RigVMAsset:
				{
					if (const IRigVMRuntimeAssetInterface* RigVMAssetInterface = RigVMAsset.GetInterface())
					{
						check(RigVMAsset.GetObject() != nullptr);
						const FAnimNextParamType ExpectedVariableType = FAnimNextParamType::FromRigVMTemplateArgument(FRigVMTemplateArgumentType(FName(*GetCPPType()), GetCPPTypeObject()));	
						FName ExpectedStructVariableName = ExpectedVariableName;
						const FProperty* Property = RigVMAssetInterface->FindGeneratedPropertyByName(ExpectedStructVariableName);

						if (Property)
						{
							const FAnimNextParamType PropertyParamType = FAnimNextParamType::FromProperty(Property);
							if (ExpectedVariableType != PropertyParamType || ExpectedStructVariableName != ExpectedVariableName)
							{
								if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
								{
									// Update with new variable name - and type if required, populating orphan pins of non-compatible data types
									AnimNextController->RefreshSharedVariableNode(GetFName(), RigVMAsset.GetObject()->GetPathName(), ExpectedStructVariableName, *Property->GetCPPType(), const_cast<UObject*>(PropertyParamType.GetValueTypeObject()), false, true, false );
								}
							}
						}
						else
						{
							// node cannot be resolved
							UE_LOGF(LogAnimation, Warning, "Unable to find struct FProperty with expected name: %ls, type: %ls in %ls, SharedVariableNode will be invalidated", *ExpectedVariableName.ToString(), *ExpectedVariableType.ToString(), *RigVMAsset.GetObject()->GetPathName());

							if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
							{
								AnimNextController->RefreshSharedVariableNode(GetFName(), RigVMAsset.GetObject()->GetPathName(), ExpectedStructVariableName, *RigVMTypeUtils::GetWildCardCPPTypeName().ToString(), RigVMTypeUtils::GetWildCardCPPTypeObject(), false, true, false );
							}
						}
					}
					else
					{
						UE_LOGF(LogAnimation, Warning, "Unable to find RigVMAsset for %ls when trying to resolve SharedVariable node source", *GetPathName());
						
						if (UAnimNextControllerBase* AnimNextController = Cast<UAnimNextControllerBase>(OuterHost->GetController(OuterGraph)))
						{
							AnimNextController->RefreshSharedVariableNode(GetFName(), TEXT(""), ExpectedVariableName, *RigVMTypeUtils::GetWildCardCPPTypeName().ToString(), RigVMTypeUtils::GetWildCardCPPTypeObject(), false, true, false );
						}
					}
				}
				break;
			default:
				{
					checkNoEntry();
				}
				break;
			}
		}
	}
}

void UUAFSharedVariableNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

void UUAFSharedVariableNode::UpdateCachedVariableGUID()
{
	CachedGuid.Invalidate();

	switch (Type)
	{
	case EAnimNextSharedVariablesType::Asset:
		{
			if (Asset)
			{
				const UUAFSharedVariables_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<const UUAFSharedVariables_EditorData>(Asset.Get());

				if (const IUAFRigVMVariableInterface* VariableInterface = EditorData != nullptr ? Cast<IUAFRigVMVariableInterface>(EditorData->FindEntry(GetVariableName())) : nullptr)
				{
					CachedGuid = VariableInterface->GetGuid();
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Unable to generate GUID for %ls as Asset %ls does not contain variable %ls", *GetPathName(), *Asset->GetPathName(), *GetVariableName().ToString());
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Unable to generate GUID for %ls as Asset is invalid", *GetPathName());
			}
		}
		break;
	case EAnimNextSharedVariablesType::Struct:
		{
			if (Struct)
			{
				if (const FProperty* Property = Struct->FindPropertyByName(GetVariableName()))
				{
					CachedGuid = UE::UAF::UncookedOnly::FUtils::GenerateScriptStructPropertyGUID(Property);
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Unable to generate GUID for %ls as Struct %ls does not contain variable %ls", *GetPathName(), *Struct->GetPathName(), *GetVariableName().ToString());
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Unable to generate GUID for %ls as Struct is invalid", *GetPathName());
			}
		}
		break;
	case EAnimNextSharedVariablesType::RigVMAsset:
		{
			if (RigVMAsset)
			{
				if (const FProperty* Property = RigVMAsset->FindGeneratedPropertyByName(GetVariableName()))
				{
					CachedGuid = UE::UAF::UncookedOnly::FUtils::GenerateScriptStructPropertyGUID(Property);
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Unable to generate GUID for %ls as RigVMAsset %ls does not contain variable %ls", *GetPathName(), RigVMAsset.GetObject() ? *RigVMAsset.GetObject()->GetPathName() : TEXT("Invalid Asset Object"), *GetVariableName().ToString());
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Unable to generate GUID for %ls as RigVMAsset is invalid", *GetPathName());
			}
		}
	break;
	default:
		{
			checkNoEntry();
		}
	}
}

FString UUAFSharedVariableNode::GetNodeSubTitle() const
{
	switch (Type)
	{
	case EAnimNextSharedVariablesType::Asset:
		return Asset ? Asset->GetFName().ToString() : FString();
	case EAnimNextSharedVariablesType::Struct:
		return Struct ? Struct->GetDisplayNameText().ToString(): FString();
	}
	return FString();
}

