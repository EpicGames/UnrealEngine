// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextUncookedOnlyModule.h"

#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "UncookedOnlyUtils.h"
#include "Engine/Blueprint.h"
#include "Modules/ModuleManager.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "MessageLogModule.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Variables/UniversalObjectLocatorBindingType.h"

#define LOCTEXT_NAMESPACE "AnimNextUncookedOnlyModule"

namespace UE::UAF::UncookedOnly
{

void FModule::StartupModule()
{
	RegisterVariableBindingType("/Script/UAFUncookedOnly.AnimNextUniversalObjectLocatorBindingData", MakeShared<FUniversalObjectLocatorBindingType>());

	// Register the compilation log (hidden from the main log set, it is displayed in the workspace editor)
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions LogInitOptions;
	LogInitOptions.bShowInLogWindow = false;
	LogInitOptions.MaxPageCount = 10;
	MessageLogModule.RegisterLogListing("AnimNextCompilerResults", LOCTEXT("CompilerResults", "UAF Compiler Results"), LogInitOptions);
}

void FModule::ShutdownModule()
{
	if(FMessageLogModule* MessageLogModule = FModuleManager::GetModulePtr<FMessageLogModule>("MessageLog"))
	{
		MessageLogModule->UnregisterLogListing("AnimNextCompilerResults");
	}

	UnregisterVariableBindingType("/Script/UAFUncookedOnly.AnimNextUniversalObjectLocatorBindingData");
}

void FModule::RegisterVariableBindingType(FName InStructName, TSharedPtr<IVariableBindingType> InType)
{
	VariableBindingTypes.Add(InStructName, InType);
}

void FModule::UnregisterVariableBindingType(FName InStructName)
{
	VariableBindingTypes.Remove(InStructName);
}

TSharedPtr<IVariableBindingType> FModule::FindVariableBindingType(const UScriptStruct* InStruct) const
{
	return VariableBindingTypes.FindRef(*InStruct->GetPathName());
}

#if WITH_EDITOR
FGuid FModule::GetVariableGuidByName(const FName InName, const UObject* InObject) const
{
	FGuid Guid;

	if (InObject)
	{
		const_cast<UObject*>(InObject)->ConditionalPreload();
		
		auto GetGuidFromEditorData = [InName, &Guid](const UUAFRigVMAssetEditorData* EditorData)
		{
			if (UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(EditorData->FindEntry(InName)))
			{
				Guid = VariableEntry->GetGuid();
			}
		};
	
		if (const UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(InObject))
		{
			GetGuidFromEditorData(FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset));
		}
		else if (const UUAFRigVMAssetEditorData* EditorData = Cast<UUAFRigVMAssetEditorData>(InObject))
		{
			GetGuidFromEditorData(EditorData);
		}
		else if (const UScriptStruct* Struct = Cast<UScriptStruct>(InObject))
		{
			const FName RedirectedName = FProperty::FindRedirectedPropertyName(Struct, InName);	
			if (const FProperty* FoundProperty = Struct->FindPropertyByName(RedirectedName != NAME_None ? RedirectedName : InName))
			{
				Guid = FUtils::GenerateScriptStructPropertyGUID(FoundProperty);
			}
		}
		else if (const IRigVMRuntimeAssetInterface* RigVMRuntimeAssetInterface = Cast<IRigVMRuntimeAssetInterface>(InObject))
		{
			// Generate a guid from the path name of the property
			if (const FProperty* Property = RigVMRuntimeAssetInterface->FindGeneratedPropertyByName(InName))
			{
				Guid = FGuid::NewDeterministicGuid(Property->GetPathName());
			}
		}
		else if (const IRigVMEditorAssetInterface* EditorAssetInterface = Cast<IRigVMEditorAssetInterface>(InObject))
		{
			// Generate a guid from the path name of the property
			if (const FProperty* Property = EditorAssetInterface->GetRuntimeAssetInterface()->FindGeneratedPropertyByName(InName))
			{
				Guid = FGuid::NewDeterministicGuid(Property->GetPathName());
			}
		}
	}
	
	
	return Guid;
}

FName GetVariableNameByGuidRigVMRuntimeAsset(const FGuid InGuid, const IRigVMRuntimeAssetInterface* RigVMRuntimeAssetInterface)
{
	check(RigVMRuntimeAssetInterface);

	// Currently no lookup available for IRigVMRuntimeAssetInterface
	// So we must do a manual search
	TArray<FRigVMExternalVariable> Variables = const_cast<IRigVMRuntimeAssetInterface*>(RigVMRuntimeAssetInterface)->GetExternalVariables();
	FRigVMExternalVariable* FoundVariable = Variables.FindByPredicate( [InGuid](const FRigVMExternalVariable& ItSearch)
		{
			FGuid CompareGuid = ItSearch.GetProperty() != nullptr ? FGuid::NewDeterministicGuid(ItSearch.GetProperty()->GetPathName()) : FGuid();
			return CompareGuid == InGuid;
		});

	if (FoundVariable != nullptr)
	{
		return FoundVariable->GetName();
	}

	return NAME_None;
}

FName FModule::GetVariableNameByGuid(const FGuid InGuid, const UObject* InObject) const
{
	auto GetNameFromEditorData = [InGuid](const UUAFRigVMAssetEditorData* EditorData)-> FName
	{
		const UAnimNextVariableEntry* Entry = nullptr;
		EditorData->ForEachEntryOfType<UAnimNextVariableEntry>([InGuid, &Entry](const UAnimNextVariableEntry* VariableEntry)
			{
				if (VariableEntry->GetGuid() == InGuid)
				{
					Entry = VariableEntry;
					return false;
				}
				
				return true;
			});
		
		return Entry ? Entry->GetVariableName() : NAME_None;
	};
	
	
	if (InObject)
	{
		const_cast<UObject*>(InObject)->ConditionalPreload();
	
		if (const UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(InObject))
		{
			return GetNameFromEditorData(FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset));
		}
		else if (const UUAFRigVMAssetEditorData* EditorData = Cast<UUAFRigVMAssetEditorData>(InObject))
		{
			return GetNameFromEditorData(EditorData);
		}
		else if (const UScriptStruct* Struct = Cast<UScriptStruct>(InObject))
		{
			// Optimize using caching map?
			for (TFieldIterator<FProperty> PropertyIt(Struct); PropertyIt; ++PropertyIt)
			{
				const FGuid PropertyGuid = FUtils::GenerateScriptStructPropertyGUID(*PropertyIt);
				if (PropertyGuid == InGuid)
				{
					return PropertyIt->GetFName();
				}
			}
		}
		else if (const UBlueprintGeneratedClass* BpGenerated = Cast<UBlueprintGeneratedClass>(InObject))
		{
			return BpGenerated->FindBlueprintPropertyNameFromGuid(InGuid);
		}
		else if (const IRigVMRuntimeAssetInterface* RigVMRuntimeAssetInterface = Cast<IRigVMRuntimeAssetInterface>(InObject))
		{
			return GetVariableNameByGuidRigVMRuntimeAsset(InGuid, RigVMRuntimeAssetInterface);
		}
		else if (const IRigVMEditorAssetInterface* RigVMEditorAssetInterface = Cast<IRigVMEditorAssetInterface>(InObject))
		{
			// We assume that if we have a editor asset interface, we will also have a runtime asset interface
			return GetVariableNameByGuidRigVMRuntimeAsset(InGuid, RigVMEditorAssetInterface->GetRuntimeAssetInterface().GetInterface());
		}
	}
	
	return NAME_None;
}

void FModule::GetExternalVariablesForAsset(const UUAFRigVMAsset* InAsset, TArray<FRigVMExternalVariable>& OutVariables) const
{
	FUtils::GetExternalVariables(FUtils::GetEditorData<UUAFRigVMAssetEditorData>(InAsset), OutVariables);
}

bool FModule::DoesAssetVariablesRequireCompilation(const UUAFRigVMAsset* InAsset) const
{
	UUAFRigVMAssetEditorData* EditorData = FUtils::GetEditorData<UUAFRigVMAssetEditorData>(InAsset);
	
	const bool bNeedsCompilation = !EditorData->bIsCompiling && (EditorData->bVMRecompilationRequired || EditorData->bVMRecompilationRequested);
	const bool bVariablesNotYetCompiled = EditorData->bIsCompiling && !EditorData->bHasCompiledVariables;
	
	return bNeedsCompilation || bVariablesNotYetCompiled;
}
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE 

IMPLEMENT_MODULE(UE::UAF::UncookedOnly::FModule, UAFUncookedOnly);
