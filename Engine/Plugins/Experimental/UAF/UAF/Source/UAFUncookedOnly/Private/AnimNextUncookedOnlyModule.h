// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAnimNextUncookedOnlyModule.h"

namespace UE::UAF::UncookedOnly
{
	struct FUtils;
}

namespace UE::UAF::UncookedOnly
{

class FModule : public IAnimNextUncookedOnlyModule
{
private:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// IAnimNextUncookedOnlyModule interface
	virtual void RegisterVariableBindingType(FName InStructName, TSharedPtr<IVariableBindingType> InType) override;
	virtual void UnregisterVariableBindingType(FName InStructName) override;
	virtual TSharedPtr<IVariableBindingType> FindVariableBindingType(const UScriptStruct* InInstanceIdStruct) const override;
#if WITH_EDITOR
	virtual FGuid GetVariableGuidByName(const FName InName, const UObject* InObject) const override;
	virtual FName GetVariableNameByGuid(const FGuid InGuid, const UObject* InObject) const override;

	virtual void GetExternalVariablesForAsset(const UUAFRigVMAsset* InAsset, TArray<FRigVMExternalVariable>& OutVariables) const override;
	virtual bool DoesAssetVariablesRequireCompilation(const UUAFRigVMAsset* InAsset) const override;
#endif // WITH_EDITOR

	TMap<FName, TSharedPtr<IVariableBindingType>> VariableBindingTypes;

	friend struct FUtils;
};

}
