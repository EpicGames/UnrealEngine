// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/AnimNextGetFunctionHeaderCompileContext.h"

FAnimNextGetFunctionHeaderCompileContext::FAnimNextGetFunctionHeaderCompileContext(FAnimNextRigVMAssetCompileContext& InCompilerContext)
	: CompilerContext(InCompilerContext)
{
}

const TArray<FAnimNextProgrammaticFunctionHeader>& FAnimNextGetFunctionHeaderCompileContext::GetFunctionHeaders() const
{
	return CompilerContext.FunctionHeaders;
}

void FAnimNextGetFunctionHeaderCompileContext::AddUniqueFunctionHeader(const FAnimNextProgrammaticFunctionHeader& InFunctionHeader)
{
	CompilerContext.FunctionHeaders.AddUnique(InFunctionHeader);
}

const UUAFRigVMAssetEditorData* FAnimNextGetFunctionHeaderCompileContext::GetOwningAssetEditorData() const
{
	return CompilerContext.OwningAssetEditorData;
}
