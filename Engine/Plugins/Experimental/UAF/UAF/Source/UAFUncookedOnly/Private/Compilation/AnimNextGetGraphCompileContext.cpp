// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/AnimNextGetGraphCompileContext.h"

FAnimNextGetGraphCompileContext::FAnimNextGetGraphCompileContext(FAnimNextRigVMAssetCompileContext& InCompilerContext)
	: CompilerContext(InCompilerContext)
{
}

const TArray<FAnimNextProgrammaticFunctionHeader>& FAnimNextGetGraphCompileContext::GetFunctionHeaders() const
{
	return CompilerContext.FunctionHeaders;
}

const TArray<URigVMGraph*>& FAnimNextGetGraphCompileContext::GetProgrammaticGraphs() const
{
	return CompilerContext.ProgrammaticGraphs;
}

TArray<URigVMGraph*>& FAnimNextGetGraphCompileContext::GetMutableProgrammaticGraphs() 
{
	return CompilerContext.ProgrammaticGraphs;
}

const UUAFRigVMAssetEditorData* FAnimNextGetGraphCompileContext::GetOwningAssetEditorData() const
{
	return CompilerContext.OwningAssetEditorData;
}
