// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/AnimNextGetVariableCompileContext.h"

FAnimNextGetVariableCompileContext::FAnimNextGetVariableCompileContext(FAnimNextRigVMAssetCompileContext& InCompilerContext)
	: CompilerContext(InCompilerContext)
{
}

const TArray<FAnimNextProgrammaticFunctionHeader>& FAnimNextGetVariableCompileContext::GetFunctionHeaders() const
{
	return CompilerContext.FunctionHeaders;
}

const TArray<FAnimNextProgrammaticVariable>& FAnimNextGetVariableCompileContext::GetProgrammaticVariables() const
{
	return CompilerContext.ProgrammaticVariables;
}

TArray<FAnimNextProgrammaticVariable>& FAnimNextGetVariableCompileContext::GetMutableProgrammaticVariables()
{
	return CompilerContext.ProgrammaticVariables;
}

const UUAFRigVMAssetEditorData* FAnimNextGetVariableCompileContext::GetOwningAssetEditorData() const
{
	return CompilerContext.OwningAssetEditorData;
}

const FAnimNextRigVMAssetCompileContext& FAnimNextGetVariableCompileContext::GetAssetCompileContext() const
{
	return CompilerContext;
}
