// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/AnimNextProcessGraphCompileContext.h"

FAnimNextProcessGraphCompileContext::FAnimNextProcessGraphCompileContext(FAnimNextRigVMAssetCompileContext& InCompilerContext)
	: CompilerContext(InCompilerContext)
{
}

const TArray<URigVMGraph*>& FAnimNextProcessGraphCompileContext::GetAllGraphs() const
{
	return CompilerContext.AllGraphs;
}

TArray<URigVMGraph*>& FAnimNextProcessGraphCompileContext::GetMutableAllGraphs() 
{
	return CompilerContext.AllGraphs;
}

const UUAFRigVMAssetEditorData* FAnimNextProcessGraphCompileContext::GetOwningAssetEditorData() const
{
	return CompilerContext.OwningAssetEditorData;
}
