// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compilation/AnimNextRigVMAssetCompileContext.h"
#include "AnimNextRigVMAssetEditorData.h"

#if WITH_EDITOR
void FAnimNextRigVMAssetCompileContext::Message(TSharedRef<FTokenizedMessage> InMessage) const
{
	check(OwningAssetEditorData);
	OwningAssetEditorData->HandleMessageFromCompiler(InMessage);	
}

void FAnimNextRigVMAssetCompileContext::Message(EMessageSeverity::Type Severity, UObject* Object, const FText& Message) const
{
	check(OwningAssetEditorData);
	OwningAssetEditorData->HandleReportFromCompiler(Severity, Object, Message.ToString());
}
#endif // WITH_EDITOR
