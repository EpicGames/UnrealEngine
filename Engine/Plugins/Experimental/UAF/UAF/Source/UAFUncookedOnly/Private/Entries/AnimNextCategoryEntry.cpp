// Copyright Epic Games, Inc. All Rights Reserved.

#include "Entries/AnimNextCategoryEntry.h"
#include "AnimNextRigVMAssetEditorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextCategoryEntry)

void UAnimNextCategoryEntry::Initialize(UUAFRigVMAssetEditorData* InEditorData)
{
	Super::Initialize(InEditorData);
}

void UAnimNextCategoryEntry::SetEntryName(FName InName, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	}

	CategoryName = InName;
	BroadcastModified(EAnimNextEditorDataNotifType::EntryRenamed);
}
