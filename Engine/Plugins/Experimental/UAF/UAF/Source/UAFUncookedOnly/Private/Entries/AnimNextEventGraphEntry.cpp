// Copyright Epic Games, Inc. All Rights Reserved.

#include "Entries/AnimNextEventGraphEntry.h"

#include "AnimNextRigVMAsset.h"
#include "UncookedOnlyUtils.h"
#include "AnimNextEdGraph.h"
#include "AnimNextRigVMAssetEditorData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextEventGraphEntry)

FText UAnimNextEventGraphEntry::GetDisplayName() const
{
	return FText::FromName(GraphName);
}

FText UAnimNextEventGraphEntry::GetDisplayNameTooltip() const
{
	return FText::FromName(GraphName);
}

void UAnimNextEventGraphEntry::SetEntryName(FName InName, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	}

	GraphName = InName;

	BroadcastModified(EAnimNextEditorDataNotifType::EntryRenamed);
}

const FName& UAnimNextEventGraphEntry::GetGraphName() const
{
	return GraphName;
}

URigVMGraph* UAnimNextEventGraphEntry::GetRigVMGraph() const
{
	return Graph;
}

URigVMEdGraph* UAnimNextEventGraphEntry::GetEdGraph() const
{
	return EdGraph;
}

void UAnimNextEventGraphEntry::SetRigVMGraph(URigVMGraph* InGraph)
{
	Graph = InGraph;
}

void UAnimNextEventGraphEntry::SetEdGraph(URigVMEdGraph* InGraph)
{
	EdGraph = CastChecked<UAnimNextEdGraph>(InGraph, ECastCheckedType::NullAllowed);
}
