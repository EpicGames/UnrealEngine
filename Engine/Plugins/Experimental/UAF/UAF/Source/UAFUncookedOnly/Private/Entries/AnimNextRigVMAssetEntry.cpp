// Copyright Epic Games, Inc. All Rights Reserved.

#include "Entries/AnimNextRigVMAssetEntry.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Misc/TransactionObjectEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextRigVMAssetEntry)

#define LOCTEXT_NAMESPACE "AnimNextRigVMAssetEntry"

void UUAFRigVMAssetEntry::Initialize(UUAFRigVMAssetEditorData* InEditorData)
{
	InEditorData->RigVMGraphModifiedEvent.RemoveAll(this);
	InEditorData->RigVMGraphModifiedEvent.AddUObject(this, &UUAFRigVMAssetEntry::HandleRigVMGraphModifiedEvent);
}

bool UUAFRigVMAssetEntry::IsAsset() const
{
	// Entries are considered assets to allow using the asset logic for save dialogs, etc.
	// Also, they return true even if pending kill, in order to show up as deleted in these dialogs.
	return IsPackageExternal() && !GetPackage()->HasAnyFlags(RF_Transient) && !HasAnyFlags(RF_Transient | RF_ClassDefaultObject);
}

#if WITH_EDITOR

void UUAFRigVMAssetEntry::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	BroadcastModified(EAnimNextEditorDataNotifType::PropertyChanged);
}

void UUAFRigVMAssetEntry::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		BroadcastModified(EAnimNextEditorDataNotifType::UndoRedo);
	}
}

void UUAFRigVMAssetEntry::PostEditUndo()
{
	// Note: Dont call super as that will default to a generic property changed event

	BroadcastModified(EAnimNextEditorDataNotifType::UndoRedo);
}

#endif

void UUAFRigVMAssetEntry::BroadcastModified(EAnimNextEditorDataNotifType InType)
{
	if (!IsValid(this))
	{
		return;
	}
	
	if(UUAFRigVMAssetEditorData* EditorData = Cast<UUAFRigVMAssetEditorData>(GetOuter()))
	{
		EditorData->BroadcastModified(InType, this);
	}
}

#undef LOCTEXT_NAMESPACE
