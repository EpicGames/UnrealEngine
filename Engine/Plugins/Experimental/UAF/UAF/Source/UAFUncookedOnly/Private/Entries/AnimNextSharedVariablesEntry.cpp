// Copyright Epic Games, Inc. All Rights Reserved.

#include "Entries/AnimNextSharedVariablesEntry.h"

#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif
#include "UAFCompilationScope.h"
#include "UncookedOnlyUtils.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"
#include "Entries/AnimNextVariableEntry.h"
#include "Logging/StructuredLog.h"
#include "Modules/ModuleManager.h"
#include "Param/ParamType.h"
#include "Variables/AnimNextSharedVariables.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextSharedVariablesEntry)

#define LOCTEXT_NAMESPACE "AnimNextSharedVariablesEntry"

void UUAFSharedVariablesEntry::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		// Make sure Asset and Struct are loaded
		if (Asset)
		{
			UUAFSharedVariables* NonConstAsset = const_cast<UUAFSharedVariables*>(Asset.Get());
			NonConstAsset->ConditionalPreload();
		}

		if (Struct)
		{
			UScriptStruct* NonConstStruct = const_cast<UScriptStruct*>(Struct.Get());
			NonConstStruct->ConditionalPreload();
		}
	}
#endif
}

void UUAFSharedVariablesEntry::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	// Make sure Asset and Struct are loaded
	if (Asset)
	{
		UUAFSharedVariables* NonConstAsset = const_cast<UUAFSharedVariables*>(Asset.Get());
		NonConstAsset->ConditionalPostLoad();
	}
	
	if (Struct)
	{
		UScriptStruct* NonConstStruct = const_cast<UScriptStruct*>(Struct.Get());
		NonConstStruct->ConditionalPostLoad();
	}
#endif
}

void UUAFSharedVariablesEntry::Initialize(UUAFRigVMAssetEditorData* InEditorData)
{
	Super::Initialize(InEditorData);

	if(Asset)
	{
		UUAFSharedVariables_EditorData* EditorData = UE::UAF::UncookedOnly::FUtils::GetEditorData<UUAFSharedVariables_EditorData>(Asset.Get());
		EditorData->ModifiedDelegate.AddUObject(this, &UUAFSharedVariablesEntry::HandleAssetModified);
		
		TWeakObjectPtr<UUAFRigVMAssetEditorData> WeakEditorData = EditorData;
		EditorData->RecompileRequiredChangedEvent.AddUObject(this, &UUAFSharedVariablesEntry::HandleAssetRequiresCompilation, WeakEditorData);
	}
	else if (Struct)
	{
#if WITH_LIVE_CODING
		if (ILiveCodingModule* LiveCoding = FModuleManager::LoadModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME))
		{
			LiveCoding->GetOnPatchCompleteDelegate().AddUObject(this, &UUAFSharedVariablesEntry::HandlePatchComplete);
		}
#endif
	}
	// todo: handle RigVMAsset modification
}

FName UUAFSharedVariablesEntry::GetEntryName() const
{
	switch (Type)
	{
	case EAnimNextSharedVariablesType::Asset:
		return Asset ? Asset->GetFName() : NAME_None;
	case EAnimNextSharedVariablesType::Struct:
		return Struct ? Struct->GetFName() : NAME_None;
	case EAnimNextSharedVariablesType::RigVMAsset:
		return RigVMAsset ? RigVMAsset.GetObject()->GetFName() : NAME_None;
	default:
		checkNoEntry();
	}
	
	return NAME_None;
}

FText UUAFSharedVariablesEntry::GetDisplayName() const
{
	switch (Type)
	{
	case EAnimNextSharedVariablesType::Asset:
		return Asset ? FText::FromName(Asset->GetFName()) : LOCTEXT("InvalidSharedVariables", "Invalid Shared Variables");
	case EAnimNextSharedVariablesType::Struct:
		return Struct ? Struct->GetDisplayNameText() : LOCTEXT("InvalidSharedVariables", "Invalid Shared Variables");
	case EAnimNextSharedVariablesType::RigVMAsset:
		return RigVMAsset ? FText::FromName(RigVMAsset.GetObject()->GetFName()) : LOCTEXT("InvalidSharedVariables", "Invalid Shared Variables");
	default:
		checkNoEntry();
	}

	return FText::GetEmpty();
}

FText UUAFSharedVariablesEntry::GetDisplayNameTooltip() const
{
	switch (Type)
	{
	case EAnimNextSharedVariablesType::Asset:
		return Asset ? FText::FromName(Asset->GetFName()) : LOCTEXT("InvalidSharedVariablesTooltip", "Invalid or deleted Shared Variables");
	case EAnimNextSharedVariablesType::Struct:
		return Struct ? FText::FromString(Struct->GetPathName()) : LOCTEXT("InvalidSharedVariables", "Invalid Shared Variables");
	case EAnimNextSharedVariablesType::RigVMAsset:
		return RigVMAsset ? FText::FromString(RigVMAsset.GetObject()->GetPathName()) : LOCTEXT("InvalidSharedVariables", "Invalid Shared Variables");
	default:
		checkNoEntry();
	}

	return FText::GetEmpty();
}

void UUAFSharedVariablesEntry::SetAsset(const UUAFSharedVariables* InAsset, bool bSetupUndoRedo)
{
	check(InAsset != nullptr);

	if(bSetupUndoRedo)
	{
		Modify();
	}

	Type = EAnimNextSharedVariablesType::Asset;
	Asset = InAsset;
	ObjectPath = FSoftObjectPath(Asset);
	Struct = nullptr;
	RigVMAsset = nullptr;
}

void UUAFSharedVariablesEntry::SetRigVMAsset(const IRigVMRuntimeAssetInterface* InRigVMAsset, bool bSetupUndoRedo)
{
	check(InRigVMAsset != nullptr);

	if(bSetupUndoRedo)
	{
		Modify();
	}

	Type = EAnimNextSharedVariablesType::RigVMAsset;
	const UObject* Object = Cast<UObject>(InRigVMAsset); 
	RigVMAsset = Object;
	ObjectPath = FSoftObjectPath(Object);
	Struct = nullptr;
	Asset = nullptr;
}

const UUAFSharedVariables* UUAFSharedVariablesEntry::GetAsset() const
{
	return Type == EAnimNextSharedVariablesType::Asset ? Asset : nullptr;
}

FSoftObjectPath UUAFSharedVariablesEntry::GetObjectPath() const
{
	return ObjectPath;
}

void UUAFSharedVariablesEntry::SetStruct(const UScriptStruct* InStruct, bool bSetupUndoRedo)
{
	check(InStruct != nullptr);

	if(bSetupUndoRedo)
	{
		Modify();
	}

	Type = EAnimNextSharedVariablesType::Struct;
	Struct = InStruct;
	ObjectPath = FSoftObjectPath(Struct);
	Asset = nullptr;
	RigVMAsset = nullptr;
}

const UScriptStruct* UUAFSharedVariablesEntry::GetStruct() const
{
	return Type == EAnimNextSharedVariablesType::Struct ? Struct : nullptr;
}

void UUAFSharedVariablesEntry::HandleAssetModified(UUAFRigVMAssetEditorData* InEditorData, EAnimNextEditorDataNotifType InType, UObject* InSubject)
{
	switch(InType)
	{
	case EAnimNextEditorDataNotifType::UndoRedo:
	case EAnimNextEditorDataNotifType::EntryAdded:
	case EAnimNextEditorDataNotifType::EntryRemoved:
	case EAnimNextEditorDataNotifType::EntryRenamed:
	case EAnimNextEditorDataNotifType::EntryAccessSpecifierChanged:
	case EAnimNextEditorDataNotifType::VariableTypeChanged:
	case EAnimNextEditorDataNotifType::VariableDefaultValueChanged:
	case EAnimNextEditorDataNotifType::VariableBindingChanged:
		if(UUAFRigVMAsset* OuterAsset = GetTypedOuter<UUAFRigVMAsset>())
		{
			// Recompile the owning asset (which references the modified asset through this entry)
			UE::UAF::UncookedOnly::Compilation::RequestAssetCompilation(OuterAsset);
		}
		break;
	default:
		break;
	}
}

void UUAFSharedVariablesEntry::HandleAssetRequiresCompilation(TWeakObjectPtr<UUAFRigVMAssetEditorData> WeakEditorData) const
{
	if(UUAFRigVMAssetEditorData* EditorData = GetTypedOuter<UUAFRigVMAssetEditorData>())
	{
		if (UUAFRigVMAssetEditorData* SharedAssetEditorData = WeakEditorData.Get())
		{
			if (SharedAssetEditorData->bVMRecompilationRequired)
			{
				EditorData->SetVMRecompilationRequired(true);
			}
		}
	}
}

#if WITH_LIVE_CODING
void UUAFSharedVariablesEntry::HandlePatchComplete()
{
	if(UUAFRigVMAssetEditorData* EditorData = GetTypedOuter<UUAFRigVMAssetEditorData>())
	{
		EditorData->RequestAutoVMRecompilation();
	}
}
#endif

#undef LOCTEXT_NAMESPACE
