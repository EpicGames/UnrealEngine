// Copyright Epic Games, Inc. All Rights Reserved.

#include "Entries/AnimNextVariableEntry.h"

#include "AnimNextRigVMAssetEditorData.h"
#include "UncookedOnlyUtils.h"
#include "Logging/StructuredLog.h"
#include "Module/AnimNextModule.h"
#include "Variables/AnimNextUniversalObjectLocatorBindingData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextVariableEntry)

const FLazyName IUAFRigVMVariableInterface::ValueName("Value");

UAnimNextVariableEntry::UAnimNextVariableEntry()
{
	Guid = FGuid::NewGuid();
}

FAnimNextParamType UAnimNextVariableEntry::GetExportType() const
{
	return GetType();
}

FName UAnimNextVariableEntry::GetExportName() const
{
	return GetVariableName();
}

EAnimNextExportAccessSpecifier UAnimNextVariableEntry::GetExportAccessSpecifier() const
{
	return Access;
}

void UAnimNextVariableEntry::SetExportAccessSpecifier(EAnimNextExportAccessSpecifier InAccessSpecifier, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	};

	Access = InAccessSpecifier;

	BroadcastModified(EAnimNextEditorDataNotifType::EntryAccessSpecifierChanged);
}

FAnimNextParamType UAnimNextVariableEntry::GetType() const
{
	return Type;
}

FName UAnimNextVariableEntry::GetEntryName() const
{
	return ParameterName;
}

bool UAnimNextVariableEntry::SetType(const FAnimNextParamType& InType, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	}

	Type = InType;

	DefaultValue.Reset();
	DefaultValue.AddProperties({ FPropertyBagPropertyDesc(IUAFRigVMVariableInterface::ValueName, Type.GetContainerType(), Type.GetValueType(), Type.GetValueTypeObject()) });

	BroadcastModified(EAnimNextEditorDataNotifType::VariableTypeChanged);

	return true;
}

bool UAnimNextVariableEntry::SetDefaultValue(TConstArrayView<uint8> InValue, bool bSetupUndoRedo)
{
	check(!InValue.IsEmpty());

	if(bSetupUndoRedo)
	{
		Modify();
	}

	const FPropertyBagPropertyDesc* Desc = DefaultValue.FindPropertyDescByName(IUAFRigVMVariableInterface::ValueName);
	if(Desc == nullptr)
	{
		UE_LOGFMT(LogAnimation, Error, "UAnimNextVariableEntry::SetDefaultValue: Could not find default value in property bag");
		return false;
	}

	check(Desc->CachedProperty);
	if(Desc->CachedProperty->GetElementSize() != InValue.Num())
	{
		UE_LOGFMT(LogAnimation, Error, "UAnimNextVariableEntry::SetDefaultValue: Mismatched buffer sizes");
		return false;
	}

	uint8* DestPtr = Desc->CachedProperty->ContainerPtrToValuePtr<uint8>(DefaultValue.GetMutableValue().GetMemory());
	const uint8* SrcPtr = InValue.GetData();
	Desc->CachedProperty->CopyCompleteValue(DestPtr, SrcPtr);

	BroadcastModified(EAnimNextEditorDataNotifType::VariableDefaultValueChanged);

	return true;
}

bool UAnimNextVariableEntry::SetDefaultValueFromString(const FString& InDefaultValue, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	}

	if(DefaultValue.SetValueSerializedString(IUAFRigVMVariableInterface::ValueName, InDefaultValue) != EPropertyBagResult::Success)
	{
		UE_LOGFMT(LogAnimation, Error, "UAnimNextVariableEntry::SetDefaultValueFromString: Could not set value from string");
		return false;
	}

	BroadcastModified(EAnimNextEditorDataNotifType::VariableDefaultValueChanged);

	return true;
}

FName UAnimNextVariableEntry::GetVariableName() const
{
	return ParameterName;
}

void UAnimNextVariableEntry::SetVariableName(FName InName, bool bSetupUndoRedo)
{
	SetEntryName(InName, bSetupUndoRedo);
}

const FInstancedPropertyBag& UAnimNextVariableEntry::GetPropertyBag() const
{
	return DefaultValue;
}

bool UAnimNextVariableEntry::GetDefaultValue(const FProperty*& OutProperty, TConstArrayView<uint8>& OutValue) const
{
	const FPropertyBagPropertyDesc* Desc = DefaultValue.FindPropertyDescByName(IUAFRigVMVariableInterface::ValueName);
	if(Desc == nullptr)
	{
		UE_LOGFMT(LogAnimation, Error, "UAnimNextVariableEntry::GetDefaultValue: Could not find default value in property bag");
		return false;
	}

	check(Desc->CachedProperty);
	OutProperty = Desc->CachedProperty;
	const uint8* ValuePtr = Desc->CachedProperty->ContainerPtrToValuePtr<uint8>(DefaultValue.GetValue().GetMemory());
	OutValue = TConstArrayView<uint8>(ValuePtr, Desc->CachedProperty->GetElementSize());
	return true;
}

bool UAnimNextVariableEntry::GetDefaultValueString(FString& OutValueString) const
{
	const FPropertyBagPropertyDesc* Desc = DefaultValue.FindPropertyDescByName(IUAFRigVMVariableInterface::ValueName);
	if(Desc == nullptr)
	{
		UE_LOGFMT(LogAnimation, Error, "UAnimNextVariableEntry::GetDefaultValueString: Could not find default value in property bag");
		return false;
	}

	check(Desc->CachedProperty);
	Desc->CachedProperty->ExportTextItem_InContainer(OutValueString, DefaultValue.GetValue().GetMemory(), nullptr, nullptr, 0);

	return true;
}

void UAnimNextVariableEntry::SetBindingType(UScriptStruct* InBindingTypeStruct, bool bSetupUndoRedo)
{
	check(InBindingTypeStruct == nullptr || InBindingTypeStruct->IsChildOf(FAnimNextVariableBindingData::StaticStruct()));

	if(bSetupUndoRedo)
	{
		Modify();
	}

	if(InBindingTypeStruct)
	{
		Binding.BindingData.InitializeAsScriptStruct(InBindingTypeStruct);
	}
	else
	{
		Binding.BindingData.Reset();
	}

	BroadcastModified(EAnimNextEditorDataNotifType::VariableBindingChanged);
}

void UAnimNextVariableEntry::SetBinding(TInstancedStruct<FAnimNextVariableBindingData>&& InBinding, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	}

	Binding.BindingData = MoveTemp(InBinding);

	BroadcastModified(EAnimNextEditorDataNotifType::VariableBindingChanged);
}

TConstStructView<FAnimNextVariableBindingData> UAnimNextVariableEntry::GetBinding() const
{
	return Binding.BindingData;
}

FStringView UAnimNextVariableEntry::GetVariableCategory() const
{
	return Category;
}

void UAnimNextVariableEntry::SetVariableCategory(FStringView InCategoryName, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	}

	Category = InCategoryName;
	
	BroadcastModified(EAnimNextEditorDataNotifType::VariableCategoryChanged);
}

void UAnimNextVariableEntry::SetEntryName(FName InName, bool bSetupUndoRedo)
{
	if(bSetupUndoRedo)
	{
		Modify();
	}

	ParameterName = InName;
	BroadcastModified(EAnimNextEditorDataNotifType::EntryRenamed);
}

FText UAnimNextVariableEntry::GetDisplayName() const
{
	return FText::FromName(ParameterName);
}

FText UAnimNextVariableEntry::GetDisplayNameTooltip() const
{
	return FText::FromString(Comment);
}

void UAnimNextVariableEntry::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextModuleRefactor)
		{
			// Add a property for this type
			DefaultValue.Reset();
			DefaultValue.AddProperties({ FPropertyBagPropertyDesc(IUAFRigVMVariableInterface::ValueName, Type.GetContainerType(), Type.GetValueType(), Type.GetValueTypeObject()) });

			// Copy any default value from the module's defaults, if found
			UUAFRigVMAsset* Asset = GetTypedOuter<UUAFRigVMAsset>();
			FString FullName = Asset->GetPathName() + TEXT(":") + GetVariableName().ToString();
			const FPropertyBagPropertyDesc* OldDesc = Asset->VariableDefaults.GetPropertyBagStruct() ? Asset->VariableDefaults.GetPropertyBagStruct()->FindPropertyDescByName(*FullName) : nullptr;
			const FPropertyBagPropertyDesc* NewDesc = DefaultValue.GetPropertyBagStruct() ? DefaultValue.GetPropertyBagStruct()->FindPropertyDescByName(IUAFRigVMVariableInterface::ValueName) : nullptr;
			if (OldDesc && NewDesc)
			{
				const void* OldValue = OldDesc->CachedProperty->ContainerPtrToValuePtr<void>(Asset->VariableDefaults.GetValue().GetMemory());
				void* NewValue = NewDesc->CachedProperty->ContainerPtrToValuePtr<void>(DefaultValue.GetMutableValue().GetMemory());
				NewDesc->CachedProperty->CopyCompleteValue(NewValue, OldValue);
			}
		}

		if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::UAFVariablesGuid)
		{
			Guid = FGuid::NewGuid();
		}
	}
}

void UAnimNextVariableEntry::PostLoad()
{
	Super::PostLoad();
}

#if WITH_EDITOR

void UAnimNextVariableEntry::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if(PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(UAnimNextVariableEntry, Type))
	{
		SetType(Type);
	}
	else
	{
		// Call super to broadcast general changed event
		Super::PostEditChangeProperty(PropertyChangedEvent);
	}
}

#endif
