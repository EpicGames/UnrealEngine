// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/AnimNextModule_EditorData.h"

#include "Compilation/AnimNextGetGraphCompileContext.h"
#include "Compilation/AnimNextProcessGraphCompileContext.h"
#include "ExternalPackageHelper.h"
#include "UncookedOnlyUtils.h"
#include "Module/AnimNextModule.h"
#include "Entries/AnimNextEventGraphEntry.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Entries/AnimNextVariableEntry.h"
#include "String/ParseTokens.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LinkerLoad.h"
#include "Variables/AnimNextUniversalObjectLocatorBindingData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextModule_EditorData)

TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> UUAFSystem_EditorData::GetEntryClasses() const
{
	static const TSubclassOf<UUAFRigVMAssetEntry> Classes[] =
	{
		UAnimNextEventGraphEntry::StaticClass(),
		UAnimNextVariableEntry::StaticClass(),
		UUAFSharedVariablesEntry::StaticClass(),
	};
	
	return Classes;
}

void UUAFSystem_EditorData::OnPreCompileAsset(FRigVMCompileSettings& InSettings)
{
	using namespace UE::UAF::UncookedOnly;

	UUAFSystem* Module = FUtils::GetAsset<UUAFSystem>(this);

	Module->Dependencies.Empty();
}

void UUAFSystem_EditorData::OnPreCompileGetProgrammaticGraphs(const FRigVMCompileSettings& InSettings, FAnimNextGetGraphCompileContext& OutCompileContext)
{
	using namespace UE::UAF::UncookedOnly;

	FUtils::CompileVariableBindings(InSettings, FUtils::GetAsset<UUAFSystem>(this), OutCompileContext.GetMutableProgrammaticGraphs());
}

void UUAFSystem_EditorData::OnPreCompileProcessGraphs(const FRigVMCompileSettings& InSettings, FAnimNextProcessGraphCompileContext& OutCompileContext)
{
	using namespace UE::UAF::UncookedOnly;

	UUAFSystem* Module = FUtils::GetAsset<UUAFSystem>(this);

	// Copy dependencies
	Module->Dependencies = Dependencies;
}

void UUAFSystem_EditorData::CustomizeNewAssetEntry(UUAFRigVMAssetEntry* InNewEntry) const
{
	Super::CustomizeNewAssetEntry(InNewEntry);
	
	UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(InNewEntry);
	if(VariableEntry == nullptr)
	{
		return;
	}

	VariableEntry->SetBindingType(FAnimNextUniversalObjectLocatorBindingData::StaticStruct(), false);
}
