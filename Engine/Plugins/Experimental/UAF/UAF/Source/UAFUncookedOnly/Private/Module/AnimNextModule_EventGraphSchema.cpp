// Copyright Epic Games, Inc. All Rights Reserved.

#include "Module/AnimNextEventGraphSchema.h"
#include "AnimNextExecuteContext.h"

bool UAnimNextEventGraphSchema::SupportsUnitFunction_NoLock(URigVMController* InController, const FRigVMFunction* InUnitFunction, FRigVMRegistryHandle& InRegistry) const
{
	if(const UScriptStruct* FunctionExecuteContextStruct = InUnitFunction->GetExecuteContextStruct())
	{
		if(FunctionExecuteContextStruct == FAnimNextExecuteContext::StaticStruct())
		{
			// Disallow trait stacks in event graphs
			// TODO: This should probably be using the regular ExecuteContext filtering (i.e. trait stacks should use a different execute context)
			static const UScriptStruct* TraitStackStruct = FindObjectChecked<UScriptStruct>(nullptr, TEXT("/Script/UAFAnimGraph.RigUnit_AnimNextTraitStack"));
			if(InUnitFunction->Struct && InUnitFunction->Struct->IsChildOf(TraitStackStruct))
			{
				return false;
			}
		}
	}

	return Super::SupportsUnitFunction_NoLock(InController, InUnitFunction, InRegistry);
}
