// Copyright Epic Games, Inc. All Rights Reserved.

#include "UAFCompilationScope.h"

#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Editor.h"
#include "StatusBarSubsystem.h"
#include "UncookedOnlyUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"

#define LOCTEXT_NAMESPACE "UAFCompilationScope"

namespace UE::UAF::UncookedOnly
{

struct FCompileJobThreadData
{
	FText JobName;
	TMap<TWeakObjectPtr<const UObject>, FCompilerResultsLog> PerObjectCompileLog;
	TArray<TWeakObjectPtr<UUAFRigVMAsset>> CompiledAssets;
	int32 ScopeDepth = 0;
	double StartTimeInSeconds = 0;
};

static thread_local FCompileJobThreadData GCompileJobData;

void Compilation::RequestAssetCompilation(UUAFRigVMAsset* InAsset)
{
	FCompilationScope Scope(InAsset);
	UUAFRigVMAssetEditorData* EditorData = FUtils::GetEditorData<UUAFRigVMAssetEditorData>(InAsset);
	EditorData->RequestVMRecompilation();
}

void Compilation::RequestAssetCompilation(TConstArrayView<UUAFRigVMAsset*> InAssets)
{
	if (InAssets.Num())
	{
		FCompilationScope Scope(InAssets);

		for (UUAFRigVMAsset* Asset : InAssets)
		{
			check(Asset);
			UUAFRigVMAssetEditorData* EditorData = FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
			EditorData->RequestVMRecompilation();
		}
	}
}

FCompilationScope::FCompilationScope(const FText& InJobName)
	: FCompilationScope(InJobName, TConstArrayView<UUAFRigVMAsset*>())
{
}

FCompilationScope::FCompilationScope(UUAFRigVMAsset* InAsset)
	: FCompilationScope(FText::Format(LOCTEXT("AssetCompilationFormat", "Compiling {0}"), FText::FromName(InAsset->GetFName())), InAsset)
{
}

FCompilationScope::FCompilationScope(TConstArrayView<UUAFRigVMAsset*> InAssets)
{
	FString AssetsList;	
	for (const UUAFRigVMAsset* Asset : InAssets)
	{
		if (AssetsList.Len())
		{
			AssetsList.Append(TEXT(", "));
		}
		
		AssetsList.Append(Asset->GetName());
	}
	
	ProcessAssets(FText::Format(LOCTEXT("MultiAssetCompilationFormat", "Compiling ({0})"), FText::FromString(AssetsList)), InAssets);
}

FCompilationScope::FCompilationScope(const FText& InJobName, UUAFRigVMAsset* InAsset)
	: FCompilationScope(InJobName, {&InAsset, 1})
{
	check(InAsset);
}

FCompilationScope::FCompilationScope(const FText& InJobName, TConstArrayView<UUAFRigVMAsset*> InAssets)
{
	ProcessAssets(InJobName, InAssets);
}

FCompilationScope::~FCompilationScope()
{
	FCompileJobThreadData* ThreadData = &GCompileJobData;

	if(ThreadData->ScopeDepth == 1)
	{
		if(ThreadData->CompiledAssets.Num())
		{
			TArray<double> PerAssetTimeCompileTimeSpan;
			for (int32 AssetIndex = 0; AssetIndex < ThreadData->CompiledAssets.Num(); ++AssetIndex)
			{
				TWeakObjectPtr<UUAFRigVMAsset> WeakAsset = ThreadData->CompiledAssets[AssetIndex];
				const double StartAssetTime = FPlatformTime::Seconds();
				if (UUAFRigVMAsset* Asset = WeakAsset.Get())
				{
					UUAFRigVMAssetEditorData* EditorData = FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
					if(EditorData != nullptr && !EditorData->bSuspendCompilationNotifications)
					{
						EditorData->DecrementVMRecompileBracket();
					}
				}
				const double FinishAssetTime = FPlatformTime::Seconds();
				PerAssetTimeCompileTimeSpan.Add(FinishAssetTime - StartAssetTime);
			}

			FMessageLog MessageLog("AnimNextCompilerResults");

			MessageLog.NewPage(FText::Format(LOCTEXT("MessageLogPageFormat", "{0}: {1}"), ThreadData->JobName, FText::AsDateTime(FDateTime::UtcNow())));

			int32 NumErrors = 0;
			int32 NumWarnings = 0;

			for (const auto& LogPair : ThreadData->PerObjectCompileLog)
			{
				NumErrors += LogPair.Value.NumErrors;
				NumWarnings += LogPair.Value.NumWarnings;
			}

			// Print summary
			FNumberFormattingOptions TimeFormat;
			TimeFormat.MaximumFractionalDigits = 2;
			TimeFormat.MinimumFractionalDigits = 0;
			TimeFormat.UseGrouping = false;
			
			const double ElapsedSeconds = FPlatformTime::Seconds() - ThreadData->StartTimeInSeconds;
			FFormatNamedArguments Args;
			Args.Add(TEXT("JobName"), ThreadData->JobName);
			Args.Add(TEXT("CompileTimeMs"), ElapsedSeconds * 1000.f);
			Args.Add(TEXT("NumAssets"), ThreadData->CompiledAssets.Num());

			const int32 AssetMessageIndentationLevel = ThreadData->PerObjectCompileLog.Num() == 1 ? 0 : 1;

			FText JobOverViewMessage;
			if (NumErrors > 0)
			{
				Args.Add(TEXT("NumErrors"), NumErrors);
				Args.Add(TEXT("NumWarnings"),  NumWarnings);
				JobOverViewMessage = FText::Format(LOCTEXT("CompileFailed", "[{JobName}] compilation has failed. {NumErrors} {NumErrors}|plural(one=Error,other=Errors), {NumWarnings} {NumWarnings}|plural(one=Warning,other=Warnings) [{NumAssets} {NumAssets}|plural(one=asset,other=assets) in {CompileTimeMs} ms]"), MoveTemp(Args));
			}
			else if(NumWarnings > 0)
			{
				Args.Add(TEXT("NumWarnings"),  NumWarnings);
				JobOverViewMessage = FText::Format(LOCTEXT("CompileWarning", "[{JobName}] compilation was successful. {NumWarnings} {NumWarnings}|plural(one=Warning,other=Warnings) [{NumAssets} {NumAssets}|plural(one=asset,other=assets) in {CompileTimeMs} ms]"), MoveTemp(Args));
			}
			else
			{
				JobOverViewMessage = FText::Format(LOCTEXT("CompileSuccess", "[{JobName}] compilation was successful! [{NumAssets} {NumAssets}|plural(one=asset,other=assets) in {CompileTimeMs} ms]"), MoveTemp(Args));
			}
			
			MessageLog.Info(JobOverViewMessage);
			
			for (const auto& LogPair : ThreadData->PerObjectCompileLog)
			{
				const int32 AssetIndex = ThreadData->CompiledAssets.IndexOfByKey(LogPair.Key);
			
				TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info);
				Message->AddToken(FUObjectToken::Create(LogPair.Key.Get()));
				Message->AddText(FText::Format(LOCTEXT("AssetCompileTimeFormat", "in {0} ms."), FText::AsNumber(AssetIndex != INDEX_NONE ? PerAssetTimeCompileTimeSpan[AssetIndex] * 1000.f : 0.f, &TimeFormat)));
				Message->SetIndentationLevel(AssetMessageIndentationLevel);
				MessageLog.AddMessage(Message);
				
				if (LogPair.Value.Messages.Num())
				{
					for (auto LogMessage : LogPair.Value.Messages)
					{
						LogMessage->SetIndentationLevel(AssetMessageIndentationLevel + 1);
						MessageLog.AddMessage(LogMessage);
					}
					
					NumErrors += LogPair.Value.NumErrors;
					NumWarnings += LogPair.Value.NumWarnings;
				}
			}
			
			// Broadcast compilation finished so reallocation can occur
			for (TWeakObjectPtr<UUAFRigVMAsset> WeakAsset : ThreadData->CompiledAssets)
			{
				UUAFRigVMAsset* Asset = WeakAsset.Get();
				if (Asset == nullptr)
				{
					continue;
				}

				UUAFRigVMAssetEditorData* EditorData = FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
				if (EditorData == nullptr)
				{
					continue;
				}

				Asset->CompilationState = [&EditorData]()
				{
					if (EditorData->bErrorsDuringCompilation)
					{
						return EAnimNextRigVMAssetState::CompiledWithErrors;
					}

					if (EditorData->bWarningsDuringCompilation)
					{
						return EAnimNextRigVMAssetState::CompiledWithWarnings;
					}

					return EAnimNextRigVMAssetState::CompiledWithSuccess;
				}();

				if (!EditorData->bSuspendCompilationNotifications)
				{
					UUAFRigVMAsset::OnCompileJobFinished().Broadcast(Asset);
					EditorData->OnCompileJobFinished();
				}
			}
		}

		ThreadData->CompiledAssets.Empty();

		ThreadData->PerObjectCompileLog.Reset();
		ThreadData->JobName = FText::GetEmpty();
		ThreadData->StartTimeInSeconds = 0.0;
	}

	--ThreadData->ScopeDepth;
}

FCompilerResultsLog& FCompilationScope::GetLogForObject(const UObject* InObject)
{
	check(InObject);

	FCompileJobThreadData* ThreadData = &GCompileJobData;
	return ThreadData->PerObjectCompileLog.FindOrAdd(InObject);
}

void FCompilationScope::ProcessAssets(const FText& InJobName, TConstArrayView<UUAFRigVMAsset*> InAssets)
{
	FCompileJobThreadData* ThreadData = &GCompileJobData;
	
	if (ThreadData->ScopeDepth == 0)
	{
		ThreadData->JobName = InJobName;
		ThreadData->StartTimeInSeconds = FPlatformTime::Seconds();
	}
		
	++ThreadData->ScopeDepth;

	for(UUAFRigVMAsset* Asset : InAssets)
	{
		check(Asset);
		UUAFRigVMAssetEditorData* EditorData = FUtils::GetEditorData<UUAFRigVMAssetEditorData>(Asset);
		if(EditorData == nullptr)
		{
			continue;
		}

		if (!ThreadData->CompiledAssets.Contains(Asset))
		{
			ThreadData->CompiledAssets.Add(Asset);
			EditorData->ClearErrorInfoForAllEdGraphs();

			if (!EditorData->bSuspendCompilationNotifications)
			{
				EditorData->OnCompileJobStarted();
				UUAFRigVMAsset::OnCompileJobStarted().Broadcast(Asset);
				EditorData->IncrementVMRecompileBracket();
			}
		}
	}
}

}

#undef LOCTEXT_NAMESPACE
