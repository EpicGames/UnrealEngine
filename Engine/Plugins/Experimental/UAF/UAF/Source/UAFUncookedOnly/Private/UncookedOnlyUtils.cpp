// Copyright Epic Games, Inc. All Rights Reserved.

#include "UncookedOnlyUtils.h"

#include "K2Node_CallFunction.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Compilation/AnimNextGetVariableCompileContext.h"
#include "Module/AnimNextModule.h"
#include "Module/AnimNextModule_EditorData.h"
#include "Serialization/MemoryReader.h"
#include "RigVMCore/RigVM.h"
#include "AnimNextRigVMAsset.h"
#include "AnimNextRigVMAssetEditorData.h"
#include "Entries/AnimNextRigVMAssetEntry.h"
#include "AnimNextUncookedOnlyModule.h"
#include "IAnimNextRigVMExportInterface.h"
#include "Logging/StructuredLog.h"
#include "AnimNextAssetWorkspaceAssetUserData.h"
#include "IWorkspaceEditor.h"
#include "IWorkspaceEditorModule.h"
#include "Misc/EnumerateRange.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Variables/AnimNextProgrammaticVariable.h"
#include "Variables/IVariableBindingType.h"
#include "Variables/RigUnit_CopyModuleProxyVariables.h"
#include "WorkspaceAssetRegistryInfo.h"
#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Variables/AnimNextSharedVariables.h"
#include "Variables/AnimNextSharedVariables_EditorData.h"
#include "Variables/AnimNextVariableReference.h"
#include "EdGraph/EdGraphPin.h"
#include "Logging/MessageLog.h"
#include "RigVMFunctions/Execution/RigVMFunction_UserDefinedEvent.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Variables/AnimNextSoftVariableReference.h"
#include "AnimNextExports.h"
#include "UAFCompilationScope.h"
#include "AnimNextSharedVariableNode.h"
#include "ScopedTransaction.h"
#include "Misc/UObjectToken.h"
#include "Styling/SlateIconFinder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ScopedTransaction.h"
#include "Misc/UObjectToken.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Compilation/AnimNextGetFunctionHeaderCompileContext.h"
#include "UObject/PropertyIterator.h"

#define LOCTEXT_NAMESPACE "AnimNextUncookedOnlyUtils"

namespace UE::UAF::UncookedOnly
{

TAutoConsoleVariable<bool> CVarDumpProgrammaticGraphs(
	TEXT("UAF.DumpProgrammaticGraphs"),
	false,
	TEXT("When true the transient programmatic graphs will be automatically opened for any that are generated."));
	
TAutoConsoleVariable<bool> CVarValidatePropertyBagAgainstExternalVariables(
	TEXT("UAF.ValidatePropertyBagAgainstExternalVariables"),
	true,
	TEXT("When true generated (compiled) property bag its properties will be validated against result of calling GetExternalVariables to enforce variable order."));

void FUtils::RecreateVM(UUAFRigVMAsset* InAsset)
{
	if (InAsset->VM == nullptr)
	{
		InAsset->VM = NewObject<URigVM>(InAsset, TEXT("VM"), RF_NoFlags);
	}
	InAsset->VM->Reset(InAsset->ExtendedExecuteContext);
	InAsset->RigVM = InAsset->VM; // Local serialization
}

FInstancedPropertyBag FUtils::MakePropertyBagForEditorData(const UUAFRigVMAssetEditorData* InEditorData, const FAnimNextGetVariableCompileContext& InCompileContext)
{
	ensureMsgf(InEditorData == InCompileContext.GetOwningAssetEditorData(), TEXT("Crossing RigVM asset with a different asset's compile context. Expect incorrect memory layout / variable generation"));

	struct FStructEntryInfo
	{
		FName Name;
		FAnimNextParamType Type;
		EAnimNextExportAccessSpecifier AccessSpecifier = EAnimNextExportAccessSpecifier::Private;
		TConstArrayView<uint8> Value;
		EPropertyFlags PropertyFlags = CPF_NativeAccessSpecifierPrivate | CPF_Edit;
		FGuid Guid;
	};

	// Gather all variables in this asset.
	TMap<FName, int32> EntryInfoIndexMap;
	TArray<FStructEntryInfo> StructEntryInfos;
	const TArray<FAnimNextProgrammaticVariable>& ProgrammaticVariables = InCompileContext.GetProgrammaticVariables();
	StructEntryInfos.Reserve(InEditorData->Entries.Num() + ProgrammaticVariables.Num());
	int32 NumPublicVariables = 0;

	UUAFRigVMAsset* Asset = GetAsset<UUAFRigVMAsset>(InEditorData);
	auto AddVariable = [Asset, &NumPublicVariables, &StructEntryInfos, &EntryInfoIndexMap, &InCompileContext](FName InName, const FAnimNextParamType& InType, EAnimNextExportAccessSpecifier InAccess, TConstArrayView<uint8> InValue, EPropertyFlags InFlags, FGuid InStableGUID)
	{
		const FRigVMTemplateArgumentType RigVmArgumentType = InType.ToRigVMTemplateArgument();
		if (!RigVmArgumentType.IsValid() || RigVmArgumentType.IsUnknownType())
		{
			InCompileContext.GetAssetCompileContext().Error(Asset, LOCTEXT("InvalidVariableTypeFound", "@@ Variable '{0}' has unsupported variable type '{1}'"), FText::FromName(InName), FText::FromString(InType.ToString()));
			return;
		}

		// Check for conflicts
		if(EntryInfoIndexMap.Contains(InName))
		{
			InCompileContext.GetAssetCompileContext().Error(Asset, LOCTEXT("DuplicateVariableFound", "@@ Variable '{0}' with duplicate name found"), FText::FromName(InName));
			return;
		}

		if(InAccess == EAnimNextExportAccessSpecifier::Public)
		{
			NumPublicVariables++;
		}

		int32 Index = StructEntryInfos.Add(
			{
				InName,
				FAnimNextParamType(InType.GetValueType(), InType.GetContainerType(), InType.GetValueTypeObject()),
				InAccess,
				InValue,
				InFlags,
				InStableGUID
			});

		EntryInfoIndexMap.Add(InName, Index);
	};

	for(UUAFRigVMAssetEntry* Entry : InEditorData->Entries)
	{
		if(const UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(Entry))
		{
			EPropertyFlags Flags = CPF_Edit;
			if (VariableEntry->GetExportAccessSpecifier() == EAnimNextExportAccessSpecifier::Private)
			{
				Flags |= CPF_NativeAccessSpecifierPrivate;
			}
			else
			{
				Flags |= CPF_NativeAccessSpecifierPublic;
			}

			AddVariable(
				VariableEntry->GetExportName(),
				VariableEntry->GetType(),
				VariableEntry->GetExportAccessSpecifier(),
				TConstArrayView<uint8>(VariableEntry->GetValuePtr(), VariableEntry->GetType().GetSize()),
				Flags,
				VariableEntry->GetGuid());
		}
	}

	for (const FAnimNextProgrammaticVariable& Variable : ProgrammaticVariables)
	{
		AddVariable(
			Variable.Name,
			Variable.Type,
			EAnimNextExportAccessSpecifier::Private,
			TConstArrayView<uint8>(Variable.GetValuePtr(), Variable.Type.GetSize()),
			CPF_AdvancedDisplay | CPF_NativeAccessSpecifierPrivate,
			Variable.GetGuid()
		);
	}

	// Sort by size, largest first, for better packing
	StructEntryInfos.Sort([](const FStructEntryInfo& InLHS, const FStructEntryInfo& InRHS)
	{
		if(InLHS.Type.GetSize() != InRHS.Type.GetSize())
		{
			return InLHS.Type.GetSize() > InRHS.Type.GetSize();
		}
		else
		{
			return InLHS.Name.LexicalLess(InRHS.Name);
		}
	});

	FInstancedPropertyBag VariableDefaults;

	if(StructEntryInfos.Num() > 0)
	{
		// Build PropertyDescs and values to batch-create the property bag
		TArray<FPropertyBagPropertyDesc> PropertyDescs;
		PropertyDescs.Reserve(StructEntryInfos.Num());
		TArray<TConstArrayView<uint8>> Values;
		Values.Reserve(StructEntryInfos.Num());

		for (const FStructEntryInfo& StructEntryInfo : StructEntryInfos)
		{
			FPropertyBagPropertyDesc& Desc = PropertyDescs.Emplace_GetRef(StructEntryInfo.Name, StructEntryInfo.Type.ContainerType, StructEntryInfo.Type.ValueType, StructEntryInfo.Type.ValueTypeObject, StructEntryInfo.PropertyFlags);
			Desc.ID = StructEntryInfo.Guid;
			Values.Add(StructEntryInfo.Value);
		}

		// Create new property bags and migrate
		EPropertyBagResult Result = VariableDefaults.ReplaceAllPropertiesAndValues(PropertyDescs, Values);
		check(Result == EPropertyBagResult::Success);
	}

	return VariableDefaults;
}

void FUtils::CompileVariables(const FRigVMCompileSettings& InSettings, UUAFRigVMAsset* InAsset, FAnimNextGetVariableCompileContext& OutCompileContext)
{
	check(InAsset);

	UUAFRigVMAssetEditorData* EditorData = GetEditorData<UUAFRigVMAssetEditorData>(InAsset);

	// Gather programmatic variables regenerated each compile
	// TODO when we need support for this, the variables will have to be available for FUtils::GetExternalVariables as well to keep them in sync with property bag
	//EditorData->OnPreCompileGetProgrammaticVariables(InSettings, OutCompileContext);

	// Generate the internal property bag
	InAsset->VariableDefaults = MakePropertyBagForEditorData(EditorData, OutCompileContext);

	TSet<const UUAFRigVMAsset*> SharedVariableAssets;
	TSet<const UScriptStruct*> SharedVariableStructs;
	TSet<TScriptInterface<const IRigVMRuntimeAssetInterface>> SharedVariableRigVMAssets;
	
	// Set of SharedVariable Assets and structs, retrieved by recursively following references
	TSet<const UUAFRigVMAsset*> RecursiveSharedVariableAssets;
	TSet<const UScriptStruct*> RecursiveSharedVariableStructs;
	TSet<TScriptInterface<const IRigVMRuntimeAssetInterface>> RecursiveSharedVariableRigVMAssets;

	for(UUAFRigVMAssetEntry* Entry : EditorData->Entries)
	{
		if(UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(Entry))
		{
			auto SharedAssetEntryError = [InAsset, EditorData, SharedVariablesEntry, &OutCompileContext](const FText& InMessage)
			{
				TWeakObjectPtr<UUAFRigVMAssetEditorData> WeakEditorData = EditorData;
				TWeakObjectPtr<UUAFSharedVariablesEntry> WeakEntry = SharedVariablesEntry;
					
				TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error);
				Message->AddToken(FUObjectToken::Create(InAsset));
				Message->AddText(LOCTEXT("InvalidSharedVariableEntryFound", "Invalid shared variables entry found ({0})"), FText::FromString(SharedVariablesEntry->GetObjectPath().ToString()));
				Message->AddToken(FActionToken::Create(LOCTEXT("RemoveSharedVariablesEntry", "Remove Shared Variables entry"), LOCTEXT("RemoveSharedVariablesEntryDesc", "Removes the Shared Variables entry from the asset."), FOnActionTokenExecuted::CreateLambda([WeakEditorData, WeakEntry]()
					{
						UUAFRigVMAssetEditorData* EditorData = WeakEditorData.Get();
						UUAFSharedVariablesEntry* Entry = WeakEntry.Get();
						if (EditorData && Entry)
						{
							FScopedTransaction Transaction(LOCTEXT("RemoveSharedVariablesTransactionDesc", "Removing Shared Variables entry"));
							EditorData->RemoveEntry(Entry);
						}						
					
					}), true));						


				OutCompileContext.GetAssetCompileContext().Message(Message);
			};
			
			switch (SharedVariablesEntry->GetType())
			{
			case EAnimNextSharedVariablesType::Asset:
				{
					const UUAFSharedVariables* SharedVariablesAsset = SharedVariablesEntry->GetAsset();
					if (SharedVariablesAsset == nullptr)
					{
						SharedAssetEntryError(FText::Format(LOCTEXT("InvalidSharedVariableAssetFound", "Invalid shared variables asset found ({0})"), FText::FromString(SharedVariablesEntry->GetObjectPath().ToString())));
						continue;
					}

					if (SharedVariableAssets.Contains(SharedVariablesAsset))
					{
						SharedAssetEntryError(FText::Format(LOCTEXT("DuplicateSharedVariableAssetFound", "Duplicate shared variables asset found ({0})"), FText::FromString(SharedVariablesEntry->GetObjectPath().ToString())));
						continue;
					}
					
					RetrieveReferencedSharedVariableSources(SharedVariablesAsset, RecursiveSharedVariableAssets, RecursiveSharedVariableStructs, RecursiveSharedVariableRigVMAssets);
					
					if (RecursiveSharedVariableAssets.Remove(InAsset) != 0)
					{
						OutCompileContext.GetAssetCompileContext().Warning(InAsset, LOCTEXT("RecursiveCircularReference", "@@ is recursively referenced by its own Shared Variable Asset reference {0} - chain should be broken."), FText::FromString(SharedVariablesEntry->GetObjectPath().ToString()));
					}

					SharedVariableAssets.Add(SharedVariablesAsset);
					break;
				}
			case EAnimNextSharedVariablesType::Struct:
				{
					const UScriptStruct* SharedVariablesStruct = SharedVariablesEntry->GetStruct();
					if (SharedVariablesStruct == nullptr)
					{
						SharedAssetEntryError(FText::Format(LOCTEXT("InvalidSharedVariableStructFound", "Invalid shared variables struct found ({0})"), FText::FromString(SharedVariablesEntry->GetObjectPath().ToString())));
						continue;
					}

					if (SharedVariableStructs.Contains(SharedVariablesStruct))
					{
						SharedAssetEntryError(FText::Format(LOCTEXT("DuplicateSharedVariableStructFound", "Duplicate shared variables struct found ({0})"), FText::FromString(SharedVariablesEntry->GetObjectPath().ToString())));
						continue;
					}

					SharedVariableStructs.Add(SharedVariablesStruct);
					break;
				}
			case EAnimNextSharedVariablesType::RigVMAsset:
				{
					TScriptInterface<const IRigVMRuntimeAssetInterface> RigVMRuntimeAssetInterface = SharedVariablesEntry->GetRigVMAsset();
					if (RigVMRuntimeAssetInterface == nullptr)
					{
						SharedAssetEntryError(FText::Format(LOCTEXT("InvalidSharedVariableRigVMAssetFound", "Invalid shared variables rig VM asset found ({0})"), FText::FromString(SharedVariablesEntry->GetObjectPath().ToString())));
						continue;
					}

					if (SharedVariableRigVMAssets.Contains(RigVMRuntimeAssetInterface))
					{
						SharedAssetEntryError(FText::Format(LOCTEXT("DuplicateSharedVariableRigVMAssetFound", "Duplicate shared variables struct found ({0})"), FText::FromString(SharedVariablesEntry->GetObjectPath().ToString())));
						continue;
					}

					SharedVariableRigVMAssets.Add(RigVMRuntimeAssetInterface);
					break;
				}
			}
		}
	}

	SharedVariableAssets.Append(RecursiveSharedVariableAssets);
	SharedVariableStructs.Append(RecursiveSharedVariableStructs);
	SharedVariableRigVMAssets.Append(RecursiveSharedVariableRigVMAssets);

	// Set shared variable assets/structs before we setup variables as GetExternalVariablesImpl relies on these
	if (SharedVariableAssets.Num() > 0)
	{
		InAsset->ReferencedVariableAssets = SharedVariableAssets.Array();
	}
	else
	{
		InAsset->ReferencedVariableAssets.Empty();
	}

	if (SharedVariableStructs.Num() > 0)
	{
		InAsset->ReferencedVariableStructs = SharedVariableStructs.Array();
	}
	else
	{
		InAsset->ReferencedVariableStructs.Empty();
	}

	if (SharedVariableRigVMAssets.Num() > 0)
	{
		InAsset->ReferencedVariableRigVMAssets = SharedVariableRigVMAssets.Array();
	}
	else
	{
		InAsset->ReferencedVariableRigVMAssets.Empty();
	}

	// Now rebuild combined the combined property bag we used for stable properties 
	InAsset->CombinedPropertyBag = EditorData->GenerateCombinedPropertyBag(InSettings, OutCompileContext);

	// Rebuild external variables
	if (InAsset->CombinedPropertyBag.GetNumPropertiesInBag() > 0)
	{
		InAsset->VM->SetExternalVariableDefs(InAsset->GetExternalVariablesImpl(false));
	}
	else
	{
		InAsset->VM->ClearExternalVariables(InAsset->ExtendedExecuteContext);
	}

	EditorData->bHasCompiledVariables = true;

	if (CVarValidatePropertyBagAgainstExternalVariables.GetValueOnAnyThread())
	{
		// Compilation might have failed
		if (const UPropertyBag* PropertyBag = InAsset->CombinedPropertyBag.GetPropertyBagStruct())
		{
			TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs = PropertyBag->GetPropertyDescs();
			TArray<FRigVMExternalVariable> ExternalVariables;
			GetExternalVariables(EditorData, ExternalVariables);

			const int32 NumDescriptions = PropertyDescs.Num();
			if (ensure(NumDescriptions == ExternalVariables.Num()))
			{
				for (int32 Index = 0; Index < NumDescriptions; ++Index)
				{
					const FPropertyBagPropertyDesc& Desc = PropertyDescs[Index];
					const FRigVMExternalVariable& Variable = ExternalVariables[Index];
					ensureMsgf(Desc.Name == Variable.GetName(),
						TEXT("Mismatch between compiled property bag variable name %s and external variable entry at same index %s"),
						*Desc.Name.ToString(), *Variable.GetName().ToString());
				}
			}
		}
	}

	// Validate injection site
	InAsset->DefaultInjectionSite.Reset();

	if(!EditorData->DefaultInjectionSiteReference.IsNone())
	{
		if (!InAsset->DefaultInjectionSite.IsValid())
		{
			OutCompileContext.GetAssetCompileContext().Error(InAsset, LOCTEXT("MissingDefaultInjectionSiteError", "@@ Could not find default injection site: {0}"), FText::FromName(EditorData->DefaultInjectionSiteReference.GetName()));
		}
		else
		{
			// Warn about using the deprecated name-based path
			if (EditorData->DefaultInjectionSiteReference.GetObject() == nullptr)
			{
				OutCompileContext.GetAssetCompileContext().Warning(InAsset, LOCTEXT("DeprecatedNamedInjectionSiteWarning", "@@ Default injection site '{0}' uses a name-based reference. Please select a full reference to an asset's variable."), FText::FromName(EditorData->DefaultInjectionSiteReference.GetName()));
			}

			InAsset->DefaultInjectionSite = EditorData->DefaultInjectionSiteReference; 
		}
	}
}

// Helper function to sort ExternalVariables according to property-size/name without needing to rely on RigVMRegistry (which is too slow)
template<typename SortType, typename ValueType, class SortPredicate>
void SortByValuesArray(TArrayView<SortType> ToSort, TConstArrayView<ValueType> Values, const SortPredicate& SortPred)
{
	const int32 NumValues = Values.Num();
	if (!ensureMsgf(ToSort.Num() == NumValues, TEXT("Values array must have same length as ToSort array, but did not (%d vs %d)"), NumValues, ToSort.Num()))
	{
		return;
	}

	// make a reference array of indices, that we will sort based on values
	TArray<int32> Indices;
	Indices.SetNumUninitialized(NumValues);
	for (int32 Idx = 0; Idx < NumValues; ++Idx)
	{
		Indices[Idx] = Idx;
	}
	
	Indices.Sort([&SortPred, &Values](int32 IdxA, int32 IdxB)
	{
		return SortPred(Values[IdxA], Values[IdxB]);
	});

	// swap the ToSort array to follow the ordering of the reference array
	for (int32 Idx = 0; Idx < NumValues; ++Idx)
	{
		int32 SwapFromIndex = Indices[Idx];
		while (SwapFromIndex < Idx)
		{
			SwapFromIndex = Indices[SwapFromIndex];
		}

		if (SwapFromIndex != Idx)
		{
			Swap(ToSort[Idx], ToSort[SwapFromIndex]);
		}
	}
}

void FUtils::GetExternalVariables(const UUAFRigVMAssetEditorData* EditorData, TArray<FRigVMExternalVariable>& OutVariables)
{
	GetInternalVariables(EditorData, OutVariables);

	TSet<const UScriptStruct*> ReferencedVariableStructs;
	TSet<const UUAFRigVMAsset*> ReferencedSharedVariableAssets;
	TSet<TScriptInterface<const IRigVMRuntimeAssetInterface>> ReferencedSharedVariableRigVMAssets;

	TSet<const UUAFRigVMAsset*> RecursiveSharedVariableAssets;
	TSet<const UScriptStruct*> RecursiveSharedVariableStructs;
	TSet<TScriptInterface<const IRigVMRuntimeAssetInterface>> RecursiveSharedVariableRigVMAssets;

	for(UUAFRigVMAssetEntry* Entry : EditorData->Entries)
	{
		if(UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(Entry))
		{
			switch (SharedVariablesEntry->GetType())
			{
			case EAnimNextSharedVariablesType::Asset:
				{
					if (const UUAFSharedVariables* SharedVariablesAsset = SharedVariablesEntry->GetAsset())
					{
						RetrieveReferencedSharedVariableSources(SharedVariablesAsset, RecursiveSharedVariableAssets, RecursiveSharedVariableStructs, RecursiveSharedVariableRigVMAssets);
						ReferencedSharedVariableAssets.Add(SharedVariablesAsset);
					}
					break;
				}
			case EAnimNextSharedVariablesType::Struct:
				{
					if (const UScriptStruct* SharedVariablesStruct = SharedVariablesEntry->GetStruct())
					{
						ReferencedVariableStructs.Add(SharedVariablesStruct);
					}
					break;
				}
			case EAnimNextSharedVariablesType::RigVMAsset:
				{
					ReferencedSharedVariableRigVMAssets.Add(SharedVariablesEntry->GetRigVMAsset());
					break;
				}
			default:
				checkNoEntry();
			}
		}
	}

	ReferencedSharedVariableAssets.Append(RecursiveSharedVariableAssets);
	ReferencedVariableStructs.Append(RecursiveSharedVariableStructs);
	ReferencedSharedVariableRigVMAssets.Append(RecursiveSharedVariableRigVMAssets);
	
	// Append each _sorted_ set of referenced shared asset variables
	for (const UUAFRigVMAsset* ReferencedVariableAsset : ReferencedSharedVariableAssets)
	{
		GetInternalVariables(GetEditorData<const UUAFRigVMAssetEditorData>(ReferencedVariableAsset), OutVariables);
	}
	
	// Append each set of referenced shared struct variables, sorted by their property offset (can differ from its declaration)
	for (const UScriptStruct* ReferencedVariableStruct : ReferencedVariableStructs)
	{
		TArray<int32> PropertyOffsets;
		TArray<FRigVMExternalVariable> StructVariables;
		for (TFieldIterator<FProperty> It(ReferencedVariableStruct); It; ++It)
		{
			const FProperty* Property = *It;
			const FGuid Guid = IAnimNextUncookedOnlyModule::Get().GetVariableGuidByName(Property->GetFName(), ReferencedVariableStruct);
			const FRigVMExternalVariable Variable = FRigVMExternalVariable::Make(Guid, Property, nullptr);
			if (!Variable.IsValid(true))
			{
				UE_LOGF(LogRigVM, Error, "%ls: Property '%ls' of type '%ls' is not valid.", *ReferencedVariableStruct->GetName(), *Property->GetName(), *Property->GetCPPType());
				continue;
			}

			StructVariables.Add(Variable);
			PropertyOffsets.Add(It->GetOffset_ForInternal());
		}

		SortByValuesArray(MakeArrayView(StructVariables), MakeConstArrayView(PropertyOffsets),
			[](const int32 OffsetA, const int32 OffsetB) -> bool
				{
					return OffsetA < OffsetB;
				});

		OutVariables.Append(StructVariables);
	}

	for (TScriptInterface<const IRigVMRuntimeAssetInterface> ReferencedVariableRigVMAsset : ReferencedSharedVariableRigVMAssets)
	{
		if (ReferencedVariableRigVMAsset.GetInterface())
		{
			TArray<FRigVMExternalVariable> RigVMExternalVariables = const_cast<IRigVMRuntimeAssetInterface*>(ReferencedVariableRigVMAsset.GetInterface())->GetExternalVariables();
			OutVariables.Append(RigVMExternalVariables);
		}
	}
}

void FUtils::GetInternalVariables(const UUAFRigVMAssetEditorData* EditorData, TArray<FRigVMExternalVariable>& OutVariables)
{
	TArray<TPair<FName, FAnimNextParamType>> VariableInfo;
	TArray<FRigVMExternalVariable> AssetVariables;
	EditorData->ForEachEntryOfType<UAnimNextVariableEntry>([AssetName = EditorData->GetAssetName(), &AssetVariables, &VariableInfo](const UAnimNextVariableEntry* Entry)
	{
		const FRigVMExternalVariable Variable = FRigVMExternalVariable::Make(Entry->GetGuid(), Entry->GetVariableName(), Entry->GetType().ToString(), const_cast<UObject*>(Entry->GetType().GetValueTypeObject()), Entry->Access == EAnimNextExportAccessSpecifier::Public, false);
		if (!Variable.IsValid(true))
		{
			UE_LOGF(LogRigVM, Error, "%ls: Property '%ls' of type '%ls' is not valid.", *AssetName, *Entry->GetName(), *Variable.GetExtendedCPPType().ToString());
			return true;
		}
		
		VariableInfo.Add({Entry->GetVariableName(), Entry->GetType() });

		AssetVariables.Add(Variable);
		return true;
	});

	// Handle programmatically generated variables
	{
		FAnimNextRigVMAssetCompileContext CompileContext = { EditorData };
		FRigVMCompileSettings Settings;
		const FAnimNextGetVariableCompileContext GetVariableCompileContext = EditorData->GetVariableCompileContext(Settings, CompileContext);
		for (const FAnimNextProgrammaticVariable& ProgrammaticVariable : GetVariableCompileContext.GetProgrammaticVariables())
		{
			VariableInfo.Add({ProgrammaticVariable.GetVariableName(), ProgrammaticVariable.GetType()});
			AssetVariables.Add(FRigVMExternalVariable::Make(ProgrammaticVariable.GetGuid(), ProgrammaticVariable.GetVariableName(), ProgrammaticVariable.GetType().ToString(), const_cast<UObject*>(ProgrammaticVariable.GetType().GetValueTypeObject()), false, false));
		}
	}

	SortByValuesArray(MakeArrayView(AssetVariables), MakeConstArrayView(VariableInfo),
		[](const TPair<FName, FAnimNextParamType>& A, const TPair<FName, FAnimNextParamType>& B)-> bool
			{
				if (A.Value.GetSize() != B.Value.GetSize())
				{
					return A.Value.GetSize() > B.Value.GetSize();
				}

				return A.Key.LexicalLess(B.Key);
			});
	OutVariables.Append(AssetVariables);
}

void FUtils::RetrieveReferencedSharedVariableSources(const UUAFRigVMAsset* InAsset, TSet<const UUAFRigVMAsset*>& ReferencedVariableAssets, TSet<const UScriptStruct*>& ReferencedVariableStructs, TSet<TScriptInterface<const IRigVMRuntimeAssetInterface>>& ReferencedVariableRigVMAssets)
{
	const UUAFRigVMAssetEditorData* EditorData = GetEditorData<UUAFRigVMAssetEditorData>(InAsset);
	EditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>(
		[&ReferencedVariableAssets, &ReferencedVariableStructs, &ReferencedVariableRigVMAssets](const UUAFSharedVariablesEntry* Entry)
			{
				if (Entry->Asset)
				{
					ensure(!Entry->HasAllFlags(RF_NeedPostLoad));
					bool IsAlreadyInSet = false;
					ReferencedVariableAssets.Add(Entry->Asset, &IsAlreadyInSet);
					if (!IsAlreadyInSet)
					{
						// We added an element, recurse
						RetrieveReferencedSharedVariableSources(Entry->Asset.Get(), ReferencedVariableAssets, ReferencedVariableStructs,
							ReferencedVariableRigVMAssets);
					}
				}
				else if (Entry->Struct)
				{
					ReferencedVariableStructs.Add(Entry->Struct);
				}
				else if (Entry->RigVMAsset)
				{
					ReferencedVariableRigVMAssets.Add(Entry->RigVMAsset);
				}
				return true;
			});
}

void FUtils::CompileVariableBindings(const FRigVMCompileSettings& InSettings, UUAFRigVMAsset* InAsset, TArray<URigVMGraph*>& OutGraphs)
{
	CompileVariableBindingsInternal(InSettings, InAsset, OutGraphs, true);
	CompileVariableBindingsInternal(InSettings, InAsset,  OutGraphs, false);
}

void FUtils::CompileVariableBindingsInternal(const FRigVMCompileSettings& InSettings, UUAFRigVMAsset* InAsset, TArray<URigVMGraph*>& OutGraphs, bool bInThreadSafe)
{
	check(InAsset);

	FModule& Module = FModuleManager::LoadModuleChecked<FModule>("UAFUncookedOnly");
	UUAFRigVMAssetEditorData* EditorData = GetEditorData(InAsset);
	TMap<IVariableBindingType*, TArray<IVariableBindingType::FBindingGraphInput>> BindingGroups;

	for(const UUAFRigVMAssetEntry* Entry : EditorData->Entries)
	{
		auto AddBindingForEntry = [bInThreadSafe, &Module, &BindingGroups](const IUAFRigVMVariableInterface* InVariable)
		{
			TConstStructView<FAnimNextVariableBindingData> Binding = InVariable->GetBinding();
			if(!Binding.IsValid() || !Binding.Get<FAnimNextVariableBindingData>().IsValid())
			{
				return;
			}

			if(Binding.Get<FAnimNextVariableBindingData>().IsThreadSafe() != bInThreadSafe)
			{
				return;
			}

			TSharedPtr<IVariableBindingType> BindingType = Module.FindVariableBindingType(Binding.GetScriptStruct());
			if(!BindingType.IsValid())
			{
				return;
			}

			TArray<IVariableBindingType::FBindingGraphInput>& Group = BindingGroups.FindOrAdd(BindingType.Get());
			FRigVMTemplateArgumentType RigVMArg = InVariable->GetType().ToRigVMTemplateArgument();
			Group.Add({ InVariable->GetVariableName(), RigVMArg.GetBaseCPPType(), RigVMArg.CPPTypeObject, Binding});
		};
		
		if(const IUAFRigVMVariableInterface* Variable = Cast<IUAFRigVMVariableInterface>(Entry))
		{
			AddBindingForEntry(Variable);
		}
		else if (const UUAFSharedVariablesEntry* SharedVariables = Cast<UUAFSharedVariablesEntry>(Entry))
		{
			if (const UUAFRigVMAsset* SharedVariablesAsset = SharedVariables->GetAsset())
			{
				const UUAFRigVMAssetEditorData* SharedVariablesEditorData = GetEditorData<const UUAFRigVMAssetEditorData>(SharedVariablesAsset);
				for(const UUAFRigVMAssetEntry* SharedVariablesEntry : SharedVariablesEditorData->Entries)
				{
					if(const IUAFRigVMVariableInterface* SharedVariable = Cast<IUAFRigVMVariableInterface>(SharedVariablesEntry))
					{
						AddBindingForEntry(SharedVariable);
					}
				}
			}
		}
	}

	const bool bHasBindings = BindingGroups.Num() > 0;
	const bool bHasPublicVariablesToCopy = EditorData->IsA<UUAFSystem_EditorData>() && EditorData->HasPublicVariables() && bInThreadSafe;
	if(!bHasBindings && !bHasPublicVariablesToCopy)
	{
		// Nothing to do here
		return;
	}

	URigVMGraph* BindingGraph = NewObject<URigVMGraph>(EditorData, NAME_None, RF_Transient);

	FRigVMClient* VMClient = EditorData->GetRigVMClient();
	URigVMController* Controller = VMClient->GetOrCreateController(BindingGraph);

	FRigVMControllerASTLinkCheckGuard LinkCheckGuard(Controller);

	UScriptStruct* BindingsNodeType = bInThreadSafe ? FRigUnit_AnimNextExecuteBindings_WT::StaticStruct() : FRigUnit_AnimNextExecuteBindings_GT::StaticStruct();
	URigVMNode* ExecuteBindingsNode = Controller->AddUnitNode(BindingsNodeType, FRigVMStruct::ExecuteName, FVector2D::ZeroVector, FString(), false);
	if(ExecuteBindingsNode == nullptr)
	{
		InSettings.ReportError(TEXT("Could not spawn Execute Bindings node"));
		return;
	}
	URigVMPin* ExecuteBindingsExecPin = ExecuteBindingsNode->FindPin(FRigVMStruct::ExecuteContextName.ToString());
	if(ExecuteBindingsExecPin == nullptr)
	{
		InSettings.ReportError(TEXT("Could not find execute pin on Execute Bindings node"));
		return;
	}
	URigVMPin* ExecPin = ExecuteBindingsExecPin;

	// Copy public vars in the WT event
	if(bHasPublicVariablesToCopy && bInThreadSafe)
	{
		URigVMNode* CopyProxyVariablesNode = Controller->AddUnitNode(FRigUnit_CopyModuleProxyVariables::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(200, 0.0f), FString(), false);
		if(CopyProxyVariablesNode == nullptr)
		{
			InSettings.ReportError(TEXT("Could not spawn Copy System Proxy Variables node"));
			return;
		}
		URigVMPin* CopyProxyVariablesExecPin = CopyProxyVariablesNode->FindPin(FRigVMStruct::ExecuteContextName.ToString());
		if(CopyProxyVariablesExecPin == nullptr)
		{
			InSettings.ReportError(TEXT("Could not find execute pin on Copy System Proxy Variables node"));
			return;
		}
		bool bLinkAdded = Controller->AddLink(ExecuteBindingsExecPin, CopyProxyVariablesExecPin, false);
		if(!bLinkAdded)
		{
			InSettings.ReportError(TEXT("Could not link Copy System Proxy Variables node"));
			return;
		}
		ExecPin = CopyProxyVariablesExecPin;
	}

	IVariableBindingType::FBindingGraphFragmentArgs Args;
	Args.Event = BindingsNodeType;
	Args.Controller = Controller;
	Args.BindingGraph = BindingGraph;
	Args.ExecTail = ExecPin;
	Args.bThreadSafe = bInThreadSafe;

	FVector2D Location(0.0f, 0.0f);
	for(const TPair<IVariableBindingType*, TArray<IVariableBindingType::FBindingGraphInput>>& BindingGroupPair : BindingGroups)
	{
		Args.Inputs = BindingGroupPair.Value;
		BindingGroupPair.Key->BuildBindingGraphFragment(InSettings, Args, ExecPin, Location);
	}

	OutGraphs.Add(BindingGraph);
}

bool FUtils::CanAddSharedVariablesReference(UUAFRigVMAssetEditorData* InEditorData, const UUAFSharedVariables* InSharedVariables, FString* OutErrorMessage)
{
	auto CheckForCircularity = [InEditorData](UUAFSharedVariables_EditorData* SourceEditorData, auto& InCheckForCircularity, TArray<const UUAFSharedVariables_EditorData*>& InOutReferenceChain)
	{
		TArray<const UUAFSharedVariables_EditorData*> ReferenceChain = InOutReferenceChain;
		ReferenceChain.Add(SourceEditorData);
		if(SourceEditorData == InEditorData)
		{
			InOutReferenceChain = ReferenceChain;
			return true;
		}

		for(UUAFRigVMAssetEntry* Entry : SourceEditorData->Entries)
		{
			if(UUAFSharedVariablesEntry* SharedVariablesEntry = Cast<UUAFSharedVariablesEntry>(Entry))
			{
				// SharedVariables could be nullptr as the entry is a Struct based source, or the underlying asset (UUAFSharedVariablesEntry::ObjectPath) has been deleted
				if (const UUAFSharedVariables* SharedVariables = SharedVariablesEntry->GetAsset())
				{
					UUAFSharedVariables_EditorData* EditorData = GetEditorData<UUAFSharedVariables_EditorData>(SharedVariables);
					
					if(InCheckForCircularity(EditorData, InCheckForCircularity, ReferenceChain))
					{
						InOutReferenceChain = ReferenceChain;
						return true;
					}
				}
			}
		}

		return false;
	};
	
	if (UUAFSharedVariables_EditorData* SharedVariablesEditorData = GetEditorData<UUAFSharedVariables_EditorData>(InSharedVariables))
	{
		TArray<const UUAFSharedVariables_EditorData*> EditorDataReferenceChain;
		EditorDataReferenceChain.Add(Cast<UUAFSharedVariables_EditorData>(InEditorData));
		const bool bResult = CheckForCircularity(SharedVariablesEditorData, CheckForCircularity, EditorDataReferenceChain);

		if (bResult && OutErrorMessage)
		{
			FString DependencyString;
			for (const UUAFSharedVariables_EditorData* EditorData : EditorDataReferenceChain)
			{
				if (!DependencyString.IsEmpty())
				{
					DependencyString += TEXT("\n\t");
				}
				DependencyString += GetAsset<UUAFSharedVariables>(EditorData)->GetPackage()->GetFName().ToString();
			}
			*OutErrorMessage = FString::Printf(TEXT("Failed to add Shared Variables dependency as this would introduce a circular reference:\n\t%s"), *DependencyString);
		}

		return !bResult;
	}

	return false;	
}

bool FUtils::RemoveSharedVariablesReference(UUAFRigVMAssetEditorData* InEditorData, const FSoftObjectPath& SharedVariablesPath)
{
	const FText AssetNameText = FText::FromName(InEditorData->GetTypedOuter<UUAFRigVMAsset>()->GetFName());
	if (UUAFSharedVariablesEntry* SharedVariableEntryToRemove = Cast<UUAFSharedVariablesEntry>(InEditorData->FindEntry(SharedVariablesPath.GetAssetFName())))
	{
		const FText SharedVariablesAssetNameText = FText::FromName(SharedVariablesPath.GetAssetFName());
		const FText FormattedMessage = FText::Format(LOCTEXT("RemovingSharedAssetEntryFormat", "Removing {0} from {1}"), SharedVariablesAssetNameText, AssetNameText);
		FScopedTransaction Transaction(FormattedMessage);
		
		if (const UUAFSharedVariables* SharedVariablesToRemove = SharedVariableEntryToRemove->GetAsset())
		{
			// Replace all referenced variables from the to-be-removed SharedVariables from the referencing Asset
			FAnimNextAssetRegistryExports VariableExports;
			GetExportedVariablesForAsset(SharedVariablesToRemove, VariableExports);
	
			VariableExports.ForEachExportOfType<FAnimNextVariableDeclarationData>([InEditorData, SharedVariablesPath](const FName& Identifier, const FAnimNextVariableDeclarationData& VariableDeclaration) -> bool
			{
				FAnimNextSoftVariableReference VariableReference = FAnimNextSoftVariableReference::FromName(Identifier, SharedVariablesPath);
				FAnimNextSoftVariableReference EmptyReference;
				ReplaceVariableReferences(InEditorData, VariableReference, EmptyReference);
			
				return true;
			});
		}
		else if (const UScriptStruct* Struct = SharedVariableEntryToRemove->GetStruct())
		{
			// Replace all referenced variables from the to-be-removed Native SharedVariables source from the referencing Asset
			for (TFieldIterator<FProperty> It(Struct); It; ++It)
			{
				const FProperty* Property = *It;
				if (Property->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPublic))
				{	
					FAnimNextSoftVariableReference VariableReference = FAnimNextSoftVariableReference::FromName(Property->GetFName(), SharedVariablesPath);
					FAnimNextSoftVariableReference EmptyReference;
					ReplaceVariableReferences(InEditorData, VariableReference, EmptyReference);
				}
			}
		}
	
		// Remove the entry itself
		return InEditorData->RemoveEntry(SharedVariableEntryToRemove);
	}
	
	return false;
}

FWorkspaceOutlinerItemExport FUtils::MakeFunctionExport(const URigVMLibraryNode* FunctionNode, const FWorkspaceOutlinerItemExport& ParentExport)
{
	FWorkspaceOutlinerItemExport Export;
	
	check(FunctionNode);
	if (const URigVMGraph* ContainedModelGraph = FunctionNode->GetContainedGraph())
	{
		if (const UUAFRigVMAssetEditorData* EditorData = FunctionNode->GetTypedOuter<UUAFRigVMAssetEditorData>())
		{
			if (URigVMEdGraph* EditorObject = Cast<URigVMEdGraph>(EditorData->GetEditorObjectForRigVMGraph(ContainedModelGraph)))
			{
				const FSoftObjectPath EditorObjectSoftObjPath = EditorObject;
				ensureMsgf(EditorObjectSoftObjPath.IsSubobject(), TEXT("EditorObject for RigVMFunctionReferenceNode Graph was not a subobject as expected."));

				const FName Identifier = EditorObjectSoftObjPath.IsSubobject() ? *EditorObjectSoftObjPath.GetSubPathUtf8String() : FunctionNode->GetFName();
				Export = FWorkspaceOutlinerItemExport(Identifier, ParentExport);

				Export.GetData().InitializeAsScriptStruct(FAnimNextGraphFunctionOutlinerData::StaticStruct());
				FAnimNextGraphFunctionOutlinerData& FnGraphData = Export.GetData().GetMutable<FAnimNextGraphFunctionOutlinerData>();
				FnGraphData.SoftEditorObject = EditorObject;
			}
		} 
	}
	
	return Export;
}

FWorkspaceOutlinerItemExport FUtils::MakeCollapsedGraphExport(const URigVMCollapseNode* CollapseNode, const FWorkspaceOutlinerItemExport& ParentExport)
{
	FWorkspaceOutlinerItemExport Export;
	if (CollapseNode)
	{		
		if (IRigVMClientHost* OuterHost = CollapseNode->GetImplementingOuter<IRigVMClientHost>())
		{
			if (URigVMEdGraph* EditorObject = Cast<URigVMEdGraph>(OuterHost->GetEditorObjectForRigVMGraph(CollapseNode->GetContainedGraph())))
			{
				const FSoftObjectPath EditorObjectSoftObjPath = EditorObject;
				ensureMsgf(EditorObjectSoftObjPath.IsSubobject(), TEXT("EditorObject for RigVMCollapseNode Graph was not a subobject as expected."));

				const FName Identifier = EditorObjectSoftObjPath.IsSubobject() ? *EditorObjectSoftObjPath.GetSubPathUtf8String() : CollapseNode->GetFName();
				Export = FWorkspaceOutlinerItemExport(Identifier, ParentExport);
				Export.GetData().InitializeAsScriptStruct(FAnimNextCollapseGraphOutlinerData::StaticStruct());
				FAnimNextCollapseGraphOutlinerData& FnGraphData = Export.GetData().GetMutable<FAnimNextCollapseGraphOutlinerData>();
				FnGraphData.SoftEditorObject = EditorObject;
			}
		}
	}
	
	return Export;
}

FWorkspaceOutlinerItemExport FUtils::MakeAssetReferenceExport(const FSoftObjectPath& ReferencedAssetPath, const FWorkspaceOutlinerItemExport& ParentExport, bool bShouldExpand)
{
	check(ReferencedAssetPath.IsValid());
	
	FWorkspaceOutlinerItemExport Export = FWorkspaceOutlinerItemExport(FName(ReferencedAssetPath.ToString()), ParentExport);
	Export.GetData().InitializeAs<FWorkspaceOutlinerAssetReferenceItemData>();
	FWorkspaceOutlinerAssetReferenceItemData& Data = Export.GetData().GetMutable<FWorkspaceOutlinerAssetReferenceItemData>();
	Data.ReferredObjectPath = ReferencedAssetPath;
	Data.bShouldExpandReference = bShouldExpand;
	
	return Export;
}

UUAFRigVMAsset* FUtils::GetAsset(UUAFRigVMAssetEditorData* InEditorData)
{
	check(InEditorData);
	return CastChecked<UUAFRigVMAsset>(InEditorData->GetOuter());
}

UUAFRigVMAssetEditorData* FUtils::GetEditorData(UUAFRigVMAsset* InAsset)
{
	check(InAsset);
	return CastChecked<UUAFRigVMAssetEditorData>(InAsset->EditorData);
}

const UUAFRigVMAssetEditorData* FUtils::GetEditorData(const UUAFRigVMAsset* InAsset)
{
	check(InAsset);
	return CastChecked<UUAFRigVMAssetEditorData>(InAsset->EditorData);
}

FAnimNextParamType FUtils::GetParamTypeFromPinType(const FEdGraphPinType& InPinType)
{
	FAnimNextParamType::EValueType ValueType = FAnimNextParamType::EValueType::None;
	FAnimNextParamType::EContainerType ContainerType = FAnimNextParamType::EContainerType::None;
	UObject* ValueTypeObject = nullptr;

	if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		ValueType = FAnimNextParamType::EValueType::Bool;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		if (UEnum* Enum = Cast<UEnum>(InPinType.PinSubCategoryObject.Get()))
		{
			ValueType = FAnimNextParamType::EValueType::Enum;
			ValueTypeObject = Enum;
		}
		else
		{
			ValueType = FAnimNextParamType::EValueType::Byte;
		}
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Int)
	{
		ValueType = FAnimNextParamType::EValueType::Int32;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Int64)
	{
		ValueType = FAnimNextParamType::EValueType::Int64;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Real)
	{
		if (InPinType.PinSubCategory == UEdGraphSchema_K2::PC_Float)
		{
			ValueType = FAnimNextParamType::EValueType::Float;
		}
		else if (InPinType.PinSubCategory == UEdGraphSchema_K2::PC_Double)
		{
			ValueType = FAnimNextParamType::EValueType::Double;
		}
		else
		{
			ensure(false);	// Reals should be either floats or doubles
		}
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Float)
	{
		ValueType = FAnimNextParamType::EValueType::Float;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Double)
	{
		ValueType = FAnimNextParamType::EValueType::Double;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		ValueType = FAnimNextParamType::EValueType::Name;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_String)
	{
		ValueType = FAnimNextParamType::EValueType::String;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Text)
	{
		ValueType = FAnimNextParamType::EValueType::Text;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Enum)
	{
		ValueType = FAnimNextParamType::EValueType::Enum;
		ValueTypeObject = Cast<UEnum>(InPinType.PinSubCategoryObject.Get());
		ensure(ValueTypeObject);
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		ValueType = FAnimNextParamType::EValueType::Struct;
		ValueTypeObject = Cast<UScriptStruct>(InPinType.PinSubCategoryObject.Get());
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Object || InPinType.PinCategory == UEdGraphSchema_K2::AllObjectTypes)
	{
		ValueType = FAnimNextParamType::EValueType::Object;
		ValueTypeObject = Cast<UClass>(InPinType.PinSubCategoryObject.Get());
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject)
	{
		ValueType = FAnimNextParamType::EValueType::SoftObject;
		ValueTypeObject = Cast<UClass>(InPinType.PinSubCategoryObject.Get());
		ensure(ValueTypeObject);
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
	{
		ValueType = FAnimNextParamType::EValueType::SoftClass;
		ValueTypeObject = Cast<UClass>(InPinType.PinSubCategoryObject.Get());
		ensure(ValueTypeObject);
	}

	if(InPinType.ContainerType == EPinContainerType::Array)
	{
		ContainerType = FAnimNextParamType::EContainerType::Array;
	}
	else if(InPinType.ContainerType == EPinContainerType::Set)
	{
		ensureMsgf(false, TEXT("Set pins are not yet supported"));
	}
	if(InPinType.ContainerType == EPinContainerType::Map)
	{
		ensureMsgf(false, TEXT("Map pins are not yet supported"));
	}
	
	return FAnimNextParamType(ValueType, ContainerType, ValueTypeObject);
}

FEdGraphPinType FUtils::GetPinTypeFromParamType(const FAnimNextParamType& InParamType)
{
	FEdGraphPinType PinType;
	PinType.PinSubCategory = NAME_None;

	// Container type
	switch (InParamType.ContainerType)
	{
	case FAnimNextParamType::EContainerType::Array:
		PinType.ContainerType = EPinContainerType::Array;
		break;
	default:
		PinType.ContainerType = EPinContainerType::None;
	}

	// Value type
	switch (InParamType.ValueType)
	{
	case EPropertyBagPropertyType::Bool:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
		break;
	case EPropertyBagPropertyType::Byte:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
		break;
	case EPropertyBagPropertyType::Int32:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
		break;
	case EPropertyBagPropertyType::Int64:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Int64;
		break;
	case EPropertyBagPropertyType::Float:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
		break;
	case EPropertyBagPropertyType::Double:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
		break;
	case EPropertyBagPropertyType::Name:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
		break;
	case EPropertyBagPropertyType::String:
		PinType.PinCategory = UEdGraphSchema_K2::PC_String;
		break;
	case EPropertyBagPropertyType::Text:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Text;
		break;
	case EPropertyBagPropertyType::Enum:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Enum;
		PinType.PinSubCategoryObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		if (UEnum* Enum = Cast<UEnum>(PinType.PinSubCategoryObject))
		{
			if (Enum->GetMaxEnumValue() <= (int64)std::numeric_limits<uint8>::max())
			{
				// Use byte for BP. It will use the correct picker and enum k2 node.
				PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
			}
		}
		break;
	case EPropertyBagPropertyType::Struct:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		PinType.PinSubCategoryObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		break;
	case EPropertyBagPropertyType::Object:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
		PinType.PinSubCategoryObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		break;
	case EPropertyBagPropertyType::SoftObject:
		PinType.PinCategory = UEdGraphSchema_K2::PC_SoftObject;
		PinType.PinSubCategoryObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		break;
	case EPropertyBagPropertyType::Class:
		PinType.PinCategory = UEdGraphSchema_K2::PC_Class;
		PinType.PinSubCategoryObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		break;
	case EPropertyBagPropertyType::SoftClass:
		PinType.PinCategory = UEdGraphSchema_K2::PC_SoftClass;
		PinType.PinSubCategoryObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		break;
	default:
		break;
	}

	return PinType;
}

FRigVMTemplateArgumentType FUtils::GetRigVMArgTypeFromParamType(const FAnimNextParamType& InParamType)
{
	FRigVMTemplateArgumentType ArgType;

	FString CPPTypeString;

	// Value type
	switch (InParamType.ValueType)
	{
	case EPropertyBagPropertyType::Bool:
		CPPTypeString = RigVMTypeUtils::BoolType;
		break;
	case EPropertyBagPropertyType::Byte:
		CPPTypeString = RigVMTypeUtils::UInt8Type;
		break;
	case EPropertyBagPropertyType::Int32:
		CPPTypeString = RigVMTypeUtils::UInt32Type;
		break;
	case EPropertyBagPropertyType::Int64:
		ensureMsgf(false, TEXT("Unhandled value type %d"), EnumToUnderlyingType(InParamType.ValueType));
		break;
	case EPropertyBagPropertyType::Float:
		CPPTypeString = RigVMTypeUtils::FloatType;
		break;
	case EPropertyBagPropertyType::Double:
		CPPTypeString = RigVMTypeUtils::DoubleType;
		break;
	case EPropertyBagPropertyType::Name:
		CPPTypeString = RigVMTypeUtils::FNameType;
		break;
	case EPropertyBagPropertyType::String:
		CPPTypeString = RigVMTypeUtils::FStringType;
		break;
	case EPropertyBagPropertyType::Text:
		ensureMsgf(false, TEXT("Unhandled value type %d"), EnumToUnderlyingType(InParamType.ValueType));
		break;
	case EPropertyBagPropertyType::Enum:
		CPPTypeString = RigVMTypeUtils::CPPTypeFromEnum(Cast<UEnum>(InParamType.ValueTypeObject.Get()));
		ArgType.CPPTypeObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		break;
	case EPropertyBagPropertyType::Struct:
		CPPTypeString = RigVMTypeUtils::GetUniqueStructTypeName(Cast<UScriptStruct>(InParamType.ValueTypeObject.Get()));
		ArgType.CPPTypeObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		break;
	case EPropertyBagPropertyType::Object:
		CPPTypeString = RigVMTypeUtils::CPPTypeFromObject(Cast<UClass>(InParamType.ValueTypeObject.Get()), RigVMTypeUtils::EClassArgType::AsObject);
		ArgType.CPPTypeObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		break;
	case EPropertyBagPropertyType::SoftObject:
		ensureMsgf(false, TEXT("Unhandled value type %d"), EnumToUnderlyingType(InParamType.ValueType));
		break;
	case EPropertyBagPropertyType::Class:
		CPPTypeString = RigVMTypeUtils::CPPTypeFromObject(Cast<UClass>(InParamType.ValueTypeObject.Get()), RigVMTypeUtils::EClassArgType::AsClass);
		ArgType.CPPTypeObject = const_cast<UObject*>(InParamType.ValueTypeObject.Get());
		break;
	case EPropertyBagPropertyType::SoftClass:
		ensureMsgf(false, TEXT("Unhandled value type %d"), EnumToUnderlyingType(InParamType.ValueType));
		break;
	default:
		ensureMsgf(false, TEXT("Unhandled value type %d"), EnumToUnderlyingType(InParamType.ValueType));
		break;
	}

	// Container type
	switch (InParamType.ContainerType)
	{
	case FAnimNextParamType::EContainerType::None:
		break;
	case FAnimNextParamType::EContainerType::Array:
		CPPTypeString = FString::Printf(RigVMTypeUtils::TArrayTemplate, *CPPTypeString);
		break;
	default:
		ensureMsgf(false, TEXT("Unhandled container type %d"), EnumToUnderlyingType(InParamType.ContainerType));
		break;
	}

	ArgType.CPPType = *CPPTypeString;

	return ArgType;
}

void FUtils::SetupEventGraph(URigVMController* InController, UScriptStruct* InEventStruct, FName InEventName, bool bPrintPythonCommand)
{
	// Clear the graph
	InController->RemoveNodes(InController->GetGraph()->GetNodes());

	if (InEventStruct->IsChildOf(FRigUnit_AnimNextUserEvent::StaticStruct()))
	{
		FRigUnit_AnimNextUserEvent Defaults;
		Defaults.Name = InEventName;
		Defaults.SortOrder = InEventName.GetNumber();
		InController->AddUnitNodeWithDefaults(InEventStruct, Defaults, FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
	}
	else if (InEventStruct == FRigVMFunction_UserDefinedEvent::StaticStruct())
	{
		FRigVMFunction_UserDefinedEvent Defaults;
		Defaults.EventName = InEventName;
		InController->AddUnitNodeWithDefaults(InEventStruct, Defaults, FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
	}
	else
	{
		InController->AddUnitNode(InEventStruct, FRigVMStruct::ExecuteName, FVector2D(0.0f, 0.0f), FString(), false);
	}
}

FAnimNextParamType FUtils::FindVariableType(const FAnimNextVariableReference& InVariableReference)
{
	// If this is likely a struct variable, just resolve & search for the member
	if (const UScriptStruct* Struct = Cast<UScriptStruct>(InVariableReference.GetObject()))
	{
		if (const FProperty* Property = Struct->FindPropertyByName(InVariableReference.GetName()))
		{
			return FAnimNextParamType::FromProperty(Property);
		}
	}
	else if (const UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(InVariableReference.GetObject()))
	{
		UUAFRigVMAssetEditorData* AssetEditorData = CastChecked<UUAFRigVMAssetEditorData>(Asset->EditorData);
		AssetEditorData->ConditionalPreload();
		if (const IUAFRigVMVariableInterface* Variable = Cast<IUAFRigVMVariableInterface>(AssetEditorData->FindEntry(InVariableReference.GetName())))
		{
			return Variable->GetType();
		}
	}
	else if (const IRigVMRuntimeAssetInterface* RigVMAsset = Cast<IRigVMRuntimeAssetInterface>(InVariableReference.GetObject()))
	{
		if (const FProperty* Property = InVariableReference.ResolveProperty())
		{
			return FAnimNextParamType::FromProperty(Property);
		}
	}

	return FAnimNextParamType();
}


FAnimNextParamType FUtils::FindVariableType(const FAnimNextSoftVariableReference& InSoftVariableReference)
{
	// If this is likely a struct variable, just resolve & search for the member
	if (InSoftVariableReference.GetSoftObjectPath().GetLongPackageName().StartsWith(TEXT("/Script/")))
	{
		if (const UScriptStruct* Struct = Cast<UScriptStruct>(InSoftVariableReference.GetSoftObjectPath().TryLoad()))
		{
			if (const FProperty* Property = Struct->FindPropertyByName(InSoftVariableReference.GetName()))
			{
				return FAnimNextParamType::FromProperty(Property);
			}
		}
	}
	else
	{
		// Query the asset registry for the asset
		FAssetData AssetData = IAssetRegistry::GetChecked().GetAssetByObjectPath(InSoftVariableReference.GetSoftObjectPath());

		// If the asset is loaded, just use it
		if (const UUAFRigVMAsset* Asset = Cast<UUAFRigVMAsset>(AssetData.FastGetAsset(false)))
		{
			const UUAFRigVMAssetEditorData* EditorData = GetEditorData<UUAFRigVMAssetEditorData>(Asset);
			FAnimNextAssetRegistryExports Exports;

			FAssetRegistryTagsContextData ContextData(EditorData, EAssetRegistryTagsCaller::Uncategorized);
			const FAssetRegistryTagsContext Context(ContextData);
			GetAssetVariableExports(EditorData, Exports, Context);

			for(const FAnimNextExport& Export : Exports.Exports)
			{
				const FAnimNextVariableDeclarationData* Data = Export.Data.GetPtr<FAnimNextVariableDeclarationData>();
				if(Export.Identifier == InSoftVariableReference.GetName() && Data)
				{
					return Data->Type;
				}
			}
		}
		else
		{
			// Otherwise use AR data to find its exports
			FAnimNextAssetRegistryExports Exports;
			if (GetExportedVariablesForAsset(AssetData, Exports))
			{
				for(const FAnimNextExport& Export : Exports.Exports)
				{
					if(Export.Identifier == InSoftVariableReference.GetName())
					{
						if (const FAnimNextVariableDeclarationData* VariableDeclaration = Export.Data.GetPtr<FAnimNextVariableDeclarationData>())
						{
							return VariableDeclaration->Type;
						}
					}
				}
			}
		}
	}

	return FAnimNextParamType();
}

bool FUtils::GetExportedVariablesForAsset(const FAssetData& InAsset, FAnimNextAssetRegistryExports& OutExports)
{
	return GetExportsOfTypeForAsset<FAnimNextVariableDeclarationData>(InAsset, OutExports);
}

bool FUtils::GetExportedVariablesFromAssetRegistry(TMap<FAssetData, FAnimNextAssetRegistryExports>& OutExports)
{	
	return GetExportsOfTypeFromAssetRegistry<FAnimNextVariableDeclarationData>(OutExports);
}

void FUtils::GetAssetFunctions(const UUAFRigVMAssetEditorData* InEditorData, FRigVMGraphFunctionHeaderArray& OutExports)
{
	for (const FRigVMGraphFunctionData& FunctionData : InEditorData->GraphFunctionStore.PublicFunctions)
	{
		// Note: By default all public functions should get added for compilation in `UUAFRigVMAssetEditorData::BuildFunctionHeadersContext`
		if (FunctionData.CompilationData.IsValid())
		{
			OutExports.Headers.Add(FunctionData.Header);
		}
	}
}

void FUtils::GetAssetPrivateFunctions(const UUAFRigVMAssetEditorData* InEditorData, FRigVMGraphFunctionHeaderArray& OutExports)
{
	for (const FRigVMGraphFunctionData& FunctionData : InEditorData->GraphFunctionStore.PrivateFunctions)
	{
		// Note: We dont check compilation data here as private functions are not compiled if they are not referenced
		OutExports.Headers.Add(FunctionData.Header);
	}
}

bool FUtils::GetExportedFunctionsFromAssetRegistry(FName Tag, TMap<FAssetData, FRigVMGraphFunctionHeaderArray>& OutExports)
{
	TArray<FAssetData> AssetData;
	IAssetRegistry::GetChecked().GetAssetsByTags({ Tag }, AssetData);

	const FArrayProperty* HeadersProperty = CastField<FArrayProperty>(FRigVMGraphFunctionHeaderArray::StaticStruct()->FindPropertyByName(TEXT("Headers")));

	for (const FAssetData& Asset : AssetData)
	{
		const FString TagValue = Asset.GetTagValueRef<FString>(Tag);
		FRigVMGraphFunctionHeaderArray AssetExports;

		if (HeadersProperty->ImportText_Direct(*TagValue, &AssetExports, nullptr, EPropertyPortFlags::PPF_None) != nullptr)
		{
			if (AssetExports.Headers.Num() > 0)
			{
				FRigVMGraphFunctionHeaderArray& AssetArray = OutExports.FindOrAdd(Asset);
				AssetArray.Headers.Append(MoveTemp(AssetExports.Headers));
			}
		}
	}

	return OutExports.Num() > 0;
}

static void AddParamToSet(const FAnimNextExport& InNewParam, TSet<FAnimNextExport>& OutExports)
{
	if(FAnimNextExport* ExistingEntry = OutExports.Find(InNewParam))
	{
		const FAnimNextVariableDeclarationData* NewData = InNewParam.Data.GetPtr<FAnimNextVariableDeclarationData>();
		FAnimNextVariableDeclarationData* ExistingData = ExistingEntry->Data.GetMutablePtr<FAnimNextVariableDeclarationData>();
		check(NewData && ExistingData);
		
		if(ExistingData->Type != NewData->Type)
		{
			UE_LOGFMT(LogAnimation, Warning, "Type mismatch between parameter {ParameterName}. {ParamType1} vs {ParamType1}", InNewParam.Identifier, NewData->Type.ToString(), ExistingData->Type.ToString());
		}
		ExistingData->Flags |= NewData->Flags;
	}
	else
	{
		OutExports.Add(InNewParam);
	}
}

void FUtils::GetAssetVariableExports(const UUAFRigVMAssetEditorData* EditorData, FAnimNextAssetRegistryExports& OutExports, FAssetRegistryTagsContext Context)
{
	OutExports.Exports.Reserve(EditorData->Entries.Num());

	TSet<FAnimNextExport> ExportSet;
	GetAssetVariableExports(EditorData, ExportSet, Context);
	OutExports.Exports.Append(ExportSet.Array());
}

void FUtils::GetAssetVariableExports(const UUAFRigVMAssetEditorData* InEditorData, TSet<FAnimNextExport>& OutExports, FAssetRegistryTagsContext Context, EAnimNextExportedVariableFlags InFlags /*= EAnimNextExportedVariableFlags::NoFlags*/)
{
	for(const UUAFRigVMAssetEntry* Entry : InEditorData->Entries)
	{
		if(const UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(Entry))
		{
			const bool bReferencedVariableEntry = EnumHasAnyFlags(InFlags, EAnimNextExportedVariableFlags::Referenced);
			// Only mark variable as declared when retrieving exports for the asset which actually contains it (non-recursive)
			EAnimNextExportedVariableFlags Flags = bReferencedVariableEntry ? InFlags : EAnimNextExportedVariableFlags::Declared | InFlags;
			if(VariableEntry->GetExportAccessSpecifier() == EAnimNextExportAccessSpecifier::Public)
			{
				Flags |= EAnimNextExportedVariableFlags::Public;
			}

			FAnimNextExport ParameterExport = FAnimNextExport::MakeExport<FAnimNextVariableDeclarationData>(VariableEntry->GetExportName(), VariableEntry->GetExportType(), VariableEntry->GetGuid(), Flags);
			AddParamToSet(ParameterExport, OutExports);
		}
		else if(const UUAFSharedVariablesEntry* DataInterfaceEntry = Cast<UUAFSharedVariablesEntry>(Entry))
		{
			switch(DataInterfaceEntry->Type)
			{
			case EAnimNextSharedVariablesType::Asset:
				if(const UUAFSharedVariables* SharedVariablesAsset = DataInterfaceEntry->Asset)
				{
					UUAFSharedVariables_EditorData* EditorData = GetEditorData<UUAFSharedVariables_EditorData>(SharedVariablesAsset);
					GetAssetVariableExports(EditorData, OutExports, Context, EAnimNextExportedVariableFlags::Referenced);
				}
				break;
			case EAnimNextSharedVariablesType::Struct:
				if(const UScriptStruct* Struct = DataInterfaceEntry->Struct)
				{
					GetStructVariableExports(Struct, OutExports);
				}
				break;
			}
		}

		if(const IUAFRigVMGraphInterface* GraphInterface = Cast<IUAFRigVMGraphInterface>(Entry))
		{
			if (URigVMEdGraph* RigVMEdGraph = GraphInterface->GetEdGraph())
			{
				GetSubGraphVariableExportsRecursive(RigVMEdGraph, OutExports);
			}
		}
	}

	for (const TObjectPtr<URigVMEdGraph>& FunctionGraph : InEditorData->FunctionEdGraphs)
	{
		GetSubGraphVariableExportsRecursive(FunctionGraph, OutExports);
	}
}

void FUtils::GetStructVariableExports(const UScriptStruct* Struct, TSet<FAnimNextExport>& OutExports)
{
	constexpr EAnimNextExportedVariableFlags Flags = EAnimNextExportedVariableFlags::Declared | EAnimNextExportedVariableFlags::Public;
	for (TFieldIterator<FProperty> It(Struct); It; ++It)
	{
		if(const FProperty* Property = *It)
		{
			const FName PropertyName = Property->GetFName();
			FAnimNextExport ParameterExport = FAnimNextExport::MakeExport<FAnimNextVariableDeclarationData>(PropertyName, FAnimNextParamType::FromProperty(Property), GenerateScriptStructPropertyGUID(Property), Flags);
			AddParamToSet(ParameterExport, OutExports);
		}
	}
}

void FUtils::GetAssetWorkspaceExports(const UUAFRigVMAssetEditorData* EditorData, FWorkspaceOutlinerItemExports& OutExports, FAssetRegistryTagsContext Context)
{
	FWorkspaceOutlinerItemExport AssetIdentifier = FWorkspaceOutlinerItemExport(EditorData->GetOuter()->GetFName(), EditorData->GetOuter());

	const int32 RootExportIndex = OutExports.Exports.Num() - 1;

	int32 SharedVariablesGroupIndex = INDEX_NONE;
	auto AddSharedVariablesGroupEntry = [&SharedVariablesGroupIndex, &OutExports, RootExportIndex]()
	{
		const FWorkspaceOutlinerItemExport& ParentExport = OutExports.Exports[RootExportIndex];
		FWorkspaceOutlinerItemExport& GroupExport = OutExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(FName("SharedVariablesGroup"), ParentExport));
		GroupExport.GetData().InitializeAs<FWorkspaceOutlinerGroupItemData>();
		GroupExport.GetData().GetMutable<FWorkspaceOutlinerGroupItemData>().GroupName = TEXT("Shared Variables");
		GroupExport.GetData().GetMutable<FWorkspaceOutlinerGroupItemData>().GroupIcon = *FSlateIconFinder::FindIconBrushForClass(UUAFSharedVariables::StaticClass());

		SharedVariablesGroupIndex = OutExports.Exports.Num() - 1;
	};

	// Referred SharedVariables
	EditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>([&OutExports, &SharedVariablesGroupIndex, AddSharedVariablesGroupEntry](const UUAFSharedVariablesEntry* SharedVariablesEntry)
	{
		if(SharedVariablesEntry)
		{
			if (SharedVariablesGroupIndex == INDEX_NONE)
			{
				AddSharedVariablesGroupEntry();
			}

			const FSoftObjectPath ObjectPath = [SharedVariablesEntry]() -> FSoftObjectPath
			{
				switch(SharedVariablesEntry->Type)
				{
				case EAnimNextSharedVariablesType::Asset:
					{
						const UUAFRigVMAsset* Asset = SharedVariablesEntry->Asset;
						return Asset;
					}
				case EAnimNextSharedVariablesType::Struct:
					{
						const UScriptStruct* Struct = SharedVariablesEntry->Struct;
						return Struct;
					}
				}
				
				return nullptr;
			}();

			if (ObjectPath.IsValid())
			{
				OutExports.Exports.Add(MakeAssetReferenceExport(ObjectPath, OutExports.Exports[SharedVariablesGroupIndex], false));
			}
		}
		return true;
	});


	int32 GraphsGroupIndex = INDEX_NONE;
	auto AddGraphsGroupEntry = [&GraphsGroupIndex, &OutExports, RootExportIndex]()
	{
		const FWorkspaceOutlinerItemExport& ParentExport = OutExports.Exports[RootExportIndex];
		FWorkspaceOutlinerItemExport& GroupExport = OutExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(FName("GraphsGroup"), ParentExport));
		GroupExport.GetData().InitializeAs<FWorkspaceOutlinerGroupItemData>();
		GroupExport.GetData().GetMutable<FWorkspaceOutlinerGroupItemData>().GroupName = TEXT("Graphs");
		GroupExport.GetData().GetMutable<FWorkspaceOutlinerGroupItemData>().GroupIcon = *FAppStyle::GetBrush(TEXT("GraphEditor.EventGraph_24x"));

		GraphsGroupIndex = OutExports.Exports.Num() - 1;
	};

	EditorData->ForEachEntryOfType<IUAFRigVMGraphInterface>([&EditorData, &OutExports, AssetIdentifier, &Context, &GraphsGroupIndex, &AddGraphsGroupEntry](IUAFRigVMGraphInterface* GraphInterface)
	{
		UUAFRigVMAssetEntry* Entry = CastChecked<UUAFRigVMAssetEntry>(GraphInterface);		
		if (Entry->IsHiddenInOutliner())
		{
			if (URigVMEdGraph* RigVMEdGraph = GraphInterface->GetEdGraph())
			{
				GetSubGraphWorkspaceExportsRecursive(EditorData, OutExports, AssetIdentifier, INDEX_NONE, RigVMEdGraph, Context);
			}
		}
		else
		{
			if (GraphsGroupIndex == INDEX_NONE)
			{
				AddGraphsGroupEntry();
			}
			
			FWorkspaceOutlinerItemExport& Export = OutExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(Entry->GetEntryName(), OutExports.Exports[GraphsGroupIndex]));

			Export.GetData().InitializeAsScriptStruct(FAnimNextGraphOutlinerData::StaticStruct());
			FAnimNextGraphOutlinerData& GraphData = Export.GetData().GetMutable<FAnimNextGraphOutlinerData>();
			GraphData.SoftEntryPtr = Entry;

			if (URigVMEdGraph* RigVMEdGraph = GraphInterface->GetEdGraph())
			{					
				const int32 ExportIndex = OutExports.Exports.Num() - 1;
				GetSubGraphWorkspaceExportsRecursive(EditorData, OutExports, OutExports.Exports[ExportIndex], ExportIndex, RigVMEdGraph, Context);
			}
		}
		return true;
	});	
}

void FUtils::ProcessPinAssetReferences(const URigVMPin* InPin, FWorkspaceOutlinerItemExports& OutExports, const FWorkspaceOutlinerItemExport& RootExport, int32 ParentExportIndex)
{
	auto HandleSoftObjectPath = [&ParentExportIndex, &RootExport, &OutExports](const FSoftObjectPath& InSoftObjectPath) 
	{
		// Only add export if object is loaded, or the path actually points to an asset
		FAssetData ReferenceAssetData;
		if (InSoftObjectPath.ResolveObject() || IAssetRegistry::GetChecked().TryGetAssetByObjectPath(InSoftObjectPath, ReferenceAssetData) == UE::AssetRegistry::EExists::Exists)
		{
			const FWorkspaceOutlinerItemExport& ParentExport = ParentExportIndex == INDEX_NONE ? RootExport : OutExports.Exports[ParentExportIndex];
			OutExports.Exports.Add(MakeAssetReferenceExport(InSoftObjectPath, ParentExport, true));
		}
	};
	
	if (InPin)
	{
		const UObject* TypeObject = InPin->GetCPPTypeObject();
		if (Cast<UClass>(TypeObject))
		{
			FSoftObjectPath ObjectPath(InPin->GetDefaultValue());
			if (ObjectPath.IsValid())
			{
				HandleSoftObjectPath(ObjectPath);
			}
			else
			{

				auto HandleDirectPinVariableLink = [&HandleSoftObjectPath](const IUAFRigVMVariableInterface* InVariableInterface)
				{
					FString DefaultValue;
					InVariableInterface->GetDefaultValueString(DefaultValue);
					const FSoftObjectPath ObjectPath = FSoftObjectPath(DefaultValue);

					if (ObjectPath.IsValid())
					{
						HandleSoftObjectPath(ObjectPath);
					}
				};
				
				// Check for variable nodes linked directly to this pin
				for (URigVMPin* LinkedPin : InPin->GetLinkedSourcePins())
				{
					const URigVMNode* LinkedNode = LinkedPin->GetNode();
					if (const UUAFSharedVariableNode* LinkedSharedVariableNode = Cast<UUAFSharedVariableNode>(LinkedNode))
					{
						if (LinkedSharedVariableNode->Type == EAnimNextSharedVariablesType::Asset && LinkedSharedVariableNode->Asset)
						{
							if (const UUAFRigVMAssetEditorData* EditorData = GetEditorData<const UUAFRigVMAssetEditorData, const UUAFSharedVariables>(LinkedSharedVariableNode->Asset))
							{
								if (IUAFRigVMVariableInterface* VariableInterface = Cast<IUAFRigVMVariableInterface>(EditorData->FindEntry(LinkedSharedVariableNode->GetVariableName())))
								{
									HandleDirectPinVariableLink(VariableInterface);
								}
							}
						}
					}
					else if (const URigVMVariableNode* LinkedVariableNode = Cast<URigVMVariableNode>(LinkedNode))
					{
						if (UUAFRigVMAssetEditorData* EditorData = InPin->GetTypedOuter<UUAFRigVMAssetEditorData>())
						{
							if (IUAFRigVMVariableInterface* VariableInterface = Cast<IUAFRigVMVariableInterface>(EditorData->FindEntry(LinkedVariableNode->GetVariableName())))
							{
								HandleDirectPinVariableLink(VariableInterface);
							}
						}
					}
				}

				// Check for variable nodes linked to this pin its parent pin
				if (InPin->GetParentPin())
				{
					auto HandleParentPinVariableLink = [&HandleSoftObjectPath](const IUAFRigVMVariableInterface* InVariableInterface, const URigVMPin* InPin)
					{
						const FAnimNextParamType& ParameterType = InVariableInterface->GetType();
						if (ParameterType.GetValueType() == FAnimNextParamType::EValueType::Struct && ParameterType.GetContainerType() == FAnimNextParamType::EContainerType::None)
						{
							if (const UScriptStruct* TypeStruct = Cast<UScriptStruct>(ParameterType.GetValueTypeObject()))
							{
								if (const FProperty* StructPinProperty = TypeStruct->FindPropertyByName(InPin->GetFName()))
								{
									FString DefaultValue;
									StructPinProperty->ExportTextItem_InContainer(DefaultValue, InVariableInterface->GetValuePtr(), nullptr, nullptr, 0 );
									const FSoftObjectPath ObjectPath = FSoftObjectPath(DefaultValue);
									if (ObjectPath.IsValid())
									{
										HandleSoftObjectPath(ObjectPath);
									}
								}
							}
						}
					};
					
					for (URigVMPin* LinkedPin : InPin->GetParentPin()->GetLinkedSourcePins())
					{
						const URigVMNode* LinkedNode = LinkedPin->GetNode();
						if (const UUAFSharedVariableNode* LinkedSharedVariableNode = Cast<UUAFSharedVariableNode>(LinkedNode))
						{
							if (LinkedSharedVariableNode->Type == EAnimNextSharedVariablesType::Asset && LinkedSharedVariableNode->Asset)
							{
								if (const UUAFRigVMAssetEditorData* EditorData = GetEditorData<const UUAFRigVMAssetEditorData, const UUAFSharedVariables>(LinkedSharedVariableNode->Asset))
								{
									if (IUAFRigVMVariableInterface* VariableInterface = Cast<IUAFRigVMVariableInterface>(EditorData->FindEntry(LinkedSharedVariableNode->GetVariableName())))
									{
										HandleParentPinVariableLink(VariableInterface, InPin);
									}
								}
							}
						}
						else if (const URigVMVariableNode* LinkedVariableNode = Cast<URigVMVariableNode>(LinkedNode))
						{
							if (UUAFRigVMAssetEditorData* EditorData = InPin->GetTypedOuter<UUAFRigVMAssetEditorData>())
							{
								if (IUAFRigVMVariableInterface* VariableInterface = Cast<IUAFRigVMVariableInterface>(EditorData->FindEntry(LinkedVariableNode->GetVariableName())))
								{
									HandleParentPinVariableLink(VariableInterface, InPin);
								}
							}
						}
					}
				}
			}
		}

		for (const URigVMPin* SubPin : InPin->GetSubPins())
		{
			ProcessPinAssetReferences(SubPin, OutExports, RootExport, ParentExportIndex);
		}
	}	
}

void FUtils::GetSubGraphWorkspaceExportsRecursive(const UUAFRigVMAssetEditorData* EditorData, FWorkspaceOutlinerItemExports& OutExports, const FWorkspaceOutlinerItemExport& RootExport, int32 ParentExportIndex, URigVMEdGraph* RigVMEdGraph, FAssetRegistryTagsContext Context)
{
	if (RigVMEdGraph == nullptr)
	{
		return;
	}

	// Handle pin asset references (disabled during save as GetMetaData can cause StaticFindFast calls which is prohibited during save)
	if (!Context.IsSaving())
	{
		for (const TObjectPtr<class UEdGraphNode>& Node : RigVMEdGraph->Nodes)
		{
			if (URigVMEdGraphNode* RigVMEdNode = Cast<URigVMEdGraphNode>(Node))
			{
				if (URigVMTemplateNode* TemplateRigVMNode = Cast<URigVMTemplateNode>(RigVMEdNode->GetModelNode()))
				{
					if (TemplateRigVMNode->GetScriptStruct() && TemplateRigVMNode->GetScriptStruct()->IsChildOf(FRigUnit_AnimNextBase::StaticStruct()))
					{
						for (URigVMPin* ModelPin : TemplateRigVMNode->GetPins())
						{
							if (ModelPin->GetDirection() == ERigVMPinDirection::Input)
							{
								auto HandlePin = [&OutExports, &RootExport, ParentExportIndex](const URigVMPin* InPin)
								{
									if (InPin->GetMetaData(TEXT("ExportAsReference")) == TEXT("true"))
									{
										ProcessPinAssetReferences(InPin, OutExports, RootExport, ParentExportIndex);
									}										
								};
								
								HandlePin(ModelPin);
								
								for (URigVMPin* TraitPin : ModelPin->GetSubPins())
								{									
									HandlePin(TraitPin);
								}
							}
						}
					}
				}
			}
		}
	}

	// ---- Collapsed graphs ---
	for (const TObjectPtr<UEdGraph>& SubGraph : RigVMEdGraph->SubGraphs)
	{
		URigVMEdGraph* EditorObject = Cast<URigVMEdGraph>(SubGraph);
		if (IsValid(EditorObject))
		{
			if(ensure(EditorObject->GetModel()))
			{
				// Ignore aggregate nodes
				const URigVMCollapseNode* CollapseNode = ExactCast<URigVMCollapseNode>(EditorObject->GetModel()->GetOuter());
				if (CollapseNode)
				{
					const FSoftObjectPath EditorObjectSoftObjPath = EditorObject;
					ensureMsgf(EditorObjectSoftObjPath.IsSubobject(), TEXT("EditorObject for RigVMCollapseNode Graph was not a subobject as expected."));

					const FWorkspaceOutlinerItemExport& ParentExport = ParentExportIndex == INDEX_NONE ? RootExport : OutExports.Exports[ParentExportIndex];
					const FName Identifier = EditorObjectSoftObjPath.IsSubobject() ? *EditorObjectSoftObjPath.GetSubPathUtf8String() : CollapseNode->GetFName();
					FWorkspaceOutlinerItemExport& Export = OutExports.Exports.Add_GetRef(FWorkspaceOutlinerItemExport(Identifier, ParentExport));
					Export.GetData().InitializeAsScriptStruct(FAnimNextCollapseGraphOutlinerData::StaticStruct());
					FAnimNextCollapseGraphOutlinerData& FnGraphData = Export.GetData().GetMutable<FAnimNextCollapseGraphOutlinerData>();
					FnGraphData.SoftEditorObject = EditorObject;

					int32 ExportIndex = OutExports.Exports.Num() - 1;
					GetSubGraphWorkspaceExportsRecursive(EditorData, OutExports, RootExport, ExportIndex, EditorObject, Context);
				}
			}
		}
	}
}

void FUtils::GetSubGraphVariableExportsRecursive(const URigVMEdGraph* RigVMEdGraph, TSet<FAnimNextExport>& OutExports)
{
	TArray<URigVMEdGraphNode*> EdNodes;
	RigVMEdGraph->GetNodesOfClass(EdNodes);

	for (URigVMEdGraphNode* EdNode : EdNodes)
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(EdNode->GetModelNode()))
		{
			OutExports.Add(
				FAnimNextExport::MakeExport<FAnimNextVariableReferenceData>
				(
					VariableNode->GetVariableName(),
					RigVMEdGraph->GetFName(),
					VariableNode->GetNodePath(),
					VariableNode->GetValuePin()->GetPinPath(),
					VariableNode->IsGetter() ? EAnimNextExportedVariableFlags::Read : EAnimNextExportedVariableFlags::Write
				)
			);
		}
		else if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(EdNode->GetModelNode()))
		{
			if (URigVMGraph* ContainedGraph = CollapseNode->GetContainedGraph())
			{
				if (IRigVMClientHost* ClientHost = ContainedGraph->GetImplementingOuter<IRigVMClientHost>())
				{
					if (URigVMEdGraph* ContainerEdGraph = Cast<URigVMEdGraph>(ClientHost->GetEditorObjectForRigVMGraph(ContainedGraph)))
					{
						GetSubGraphVariableExportsRecursive(ContainerEdGraph, OutExports);
					}
				}
			}
		}
	}
}

const FText& FUtils::GetFunctionLibraryDisplayName()
{
	static const FText FunctionLibraryName = LOCTEXT("WorkspaceFunctionLibraryName", "Function Library");
	return FunctionLibraryName;
}

#if WITH_EDITOR
void FUtils::OpenProgrammaticGraphs(UUAFRigVMAssetEditorData* EditorData, const TArray<URigVMGraph*>& ProgrammaticGraphs)
{
	UUAFRigVMAsset* OwningAsset = FUtils::GetAsset(EditorData);
	UE::Workspace::IWorkspaceEditorModule& WorkspaceEditorModule = FModuleManager::LoadModuleChecked<UE::Workspace::IWorkspaceEditorModule>("WorkspaceEditor");
	if(UE::Workspace::IWorkspaceEditor* WorkspaceEditor = WorkspaceEditorModule.OpenWorkspaceForObject(OwningAsset, UE::Workspace::EOpenWorkspaceMethod::Default))
	{
		TArray<UObject*> Graphs;
		for(URigVMGraph* ProgrammaticGraph : ProgrammaticGraphs)
		{
			// Some explanation needed here!
			// URigVMEdGraph caches its underlying model internally in GetModel depending on its outer if it is no attached to a RigVMClient
			// So here we rename the graph into the transient package so we dont get any notifications
			ProgrammaticGraph->Rename(nullptr, GetTransientPackage(), REN_AllowPackageLinkerMismatch | REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);

			// then create the graph (transient so it outers to the RigVMGraph)
			URigVMEdGraph* EdGraph = CastChecked<URigVMEdGraph>(EditorData->CreateEdGraph(ProgrammaticGraph, true));

			// Then cache the model
			EdGraph->GetModel();
			Graphs.Add(EdGraph);

			// Now rename into this asset again to be able to correctly create a controller (needed to view the graph and interact with it)
			ProgrammaticGraph->Rename(nullptr, EditorData, REN_AllowPackageLinkerMismatch | REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			URigVMController* ProgrammaticController = EditorData->GetOrCreateController(ProgrammaticGraph);

			// Resend notifications to rebuild the EdGraph
			ProgrammaticController->ResendAllNotifications();
		}

		WorkspaceEditor->OpenObjects(Graphs);
	}
}
#endif // WITH_EDITOR

FString FUtils::MakeFunctionWrapperVariableName(FName InFunctionName, FName InVariableName)
{
	// We assume the function name is enough for variable name uniqueness in this graph (We don't yet desire global uniqueness).
	return InternalVariablePrefix + InFunctionName.ToString() + "_" + InVariableName.ToString();
}

FString FUtils::MakeFunctionWrapperEventName(FName InFunctionName)
{
	return TEXT("__InternalCall_") + InFunctionName.ToString();
}

void FUtils::GetVariableNames(UUAFRigVMAssetEditorData* InEditorData, TArray<FName>& OutVariableNames, bool bRecursive)
{
	for(UUAFRigVMAssetEntry* Entry : InEditorData->Entries)
	{
		if(UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(Entry))
		{
			if(VariableEntry->GetExportAccessSpecifier() == EAnimNextExportAccessSpecifier::Public)
			{
				OutVariableNames.Add(Entry->GetEntryName());
			}
		}
		else if(UUAFSharedVariablesEntry* DataInterfaceEntry = Cast<UUAFSharedVariablesEntry>(Entry))
		{
			if (bRecursive && DataInterfaceEntry->GetAsset())
			{
				UUAFSharedVariables_EditorData* EditorData = GetEditorData<UUAFSharedVariables_EditorData>(DataInterfaceEntry->GetAsset());
				GetVariableNames(EditorData, OutVariableNames);
			}
		}
	}
}

FName FUtils::GetValidVariableName(UUAFRigVMAssetEditorData* InEditorData, FName InBaseName)
{
	TArray<FName> ExistingNames;
	GetVariableNames(InEditorData, ExistingNames, true);	
	return GetValidVariableName(InBaseName, ExistingNames);	
}

FName FUtils::GetValidVariableName(FName InBaseName, const TArrayView<FName> ExistingNames)
{
	auto NameExists = [&ExistingNames](FName InName)
	{
		for(FName AdditionalName : ExistingNames)
		{
			if(AdditionalName == InName)
			{
				return true;
			}
		}

		return false;
	};

	if(!NameExists(InBaseName))
	{
		// Early out - name is valid
		return InBaseName;
	}

	int32 PostFixIndex = 0;
	TStringBuilder<128> StringBuilder;
	while(true)
	{
		StringBuilder.Reset();
		InBaseName.GetDisplayNameEntry()->AppendNameToString(StringBuilder);
		StringBuilder.Appendf(TEXT("_%d"), PostFixIndex++);

		FName TestName(StringBuilder.ToString()); 
		if(!NameExists(TestName))
		{
			return TestName;
		}
	}
}

void FUtils::DeleteVariable(UAnimNextVariableEntry* VariableEntry, bool bSetupUndoRedo, bool bPrintPythonCommands)
{
	if (VariableEntry)
	{
		UUAFRigVMAsset* OuterRigVMAsset = VariableEntry->GetTypedOuter<UUAFRigVMAsset>();
		check(OuterRigVMAsset);
		
		FCompilationScope CompilerResults(LOCTEXT("ModifiedAssets_DeleteVariable", "Modified Assets Delete Variable"), OuterRigVMAsset);
		
		const FAnimNextVariableReference FindReference = FAnimNextVariableReference::FromName(VariableEntry->GetEntryName(), OuterRigVMAsset);

		// Replace references with None
		UUAFRigVMAssetEditorData* EditorData = GetEditorData<UUAFRigVMAssetEditorData>(OuterRigVMAsset);

		// Replace reference across project for public variables
		const bool bIsPublicVariable = VariableEntry->GetExportAccessSpecifier() == EAnimNextExportAccessSpecifier::Public;
		if (bIsPublicVariable)
		{
			constexpr bool bForceLoadReferencingAssets = false;
			ReplaceVariableReferencesAcrossProject(FAnimNextSoftVariableReference(FindReference), FAnimNextSoftVariableReference(), bForceLoadReferencingAssets, bSetupUndoRedo, bPrintPythonCommands);
		}
		else
		{
			// Otherwise replace, potential, local references only 
			ReplaceVariableReferences(EditorData, FAnimNextSoftVariableReference(FindReference), FAnimNextSoftVariableReference(), bSetupUndoRedo, bPrintPythonCommands);
		}

		EditorData->RemoveEntry(VariableEntry);
	}
}

void FUtils::RenameVariable(UAnimNextVariableEntry* VariableEntry, const FName NewName, bool bSetupUndoRedo, bool bPrintPythonCommands)
{
	if (VariableEntry)
	{
		UUAFRigVMAsset* OuterRigVMAsset = VariableEntry->GetTypedOuter<UUAFRigVMAsset>();
		check(OuterRigVMAsset);

		FCompilationScope CompilerResults(LOCTEXT("ModifiedAssets_RenameVariable", "Modified Assets Rename Variable"), OuterRigVMAsset);
		
		const FAnimNextSoftVariableReference FindReference = FAnimNextSoftVariableReference::FromNameAndType(VariableEntry->GetEntryName(), OuterRigVMAsset, VariableEntry->GetType());
		
		// Rename variable first
		VariableEntry->SetEntryName(NewName);
		
		const FAnimNextSoftVariableReference ReplaceReference = FAnimNextSoftVariableReference::FromName(NewName, OuterRigVMAsset);
		
		// Replace reference across project for public variables
		const bool bIsPublicVariable = VariableEntry->GetExportAccessSpecifier() == EAnimNextExportAccessSpecifier::Public;
		if (bIsPublicVariable)
		{
			constexpr bool bForceLoadReferencingAssets = false;
			ReplaceVariableReferencesAcrossProject(FAnimNextSoftVariableReference(FindReference), FAnimNextSoftVariableReference(ReplaceReference), bForceLoadReferencingAssets, bSetupUndoRedo, bPrintPythonCommands);
		}
		else
		{
			// Otherwise replace, potential, local references only 
			ReplaceVariableReferences(GetEditorData(OuterRigVMAsset), FAnimNextSoftVariableReference(FindReference), FAnimNextSoftVariableReference(ReplaceReference), bSetupUndoRedo, bPrintPythonCommands);		
		}
	}	
}

UAnimNextVariableEntry* FUtils::MoveVariableToAsset(UAnimNextVariableEntry* VariableEntry, UUAFRigVMAsset* NewOuter, bool bSetupUndoRedo, bool bPrintPythonCommands)
{
	if (VariableEntry && NewOuter)
	{
		if (UUAFRigVMAsset* CurrentOuter = VariableEntry->GetTypedOuter<UUAFRigVMAsset>())
		{
			FCompilationScope CompilerResults(LOCTEXT("ModifiedAssets_MoveVariableToAsset", "Modified Assets Move Variable"), MakeConstArrayView({ CurrentOuter, NewOuter }));

			UUAFRigVMAssetEditorData* CurrentEditorData = GetEditorData(CurrentOuter);
			UUAFRigVMAssetEditorData* NewEditorData = GetEditorData(NewOuter);
						
			FString DefaultValueString;
			VariableEntry->GetDefaultValueString(DefaultValueString);

			UAnimNextVariableEntry* NewEntry = NewEditorData->AddVariable(VariableEntry->GetVariableName(), VariableEntry->GetType(), DefaultValueString, bSetupUndoRedo, bPrintPythonCommands);
			check(NewEntry);

			TInstancedStruct<FAnimNextVariableBindingData> BindingCopy = VariableEntry->Binding.BindingData;
			NewEntry->SetBinding(MoveTemp(BindingCopy));

			NewEntry->SetExportAccessSpecifier(NewOuter->IsA<UUAFSharedVariables>()
				? EAnimNextExportAccessSpecifier::Public
				: VariableEntry->GetExportAccessSpecifier());

			const FAnimNextSoftVariableReference FindReference = FAnimNextSoftVariableReference::FromName(VariableEntry->GetEntryName(), CurrentOuter);
			const FAnimNextSoftVariableReference ReplaceReference = FAnimNextSoftVariableReference::FromName(VariableEntry->GetEntryName(), NewOuter);

			// Replace reference across project for public variables
			const bool bIsPublicVariable = VariableEntry->GetExportAccessSpecifier() == EAnimNextExportAccessSpecifier::Public;
			if (bIsPublicVariable)
			{
				constexpr bool bForceLoadReferencingAssets = false;
				constexpr bool bForceReplace = true;
				ReplaceVariableReferencesAcrossProject(FindReference, ReplaceReference, bForceLoadReferencingAssets, bForceReplace, bSetupUndoRedo, bPrintPythonCommands);
			}
			else
			{
				// Otherwise replace, potential, local references only
				constexpr bool bForceReplace = true;
				ReplaceVariableReferences(CurrentEditorData, FindReference, ReplaceReference, bForceReplace, bSetupUndoRedo, bPrintPythonCommands);
			}

			CurrentEditorData->RemoveEntry(VariableEntry, bSetupUndoRedo, bPrintPythonCommands);
			
			return NewEntry;
		}
	}
	
	return nullptr;
}

void FUtils::SetVariableType(UAnimNextVariableEntry* VariableEntry, const FAnimNextParamType NewType, bool bSetupUndoRedo, bool bPrintPythonCommands)
{
	if (VariableEntry)
	{
		UUAFRigVMAsset* OuterRigVMAsset = VariableEntry->GetTypedOuter<UUAFRigVMAsset>();
		check(OuterRigVMAsset);

		FCompilationScope CompilerResults(LOCTEXT("ModifiedAssets_SetVariableType", "Modified Assets Set Variable Type"), OuterRigVMAsset);
		
		const FAnimNextSoftVariableReference OldReference = FAnimNextSoftVariableReference::FromNameAndType(VariableEntry->GetEntryName(), OuterRigVMAsset, VariableEntry->GetType());
		const FAnimNextSoftVariableReference NewReference = FAnimNextSoftVariableReference::FromNameAndType(VariableEntry->GetEntryName(), OuterRigVMAsset, NewType);
		
		// Change variable type first
		VariableEntry->SetType(NewType);

		// Replace reference across project for public variables
		const bool bIsAPublicVariable = VariableEntry->GetExportAccessSpecifier() == EAnimNextExportAccessSpecifier::Public;
		if (bIsAPublicVariable)
		{
			constexpr bool bForceLoadReferencingAssets = false;
			ReplaceVariableReferencesAcrossProject(OldReference, NewReference, bForceLoadReferencingAssets, bSetupUndoRedo, bPrintPythonCommands);
		}
		else
		{
			// Otherwise replace, potential, local references only 
			ReplaceVariableReferences(GetEditorData(OuterRigVMAsset), OldReference, NewReference, bSetupUndoRedo, bPrintPythonCommands);
		}
	}	
}

void FUtils::ReplaceVariableReferences(UUAFRigVMAssetEditorData* InEditorData, const FAnimNextSoftVariableReference& SoftReferenceToFind, const FAnimNextSoftVariableReference& SoftReferenceToReplaceWith, bool bForceReplace /*= false*/, bool bSetupUndoRedo /*= true*/, bool bPrintPythonCommands /*= true*/)
{
	if (InEditorData == nullptr)
	{
		return;
	}

	const FAnimNextVariableReference ReferenceToFind(SoftReferenceToFind);	
	if (ReferenceToFind.IsNone())
	{
		// Cannot replace an empty reference
		return;
	}
		
	UUAFRigVMAsset* ThisOuterAsset = GetAsset(InEditorData);
	const UUAFRigVMAsset* FindOuterAsset = Cast<UUAFRigVMAsset>(ReferenceToFind.GetObject());
	const FAnimNextVariableReference ReferenceToReplaceWith(SoftReferenceToReplaceWith);
	const UUAFRigVMAsset* ReplaceOuterAsset = Cast<UUAFRigVMAsset>(ReferenceToReplaceWith.GetObject());

	const bool bReplacingWithNone = ReferenceToReplaceWith.IsNone();
	const FGuid ReferenceGuid = bReplacingWithNone  
		? UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule::Get().GetVariableGuidByName(ReferenceToFind.GetName(), FindOuterAsset)
		: UE::UAF::UncookedOnly::IAnimNextUncookedOnlyModule::Get().GetVariableGuidByName(ReferenceToReplaceWith.GetName(), FindOuterAsset);
	
	// Replacing variable ref local to this asset, with another local variable or none
	const bool bInternalReplace = (ThisOuterAsset == FindOuterAsset) && (FindOuterAsset == ReplaceOuterAsset || bReplacingWithNone);
	
	// Replacing with variable from a different object vs existing reference
	const bool bDifferentVariableSource = FindOuterAsset != ReplaceOuterAsset;

	FAnimNextParamType FindReferenceType = SoftReferenceToFind.HasType() ? SoftReferenceToFind.GetType() : FindVariableType(ReferenceToFind);
	const FAnimNextParamType ReplaceReferenceType = SoftReferenceToReplaceWith.HasType() ? SoftReferenceToReplaceWith.GetType() : FindVariableType(ReferenceToReplaceWith);

	// In case ReferenceToFind has already been removed/changed match it up with the replacing type (if not None) 
	if (!FindReferenceType.IsValid() && !ReferenceToReplaceWith.IsNone())
	{
		FindReferenceType = ReplaceReferenceType;
	}
	
	const bool bVariableTypeChanged = FindReferenceType != ReplaceReferenceType;
	const bool bVariableNameChanged = ReferenceToFind.GetName() != ReferenceToReplaceWith.GetName();

	// Conditionally adds a SharedVariables dependency when required (no reference exists yet, and asset reference requiring one is introduced)
	auto EnsureSharedVariableDependencyExists = [ReplaceOuterAsset, ThisOuterAsset, InEditorData, bSetupUndoRedo, bPrintPythonCommands]()
	{
		// Ensure that SharedVariables have been included for replace variable (if from another asset)
		const UUAFSharedVariables* SharedVariablesReplaceOuter = Cast<UUAFSharedVariables>(ReplaceOuterAsset);
		if (SharedVariablesReplaceOuter && ThisOuterAsset != SharedVariablesReplaceOuter)
		{
			bool bHasEntry = false;
			InEditorData->ForEachEntryOfType<UUAFSharedVariablesEntry>([&bHasEntry, SharedVariablesReplaceOuter](const UUAFSharedVariablesEntry* Entry)
			{
				if (Entry->GetType() == EAnimNextSharedVariablesType::Asset)
				{
					if (Entry->GetAsset() == SharedVariablesReplaceOuter)
					{
						bHasEntry = true;
						return false;
					}
				}

				return true;
			});

			if (!bHasEntry)
			{
				FString ErrorMessage;
				if (CanAddSharedVariablesReference(InEditorData, SharedVariablesReplaceOuter, &ErrorMessage))
				{
					FCompilationScope CompilerResults(LOCTEXT("AddSharedAssetJobName", "Add Shared Asset dependency"), ThisOuterAsset);
					InEditorData->AddSharedVariables(SharedVariablesReplaceOuter, bSetupUndoRedo, bPrintPythonCommands);
				}
				else
				{
					FText NotificationText = FText::FromString(ErrorMessage);
					FNotificationInfo Notification(NotificationText);
					Notification.ExpireDuration = 3.0f;
					FSlateNotificationManager::Get().AddNotification(Notification);
					UE_LOGFMT(LogAnimation, Error, "{0}", ErrorMessage);
				}
			}
		}
	};	
	
	TArray<URigVMEdGraphNode*> EdNodes;

	// Gather all the RigVM graphs to replace variables inside of
	TArray<URigVMEdGraph*> RigVMEdGraphsToTraverse;
	{
		for (const UUAFRigVMAssetEntry* Entry : InEditorData->Entries)
		{
			if (const IUAFRigVMGraphInterface* GraphInterface = Cast<IUAFRigVMGraphInterface>(Entry))
			{
				if (URigVMEdGraph* RigVMEdGraph = GraphInterface->GetEdGraph())
				{
					RigVMEdGraphsToTraverse.Add(RigVMEdGraph);
				}
			}
		}

		RigVMEdGraphsToTraverse.Append(InEditorData->FunctionEdGraphs);
	}

	// Non-range-based loop as RigVMEdGraphsToTraverse can expand during iteration
	for (int32 GraphToTraverseIndex = 0; GraphToTraverseIndex < RigVMEdGraphsToTraverse.Num(); ++GraphToTraverseIndex)
	{
		URigVMEdGraph* RigVMEdGraph = RigVMEdGraphsToTraverse[GraphToTraverseIndex];
		if (!RigVMEdGraph)
		{
			continue;
		}

		UAnimNextControllerBase* Controller = CastChecked<UAnimNextControllerBase>(RigVMEdGraph->GetController());
		// Rely on RigVM controller to handle local variable references
		if (bInternalReplace)
		{
			if (bReplacingWithNone)
			{
				Controller->OnExternalVariableRemoved(ReferenceGuid, bSetupUndoRedo);
			}
			else
			{
				if (bVariableTypeChanged)
				{
					const FRigVMTemplateArgumentType NewRigVMType = ReplaceReferenceType.ToRigVMTemplateArgument();
					Controller->OnExternalVariableTypeChanged(ReferenceGuid, NewRigVMType.CPPType.ToString(), NewRigVMType.CPPTypeObject.Get(), bSetupUndoRedo);
				}

				if (bVariableNameChanged)
				{
					Controller->OnExternalVariableRenamed(ReferenceGuid, ReferenceToReplaceWith.GetName(), bSetupUndoRedo);
				}
			}
		}
		else
		{
			RigVMEdGraph->GetNodesOfClass(EdNodes);

			for (URigVMEdGraphNode* EdNode : EdNodes)
			{
				if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(EdNode->GetModelNode()))
				{
					const bool bNameMatch = VariableNode->GetVariableName() == ReferenceToFind.GetName();

					if (!bNameMatch)
					{
						continue;
					}

					if (UUAFSharedVariableNode* SharedVariableNode = Cast<UUAFSharedVariableNode>(VariableNode))
					{
						const UObject* SearchObject = ReferenceToFind.GetObject();
						const bool bPropertySourceMatch = (SharedVariableNode->Asset && SharedVariableNode->Asset == SearchObject) || (SharedVariableNode->Struct && SharedVariableNode->Struct == SearchObject);
						const bool bValidProperty =ReferenceToReplaceWith.ResolveProperty() != nullptr;

						if (bPropertySourceMatch && (bValidProperty || bReplacingWithNone || bForceReplace))
						{
							if (bReplacingWithNone)
							{
								// [TODO] decide to either remove node or replace with None variable references (keeps node in place but will fail compilation until fixed up)
								Controller->RemoveNode(SharedVariableNode, bSetupUndoRedo, bPrintPythonCommands);
								//Controller->RefreshSharedVariableNode(SharedVariableNode->GetFName(), TEXT(""), ReferenceToReplaceWith.GetName(), TEXT(""), nullptr, true, true);	
							}
							else
							{
								if (bDifferentVariableSource && ReplaceOuterAsset == ThisOuterAsset)
								{
									Controller->ReplaceSharedVariableNodeWithVariableNode(SharedVariableNode, ReferenceToReplaceWith.GetName(), bSetupUndoRedo, bPrintPythonCommands);
								}
								else
								{
									const FRigVMTemplateArgumentType NewRigVMType = ReplaceReferenceType.ToRigVMTemplateArgument();

									EnsureSharedVariableDependencyExists();

									constexpr bool bSetupOrphanPins = true;
									Controller->RefreshSharedVariableNode(SharedVariableNode->GetFName(), ReferenceToReplaceWith.GetObject().GetPath(), ReferenceToReplaceWith.GetName(), NewRigVMType.CPPType.ToString(), NewRigVMType.CPPTypeObject.Get(), bSetupUndoRedo, bSetupOrphanPins, bPrintPythonCommands);
								}
							}
						}
					}
					// For regular variable nodes, the FindOuterAsset has to match the asset being processed
					else if (FindOuterAsset == ThisOuterAsset)
					{
						if (bReplacingWithNone)
						{
							// Remove the node for now (can also leave an invalid one causing compile error/warning)
							Controller->RemoveNode(VariableNode, bSetupUndoRedo, bPrintPythonCommands);
						}
						else
						{
							if (bDifferentVariableSource)
							{
								if (FindOuterAsset == ThisOuterAsset && ReplaceOuterAsset != ThisOuterAsset)
								{
									EnsureSharedVariableDependencyExists();
									Controller->ReplaceVariableNodeWithSharedVariableNode(VariableNode, ReferenceToReplaceWith.GetName(), ReferenceToReplaceWith.GetObject(), bSetupUndoRedo, bPrintPythonCommands);
								}
								else
								{
									UE_LOGFMT(LogAnimation, Error, "Found matching instance of Variable Type/Name ({0}/{1}) combination on {2} which differs from {3} from which the replace operation is being executed, this instance will be skipped.",
										FindReferenceType.ToString(), SoftReferenceToFind.GetName().ToString(), FindOuterAsset->GetPathName(), ThisOuterAsset->GetPathName());
								}
							}
							else
							{
								if (bVariableTypeChanged)
								{
									const FRigVMTemplateArgumentType NewRigVMType = ReplaceReferenceType.ToRigVMTemplateArgument();
									Controller->OnExternalVariableTypeChanged(ReferenceGuid, NewRigVMType.CPPType.ToString(), NewRigVMType.CPPTypeObject.Get(), bSetupUndoRedo);
								}

								if (bVariableNameChanged)
								{
									Controller->OnExternalVariableRenamed(ReferenceGuid, ReferenceToReplaceWith.GetName(), bSetupUndoRedo);
								}
							}
						}
					}
				}
				else if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(EdNode->GetModelNode()))
				{
					URigVMGraph* ContainedGraph = CollapseNode->GetContainedGraph();
					if (IRigVMClientHost* ClientHost = ContainedGraph->GetImplementingOuter<IRigVMClientHost>())
					{
						if (URigVMEdGraph* ContainedEdGraph = Cast<URigVMEdGraph>(ClientHost->GetEditorObjectForRigVMGraph(ContainedGraph)))
						{
							if (!RigVMEdGraphsToTraverse.Contains(ContainedEdGraph))
							{
								RigVMEdGraphsToTraverse.Add(ContainedEdGraph);
							}
						}
					}
				}
			}

			EdNodes.Reset();
		}
	}
}

void FUtils::ReplaceVariableReferencesAcrossProject(const FAnimNextSoftVariableReference& ReferenceToFind, const FAnimNextSoftVariableReference& ReferenceToReplaceWith,  bool bForceLoadAssets, bool bForceReplace /*= false*/, bool bSetupUndoRedo, bool bPrintPythonCommands)
{
	if (ReferenceToFind.IsNone())
	{
		return;
	}
	
	FARFilter AssetFilter;
	AssetFilter.ClassPaths = { UUAFRigVMAsset::StaticClass()->GetClassPathName() };
	AssetFilter.bRecursiveClasses = true;

	// [TODO] would this miss out assets
	//AssetFilter.TagsAndValues.Add(UE::UAF::ExportsAnimNextAssetRegistryTag, TOptional<FString>());
		
	TArray<FAssetData> AssetDatas;
	FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get().GetAssets(AssetFilter, AssetDatas);

	TArray<UUAFRigVMAssetEditorData*> EditorDataRequiringReplacing;	
	for (const FAssetData& AssetData : AssetDatas)
	{
		// Relates to [TODO] above
		ensure(AssetData.FindTag(UE::UAF::ExportsAnimNextAssetRegistryTag));
		
		FAnimNextAssetRegistryExports AnimNextExports;
		GetExportsOfTypeForAsset<FAnimNextVariableReferenceData>(AssetData, AnimNextExports);

		for (const FAnimNextExport& Export : AnimNextExports.Exports)
		{
			if (ReferenceToFind.GetName() == Export.Identifier)
			{
				if (AssetData.IsAssetLoaded() || (bForceLoadAssets && AssetData.GetAsset()))
				{
					if (UUAFRigVMAsset* RigVMAsset = Cast<UUAFRigVMAsset>(AssetData.GetAsset()))
					{
						if (UUAFRigVMAssetEditorData* EditorData = GetEditorData(RigVMAsset))
						{
							EditorDataRequiringReplacing.Add(EditorData);
						}
					}
				}
			}
		}
	}

	for (UUAFRigVMAssetEditorData* EditorData : EditorDataRequiringReplacing)
	{
		ReplaceVariableReferences(EditorData, ReferenceToFind, ReferenceToReplaceWith, bForceReplace, bSetupUndoRedo, bPrintPythonCommands);
	}
}

FGuid FUtils::GenerateScriptStructPropertyGUID(const FProperty* Property)
{
	checkf(Property, TEXT("Expected valid property for GUID generation"));
	checkf(Property->GetOwnerChecked<UStruct>(), TEXT("Expected valid owning UStruct for GUID generation"));
	return FGuid::NewDeterministicGuid(Property->GetPathName(), FCrc::StrCrc32<TCHAR>(TEXT("StructBasedSharedVariableEntry")));
}

FAnimNextVariableReference FUtils::ParseVariableReferenceFromPin(const UEdGraphPin* InPin)
{
	FAnimNextVariableReference VariableReference;

	if (InPin)
	{
		FAnimNextVariableReference::StaticStruct()->ImportText(*InPin->DefaultValue, &VariableReference, nullptr, PPF_None, nullptr, FAnimNextVariableReference::StaticStruct()->GetName());
	}
	
	return VariableReference;
}

void FUtils::PreloadVariableReferenceAssets(const UEdGraphPin* InVariablePin)
{
	const FAnimNextVariableReference VariableReference = ParseVariableReferenceFromPin(InVariablePin);
	if (!VariableReference.IsNone() && VariableReference.GetObject())
	{
		if (UObject* Object = const_cast<UObject*>(VariableReference.GetObject().Get()))
		{
			Object->ConditionalPreload();
		}
	}
}

}

#undef LOCTEXT_NAMESPACE
