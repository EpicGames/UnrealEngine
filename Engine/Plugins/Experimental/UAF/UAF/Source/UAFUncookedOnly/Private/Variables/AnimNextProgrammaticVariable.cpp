// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/AnimNextProgrammaticVariable.h"

#include "Logging/StructuredLog.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMCore/RigVMTemplate.h"
#include "Variables/IAnimNextRigVMVariableInterface.h"

FAnimNextParamType FAnimNextProgrammaticVariable::GetType() const
{
	return Type;
}

bool FAnimNextProgrammaticVariable::SetType(const FAnimNextParamType& InType, bool bSetupUndoRedo)
{
	Type = InType;

	DefaultValue.Reset();
	DefaultValue.AddProperties({ FPropertyBagPropertyDesc(IUAFRigVMVariableInterface::ValueName, Type.GetContainerType(), Type.GetValueType(), Type.GetValueTypeObject()) });

	return true;
}

FName FAnimNextProgrammaticVariable::GetVariableName() const
{
	return Name;
}

void FAnimNextProgrammaticVariable::SetVariableName(FName InName, bool bSetupUndoRedo)
{
	Name = InName;
}

bool FAnimNextProgrammaticVariable::SetDefaultValue(TConstArrayView<uint8> InValue, bool bSetupUndoRedo)
{
	const FPropertyBagPropertyDesc* Desc = DefaultValue.FindPropertyDescByName(IUAFRigVMVariableInterface::ValueName);
	if (Desc == nullptr)
	{
		UE_LOGFMT(LogAnimation, Error, "FAnimNextProgrammaticVariable::SetDefaultValue: Could not find default value in property bag");
		return false;
	}

	check(Desc->CachedProperty);
	if (Desc->CachedProperty->GetElementSize() != InValue.Num())
	{
		UE_LOGFMT(LogAnimation, Error, "FAnimNextProgrammaticVariable::SetDefaultValue: Mismatched buffer sizes");
		return false;
	}

	uint8* DestPtr = Desc->CachedProperty->ContainerPtrToValuePtr<uint8>(DefaultValue.GetMutableValue().GetMemory());
	const uint8* SrcPtr = InValue.GetData();
	Desc->CachedProperty->CopyCompleteValue(DestPtr, SrcPtr);

	return true;
}

bool FAnimNextProgrammaticVariable::SetDefaultValueFromString(const FString& InDefaultValue, bool bSetupUndoRedo)
{
	if (DefaultValue.SetValueSerializedString(IUAFRigVMVariableInterface::ValueName, InDefaultValue) != EPropertyBagResult::Success)
	{
		UE_LOGFMT(LogAnimation, Error, "FAnimNextProgrammaticVariable::FromRigVMGraphFunctionArgument: Could not set value from string");
		return false;
	}

	return true;
}

const FInstancedPropertyBag& FAnimNextProgrammaticVariable::GetPropertyBag() const
{
	return DefaultValue;
}

FInstancedPropertyBag& FAnimNextProgrammaticVariable::GetMutablePropertyBag()
{
	return DefaultValue;
}

bool FAnimNextProgrammaticVariable::GetDefaultValue(const FProperty*& OutProperty, TConstArrayView<uint8>& OutValue) const
{
	const FPropertyBagPropertyDesc* Desc = DefaultValue.FindPropertyDescByName(IUAFRigVMVariableInterface::ValueName);
	if (Desc == nullptr)
	{
		UE_LOGFMT(LogAnimation, Error, "FAnimNextProgrammaticVariable::FromRigVMGraphFunctionArgument: Could not find default value in property bag");
		return false;
	}

	check(Desc->CachedProperty);
	OutProperty = Desc->CachedProperty;
	const uint8* ValuePtr = Desc->CachedProperty->ContainerPtrToValuePtr<uint8>(DefaultValue.GetValue().GetMemory());
	OutValue = TConstArrayView<uint8>(ValuePtr, Desc->CachedProperty->GetElementSize());
	return true;
}

const uint8* FAnimNextProgrammaticVariable::GetValuePtr() const
{
	const FInstancedPropertyBag& PropertyBag = GetPropertyBag();
	const UPropertyBag* PropertyBagStruct = PropertyBag.GetPropertyBagStruct();
	const uint8* Memory = PropertyBag.GetValue().GetMemory();
	return PropertyBagStruct && Memory ? PropertyBagStruct->GetPropertyDescs()[0].CachedProperty->ContainerPtrToValuePtr<uint8>(Memory) : nullptr;
}

FGuid FAnimNextProgrammaticVariable::GetGuid() const
{
	// [TODO] this assumes Variable.Name being unique yet consistent across compilations
	return FGuid::NewDeterministicGuid(Name.ToString(), FCrc::StrCrc32<TCHAR>(TEXT("ProgrammaticVariableGUID")));
}

FAnimNextProgrammaticVariable FAnimNextProgrammaticVariable::FromRigVMGraphFunctionArgument(const FRigVMGraphFunctionArgument& RigVMGraphFunctionArgument)
{
	FAnimNextProgrammaticVariable Result = {};
	Result.Name = RigVMGraphFunctionArgument.Name;
	Result.SetType(FAnimNextParamType::FromRigVMTemplateArgument(FRigVMTemplateArgumentType(RigVMGraphFunctionArgument.CPPType, RigVMGraphFunctionArgument.CPPTypeObject.Get())));
	
	if (RigVMGraphFunctionArgument.DefaultValue.Len() > 0)
	{
		// No need to log error on fail, setter will do that
		Result.SetDefaultValueFromString(RigVMGraphFunctionArgument.DefaultValue);
	}

	return Result;
}
