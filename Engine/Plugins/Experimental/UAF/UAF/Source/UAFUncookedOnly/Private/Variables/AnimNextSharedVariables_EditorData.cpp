// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/AnimNextSharedVariables_EditorData.h"

#include "Entries/AnimNextSharedVariablesEntry.h"
#include "Entries/AnimNextVariableEntry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextSharedVariables_EditorData)

TConstArrayView<TSubclassOf<UUAFRigVMAssetEntry>> UUAFSharedVariables_EditorData::GetEntryClasses() const
{
	static const TSubclassOf<UUAFRigVMAssetEntry> Classes[] =
	{
		UAnimNextVariableEntry::StaticClass(),
		UUAFSharedVariablesEntry::StaticClass(),
	};

	return Classes;
}

void UUAFSharedVariables_EditorData::CustomizeNewAssetEntry(UUAFRigVMAssetEntry* InNewEntry) const
{
	UAnimNextVariableEntry* VariableEntry = Cast<UAnimNextVariableEntry>(InNewEntry);
	if(VariableEntry == nullptr)
	{
		return;
	}
	
	const bool bIsSharedVariables = ExactCast<UUAFSharedVariables_EditorData>(this) != nullptr;
	if(!bIsSharedVariables)
	{
		return;
	}

	// Force all variables in 'pure' shared variables assets to be public
	VariableEntry->SetExportAccessSpecifier(EAnimNextExportAccessSpecifier::Public, false);
}
