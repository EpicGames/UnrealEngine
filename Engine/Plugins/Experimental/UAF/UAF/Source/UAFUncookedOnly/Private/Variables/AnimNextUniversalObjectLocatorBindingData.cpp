// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/AnimNextUniversalObjectLocatorBindingData.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNextUniversalObjectLocatorBindingData)

#if WITH_EDITORONLY_DATA

bool FAnimNextUniversalObjectLocatorBindingData::IsThreadSafe() const
{
	// TODO: General UOL resolves are not thread-safe, so returning false here for now.
	// To address object data on worker threads we could assume that object-graphs do not change and cache the UOL result. Users would be free to
	// set the UOL as 'dynamic' to allow for use cases where per-frame resolves are required. These 'dynamic' UOLs would be run on the GT, where all
	// 'static' UOLs would be run on the WT.
	return false;

	/*
	switch(Type)
	{
	case FAnimNextUniversalObjectLocatorBindingType::Property:
		return false;
	case FAnimNextUniversalObjectLocatorBindingType::Function:
	case FAnimNextUniversalObjectLocatorBindingType::HoistedFunction:
		if(UFunction* ResolvedFunction = Function.Get())
		{
			return FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(ResolvedFunction);
		}
		break;
	}

	return false;
	*/
}

#endif
