// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClassProxy.h"
#include "Logging/StructuredLog.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Param/ParamUtils.h"

namespace UE::UAF::UncookedOnly
{

FClassProxy::FClassProxy(const UClass* InClass)
{
	Refresh(InClass);
}

void FClassProxy::Refresh(const UClass* InClass)
{
	Class = InClass;
	ParameterNameMap.Reset();

	// Add any additional extension libraries that extend this class first
	{
		TArray<UClass*> Classes;
		GetDerivedClasses(UBlueprintFunctionLibrary::StaticClass(), Classes);
		for(UClass* ProxyClass : Classes)
		{
			for(TFieldIterator<UFunction> It(ProxyClass); It; ++It)
			{
				UFunction* Function = *It;
				if(FParamUtils::CanUseFunction(Function, InClass))
				{
					const FProperty* ReturnProperty = Function->GetReturnProperty();
					FAnimNextParamType Type = FAnimNextParamType::FromProperty(ReturnProperty);
					if(Type.IsValid())
					{
						FClassProxyParameter Parameter;
						Parameter.AccessType = EClassProxyParameterAccessType::HoistedFunction;
						Parameter.ParameterName = *Function->GetPathName();
						Parameter.Function = Function;
						Parameter.Type = Type;
						Parameter.DisplayName = Function->GetDisplayNameText();
						Parameter.Tooltip = Function->GetToolTipText();
						Parameter.bThreadSafe = Function->HasMetaData("BlueprintThreadSafe");

						ParameterNameMap.Add(Parameter.ParameterName, Parameters.Add(Parameter));
					}
				}
			}
		}
	}

	// Add functions as the next priority (extensions have already been added above so will take priority with duplicate names)
	for(TFieldIterator<UFunction> It(InClass, EFieldIterationFlags::IncludeSuper | EFieldIterationFlags::IncludeInterfaces); It; ++It)
	{
		UFunction* Function = *It;
		if(FParamUtils::CanUseFunction(Function, InClass))
		{
			const FProperty* ReturnProperty = Function->GetReturnProperty();
			FAnimNextParamType Type = FAnimNextParamType::FromProperty(ReturnProperty);
			if(Type.IsValid())
			{
				FName ParameterName = *Function->GetPathName();
				if(!ParameterNameMap.Contains(ParameterName))
				{
					FClassProxyParameter Parameter;
					Parameter.AccessType = EClassProxyParameterAccessType::AccessorFunction;
					Parameter.ParameterName = ParameterName;
					Parameter.Function = Function;
					Parameter.Type = Type;
					Parameter.DisplayName = Function->GetDisplayNameText();
					Parameter.Tooltip = Function->GetToolTipText();
					Parameter.bThreadSafe = Function->HasMetaData("BlueprintThreadSafe");

					ParameterNameMap.Add(Parameter.ParameterName, Parameters.Add(Parameter));
				}
			}
		}
	}

	// Finally add properties (accessors and extensions have already been added above so will take priority with duplicate names)
	for(TFieldIterator<FProperty> It(InClass, EFieldIterationFlags::IncludeSuper | EFieldIterationFlags::IncludeInterfaces); It; ++It)
	{
		FProperty* Property = *It;
		if(FParamUtils::CanUseProperty(Property))
		{
			FAnimNextParamType Type = FAnimNextParamType::FromProperty(Property);
			if(Type.IsValid())
			{
				FName ParameterName = *Property->GetPathName();
				if(!ParameterNameMap.Contains(ParameterName))
				{
					FClassProxyParameter Parameter;
					Parameter.AccessType = EClassProxyParameterAccessType::Property;
					Parameter.ParameterName = ParameterName;
					Parameter.Property = Property;
					Parameter.Type = Type;
					Parameter.DisplayName = Property->GetDisplayNameText();
					Parameter.Tooltip = Property->GetToolTipText();
					Parameter.bThreadSafe = false;

					ParameterNameMap.Add(Parameter.ParameterName, Parameters.Add(Parameter));
				}
			}
		}
	}
}

}
