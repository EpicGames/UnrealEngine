// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Param/ParamType.h"

namespace UE::UAF::UncookedOnly
{

enum class EClassProxyParameterAccessType : int32
{
	// Access via FProperty
	Property,

	// Access via UFunction with accessor signature Val = Obj.Func() 
	AccessorFunction,

	// Access via UFunction with hoisted signature Val = Func(Obj) 
	HoistedFunction,
};

// Cached info about how a class property or function maps to a parameter
struct FClassProxyParameter
{
	// The parameter name that this proxy corresponds to
	FName ParameterName;

	// The function to call to get this parameter
	TWeakObjectPtr<UFunction> Function;

	// The property to copy to get this parameter
	// TFieldPath to accomodate potential reinstancing
	TFieldPath<FProperty> Property;

	// The type of the property
	FAnimNextParamType Type;

	// How this parameter is accessed
	EClassProxyParameterAccessType AccessType;

#if WITH_EDITOR
	// Name to display in editor
	FText DisplayName;

	// Tooltip to display in editor
	FText Tooltip;

	// Whether this parameter is safe to access on worker threads
	bool bThreadSafe = false;
#endif
};

// Proxy struct used to hold data about a kind of UClass to cache data for
struct FClassProxy
{
	explicit FClassProxy(const UClass* InClass);

	// Refresh the cached class info
	void Refresh(const UClass* InClass);

	// The class that this proxy wraps
	TWeakObjectPtr<const UClass> Class;

	// Cache of properties, fetched from Class
	TArray<FClassProxyParameter> Parameters;

	// Map of parameter name to index in Parameters array
	TMap<FName, int32> ParameterNameMap;

#if WITH_EDITOR
	// Whether any parameters are safe to access on worker threads
	bool bHasThreadSafeParameters = false;

	// Whether any parameters are not safe to access on worker threads
	bool bHasNonThreadSafeParameters = true;
#endif
};

}