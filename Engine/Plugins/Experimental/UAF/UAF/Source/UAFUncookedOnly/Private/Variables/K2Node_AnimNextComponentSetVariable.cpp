// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/K2Node_AnimNextComponentSetVariable.h"

#include "AnimNextRigVMAsset.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_Literal.h"
#include "KismetCompiler.h"
#include "UncookedOnlyUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Component/AnimNextComponent.h"
#include "Param/ParamCompatibility.h"
#include "Param/ParamUtils.h"
#include "Variables/AnimNextVariableReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AnimNextComponentSetVariable)

#define LOCTEXT_NAMESPACE "K2Node_AnimNextComponentSetVariable"

UK2Node_AnimNextComponentSetVariable::UK2Node_AnimNextComponentSetVariable()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UUAFComponent, BlueprintSetVariableReference), UUAFComponent::StaticClass());
}

void UK2Node_AnimNextComponentSetVariable::ReconstructNode()
{
	Super::ReconstructNode();

	PropagateCachedTypeToPin();
}

void UK2Node_AnimNextComponentSetVariable::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);

	if (Pin == FindPinChecked(TEXT("Variable")))
	{
		CachePinType();
		PropagateCachedTypeToPin();
	}
}

void UK2Node_AnimNextComponentSetVariable::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);
	
	if (Pin == FindPinChecked(TEXT("Value")))
	{
		CachePinType();
		PropagateCachedTypeToPin();
	}
}

void UK2Node_AnimNextComponentSetVariable::CachePinType()
{
	// Cache the pin type if it has been set as a literal type via default value
	UEdGraphPin* VariablePin = FindPinChecked(TEXT("Variable"));
	if (VariablePin->LinkedTo.Num() == 0)
	{
		FAnimNextVariableReference VariableReference;
		FAnimNextVariableReference::StaticStruct()->ImportText(*VariablePin->DefaultValue, &VariableReference, nullptr, PPF_None, nullptr, FAnimNextVariableReference::StaticStruct()->GetName());
		if (VariableReference.IsValid())
		{
			FAnimNextParamType Type = UE::UAF::UncookedOnly::FUtils::FindVariableType(VariableReference);
			if (Type.IsValid())
			{
				CachedPinType = UE::UAF::UncookedOnly::FUtils::GetPinTypeFromParamType(Type);
			}
		}
	}
}

void UK2Node_AnimNextComponentSetVariable::PropagateCachedTypeToPin()
{
	// Find the value of the variable pin
	UEdGraphPin* VariablePin = FindPinChecked(TEXT("Variable"));
	UEdGraphPin* ValuePin = FindPinChecked(TEXT("Value"));

	if (!CachedPinType.PinCategory.IsNone() && ValuePin->PinType != CachedPinType)
	{
		ValuePin->BreakAllPinLinks();
		ValuePin->PinType = CachedPinType;
	}
	
	if (!CachedPinType.PinCategory.IsNone() && VariablePin->LinkedTo.Num() == 0 && ValuePin->LinkedTo.Num() == 0)
	{
		if (!ValuePin->GetSchema()->IsCurrentPinDefaultValid(ValuePin).IsEmpty())
		{
			ValuePin->GetSchema()->ResetPinToAutogeneratedDefaultValue(ValuePin);
		}
		GetGraph()->NotifyNodeChanged(this);

		UBlueprint* Blueprint = GetBlueprint();
		if (!Blueprint->bBeingCompiled)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

void UK2Node_AnimNextComponentSetVariable::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UK2Node* SourceNode = Cast<UK2Node>(MessageLog.FindSourceObject(this));
	if (SourceNode == nullptr)
	{
		return;
	}
	
	UEdGraphPin* VariablePin = SourceNode->FindPinChecked(TEXT("Variable"));
	UEdGraphPin* ValuePin = SourceNode->FindPinChecked(TEXT("Value"));

	const FAnimNextVariableReference VariableReference = UE::UAF::UncookedOnly::FUtils::ParseVariableReferenceFromPin(VariablePin);
	if (VariableReference.IsValid())
	{
		const FAnimNextParamType Type = UE::UAF::UncookedOnly::FUtils::FindVariableType(VariableReference);				// The variable from the UAF Module
		const FAnimNextParamType PinType = UE::UAF::UncookedOnly::FUtils::GetParamTypeFromPinType(ValuePin->PinType);	// The variable from BP pin
		const UE::UAF::FParamCompatibility Compatibility = UE::UAF::FParamUtils::GetCompatibility(Type, PinType);
		if (Compatibility == UE::UAF::EParamCompatibility::Incompatible || Compatibility == UE::UAF::EParamCompatibility::Incompatible_DataLoss)
		{
			MessageLog.Error(*FString::Printf(TEXT("@@ mismatched type, setting '%s' type, but variable is '%s' type"), *PinType.ToString(), *Type.ToString()), this);
		}
	}
	else if (VariableReference.IsNone())
	{
		MessageLog.Error(TEXT("@@ variable reference not set"), this);
	}
	else
	{
		MessageLog.Error(*FString::Printf(TEXT("@@ variable reference '%s' was not found in '%s'"), *VariableReference.GetName().ToString(), VariableReference.GetObject() ? TEXT("None") : *VariableReference.GetObject()->GetPathName()), this);
	}
}

void UK2Node_AnimNextComponentSetVariable::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	const UClass* ActionKey = GetClass();
	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UK2Node_AnimNextComponentSetVariable::PreloadRequiredAssets()
{
	Super::PreloadRequiredAssets();
	UE::UAF::UncookedOnly::FUtils::PreloadVariableReferenceAssets(FindPinChecked(TEXT("Variable")));
}

#undef LOCTEXT_NAMESPACE
