// Copyright Epic Games, Inc. All Rights Reserved.

#include "Variables/K2Node_UAFComponentSetInputBinding.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "UncookedOnlyUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Component/AnimNextComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Param/ParamCompatibility.h"
#include "Param/ParamUtils.h"
#include "Param/ParamType.h"
#include "Variables/AnimNextVariableReference.h"
#include "UAF/ValueRuntime/ValueBundle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_UAFComponentSetInputBinding)

#define LOCTEXT_NAMESPACE "K2Node_UAFComponentSetInputBinding"

const FName UK2Node_UAFComponentSetInputBinding::VariablePinName(TEXT("Variable"));
const FName UK2Node_UAFComponentSetInputBinding::ComponentPinName(TEXT("Component"));
const TArray<const UClass*> UK2Node_UAFComponentSetInputBinding::AllowedComponentClasses = { UUAFComponent::StaticClass(), USkeletalMeshComponent::StaticClass() };

UK2Node_UAFComponentSetInputBinding::UK2Node_UAFComponentSetInputBinding()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UUAFComponent, BlueprintSetInputBinding), UUAFComponent::StaticClass());
}

FString UK2Node_UAFComponentSetInputBinding::GetPinMetaData(FName InPinName, FName InKey)
{
	if (InPinName == VariablePinName && InKey == TEXT("AllowedType"))
	{
		return FAnimNextParamType::GetType<FUAFValueBundle>().ToString();
	}

	return Super::GetPinMetaData(InPinName, InKey);
}

bool UK2Node_UAFComponentSetInputBinding::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	if (MyPin->GetFName() == ComponentPinName && MyPin->Direction == EGPD_Input)
	{
		if (const UClass* ConnectedClass = Cast<UClass>(OtherPin->PinType.PinSubCategoryObject.Get()))
		{
			if (!AllowedComponentClasses.ContainsByPredicate([ConnectedClass](const UClass* AllowedClass)
			{
				return ConnectedClass->IsChildOf(AllowedClass);
			}))
			{
				OutReason = LOCTEXT("DisallowedComponentType", "Only UAFComponent and SkeletalMeshComponent types are allowed for input bindings.").ToString();
				return true;
			}
		}
	}

	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_UAFComponentSetInputBinding::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UK2Node* SourceNode = Cast<UK2Node>(MessageLog.FindSourceObject(this));
	if (SourceNode == nullptr)
	{
		return;
	}

	const UEdGraphPin* VariablePin = SourceNode->FindPinChecked(VariablePinName);

	FAnimNextVariableReference VariableReference = UE::UAF::UncookedOnly::FUtils::ParseVariableReferenceFromPin(VariablePin);
	if (VariableReference.IsValid())
	{
		const FAnimNextParamType Type = UE::UAF::UncookedOnly::FUtils::FindVariableType(VariableReference);
		const FAnimNextParamType ExpectedType = FAnimNextParamType::GetType<FUAFValueBundle>();
		const UE::UAF::FParamCompatibility Compatibility = UE::UAF::FParamUtils::GetCompatibility(ExpectedType, Type);
		if (Compatibility == UE::UAF::EParamCompatibility::Incompatible || Compatibility == UE::UAF::EParamCompatibility::Incompatible_DataLoss)
		{
			MessageLog.Error(*FString::Printf(TEXT("@@ variable '%s' is of type '%s', but only ValueBundle type variables are allowed"), *VariableReference.GetName().ToString(), *Type.ToString()), this);
		}
	}
	else if (VariableReference.IsNone())
	{
		MessageLog.Error(TEXT("@@ variable reference not set"), this);
	}
	else
	{
		MessageLog.Error(*FString::Printf(TEXT("@@ variable reference '%s' was not found in '%s'"), *VariableReference.GetName().ToString(), VariableReference.GetObject() ? *VariableReference.GetObject()->GetPathName() : TEXT("None")), this);
	}

	// Validate the component pin type if it is connected
	UEdGraphPin* ComponentPin = SourceNode->FindPinChecked(ComponentPinName);
	if (ComponentPin->LinkedTo.Num() > 0)
	{
		if (const UEdGraphPin* LinkedPin = ComponentPin->LinkedTo[0])
		{
			if (const UClass* ConnectedClass = Cast<UClass>(LinkedPin->PinType.PinSubCategoryObject.Get()))
			{
				const bool bIsAllowed = AllowedComponentClasses.ContainsByPredicate([ConnectedClass](const UClass* AllowedClass)
				{
					return ConnectedClass->IsChildOf(AllowedClass);
				});
				if (!bIsAllowed)
				{
					MessageLog.Error(*FString::Printf(TEXT("@@ component type '%s' is not allowed, only UAFComponent and SkeletalMeshComponent are supported"), *ConnectedClass->GetName()), this);
				}
			}
		}
	}
}

void UK2Node_UAFComponentSetInputBinding::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	const UClass* ActionKey = GetClass();
	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UK2Node_UAFComponentSetInputBinding::PreloadRequiredAssets()
{
	Super::PreloadRequiredAssets();
	UE::UAF::UncookedOnly::FUtils::PreloadVariableReferenceAssets(FindPinChecked(VariablePinName));
}

#undef LOCTEXT_NAMESPACE
