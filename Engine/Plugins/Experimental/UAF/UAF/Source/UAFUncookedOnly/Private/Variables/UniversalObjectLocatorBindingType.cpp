// Copyright Epic Games, Inc. All Rights Reserved.

#include "UniversalObjectLocatorBindingType.h"

#include "AnimNextModuleImpl.h"
#include "ClassProxy.h"
#include "InstancedStructDetails.h"
#include "IStructureDataProvider.h"
#include "IUniversalObjectLocatorEditorModule.h"
#include "PropertyHandle.h"
#include "UniversalObjectLocatorEditor.h"
#include "UniversalObjectLocatorFragmentType.h"
#include "Component/AnimNextComponent.h"
#include "Components/VerticalBox.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/PropertyViewer/IFieldExpander.h"
#include "Framework/PropertyViewer/IFieldIterator.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Module/RigUnit_AnimNextModuleEvents.h"
#include "Modules/ModuleManager.h"
#include "Param/ParamCompatibility.h"
#include "Param/ParamUtils.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMModel/RigVMController.h"
#include "Variables/AnimNextFieldPath.h"
#include "Variables/AnimNextSoftFunctionPtr.h"
#include "Variables/AnimNextUniversalObjectLocatorBindingData.h"
#include "Variables/RigUnit_ResolveUniversalObjectLocator.h"
#include "Variables/RigVMDispatch_CallHoistedAccessorFunction.h"
#include "Variables/RigVMDispatch_CallObjectAccessorFunction.h"
#include "Variables/RigVMDispatch_GetObjectProperty.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/PropertyViewer/SPropertyViewer.h"

#define LOCTEXT_NAMESPACE "UniversalObjectLocatorBindingType"

namespace UE::UAF::UncookedOnly
{

TSharedRef<SWidget> FUniversalObjectLocatorBindingType::CreateEditWidget(const TSharedRef<IPropertyHandle>& InPropertyHandle, const FAnimNextParamType& InType) const
{
	class SEditWidget : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SEditWidget) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle, const FAnimNextParamType& InType)
		{
			FilterType = InType;
			StructProvider = MakeShared<FInstancedStructProvider>(InPropertyHandle);
			InPropertyHandle->AddChildStructure(StructProvider.ToSharedRef());
			FieldIterator = MakeUnique<FFieldIterator>(InType);
			PropertyHandle = InPropertyHandle;
			FilterType = InType;

			LocatorHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNextUniversalObjectLocatorBindingData, Locator));
			check(LocatorHandle.IsValid());

			LocatorHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this, InType]()
			{
				// Reset property/function/type when setting container
				PropertyHandle->NotifyPreChange();
				PropertyHandle->EnumerateRawData([InType](void* RawData, int32 Index, int32 Num)
				{
					TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>* InstancedStruct = static_cast<TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>*>(RawData);
					FAnimNextUniversalObjectLocatorBindingData& Data = InstancedStruct->GetMutable();
					Data.Type = InType.IsObjectType() ? FAnimNextUniversalObjectLocatorBindingType::UOL : FAnimNextUniversalObjectLocatorBindingType::Property;
					Data.Property.Reset();
					Data.Function.Reset();
					return true;
				});
				PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
				PropertyHandle->NotifyFinishedChangingProperties();
				
				// Now refresh the entries to correspond to the new locator class
				RefreshEntries();
			}));

			ChildSlot
			[
				SNew(SBox)
				.WidthOverride(300.0f)
				.HeightOverride(400.0f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(0.0f, 2.0f))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							LocatorHandle->CreatePropertyValueWidgetWithCustomization(nullptr)
						]
					]
					+SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(2.0f)
					[
						SAssignNew(PropertyViewer, UE::PropertyViewer::SPropertyViewer)
						.OnSelectionChanged(this, &SEditWidget::HandleFieldPicked)
						.OnGenerateContainer(this, &SEditWidget::HandleGenerateContainer)
						.FieldIterator(FieldIterator.Get())
						.FieldExpander(&FieldExpander)
						.bShowSearchBox(true)
					]
				]
			];

			RefreshEntries();
		}

		void RefreshEntries()
		{
			using namespace UE::UniversalObjectLocator;

			PropertyViewer->RemoveAll();
			CachedContainers.Reset();
			ContainerMap.Reset();

			TOptional<FUniversalObjectLocator> CommonLocator;
			LocatorHandle->EnumerateConstRawData([&CommonLocator](const void* RawData, const int32 DataIndex, const int32 NumDatas)
			{
				const FUniversalObjectLocator& Locator = *static_cast<const FUniversalObjectLocator*>(RawData);
				if(!CommonLocator.IsSet())
				{
					CommonLocator = Locator;
				}
				else if(CommonLocator.GetValue() != Locator)
				{
					// No common locator, use empty
					CommonLocator = FUniversalObjectLocator();
					return false;
				}
				return true;
			});

			if(!CommonLocator.IsSet())
			{
				return;
			}

			FUniversalObjectLocator& Locator = CommonLocator.GetValue();

			IUniversalObjectLocatorEditorModule& UolEditorModule = FModuleManager::LoadModuleChecked<IUniversalObjectLocatorEditorModule>("UniversalObjectLocatorEditor");
			const FFragmentType* FragmentType = Locator.GetLastFragmentType();
			if(FragmentType == nullptr)
			{
				return;
			}

			TSharedPtr<ILocatorFragmentEditor> LocatorEditor = UolEditorModule.FindLocatorEditor(FragmentType->PrimaryEditorType);
			if(!LocatorEditor.IsValid())
			{
				return;
			}

			UObject* Context = UUAFComponent::StaticClass()->GetDefaultObject();
			const UStruct* Struct = LocatorEditor->ResolveClass(*Locator.GetLastFragment(), Context);
			if(Struct == nullptr)
			{
				return;
			}

			FieldIterator->CurrentStruct = Struct;

			if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Struct))
			{
				CachedContainers.Emplace(ScriptStruct->GetDisplayNameText(), ScriptStruct->GetToolTipText(), ScriptStruct);
				UE::PropertyViewer::SPropertyViewer::FHandle Handle = PropertyViewer->AddContainer(ScriptStruct);
				ContainerMap.Add(Handle, CachedContainers.Num() - 1);
			}
			else if(const UClass* Class = Cast<UClass>(Struct))
			{
				// Add this class
				{
					CachedContainers.Emplace(Class->GetDisplayNameText(), Class->GetToolTipText(), Class);
					UE::PropertyViewer::SPropertyViewer::FHandle Handle = PropertyViewer->AddContainer(Class);
					ContainerMap.Add(Handle, CachedContainers.Num() - 1);
				}

				// Find any UBlueprintFunctionLibrary classes to extend this class
				TArray<UClass*> Classes;
				GetDerivedClasses(UBlueprintFunctionLibrary::StaticClass(), Classes, true);
				for (const UClass* LibraryClass : Classes)
				{
					if(LibraryClass->HasAnyClassFlags(CLASS_Abstract))
					{
						continue;
					}

					if(LibraryClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
					{
						const UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(LibraryClass);
						
						// Make sure we skip any out-of-date or skeleton classes
						if(LibraryClass->HasAnyClassFlags(CLASS_NewerVersionExists) ||
							(BPClass && BPClass->IsInternalClass()))
						{
							continue;
						}
					}

					auto PassesFilterChecks = [this](const FProperty* InProperty)
					{
						if(InProperty && FilterType.IsValid())
						{
							FAnimNextParamType Type = FAnimNextParamType::FromProperty(InProperty);
							return FParamUtils::GetCompatibility(FilterType, Type).IsCompatible();
						}

						return false;
					};

					for (TFieldIterator<UFunction> FieldIt(LibraryClass); FieldIt; ++FieldIt)
					{
						if (FParamUtils::CanUseFunction(*FieldIt, Class))
						{
							if(PassesFilterChecks(FieldIt->GetReturnProperty()))
							{
								CachedContainers.Emplace(LibraryClass->GetDisplayNameText(), LibraryClass->GetToolTipText(), LibraryClass);
								UE::PropertyViewer::SPropertyViewer::FHandle Handle = PropertyViewer->AddContainer(LibraryClass);
								ContainerMap.Add(Handle, CachedContainers.Num() - 1);
								break;
							}
						}
					}
				}
			}
		}

		void HandleFieldPicked(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, TArrayView<const FFieldVariant> InFields, ESelectInfo::Type InSelectionType)
		{
			FSlateApplication::Get().DismissAllMenus();
			if (InFields.Num() == 0 && FilterType.IsObjectType())
			{
				// Container picked, so set to 'UOL' if possible
				if (int32* ContainerIndexPtr = ContainerMap.Find(InHandle))
				{
					FContainerInfo& ContainerInfo = CachedContainers[*ContainerIndexPtr];
					if(ContainerInfo.Struct)
					{
						PropertyHandle->NotifyPreChange();
						PropertyHandle->EnumerateRawData([](void* RawData, int32 Index, int32 Num)
						{
							TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>* InstancedStruct = static_cast<TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>*>(RawData);
							FAnimNextUniversalObjectLocatorBindingData& Data = InstancedStruct->GetMutable();
							Data.Type = FAnimNextUniversalObjectLocatorBindingType::UOL;
							Data.Property.Reset();
							Data.Function.Reset();
							return true;
						});
						PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
						PropertyHandle->NotifyFinishedChangingProperties();
					}
				}
			}
			else if(InFields.Num() == 1)
			{
				if (const UClass* Class = InFields[0].Get<UClass>())
				{
					PropertyHandle->NotifyPreChange();
					PropertyHandle->EnumerateRawData([Class](void* RawData, int32 Index, int32 Num)
					{
						TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>* InstancedStruct = static_cast<TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>*>(RawData);
						FAnimNextUniversalObjectLocatorBindingData& Data = InstancedStruct->GetMutable();
						Data.Type = FAnimNextUniversalObjectLocatorBindingType::UOL;
						Data.Property.Reset();
						Data.Function.Reset();
						return true;
					});
					PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
					PropertyHandle->NotifyFinishedChangingProperties();
				}
				else if(const FProperty* Property = InFields[0].Get<FProperty>())
				{
					PropertyHandle->NotifyPreChange();
					PropertyHandle->EnumerateRawData([Property](void* RawData, int32 Index, int32 Num)
					{
						TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>* InstancedStruct = static_cast<TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>*>(RawData);
						FAnimNextUniversalObjectLocatorBindingData& Data = InstancedStruct->GetMutable();
						Data.Type = FAnimNextUniversalObjectLocatorBindingType::Property;
						Data.Property = const_cast<FProperty*>(Property);
						Data.Function.Reset();
						return true;
					});
					PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
					PropertyHandle->NotifyFinishedChangingProperties();
				}
				else if(const UFunction* Function = InFields[0].Get<UFunction>())
				{
					PropertyHandle->NotifyPreChange();
					PropertyHandle->EnumerateRawData([Function](void* RawData, int32 Index, int32 Num)
					{
						TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>* InstancedStruct = static_cast<TInstancedStruct<FAnimNextUniversalObjectLocatorBindingData>*>(RawData);
						FAnimNextUniversalObjectLocatorBindingData& Data = InstancedStruct->GetMutable();
						if(Function->GetOuterUClass()->IsChildOf(UBlueprintFunctionLibrary::StaticClass()))
						{
							Data.Type = FAnimNextUniversalObjectLocatorBindingType::HoistedFunction;
						}
						else
						{
							Data.Type = FAnimNextUniversalObjectLocatorBindingType::Function;
						}
						Data.Property.Reset();
						Data.Function = const_cast<UFunction*>(Function);
						return true;
					});
					PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
					PropertyHandle->NotifyFinishedChangingProperties();
				}
			}
		}

		TSharedRef<SWidget> HandleGenerateContainer(UE::PropertyViewer::SPropertyViewer::FHandle InHandle, TOptional<FText> InDisplayName)
		{
			if(int32* ContainerIndexPtr = ContainerMap.Find(InHandle))
			{
				if(CachedContainers.IsValidIndex(*ContainerIndexPtr))
				{
					FContainerInfo& ContainerInfo = CachedContainers[*ContainerIndexPtr];

					return SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("ClassIcon.Object"))
					]
					+SHorizontalBox::Slot()
					.Padding(4.0f)
					[
						SNew(STextBlock)
						.Text(ContainerInfo.DisplayName)
						.ToolTipText(ContainerInfo.TooltipText)
					];
				}
			}

			return SNullWidget::NullWidget;
		}

		TSharedPtr<IPropertyHandle> LocatorHandle;
		TSharedPtr<IPropertyHandle> PropertyHandle;

		TSharedPtr<UE::PropertyViewer::SPropertyViewer> PropertyViewer;

		FAnimNextParamType FilterType;

		struct FFieldIterator : UE::PropertyViewer::IFieldIterator
		{
			FFieldIterator(const FAnimNextParamType& InFilterType)
				: FilterType(InFilterType)
			{}
		
			virtual TArray<FFieldVariant> GetFields(const UStruct* InStruct, const FName InFieldName, const UStruct* InContainerStruct) const
			{
				auto PassesFilterChecks = [this](const FProperty* InProperty)
				{
					if(InProperty && FilterType.IsValid())
					{
						FAnimNextParamType Type = FAnimNextParamType::FromProperty(InProperty);
						return FParamUtils::GetCompatibility(FilterType, Type).IsCompatible();
					}

					return true;
				};

				TArray<FFieldVariant> Result;
				for (TFieldIterator<FProperty> PropertyIt(InStruct, EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); PropertyIt; ++PropertyIt)
				{
					FProperty* Property = *PropertyIt;
					if (FParamUtils::CanUseProperty(Property))
					{
						if(PassesFilterChecks(Property))
						{
							Result.Add(FFieldVariant(Property));
						}
					}
				}
				for (TFieldIterator<UFunction> FunctionIt(InStruct, EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); FunctionIt; ++FunctionIt)
				{
					UFunction* Function = *FunctionIt;
					if (FParamUtils::CanUseFunction(Function, CastChecked<UClass>(CurrentStruct)))
					{
						if(PassesFilterChecks(Function->GetReturnProperty()))
						{
							Result.Add(FFieldVariant(Function));
						}
					}
				}
				return Result;
			}

			FAnimNextParamType FilterType;
			const UStruct* CurrentStruct = nullptr;
		};

		struct FFieldExpander : UE::PropertyViewer::IFieldExpander
		{
			virtual TOptional<const UClass*> CanExpandObject(const FObjectPropertyBase* Property, const UObject* Instance) const override
			{
				return TOptional<const UClass*>();
			}
			virtual bool CanExpandScriptStruct(const FStructProperty* StructProperty) const override
			{
				return false;
			}
			virtual TOptional<const UStruct*> GetExpandedFunction(const UFunction* Function) const override
			{
				return TOptional<const UStruct*>();
			}
		} FieldExpander;

		TUniquePtr<FFieldIterator> FieldIterator;

		struct FContainerInfo
		{
			FContainerInfo(const FText& InDisplayName, const FText& InTooltipText, const UStruct* InStruct)
				: DisplayName(InDisplayName)
				, TooltipText(InTooltipText)
				, Struct(InStruct)
			{}

			FText DisplayName;
			FText TooltipText;
			const UStruct* Struct = nullptr;
		};

		TArray<FContainerInfo> CachedContainers;

		TMap<UE::PropertyViewer::SPropertyViewer::FHandle, int32> ContainerMap;

		TSharedPtr<FInstancedStructProvider> StructProvider;
	};

	return SNew(SEditWidget, InPropertyHandle, InType);
}

FText FUniversalObjectLocatorBindingType::GetDisplayText(TConstStructView<FAnimNextVariableBindingData> InBindingData) const
{
	using namespace UE::UniversalObjectLocator;
	
	const FAnimNextUniversalObjectLocatorBindingData* LocatorBinding = InBindingData.GetPtr<FAnimNextUniversalObjectLocatorBindingData>();
	if(LocatorBinding == nullptr)
	{
		return LOCTEXT("NoLocatorLabel", "None");
	}

	TStringBuilder<256> StringBuilder;
	IUniversalObjectLocatorEditorModule& UolEditorModule = FModuleManager::LoadModuleChecked<IUniversalObjectLocatorEditorModule>("UniversalObjectLocatorEditor");
	LocatorBinding->Locator.ForEachFragment([&UolEditorModule, &StringBuilder](int32 FragmentIndex, int32 NumFragments, const FUniversalObjectLocatorFragment& InFragment)
	{
		if(const FFragmentType* FragmentType = InFragment.GetFragmentType())
		{
			TSharedPtr<ILocatorFragmentEditor> LocatorEditor = UolEditorModule.FindLocatorEditor(FragmentType->PrimaryEditorType);
			if(LocatorEditor.IsValid())
			{
				FText Text = LocatorEditor->GetDisplayText(&InFragment);
				if(FragmentIndex != 0)
				{
					StringBuilder.Append(TEXT("."));
				}
				StringBuilder.Append(Text.ToString());
				return true;
			}
		}

		return false;
	});

	switch(LocatorBinding->Type)
	{
	case FAnimNextUniversalObjectLocatorBindingType::UOL:
		break;
	case FAnimNextUniversalObjectLocatorBindingType::Property:
		if (StringBuilder.Len() > 0)
		{
			StringBuilder.Append(TEXT("."));
		}
		if(FProperty* Property = LocatorBinding->Property.Get())
		{
			Property->GetFName().AppendString(StringBuilder);
		}
		else if (!LocatorBinding->Property.IsPathToFieldEmpty())
		{
			StringBuilder.Append(TEXT("???"));
		}
		else
		{
			StringBuilder.Append(TEXT("None"));
		}
		break;
	case FAnimNextUniversalObjectLocatorBindingType::Function:
	case FAnimNextUniversalObjectLocatorBindingType::HoistedFunction:
		if (StringBuilder.Len() > 0)
		{
			StringBuilder.Append(TEXT("."));
		}
		if(UFunction* Function = LocatorBinding->Function.Get())
		{
			Function->GetFName().AppendString(StringBuilder);
		}
		else if (!LocatorBinding->Function.IsNull())
		{
			StringBuilder.Append(TEXT("???"));
		}
		else
		{
			StringBuilder.Append(TEXT("None"));
		}
		break;
	}

	return FText::FromStringView(StringBuilder);
}

FText FUniversalObjectLocatorBindingType::GetTooltipText(TConstStructView<FAnimNextVariableBindingData> InBindingData) const
{
	using namespace UE::UniversalObjectLocator;

	const FAnimNextUniversalObjectLocatorBindingData* LocatorBinding = InBindingData.GetPtr<FAnimNextUniversalObjectLocatorBindingData>();
	if(LocatorBinding == nullptr)
	{
		return FText::GetEmpty();
	}

	TStringBuilder<256> StringBuilder;
	IUniversalObjectLocatorEditorModule& UolEditorModule = FModuleManager::LoadModuleChecked<IUniversalObjectLocatorEditorModule>("UniversalObjectLocatorEditor");
	bool bSuccess = LocatorBinding->Locator.ForEachFragment([&UolEditorModule, &StringBuilder](int32 FragmentIndex, int32 NumFragments, const FUniversalObjectLocatorFragment& InFragment)
	{
		if(const FFragmentType* FragmentType = InFragment.GetFragmentType())
		{
			TSharedPtr<ILocatorFragmentEditor> LocatorEditor = UolEditorModule.FindLocatorEditor(FragmentType->PrimaryEditorType);
			if(LocatorEditor.IsValid())
			{
				FText Text = LocatorEditor->GetDisplayText(&InFragment);
				if(FragmentIndex != 0)
				{
					StringBuilder.Append(TEXT("."));
				}
				StringBuilder.Append(Text.ToString());
				return true;
			}
		}

		return false;
	});

	if(StringBuilder.Len() > 0)
	{
		StringBuilder.Append(TEXT("."));
	}

	switch(LocatorBinding->Type)
	{
	case FAnimNextUniversalObjectLocatorBindingType::Property:
		if(FProperty* Property = LocatorBinding->Property.Get())
		{
			Property->GetFName().AppendString(StringBuilder);
		}
		else if (!LocatorBinding->Property.IsPathToFieldEmpty())
		{
			StringBuilder.Append(TEXT("???"));
		}
		else
		{
			StringBuilder.Append(TEXT("None"));
		}
		break;
	case FAnimNextUniversalObjectLocatorBindingType::Function:
	case FAnimNextUniversalObjectLocatorBindingType::HoistedFunction:
		if(UFunction* Function = LocatorBinding->Function.Get())
		{
			Function->GetFName().AppendString(StringBuilder);
		}
		else if (!LocatorBinding->Function.IsNull())
		{
			StringBuilder.Append(TEXT("???"));
		}
		else
		{
			StringBuilder.Append(TEXT("None"));
		}
		break;
	}

	FTextBuilder TextBuilder;
	if(bSuccess)
	{
		TextBuilder.AppendLine(FText::FromStringView(StringBuilder));
	}

	TStringBuilder<256> ScopeStringBuilder;
	LocatorBinding->Locator.ToString(ScopeStringBuilder);
	TextBuilder.AppendLine(FText::Format(LOCTEXT("ParameterUOLTooltipFormat", "UOL: {0}"), FText::FromStringView(ScopeStringBuilder)));

	FText ValidationError;
	if (!Validate(InBindingData, ValidationError))
	{
		TextBuilder.AppendLine(FText::Format(LOCTEXT("ValidationErrorFormat", "Error: {0}"), ValidationError));
	}

	return TextBuilder.ToText();
}

bool FUniversalObjectLocatorBindingType::Validate(TConstStructView<FAnimNextVariableBindingData> InBindingData, FText& OutError) const
{
	const FAnimNextUniversalObjectLocatorBindingData& LocatorBinding = InBindingData.Get<FAnimNextUniversalObjectLocatorBindingData>();
	if (LocatorBinding.Locator.IsEmpty())
	{
		return true;
	}

	switch(LocatorBinding.Type)
	{
	case FAnimNextUniversalObjectLocatorBindingType::UOL:
		return true;
	case FAnimNextUniversalObjectLocatorBindingType::Property:
	{
		if (LocatorBinding.Property.Get() == nullptr)
		{
			OutError = FText::Format(LOCTEXT("PropertyBindingError", "Property binding '{0}' could not be found"), FText::FromString(LocatorBinding.Property.ToString()));
			return false;
		}
		return true;
	}
	case FAnimNextUniversalObjectLocatorBindingType::Function:
	{
		if(LocatorBinding.Function.Get() == nullptr)
		{
			OutError = FText::Format(LOCTEXT("FunctionBindingError", "Function '{0}' could not be found"), FText::FromString(LocatorBinding.Function.ToString()));
			return false;
		}
		return true;
	}
	case FAnimNextUniversalObjectLocatorBindingType::HoistedFunction:
	{
		if(LocatorBinding.Function.Get() == nullptr)
		{
			OutError = FText::Format(LOCTEXT("HoistedFunctionBindingError", "Function '{0}' could not be found"), FText::FromString(LocatorBinding.Function.ToString()));
			return false;
		}
		return true;
	}
	default:
		checkNoEntry();
		return false;
	}
}

void FUniversalObjectLocatorBindingType::BuildBindingGraphFragment(const FRigVMCompileSettings& InSettings, const FBindingGraphFragmentArgs& InArgs, URigVMPin*& OutExecTail, FVector2D& OutLocation) const
{
	if( InArgs.Event != FRigUnit_AnimNextExecuteBindings_GT::StaticStruct() &&
		InArgs.Event != FRigUnit_AnimNextExecuteBindings_WT::StaticStruct())
	{
		return;
	}
	
	URigVMPin* ExecTail = InArgs.ExecTail;

	struct FBindingData
	{
		URigVMNode* GetterNode = nullptr;
		TArray<const FBindingGraphInput*> Inputs;
	};

	// Cache all unique locators to resolve to objects
	TMap<FUniversalObjectLocator, FBindingData> LocatorDataMap;
	for(const FBindingGraphInput& Input : InArgs.Inputs)
	{
		const FAnimNextUniversalObjectLocatorBindingData& LocatorBinding = Input.BindingData.Get<FAnimNextUniversalObjectLocatorBindingData>();

		// Check if thread safety matches
		if(LocatorBinding.IsThreadSafe() == InArgs.bThreadSafe)
		{
			FBindingData& BindingData = LocatorDataMap.FindOrAdd(LocatorBinding.Locator);
			BindingData.Inputs.Add(&Input);
		}
	}

	// Spawn object getter nodes for locators
	float YOffset = 100.0f;
	for(const TPair<FUniversalObjectLocator, FBindingData>& LocatorDataPair : LocatorDataMap)
	{
		URigVMNode* ResolverNode = InArgs.Controller->AddUnitNode(FRigUnit_ResolveUniversalObjectLocator::StaticStruct(), FRigVMStruct::ExecuteName, FVector2D(0.0f, YOffset), TEXT(""), false);
		if(ResolverNode == nullptr)
		{
			InSettings.ReportError(TEXT("Could not spawn UOL resolver function"));
			continue;
		}
		URigVMPin* LocatorPin = ResolverNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_ResolveUniversalObjectLocator, Locator));
		if(LocatorPin == nullptr)
		{
			InSettings.ReportError(TEXT("Could find Locator pin"));
			continue;
		}
		FString DefaultLocatorValue;
		FUniversalObjectLocator::StaticStruct()->ExportText(DefaultLocatorValue, &LocatorDataPair.Key, nullptr, nullptr, PPF_None, nullptr);
		bool bLocatorPinDefaultValueSet = InArgs.Controller->SetPinDefaultValue(LocatorPin, DefaultLocatorValue, true, false, false);
		if(!bLocatorPinDefaultValueSet)
		{
			InSettings.ReportError(TEXT("Could not set Locator pin's default value"));
			continue;
		}
		URigVMPin* ResolvedObjectPin = ResolverNode->FindPin(GET_MEMBER_NAME_STRING_CHECKED(FRigUnit_ResolveUniversalObjectLocator, Object));
		if(ResolvedObjectPin == nullptr)
		{
			InSettings.ReportError(TEXT("Could not find UOL resolver object pin"));
			continue;
		}

		// Add unit nodes to call function or fetch property and set appropriate variables
		float AccumulatedYOffset = 0.0f;
		for(const FBindingGraphInput* Input : LocatorDataPair.Value.Inputs)
		{
			FText ValidationError;
			if (!Validate(Input->BindingData, ValidationError))
			{
				FText ValidationErrorReport = FText::Format(LOCTEXT("ValidationErrorReport", "Variable '{0}': {1}"), FText::FromName(Input->VariableName), ValidationError);
				InSettings.ReportError(ValidationErrorReport.ToString());
				continue;
			}

			const FAnimNextUniversalObjectLocatorBindingData& LocatorBinding = Input->BindingData.Get<FAnimNextUniversalObjectLocatorBindingData>();
			URigVMTemplateNode* GetterNode = nullptr;
			switch(LocatorBinding.Type)
			{
			case FAnimNextUniversalObjectLocatorBindingType::UOL:
				break;
			case FAnimNextUniversalObjectLocatorBindingType::Property:
			{
				if (LocatorBinding.Property.Get() == nullptr)
				{
					InSettings.ReportErrorf(TEXT("Property binding '%s' for variable '%s' could not be found"), *LocatorBinding.Property.ToString(), *Input->VariableName.ToString());
					continue;
				}

				GetterNode = InArgs.Controller->AddTemplateNode(FRigVMDispatch_GetObjectProperty().GetTemplateNotation(), FVector2D(200.0f, YOffset + AccumulatedYOffset), TEXT(""), false);
				if(GetterNode == nullptr)
				{
					InSettings.ReportError(TEXT("Could not spawn Get Object Property node"));
					continue;
				}
				URigVMPin* PropertyPin = GetterNode->FindPin(FRigVMDispatch_GetObjectProperty::PropertyName.ToString());
				if(PropertyPin == nullptr)
				{
					InSettings.ReportError(TEXT("Could not find Property pin"));
					continue;
				}

				FString DefaultPathValue;
				FAnimNextFieldPath AnimNextFieldPath;
				AnimNextFieldPath.FieldPath = LocatorBinding.Property;
				FAnimNextFieldPath::StaticStruct()->ExportText(DefaultPathValue, &AnimNextFieldPath, nullptr, nullptr, PPF_None, nullptr);
				bool bPropertyDefaultValueSet = InArgs.Controller->SetPinDefaultValue(PropertyPin, DefaultPathValue, true, false, false);
				if(!bPropertyDefaultValueSet)
				{
					InSettings.ReportError(TEXT("Could not set Property pin's default value"));
					continue;
				}

				URigVMPin* ValuePin = GetterNode->FindPin(FRigVMDispatch_GetObjectProperty::ValueName.ToString());
				if (ValuePin == nullptr)
				{
					InSettings.ReportError(TEXT("Could not find Value pin"));
					continue;
				}

				if (!InArgs.Controller->ResolveWildCardPin(ValuePin->GetPinPath(), FRigVMRegistry::Get().GetTypeIndex(*Input->CPPType, Input->CPPTypeObject), false))
				{
					InSettings.ReportError(TEXT("Could not resolve wildcard pin to function's return type"));
					continue;
				}

				break;
			}
			case FAnimNextUniversalObjectLocatorBindingType::Function:
			{
				UFunction* Function = LocatorBinding.Function.Get();
				if(Function == nullptr)
				{
					InSettings.ReportErrorf(TEXT("Could not resolve function call %s"), *LocatorBinding.Function.ToString());
					continue;
				}

				// Determine the call specialization we need (native vs script)
				FName Notation;
				if(Function->HasAllFunctionFlags(FUNC_Native))
				{
					Notation = FRigVMDispatch_CallObjectAccessorFunctionNative().GetTemplateNotation();
				}
				else
				{
					Notation = FRigVMDispatch_CallObjectAccessorFunctionScript().GetTemplateNotation();
				}

				GetterNode = InArgs.Controller->AddTemplateNode(Notation, FVector2D(200.0f, YOffset + AccumulatedYOffset), TEXT(""), false);
				if(GetterNode == nullptr)
				{
					InSettings.ReportError(TEXT("Could not spawn Call Object Accessor Function node"));
					continue;
				}
				URigVMPin* FunctionPin = GetterNode->FindPin(FRigVMDispatch_CallObjectAccessorFunctionBase::FunctionName.ToString());
				if(FunctionPin == nullptr)
				{
					InSettings.ReportError(TEXT("Could not find Function pin"));
					continue;
				}
				FString DefaultPtrValue;
				FAnimNextSoftFunctionPtr AnimNextSoftFunctionPtr;
				AnimNextSoftFunctionPtr.SoftObjectPtr = LocatorBinding.Function; 
				FAnimNextSoftFunctionPtr::StaticStruct()->ExportText(DefaultPtrValue, &AnimNextSoftFunctionPtr, nullptr, nullptr, PPF_None, nullptr);
				bool bFunctionDefaultValueSet = InArgs.Controller->SetPinDefaultValue(FunctionPin, DefaultPtrValue, true, false, false);
				if(!bFunctionDefaultValueSet)
				{
					InSettings.ReportError(TEXT("Could not set Function pin's default value"));
					continue;
				}

				URigVMPin* ValuePin = GetterNode->FindPin(FRigVMDispatch_CallObjectAccessorFunctionBase::ValueName.ToString());
				if (ValuePin == nullptr)
				{
					InSettings.ReportError(TEXT("Could not find Value pin"));
					continue;
				}

				if (!InArgs.Controller->ResolveWildCardPin(ValuePin->GetPinPath(), FRigVMRegistry::Get().GetTypeIndex(*Input->CPPType, Input->CPPTypeObject), false))
				{
					InSettings.ReportError(TEXT("Could not resolve wildcard pin to function's return type"));
					continue;
				}

				break;
			}
			case FAnimNextUniversalObjectLocatorBindingType::HoistedFunction:
			{
				UFunction* Function = LocatorBinding.Function.Get();
				if(Function == nullptr)
				{
					InSettings.ReportErrorf(TEXT("Could not resolve hoisted function call %s"), *LocatorBinding.Function.ToString());
					continue;
				}

				// Determine the call specialization we need (native vs script)
				FName Notation;
				if(Function->HasAllFunctionFlags(FUNC_Native))
				{
					Notation = FRigVMDispatch_CallHoistedAccessorFunctionNative().GetTemplateNotation();
				}
				else
				{
					Notation = FRigVMDispatch_CallHoistedAccessorFunctionScript().GetTemplateNotation();
				}

				GetterNode = InArgs.Controller->AddTemplateNode(Notation, FVector2D(200.0f, YOffset + AccumulatedYOffset), TEXT(""), false);
				if(GetterNode == nullptr)
				{
					InSettings.ReportError(TEXT("Could not spawn Call Hoisted Accessor Function node"));
					continue;
				}
				URigVMPin* FunctionPin = GetterNode->FindPin(FRigVMDispatch_CallHoistedAccessorFunctionBase::FunctionName.ToString());
				if(FunctionPin == nullptr)
				{
					InSettings.ReportError(TEXT("Could not find Function pin"));
					continue;
				}
				FString DefaultValue;
				FAnimNextSoftFunctionPtr AnimNextSoftFunctionPtr;
				AnimNextSoftFunctionPtr.SoftObjectPtr = LocatorBinding.Function; 
				FAnimNextSoftFunctionPtr::StaticStruct()->ExportText(DefaultValue, &AnimNextSoftFunctionPtr, nullptr, nullptr, PPF_None, nullptr);
				bool bFunctionDefaultValueSet = InArgs.Controller->SetPinDefaultValue(FunctionPin, DefaultValue, true, false, false);
				if(!bFunctionDefaultValueSet)
				{
					InSettings.ReportError(TEXT("Could not set Function pin's default value"));
					continue;
				}

				URigVMPin* ValuePin = GetterNode->FindPin(FRigVMDispatch_CallHoistedAccessorFunctionBase::ValueName.ToString());
				if (ValuePin == nullptr)
				{
					InSettings.ReportError(TEXT("Could not find Value pin"));
					continue;
				}

				if (!InArgs.Controller->ResolveWildCardPin(ValuePin->GetPinPath(), FRigVMRegistry::Get().GetTypeIndex(*Input->CPPType, Input->CPPTypeObject), false))
				{
					InSettings.ReportError(TEXT("Could not resolve wildcard pin to function's return type"));
					continue;
				}

				break;  
			}
			default:
				checkNoEntry();
				break;
			}

			URigVMPin* OutputValuePin = nullptr;
			if(LocatorBinding.Type != FAnimNextUniversalObjectLocatorBindingType::UOL)
			{
				check(GetterNode);
				URigVMPin* ObjectPin = GetterNode->FindPin(FRigVMDispatch_GetObjectProperty::ObjectName.ToString());
				if(ObjectPin == nullptr)
				{
					InSettings.ReportError(TEXT("Could not find object input pin"));
					continue;
				}
				bool bObjectPinLinked = InArgs.Controller->AddLink(ResolvedObjectPin, ObjectPin, false);
				if(!bObjectPinLinked)
				{
					InSettings.ReportError(TEXT("Could not link object pins"));
					continue;
				}
				OutputValuePin = GetterNode->FindPin(FRigVMDispatch_GetObjectProperty::ValueName.ToString());
				if(OutputValuePin == nullptr)
				{
					InSettings.ReportError(TEXT("Could find output value pin"));
					continue;
				}
			}
			else
			{
				OutputValuePin = ResolvedObjectPin;
			}
			check(OutputValuePin);

			URigVMVariableNode* VariableSetNode = InArgs.Controller->AddVariableNode(Input->VariableName, Input->CPPType, Input->CPPTypeObject, false, TEXT(""), FVector2D(400.0f, YOffset + AccumulatedYOffset), TEXT(""), false);
			if(VariableSetNode == nullptr)
			{
				InSettings.ReportError(TEXT("Could not spawn Variable Set node"));
				continue;
			}
			URigVMPin* VariableExecPin = VariableSetNode->FindPin(FRigVMStruct::ExecuteContextName.ToString());
			if(VariableExecPin == nullptr)
			{
				InSettings.ReportError(TEXT("Could not find Variable Set Execute pin"));
				continue;
			}
			bool bExecPinLinked = InArgs.Controller->AddLink(ExecTail, VariableExecPin, false);
			if(!bExecPinLinked)
			{
				InSettings.ReportError(TEXT("Could not link Variable Set Execute pin"));
				continue;
			}
			ExecTail = VariableExecPin;
			URigVMPin* VariableInputPin = VariableSetNode->GetValuePin();
			if(VariableInputPin == nullptr)
			{
				InSettings.ReportError(TEXT("Could not find Variable Set Value pin"));
				continue;
			}
			bool bValuePinLinked = InArgs.Controller->AddLink(OutputValuePin, VariableInputPin, false, ERigVMPinDirection::Invalid, /*bCreateCastNode*/true);
			if(!bValuePinLinked)
			{
				InSettings.ReportError(TEXT("Could not link Variable Set Value pin"));
				continue;
			}

			AccumulatedYOffset += 100.0f;
		}

		YOffset += AccumulatedYOffset;
	}

	OutExecTail = ExecTail;
	OutLocation = FVector2D(0.0f, YOffset);
}

const UClass* FUniversalObjectLocatorBindingType::GetClass(TConstStructView<FAnimNextVariableBindingData> InBindingData)
{
	using namespace UE::UniversalObjectLocator;

	const FAnimNextUniversalObjectLocatorBindingData* LocatorBinding = InBindingData.GetPtr<FAnimNextUniversalObjectLocatorBindingData>();
	if(LocatorBinding == nullptr)
	{
		return nullptr;
	}

	IUniversalObjectLocatorEditorModule& UolEditorModule = FModuleManager::LoadModuleChecked<IUniversalObjectLocatorEditorModule>("UniversalObjectLocatorEditor");
	const FFragmentType* FragmentType = LocatorBinding->Locator.GetLastFragmentType();
	if(FragmentType == nullptr)
	{
		return nullptr;
	}
	
	TSharedPtr<ILocatorFragmentEditor> LocatorEditor = UolEditorModule.FindLocatorEditor(FragmentType->PrimaryEditorType);
	if(!LocatorEditor.IsValid())
	{
		return nullptr;
	}

	UObject* Context = UUAFComponent::StaticClass()->GetDefaultObject();
	return LocatorEditor->ResolveClass(*LocatorBinding->Locator.GetLastFragment(), Context);
}

}

#undef LOCTEXT_NAMESPACE
