// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Variables/IVariableBindingType.h"

namespace UE::UAF::UncookedOnly
{
	struct FClassProxy;
}

namespace UE::UAF::UncookedOnly
{

// Provides information about object proxy parameter sources
class FUniversalObjectLocatorBindingType : public IVariableBindingType
{
private:
	// IVariableBindingType interface
	virtual TSharedRef<SWidget> CreateEditWidget(const TSharedRef<IPropertyHandle>& InPropertyHandle, const FAnimNextParamType& InType) const override;
	virtual FText GetDisplayText(TConstStructView<FAnimNextVariableBindingData> InBindingData) const override;
	virtual FText GetTooltipText(TConstStructView<FAnimNextVariableBindingData> InBindingData) const override;
	virtual bool Validate(TConstStructView<FAnimNextVariableBindingData> InBindingData, FText& OutError) const override;
	virtual void BuildBindingGraphFragment(const FRigVMCompileSettings& InSettings, const FBindingGraphFragmentArgs& InArgs, URigVMPin*& OutExecTail, FVector2D& OutLocation) const override;

	static const UClass* GetClass(TConstStructView<FAnimNextVariableBindingData> InBindingData);
};

}
