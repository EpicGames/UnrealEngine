// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "Param/ParamType.h"
#include "RigVMModel/RigVMGraph.h"

#include "AnimNextExports.generated.h"

namespace UE::UAF
{
static const FLazyName ExportsAnimNextAssetRegistryTag = TEXT("AnimNextExports");
}

UENUM()
enum class EAnimNextExportedVariableFlags : uint32
{
	NoFlags = 0x0,
	Public = 0x1,
	Read = 0x02,
	Write = 0x04,
	Declared = 0x08,
	Referenced = 0x10,
	Max
};

ENUM_CLASS_FLAGS(EAnimNextExportedVariableFlags)

USTRUCT()
struct FAnimNextExportData
{
	GENERATED_BODY()

	FAnimNextExportData() = default;
};

USTRUCT()
struct FAnimNextExport
{
	GENERATED_BODY()

	FAnimNextExport() = default;

	UPROPERTY()
	FName Identifier;

	UPROPERTY()
	TInstancedStruct<FAnimNextExportData> Data;

	bool operator==(const FAnimNextExport& Other) const
	{
		const UScriptStruct* ThisDataType = Data.GetScriptStruct();
		const UScriptStruct* OtherDataType = Other.Data.GetScriptStruct();
		return Identifier == Other.Identifier && ThisDataType == OtherDataType && (ThisDataType == nullptr || ThisDataType->CompareScriptStruct(Data.GetMemory(), Other.Data.GetMemory(), 0));
	}

	friend uint32 GetTypeHash(const FAnimNextExport& Entry)
	{		
		return Entry.Data.IsValid() ? HashCombine(GetTypeHash(Entry.Identifier), GetTypeHash(Entry.Data.GetScriptStruct())) : GetTypeHash(Entry.Identifier);
	}

	template<typename ItemDataType, typename... TArgs>
	static FAnimNextExport MakeExport(const FName& InIdentifier, TArgs&&... InArgs)
	{
		FAnimNextExport Export;
		Export.Identifier = InIdentifier;
		Export.Data.InitializeAs<ItemDataType>(Forward<TArgs>(InArgs)...);

		return Export;
	}
};

USTRUCT()
struct FAnimNextVariableReferenceData : public FAnimNextExportData
{
	GENERATED_BODY()

	FAnimNextVariableReferenceData() = default;
	FAnimNextVariableReferenceData(const FName& InGraphName, FStringView InNodePath, FStringView InPinPath, EAnimNextExportedVariableFlags InFlags) : GraphName(InGraphName), NodePath(InNodePath), PinPath(InPinPath), Flags(static_cast<uint32>(InFlags)) {}

	UPROPERTY()
	FName GraphName;
	
	UPROPERTY()
	FString NodePath;
	
	UPROPERTY()
	FString PinPath;
	
	UPROPERTY()
	uint32 Flags = static_cast<uint32>(EAnimNextExportedVariableFlags::NoFlags);
};

USTRUCT()
struct FAnimNextVariableDeclarationData : public FAnimNextExportData
{
	GENERATED_BODY()

	FAnimNextVariableDeclarationData() = default;
	FAnimNextVariableDeclarationData(const FAnimNextParamType& InType, const FGuid& InGuid, const EAnimNextExportedVariableFlags& InFlags) : Type(InType), Guid(InGuid), Flags(static_cast<uint32>(InFlags)) {}
	
	UPROPERTY()
	FAnimNextParamType Type;
	
	UPROPERTY()
	FGuid Guid;

	UPROPERTY()
	uint32 Flags = static_cast<uint32>(EAnimNextExportedVariableFlags::NoFlags);
};

USTRUCT()
struct FAnimNextManifestNodeData : public FAnimNextExportData
{
	GENERATED_BODY()

	FAnimNextManifestNodeData() = default;

	FAnimNextManifestNodeData(const TObjectPtr<URigVMGraph>& InModelGraph, const FString& InNodeName, const FString& InNodeCategory, const FString& InMenuDesc, const FString& InToolTip)
		: ModelGraph(FSoftObjectPath::ConstructFromObject(InModelGraph))
		, NodeName(InNodeName)
		, NodeCategory(InNodeCategory)
		, MenuDesc(InMenuDesc)
		, ToolTip(InToolTip)
	{}

	bool operator==(const FAnimNextManifestNodeData& Other) const
	{
		return ModelGraph == Other.ModelGraph && NodeName == Other.NodeName;
	}

	UPROPERTY()
	FSoftObjectPath ModelGraph;

	UPROPERTY()
	FString NodeName;

	UPROPERTY()
	FString NodeCategory;

	UPROPERTY()
	FString MenuDesc;

	UPROPERTY()
	FString ToolTip;
};

USTRUCT()
struct FAnimNextAssetRegistryExports
{
	GENERATED_BODY()

	FAnimNextAssetRegistryExports() = default;

	UPROPERTY()
	TArray<FAnimNextExport> Exports;

	template<typename ExportType>
	void ForEachExportOfType(TFunctionRef<bool(const FName&, const ExportType&)> FunctionRef)
	{
		for (const FAnimNextExport& Export : Exports)
		{
			if (Export.Data.IsValid())
			{
				if (Export.Data.GetScriptStruct() && Export.Data.GetScriptStruct()->IsChildOf(ExportType::StaticStruct()))
				{
					if (!FunctionRef(Export.Identifier, Export.Data.Get<ExportType>()))
					{
						return;
					}
				}
			}
		}
	}

	template<typename ExportType>
	void ForEachExportOfType(TFunctionRef<bool(const FName&, ExportType&)> FunctionRef)
	{
		for (const FAnimNextExport& Export : Exports)
		{
			if (Export.Data.IsValid())
			{
				if (Export.Data.GetScriptStruct() && Export.Data.GetScriptStruct()->IsChildOf(ExportType::StaticStruct()))
				{
					if (!FunctionRef(Export.Identifier, Export.Data.Get<ExportType>()))
					{
						return;
					}
				}
			}
		}
	}
};
