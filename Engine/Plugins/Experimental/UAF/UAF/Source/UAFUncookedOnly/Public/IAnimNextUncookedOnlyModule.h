// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "RigVMCore/RigVMExternalVariable.h"

template<typename T> struct TInstancedStruct;

class UUAFRigVMAsset;

namespace UE::UAF::UncookedOnly
{
	class IVariableBindingType;
}

namespace UE::UAF::UncookedOnly
{

class IAnimNextUncookedOnlyModule : public IModuleInterface
{
public:
	// Register a variable binding type, used to query information and process variable bindings
	// @param   InStructName     The full path name of the struct type for this variable binding's data (must be a child struct of FAnimNextVariableBindingData)
	// @param   InType           The type to register
	virtual void RegisterVariableBindingType(FName InStructName, TSharedPtr<IVariableBindingType> InType) = 0;
	
	// Unregister a variable binding type previously passed to RegisterVariableBindingType
	// @param   InStructName     The full path name of the struct type for this variable binding's data
	virtual void UnregisterVariableBindingType(FName InStructName) = 0;

	// Find a variable binding type previously passed to RegisterVariableBindingType
	// @param   InStruct         The struct type for this variable binding's data (must be a child struct of FAnimNextVariableBindingData)
	virtual TSharedPtr<IVariableBindingType> FindVariableBindingType(const UScriptStruct* InInstanceIdStruct) const = 0;
	
#if WITH_EDITOR
	// Try and retrieve a valid FGuid representing the provided variable (by name) within the source object
	// @param	InName			Name of the variable to retrieve the FGuid for
	// @param	InObject		Object, supposed to be, containing the variable (UScriptStruct, UUAFRigVMAsset or UUAFRigVMAssetEditorData)
	virtual FGuid GetVariableGuidByName(const FName InName, const UObject* InObject) const = 0; 
	// Try and retrieve a valid FName representing the provided variable (by Guid) within the source object 

	// @param	InGuid			FGuid of the variable to retrieve the FName for
	// @param	InObject		Object, supposed to be, containing the variable (UScriptStruct, UUAFRigVMAsset or UUAFRigVMAssetEditorData)
	virtual FName GetVariableNameByGuid(const FGuid InGuid, const UObject* InObject) const = 0;
	
	// Try and retrieve all local and referenced (through shared variable asset/struct(s)) variables for provided asset
	// @param	InAsset			Asset to retrieve variables for
	// @param	OutVariables	Output array of (referenced) external variables
	virtual void GetExternalVariablesForAsset(const UUAFRigVMAsset* InAsset, TArray<FRigVMExternalVariable>& OutVariables) const = 0;
	
	// Whether an Asset its compiled data is out-of-date, indicating it should not be relied upon
	// @param	InAsset			Asset to check for
	virtual bool DoesAssetVariablesRequireCompilation(const UUAFRigVMAsset* InAsset) const = 0;
#endif // WITH_EDITOR
	
	static IAnimNextUncookedOnlyModule& Get() { return FModuleManager::Get().LoadModuleChecked<IAnimNextUncookedOnlyModule>("UAFUncookedOnly"); }
};

}
