// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet2/CompilerResultsLog.h"

class UUAFRigVMAsset;
class UUAFRigVMAssetEditorData;
class FCompilerResultsLog;

namespace UE::UAF::UncookedOnly
{

struct Compilation
{
	// Request to synchronously compile the asset
	UAFUNCOOKEDONLY_API static void RequestAssetCompilation(UUAFRigVMAsset* InAsset);
	// Request to synchronously compile a set of assets
	UAFUNCOOKEDONLY_API static void RequestAssetCompilation(TConstArrayView<UUAFRigVMAsset*> InAssets);
};

// RAII helper for scoping compilation batches
// The re-allocation of compiled assets and their dependencies is deferred until the outermost scope exits.
class FCompilationScope
{
public:
	// Standalone scope with no immediate related assets
	UAFUNCOOKEDONLY_API explicit FCompilationScope(const FText& InJobName);

	// Scope with a single asset which the caller will compile
	UAFUNCOOKEDONLY_API explicit FCompilationScope(UUAFRigVMAsset* InAsset);
	
	// Scope with a set of assets which the caller will compile
	UAFUNCOOKEDONLY_API explicit FCompilationScope(TConstArrayView<UUAFRigVMAsset*> InAssets);
	
	// Scope with a explicit name and set of assets which the caller will compile
	UAFUNCOOKEDONLY_API explicit FCompilationScope(const FText& InJobName, TConstArrayView<UUAFRigVMAsset*> InAssets);

	// Scope with explicit name and a single asset which the caller will compile
	UAFUNCOOKEDONLY_API explicit FCompilationScope(const FText& InJobName, UUAFRigVMAsset* InAsset);

	UAFUNCOOKEDONLY_API ~FCompilationScope();

	// Returns a CompilerResultsLog instance specifically for the provided UObject
	UAFUNCOOKEDONLY_API static FCompilerResultsLog& GetLogForObject(const UObject* InObject);
private:
	static void ProcessAssets(const FText& InJobName, TConstArrayView<UUAFRigVMAsset*, int> InAssets);
};

}
