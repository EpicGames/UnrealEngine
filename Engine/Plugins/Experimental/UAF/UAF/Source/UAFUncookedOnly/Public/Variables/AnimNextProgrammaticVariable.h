// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Param/ParamType.h"
#include "StructUtils/PropertyBag.h"
#include "Variables/AnimNextVariableBinding.h"

#define UE_API UAFUNCOOKEDONLY_API

struct FRigVMGraphFunctionArgument;

/**
 * Struct wrapping a graph variable. Includes default value.
 */
struct FAnimNextProgrammaticVariable 
{

	/** Get arg AnimNext param type */
	UE_API FAnimNextParamType GetType() const;

	/** Set AnimNext param type, also sets propertybag to hold that type */
	UE_API bool SetType(const FAnimNextParamType& InType, bool bSetupUndoRedo = true);

	/** Get VM variable name */
	UE_API FName GetVariableName() const;

	/** Set VM variable name */
	UE_API void SetVariableName(FName InName, bool bSetupUndoRedo = true);

	/** Set variable default value in propertybag */
	UE_API bool SetDefaultValue(TConstArrayView<uint8> InValue, bool bSetupUndoRedo = true);

	/** Set variable default value in propertybag via string */
	UE_API bool SetDefaultValueFromString(const FString& InDefaultValue, bool bSetupUndoRedo = true);

	/** Get inner propertybag storing values */
	UE_API const FInstancedPropertyBag& GetPropertyBag() const;

	/** Get inner propertybag storing values, mutable */
	UE_API FInstancedPropertyBag& GetMutablePropertyBag();

	/** Get default value via property */
	UE_API bool GetDefaultValue(const FProperty*& OutProperty, TConstArrayView<uint8>& OutValue) const;

	/** Access the memory for the internal value */
	UE_API const uint8* GetValuePtr() const;
	
	/** Returns deterministic GUID */
	UE_API FGuid GetGuid() const;
public:

	/** Name of the variable */
	FName Name;

	/** The variable's type */
	FAnimNextParamType Type = FAnimNextParamType::GetType<bool>();

	/** Property bag holding the default value of the variable */
	FInstancedPropertyBag DefaultValue;

public: 

	/** Construct a parameter type from the passed in FRigVMTemplateArgumentType. */
	static UE_API FAnimNextProgrammaticVariable FromRigVMGraphFunctionArgument(const FRigVMGraphFunctionArgument& RigVMGraphFunctionArgument);
};

#undef UE_API
