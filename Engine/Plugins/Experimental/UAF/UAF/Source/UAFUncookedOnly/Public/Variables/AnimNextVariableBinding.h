// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextVariableBindingData.h"
#include "StructUtils/InstancedStruct.h"
#include "AnimNextVariableBinding.generated.h"

USTRUCT()
struct FAnimNextVariableBinding
{
	GENERATED_BODY()

	FAnimNextVariableBinding() = default;

	// Returns whether this binding is valid
	bool IsValid() const
	{
		return BindingData.IsValid() && BindingData.Get<FAnimNextVariableBindingData>().IsValid();
	}

	UPROPERTY(EditAnywhere, Category = "Binding", meta=(ExcludeBaseStruct))
	TInstancedStruct<FAnimNextVariableBindingData> BindingData;
};

