// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimNextVariableBindingData.generated.h"

// Base struct that abstracts binding source data
USTRUCT(meta=(Hidden))
struct FAnimNextVariableBindingData
{
	GENERATED_BODY()

	virtual ~FAnimNextVariableBindingData() = default;
	
	// Returns whether this binding data is valid or not
	virtual bool IsValid() const { return false; }

	// Returns whether this binding data is thread safe or not
	virtual bool IsThreadSafe() const { return false; }
};
