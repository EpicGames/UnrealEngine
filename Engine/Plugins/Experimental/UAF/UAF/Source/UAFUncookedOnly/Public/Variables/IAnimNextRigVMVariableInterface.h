// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "StructUtils/PropertyBag.h"
#include "IAnimNextRigVMVariableInterface.generated.h"

#define UE_API UAFUNCOOKEDONLY_API

struct FAnimNextParamType;
struct FAnimNextVariableBinding;
struct FAnimNextVariableBindingData;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UUAFRigVMVariableInterface : public UInterface
{
	GENERATED_BODY()
};

class IUAFRigVMVariableInterface
{
	GENERATED_BODY()

public:
	// Get the variable type
	virtual FAnimNextParamType GetType() const = 0;

	// Set the variable type
	virtual bool SetType(const FAnimNextParamType& InType, bool bSetupUndoRedo = true) = 0;

	// Get the variable name
	virtual FName GetVariableName() const = 0;

	// Set the variable name
	virtual void SetVariableName(FName InName, bool bSetupUndoRedo = true) = 0;

	// Set the default value
	virtual bool SetDefaultValue(TConstArrayView<uint8> InValue, bool bSetupUndoRedo = true) = 0;

	// Set the default value from a string
	virtual bool SetDefaultValueFromString(const FString& InDefaultValue, bool bSetupUndoRedo = true) = 0;

	// Access the backing storage property bag for the parameter
	virtual const FInstancedPropertyBag& GetPropertyBag() const = 0;

	// Access the mutable backing storage property bag for the parameter
	FInstancedPropertyBag& GetMutablePropertyBag()
	{
		return const_cast<FInstancedPropertyBag&>(GetPropertyBag());
	}

	virtual bool GetDefaultValue(const FProperty*& OutProperty, TConstArrayView<uint8>& OutValue) const = 0;
	virtual bool GetDefaultValueString(FString& OutValueString) const = 0;

	// Set the binding type for this variable (initializes it to default if the struct type is valid)
	virtual void SetBindingType(UScriptStruct* InBindingTypeStruct, bool bSetupUndoRedo = true) = 0;

	// Set the binding for this variable
	virtual void SetBinding(TInstancedStruct<FAnimNextVariableBindingData>&& InBinding, bool bSetupUndoRedo = true) = 0;

	// Get the binding for this variable, if any
	virtual TConstStructView<FAnimNextVariableBindingData> GetBinding() const = 0;

	// Get the category for this variable, empty string if none assigned
	virtual FStringView GetVariableCategory() const = 0;
	
	// Set the category for this variable
	virtual void SetVariableCategory(FStringView InCategoryName, bool bSetupUndoRedo = true) = 0;
	
	// Returns (stable and unique) GUID representing this variable
	virtual const FGuid& GetGuid() const = 0;

	// Access the memory for the internal value
	const uint8* GetValuePtr() const
	{
		const FInstancedPropertyBag& PropertyBag = GetPropertyBag();
		const UPropertyBag* PropertyBagStruct = PropertyBag.GetPropertyBagStruct();
		const uint8* Memory = PropertyBag.GetValue().GetMemory();
		return PropertyBagStruct && Memory ? PropertyBagStruct->GetPropertyDescs()[0].CachedProperty->ContainerPtrToValuePtr<uint8>(Memory) : nullptr;
	}

	// The name of the value property in the internal property bag
	static UE_API const FLazyName ValueName;
};

#undef UE_API
