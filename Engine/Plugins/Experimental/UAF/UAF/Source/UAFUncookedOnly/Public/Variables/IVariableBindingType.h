// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Param/ParamType.h"

class IUAFRigVMVariableInterface;
struct FRigVMCompileSettings;
class IPropertyHandle;
class URigVMController;
class URigVMGraph;
class URigVMNode;
class URigVMPin;
struct FAnimNextVariableBindingData;
template<typename T> struct TInstancedStruct;

namespace UE::UAF::UncookedOnly
{
	class FModule;
	struct FUtils;
}

namespace UE::UAF::Editor
{
	class FVariableBindingPropertyCustomization;
}

namespace UE::UAF::UncookedOnly
{

// Interface used in editor/uncooked situations to determine the characteristics of a variable binding
class IVariableBindingType
{
public:
	virtual ~IVariableBindingType() = default;

protected:
	struct FBindingGraphInput
	{
		// Name of the target variable that this binding is bound to
		FName VariableName;
		// CPPType of the variable
		FString CPPType;
		// CPPTypeObject of the variable
		TObjectPtr<UObject> CPPTypeObject;
		// Binding data of the type that this processor is registered against
		TConstStructView<FAnimNextVariableBindingData> BindingData;
	};

	struct FBindingGraphFragmentArgs
	{
		// The event (e.g. FRigUnit_AnimNextExecuteBindings_GT) that is currently being processed
		UScriptStruct* Event;
		// All inputs, corresponding to variables
		TConstArrayView<FBindingGraphInput> Inputs;
		// Controller to use for instantiation 
		URigVMController* Controller;
		// Graph to instantiate nodes into
		URigVMGraph* BindingGraph;
		// The exec pin of the last node that was instantiated, for chaining 
		URigVMPin* ExecTail;
		// The current spawn location, useful for making user-readable graphs
		FVector2D CurrentLocation;
		// Whether we should be processing thread-safe bindings
		bool bThreadSafe;
	};
	
private:
	// Create a widget used to edit the binding (displayed in a submenu from a combobox)
	virtual TSharedRef<SWidget> CreateEditWidget(const TSharedRef<IPropertyHandle>& InPropertyHandle, const FAnimNextParamType& InType) const = 0;

	// Get the display text for the specified instance ID
	virtual FText GetDisplayText(TConstStructView<FAnimNextVariableBindingData> InBindingData) const = 0;

	// Get the tooltip text for the specified instance ID
	virtual FText GetTooltipText(TConstStructView<FAnimNextVariableBindingData> InBindingData) const = 0;

	// Validate the binding data
	virtual bool Validate(TConstStructView<FAnimNextVariableBindingData> InBindingData, FText& OutError) const = 0;

	// Transforms the inputs into graph fragments. Called to convert variable bindings (derived from FAnimNextVariableBindingData) into intermediate
	// RigVM graphs for consumption by the compiler  
	virtual void BuildBindingGraphFragment(const FRigVMCompileSettings& InSettings, const FBindingGraphFragmentArgs& InArgs, URigVMPin*& OutExecTail, FVector2D& OutLocation) const = 0;

	friend class FModule;
	friend struct FUtils;
	friend class Editor::FVariableBindingPropertyCustomization;
};

}