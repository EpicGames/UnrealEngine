// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UAFUncookedOnly : ModuleRules
	{
		public UAFUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"RigVM",
					"RigVMDeveloper",
					"UAF",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"BlueprintGraph",	// For K2 schema
					"AnimationCore",
					"AnimGraph",
					"Kismet",
					"Slate",
					"SlateCore",
					"StructUtilsEditor",
					"ToolMenus",
					"UniversalObjectLocator", 
					"UniversalObjectLocatorEditor",
					"WorkspaceEditor",
					"MessageLog", 
					"AdvancedWidgets",
				}
			);

			if(Target.bWithLiveCoding)
			{
				PrivateDependencyModuleNames.Add("LiveCoding");
			}
			
			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"SourceControl",
					}
				);

				PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"WorkspaceEditor",
					}
				);
				
				PrivateIncludePathModuleNames.AddRange(
					new string[]
					{
						"UAFEditor",
					}
				);

				DynamicallyLoadedModuleNames.AddRange(
					new string[]
					{
						"UAFEditor",
					}
				);
			}
		}
	}
}