{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Unreal Animation Framework (UAF)",
	"Description" : "Framework for defining functional data flow for animation systems",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins": [
		{
			"Name": "RigVM",
			"Enabled": true
		},
		{
			"Name": "ContentBrowserAssetDataSource",
			"Enabled": true
		},
		{
			"Name": "ContentBrowserFileDataSource",
			"Enabled": true
		},
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true
		},
		{
			"Name": "Workspace",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "HierarchyTable",
			"Enabled": true
		},
		{
			"Name": "HierarchyTableAnimation",
			"Enabled": true
		},
		{
			"Name": "GameplayInsights",
			"Enabled": true
		},
		{
			"Name": "PropertyBindingUtils",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "UAF",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "UAFEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "UAFUncookedOnly",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "UAFTestData",
			"Type" : "DeveloperTool",
			"LoadingPhase" : "Default"
		}
	]
}
